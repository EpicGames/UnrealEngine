// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BackgroundHTTP : ModuleRules
{
    public BackgroundHTTP(ReadOnlyTargetRules Target) : base(Target)
    {
		bRequiresPlatformSDK = true;


        PublicIncludePathModuleNames.AddRange(
            new string[] {
                "HTTP",
            }
        );

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "ApplicationCore",
                "Engine",
                "HTTP",
                "BackgroundHTTPFileHash"
            }
        );

 		if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PublicDependencyModuleNames.Add("BackgroundHttpIOS");
		}
    }
}
