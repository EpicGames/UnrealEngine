// Copyright Epic Games, Inc. All Rights Reserved.
#include "BackgroundHttpManagerImpl.h"

#include "CoreTypes.h"
#include "PlatformBackgroundHttp.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformFile.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"
#include "Stats/Stats.h"
#include "ProfilingDebugging/CsvProfiler.h"

#if PLATFORM_ANDROID
#include "Android/AndroidPlatformMisc.h"
#endif

DEFINE_LOG_CATEGORY(LogBackgroundHttpManager);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(BACKGROUNDHTTP_API, BackgroundDownload);
CSV_DEFINE_CATEGORY(BackgroundDownload, true);

FBackgroundHttpManagerImpl::FBackgroundHttpManagerImpl()
	: PendingStartRequests()
	, PendingRequestLock()
	, ActiveRequests()
	, ActiveRequestLock()
	, NumCurrentlyActiveRequests(0)
	, MaxActiveDownloads(4)
	, FileHashHelper(MakeShared<FBackgroundHttpFileHashHelper, ESPMode::ThreadSafe>())
{
}

FBackgroundHttpManagerImpl::~FBackgroundHttpManagerImpl()
{
}

void FBackgroundHttpManagerImpl::Initialize(bool bClearPreExistingRequestsAtStartup)
{
	//Make sure we have attempted to load data at initialize
	GetFileHashHelper()->LoadData();
	
	DeleteStaleTempFiles();
	
	// Can't read into an atomic int directly
	int MaxActiveDownloadsConfig = 4;
	ensureAlwaysMsgf(GConfig->GetInt(TEXT("BackgroundHttp"), TEXT("MaxActiveDownloads"), MaxActiveDownloadsConfig, GEngineIni), TEXT("No value found for MaxActiveDownloads! Defaulting to 4!"));
	MaxActiveDownloads = MaxActiveDownloadsConfig;
}

void FBackgroundHttpManagerImpl::Shutdown()
{
	//Pending Requests Clear
	{
		FRWScopeLock ScopeLock(PendingRequestLock, SLT_Write);
		PendingStartRequests.Empty();
	}

	//Active Requests Clear
	{
		FRWScopeLock ScopeLock(ActiveRequestLock, SLT_Write);
		ActiveRequests.Empty();
		NumCurrentlyActiveRequests = 0;
	}
}

void FBackgroundHttpManagerImpl::DeleteStaleTempFiles()
{
	//Parse our .ini values to determine how much we clean in this fuction
	double FileAgeTimeOutSettings = -1;
	bool bDeleteTimedOutFiles = (FileAgeTimeOutSettings >= 0);
	bool bDeleteTempFilesWithoutURLMappings = false;
	bool bRemoveURLMappingEntriesWithoutPhysicalTempFiles = false;
	{
		GConfig->GetDouble(TEXT("BackgroundHttp"), TEXT("TempFileTimeOutSeconds"), FileAgeTimeOutSettings, GEngineIni);
		bDeleteTimedOutFiles = (FileAgeTimeOutSettings >= 0);
		
		GConfig->GetBool(TEXT("BackgroundHttp"), TEXT("DeleteTempFilesWithoutURLMappingEntries"), bDeleteTempFilesWithoutURLMappings, GEngineIni);
		
		GConfig->GetBool(TEXT("BackgroundHttp"), TEXT("RemoveURLMappingEntriesWithoutPhysicalTempFiles"), bRemoveURLMappingEntriesWithoutPhysicalTempFiles, GEngineIni);
		
		UE_LOGF(LogBackgroundHttpManager, Log, "Stale Settings -- TempFileTimeOutSeconds:%f DeleteTempFilesWithoutURLMappingEntries:%d RemoveURLMappingEntriesWithoutPhysicalTempFiles:%d", static_cast<float>(FileAgeTimeOutSettings),static_cast<int>(bDeleteTempFilesWithoutURLMappings),static_cast<int>(bRemoveURLMappingEntriesWithoutPhysicalTempFiles));
	}
	const bool bWillDoAnyWork = bDeleteTimedOutFiles || bDeleteTempFilesWithoutURLMappings || bRemoveURLMappingEntriesWithoutPhysicalTempFiles;
	
	//Only bother gathering temp files if we will actually be doing something with them
	TArray<FString> AllTempFilesToCheck;
	if (bWillDoAnyWork)
	{
		GatherAllTempFilenames(AllTempFilesToCheck);
		UE_LOGF(LogBackgroundHttpManager, Display, "Found %d temp download files.", AllTempFilesToCheck.Num());
	}
	
	//Handle all timed out files based on the .ini time out settings
	//can be turned off by setting
	if (bDeleteTimedOutFiles)
	{
		TArray<FString> TimedOutFiles;
		GatherTempFilesOlderThen(TimedOutFiles, FileAgeTimeOutSettings, &AllTempFilesToCheck);
	
		TArray<FString> TimeOutDeleteFullPaths;
		ConvertAllTempFilenamesToFullPaths(TimeOutDeleteFullPaths, TimedOutFiles);
		
		for (const FString& FullFilePath : TimeOutDeleteFullPaths)
		{
			if (IFileManager::Get().Delete(*FullFilePath))
			{
				UE_LOGF(LogBackgroundHttpManager, Log, "Successfully deleted %ls due to time out settings", *FullFilePath);
			}
			else
			{
				UE_LOGF(LogBackgroundHttpManager, Error, "Failed to delete timed out file %ls", *FullFilePath);
			}
		}
		
		//Should remove these files from the list of files we are checking as we know they are already invalid from timing out, so we shouldn't check them twice
		for(const FString& RemovedFile : TimedOutFiles)
		{
			AllTempFilesToCheck.Remove(RemovedFile);
		}
	}
	
	//Handle all temp files that should be removed because they are missing a corresponding URL mapping
	if (bDeleteTempFilesWithoutURLMappings)
	{
		TArray<FString> MissingURLMappingFiles;
		GatherTempFilesWithoutURLMappings(MissingURLMappingFiles, &AllTempFilesToCheck);
		
		TArray<FString> MissingURLDeleteFullPaths;
		ConvertAllTempFilenamesToFullPaths(MissingURLDeleteFullPaths, MissingURLMappingFiles);
		
		for (const FString& FullFilePath : MissingURLDeleteFullPaths)
		{
			if (IFileManager::Get().Delete(*FullFilePath))
			{
				UE_LOGF(LogBackgroundHttpManager, Log, "Successfully deleted %ls due to missing a URL mapping for this temp data", *FullFilePath);
			}
			else
			{
				UE_LOGF(LogBackgroundHttpManager, Error, "Failed to delete file %ls that was being deleted due to a missing URL mapping", *FullFilePath);
			}
		}
		
		//Should remove these files from the list of files we are checking as we know they are already invalid from timing out, so we shouldn't check them twice
		for(const FString& RemovedFile : MissingURLMappingFiles)
		{
			AllTempFilesToCheck.Remove(RemovedFile);
		}
	}
	
	if (bRemoveURLMappingEntriesWithoutPhysicalTempFiles)
	{
		//Remove all URL map entries that don't correspond to a physical file on disk
		GetFileHashHelper()->DeleteURLMappingsWithoutTempFiles();
	}
	
	UE_LOGF(LogBackgroundHttpManager, Log, "Kept %d temp download files:", AllTempFilesToCheck.Num());
	for (const FString& ValidFile : AllTempFilesToCheck)
	{
		UE_LOGF(LogBackgroundHttpManager, Verbose, "Kept: %ls", *ValidFile);
	}
}

void FBackgroundHttpManagerImpl::GatherTempFilesOlderThen(TArray<FString>& OutTimedOutTempFilenames,double SecondsToConsiderOld, TArray<FString>* OptionalFileList /* = nullptr */) const
{
	OutTimedOutTempFilenames.Empty();
	
	TArray<FString> GatheredFullFilePathFiles;
	
	//OptionalFileList was not supplied so we need to gather all temp files to check as full file paths
	if (nullptr == OptionalFileList)
	{
		GatherAllTempFilenames(GatheredFullFilePathFiles, true);
	}
	//We supplied an OptionalFileList, but we still need a full file path list for this operation
	else
	{
		ConvertAllTempFilenamesToFullPaths(GatheredFullFilePathFiles, *OptionalFileList);
	}

	if (SecondsToConsiderOld >= 0)
	{
		UE_LOGF(LogBackgroundHttpManager, Verbose, "Checking for BackgroundHTTP temp files that are older then: %lf", SecondsToConsiderOld);
		
		for (const FString& FullFilePath : GatheredFullFilePathFiles)
		{
			FFileStatData FileData = IFileManager::Get().GetStatData(*FullFilePath);
			FTimespan TimeSinceCreate = FDateTime::UtcNow() - FileData.CreationTime;

			const double FileAge = TimeSinceCreate.GetTotalSeconds();
			const bool bShouldReturn = (FileAge > SecondsToConsiderOld);
			if (bShouldReturn)
			{
				UE_LOGF(LogBackgroundHttpManager, Verbose, "FoundTempFile: %ls with age %lf", *FullFilePath, FileAge);
				
				//Need to save output as just filename to be consistent with other functions
				OutTimedOutTempFilenames.Add(FPaths::GetCleanFilename(FullFilePath));
			}
		}
	}
}

void FBackgroundHttpManagerImpl::GatherTempFilesWithoutURLMappings(TArray<FString>& OutTempFilesMissingURLMappings, TArray<FString>* OptionalFileList /*= nullptr */) const
{
	OutTempFilesMissingURLMappings.Empty();

	TArray<FString>* FileListToCheckPtr = OptionalFileList;

	//OptionalFileList was not supplied so we need to gather all temp files to check
	TArray<FString> GatheredFiles;
	if (nullptr == FileListToCheckPtr)
	{
		GatherAllTempFilenames(GatheredFiles, false);
		FileListToCheckPtr = &GatheredFiles;
	}
	
	TArray<FString>& FilesToCheckRef = *FileListToCheckPtr;
	for (const FString& File : FilesToCheckRef)
	{
		FString FoundURL = {};
		if (!GetFileHashHelper()->FindMappedURLForTempFilename(File, FoundURL))
		{
			OutTempFilesMissingURLMappings.Add(File);
		}
	}
}

int FBackgroundHttpManagerImpl::DeleteAllTemporaryFiles(bool Recursively, const TCHAR* FileExtension)
{
	TArray<FString> FilesToDelete;

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (Recursively)
		PlatformFile.FindFilesRecursively(FilesToDelete, *FBackgroundHttpFileHashHelper::GetTemporaryDownloadPath(), FileExtension);
	else
		PlatformFile.FindFiles(FilesToDelete, *FBackgroundHttpFileHashHelper::GetTemporaryRootPath(), FileExtension);

	for (const FString& File : FilesToDelete)
	{
		UE_LOGF(LogBackgroundHttpManager, Log, "Deleting File:%ls", *File);
		const bool bDidDelete = PlatformFile.DeleteFile(*File);

		if (!bDidDelete)
		{
			UE_LOGF(LogBackgroundHttpManager, Warning, "Failure to Delete Temp File:%ls", *File);
		}
	}
	return FilesToDelete.Num();
}

void FBackgroundHttpManagerImpl::GatherAllTempFilenames(TArray<FString>& OutAllTempFilenames, bool bOutputAsFullPaths /* = false */) const
{
	OutAllTempFilenames.Empty();
	
	const FString& DirectoryToCheck = FBackgroundHttpFileHashHelper::GetTemporaryDownloadPath();
	
	TArray<FString> AllFilenames;
	IFileManager::Get().FindFiles(AllFilenames, *DirectoryToCheck, *FBackgroundHttpFileHashHelper::GetTempFileExtension());
	
	//Make into full paths for output
	for (const FString& Filename : AllFilenames)
	{
		if (bOutputAsFullPaths)
		{
			OutAllTempFilenames.Add(FPaths::Combine(DirectoryToCheck, Filename));
		}
		else
		{
			OutAllTempFilenames.Add(Filename);
		}
	}
}

void FBackgroundHttpManagerImpl::ConvertAllTempFilenamesToFullPaths(TArray<FString>& OutFilenamesAsFullPaths, const TArray<FString>& FilenamesToConvertToFullPaths) const
{
	//Store this separetly so we don't get bad behavior if the same Array is supplied for both parameters
	TArray<FString> FilenamesToOutput;
	
	for(const FString& ExistingFilename : FilenamesToConvertToFullPaths)
	{
		FilenamesToOutput.Add(FBackgroundHttpFileHashHelper::GetFullPathOfTempFilename(ExistingFilename));
	}
	
	OutFilenamesAsFullPaths = FilenamesToOutput;
}

void FBackgroundHttpManagerImpl::DeleteAllTemporaryFiles()
{
	UE_LOGF(LogBackgroundHttpManager, Log, "Cleaning Up Temporary Files");

	//Default implementation is to just delete everything in the Root Folder non-recursively.
	DeleteAllTemporaryFiles(false, *FBackgroundHttpFileHashHelper::GetTempFileExtension());
}

int FBackgroundHttpManagerImpl::GetMaxActiveDownloads() const
{
	return MaxActiveDownloads;
}

void FBackgroundHttpManagerImpl::SetMaxActiveDownloads(int InMaxActiveDownloads)
{
	MaxActiveDownloads = InMaxActiveDownloads;
}

void FBackgroundHttpManagerImpl::AddRequest(const FBackgroundHttpRequestPtr Request)
{
	UE_LOGF(LogBackgroundHttpManager, VeryVerbose, "AddRequest Called - RequestID:%ls", *Request->GetRequestID());

	//If we don't associate with any existing requests, go into our pending list. These will be moved into the ActiveRequest list during our Tick
	if (!AssociateWithAnyExistingRequest(Request))
	{
		FRWScopeLock ScopeLock(PendingRequestLock, SLT_Write);
		PendingStartRequests.Add(Request);

		UE_LOGF(LogBackgroundHttpManager, Verbose, "Adding BackgroundHttpRequest to PendingStartRequests - RequestID:%ls", *Request->GetRequestID());
	}
}

void FBackgroundHttpManagerImpl::RemoveRequest(const FBackgroundHttpRequestPtr Request)
{
	int NumRequestsRemoved = 0;

	//Check if this request was in active list first
	{
		FRWScopeLock ScopeLock(ActiveRequestLock, SLT_Write);
		NumRequestsRemoved = ActiveRequests.Remove(Request);

		//If we removed an active request, lets decrement the NumCurrentlyActiveRequests accordingly
		if (NumRequestsRemoved != 0)
		{
			NumCurrentlyActiveRequests = NumCurrentlyActiveRequests - NumRequestsRemoved;
		}
	}

	//Only search the PendingRequestList if we didn't remove it in our ActiveRequest List
	if (NumRequestsRemoved == 0)
	{
		FRWScopeLock ScopeLock(PendingRequestLock, SLT_Write);
		NumRequestsRemoved = PendingStartRequests.Remove(Request);
	}
	
	UE_LOGF(LogBackgroundHttpManager, Verbose, "FGenericPlatformBackgroundHttpManager::RemoveRequest Called - RequestID:%ls | NumRequestsActuallyRemoved:%d | NumCurrentlyActiveRequests:%d", *Request->GetRequestID(), NumRequestsRemoved, NumCurrentlyActiveRequests);
}

void FBackgroundHttpManagerImpl::CleanUpDataAfterCompletingRequest(const FBackgroundHttpRequestPtr Request)
{
	//Need to free up all these URL's hashes in FileHashHelper so that future URLs can use those temp files
	BackgroundHttpFileHashHelperRef OurFileHashHelper = GetFileHashHelper();
	const TArray<FString>& URLList = Request->GetURLList();
	for (const FString& URL : URLList)
	{
		OurFileHashHelper->RemoveURLMapping(URL, Request->GetRequestID());
	}
}

void FBackgroundHttpManagerImpl::SetCellularPreference(int32 Value)
{

}

void FBackgroundHttpManagerImpl::SetLimitedDataPreference(int32 Value)
{
	
}

bool FBackgroundHttpManagerImpl::AssociateWithAnyExistingRequest(const FBackgroundHttpRequestPtr Request)
{
	bool bDidAssociateWithExistingRequest = false;

	FString ExistingFilePath;
	int64 ExistingFileSize;
	if (CheckForExistingCompletedDownload(Request, ExistingFilePath, ExistingFileSize))
	{
		FBackgroundHttpResponsePtr NewResponseWithExistingFile = FPlatformBackgroundHttp::ConstructBackgroundResponse(EHttpResponseCodes::Ok, ExistingFilePath);
		if (ensureAlwaysMsgf(NewResponseWithExistingFile.IsValid(), TEXT("Failure to create FBackgroundHttpResponsePtr in FPlatformBackgroundHttp::ConstructBackgroundResponse! Can not associate new download with found finished download!")))
		{
			bDidAssociateWithExistingRequest = true;
			UE_LOGF(LogBackgroundHttpManager, Display, "Found existing background task to associate with! RequestID:%ls | ExistingFileSize:%lld | ExistingFilePath:%ls", *Request->GetRequestID(), ExistingFileSize, *ExistingFilePath);

			//First send progress update for the file size so anything monitoring this download knows we are about to update this progress
			Request->OnProgressUpdated().ExecuteIfBound(Request, ExistingFileSize, ExistingFileSize);

			//Now complete with this response data.  Since we have no time data fill everything in with zeros
			FBackgroundHttpRequestMetricsExtended MetricsExtended;

			MetricsExtended.TotalBytesDownloaded = ExistingFileSize;
			MetricsExtended.DownloadDuration = 0.0f;
			MetricsExtended.FetchStartTimeUTC = FDateTime::FromUnixTimestamp(0);
			MetricsExtended.FetchEndTimeUTC = FDateTime::FromUnixTimestamp(0);

			Request->SetMetricsExtended(MetricsExtended);
			Request->CompleteWithExistingResponseData(NewResponseWithExistingFile);
		}
	}

	return bDidAssociateWithExistingRequest;
}

bool FBackgroundHttpManagerImpl::CheckForExistingCompletedDownload(const FBackgroundHttpRequestPtr Request, FString& ExistingFilePathOut, int64& ExistingFileSizeOut)
{
	bool bDidFindExistingDownload = false;

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	const TArray<FString>& URLList = Request->GetURLList();
	for (const FString& URL : URLList)
	{
		FString FoundTempFilename = {};
		if (GetFileHashHelper()->FindTempFilenameMappingForURL(URL, Request->GetRequestID(), FoundTempFilename))
		{
			const FString& FileDestination = FBackgroundHttpFileHashHelper::GetFullPathOfTempFilename(FoundTempFilename);
			if (PlatformFile.FileExists(*FileDestination))
			{
				bDidFindExistingDownload = true;
				ExistingFilePathOut = FileDestination;

				ExistingFileSizeOut = PlatformFile.FileSize(*FileDestination);
				break;
			}
		}
	}	

	return bDidFindExistingDownload;
}

bool FBackgroundHttpManagerImpl::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FBackgroundHttpManagerImpl_Tick);

	ensureAlwaysMsgf(IsInGameThread(), TEXT("Called from un-expected thread! Potential error in an implementation of background downloads!"));
	CSV_CUSTOM_STAT(BackgroundDownload, MaxActiveDownloads, MaxActiveDownloads, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BackgroundDownload, PendingStartRequests, PendingStartRequests.Num(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BackgroundDownload, NumCurrentlyActiveRequests, NumCurrentlyActiveRequests, ECsvCustomStatOp::Set);
	ActivatePendingRequests();
	
	//for now we are saving data every tick, could change this to be on an interval later if required
	GetFileHashHelper()->SaveData();
	
	//Keep ticking in all cases, so just return true
	return true;
}

void FBackgroundHttpManagerImpl::ActivatePendingRequests()
{
	FBackgroundHttpRequestPtr HighestPriorityRequestToStart = nullptr;
	EBackgroundHTTPPriority HighestRequestPriority = EBackgroundHTTPPriority::Num;

	//Go through and find the highest priority request
	{
		FRWScopeLock ActiveScopeLock(ActiveRequestLock, SLT_ReadOnly);
		FRWScopeLock PendingScopeLock(PendingRequestLock, SLT_ReadOnly);
		const int NumRequestsWeCanProcess = (MaxActiveDownloads - NumCurrentlyActiveRequests);
		if (NumRequestsWeCanProcess > 0)
		{
			if (PendingStartRequests.Num() > 0)
			{
				UE_LOGF(LogBackgroundHttpManager, Verbose, "Populating Requests to Start from PendingStartRequests - MaxActiveDownloads:%d | NumCurrentlyActiveRequests:%d | NumPendingStartRequests:%d", MaxActiveDownloads.Load(), NumCurrentlyActiveRequests, PendingStartRequests.Num());

				HighestPriorityRequestToStart = PendingStartRequests[0];
				HighestRequestPriority = HighestPriorityRequestToStart.IsValid() ? HighestPriorityRequestToStart->GetRequestPriority() : EBackgroundHTTPPriority::Num;

				//See how many more requests we can process and only do anything if we can still process more
				{
					for (int RequestIndex = 1; RequestIndex < PendingStartRequests.Num(); ++RequestIndex)
					{
						FBackgroundHttpRequestPtr PendingRequestToCheck = PendingStartRequests[RequestIndex];
						EBackgroundHTTPPriority PendingRequestPriority = PendingRequestToCheck.IsValid() ? PendingRequestToCheck->GetRequestPriority() : EBackgroundHTTPPriority::Num;

						//Found a higher priority request, so track that one
						if (PendingRequestPriority < HighestRequestPriority)
						{
							HighestPriorityRequestToStart = PendingRequestToCheck;
							HighestRequestPriority = PendingRequestPriority;
						}
					}
				}
			}
		}
	}

	if (HighestPriorityRequestToStart.IsValid())
	{
		UE_LOGF(LogBackgroundHttpManager, Display, "Activating Request: %ls Priority:%ls", *HighestPriorityRequestToStart->GetRequestID(), LexToString(HighestRequestPriority));

		//Actually move request to Active list now
		FRWScopeLock ActiveScopeLock(ActiveRequestLock, SLT_Write);
		FRWScopeLock PendingScopeLock(PendingRequestLock, SLT_Write);
		ActiveRequests.Add(HighestPriorityRequestToStart);
		PendingStartRequests.RemoveSingle(HighestPriorityRequestToStart);

		++NumCurrentlyActiveRequests;

		//Call Handle for that task to now kick itself off
		HighestPriorityRequestToStart->HandleDelayedProcess();
	}
}

FString FBackgroundHttpManagerImpl::GetTempFileLocationForURL(const FString& URL, const FString& RequestID)
{
	if (ensureAlwaysMsgf(IsInGameThread(), TEXT("Should only call GetTempFileLocationForURL from the GameThread as this is not thread-safe otherwise!")))
	{
		const FString& TempLocation = GetFileHashHelper()->FindOrAddTempFilenameMappingForURL(URL, RequestID);
		return FBackgroundHttpFileHashHelper::GetFullPathOfTempFilename(TempLocation);
	}
	
	return TEXT("");
}
