// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackgroundHttpModule.h"
#include "BackgroundHttp.h"
#include "PlatformBackgroundHttp.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IHttpRequest.h"

IMPLEMENT_MODULE(FBackgroundHttpModule, BackgroundHTTP);

FBackgroundHttpModule* FBackgroundHttpModule::Singleton = NULL;
bool FBackgroundHttpModule::bClearPreExistingRequestsAtStartup = false;

void FBackgroundHttpModule::StartupModule()
{
	Singleton = this;

	FPlatformBackgroundHttp::Initialize();

	BackgroundHttpManager = FPlatformBackgroundHttp::CreatePlatformBackgroundHttpManager();
	if (ensureAlwaysMsgf(BackgroundHttpManager.IsValid(), TEXT("Invalid result from FPlatformBackgroundHttp::CreatePlatformBackgroundHttpManager! Ensure this platform has an implementation supplied!")))
	{
		BackgroundHttpManager->Initialize(bClearPreExistingRequestsAtStartup);
	}
}

void FBackgroundHttpModule::ShutdownModule()
{
	FPlatformBackgroundHttp::Shutdown();

	BackgroundHttpManager.Reset();
	Singleton = nullptr;
}

FBackgroundHttpModule& FBackgroundHttpModule::Get()
{
	if (Singleton == NULL)
	{
		check(IsInGameThread());
		FModuleManager::LoadModuleChecked<FBackgroundHttpModule>("BackgroundHTTP");
	}
	check(Singleton != NULL);
	return *Singleton;
}

FBackgroundHttpRequestPtr FBackgroundHttpModule::CreateBackgroundRequest()
{
	// Create the platform specific Background Http request instance
	return FPlatformBackgroundHttp::ConstructBackgroundRequest();
}

FBackgroundHttpManagerPtr FBackgroundHttpModule::GetBackgroundHttpManager()
{
	return BackgroundHttpManager;
}

void FBackgroundHttpModule::ClearPreExistingRequestsAtStartup(bool bEnable)
{
	// This is only meaningful to be called before the module is instantiated.
	ensure(!Singleton);
	bClearPreExistingRequestsAtStartup = bEnable;
}
