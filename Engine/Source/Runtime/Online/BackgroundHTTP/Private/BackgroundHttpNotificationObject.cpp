// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackgroundHttpNotificationObject.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "LocalNotification.h"

#if PLATFORM_IOS || PLATFORM_TVOS
#import <UIKit/UIKit.h>
#endif

FBackgroundHttpNotificationObject::FBackgroundHttpNotificationObject(FText InNotificationTitle, FText InNotificationBody, FText InNotificationAction, const FString& InNotificationActivationString, bool InNotifyOnlyOnFullSuccess)
	: FBackgroundHttpNotificationObject(InNotificationTitle, InNotificationBody, InNotificationAction, InNotificationActivationString, InNotifyOnlyOnFullSuccess, true, -1)
{
}

FBackgroundHttpNotificationObject::FBackgroundHttpNotificationObject(FText InNotificationTitle, FText InNotificationBody, FText InNotificationAction, const FString& InNotificationActivationString, bool InNotifyOnlyOnFullSuccess, bool InbOnlySendNotificationInBackground, int32 InIdOverride, bool bUnused)
	: NotificationTitle(InNotificationTitle)
    , NotificationAction(InNotificationAction)
    , NotificationBody(InNotificationBody)
	, NotificationActivationString(InNotificationActivationString)
    , bOnlySendNotificationInBackground(InbOnlySendNotificationInBackground)
    , bNotifyOnlyOnFullSuccess(InNotifyOnlyOnFullSuccess)
	, bIsInBackground(false)
	, NumFailedDownloads(0)
	, IdOverride(InIdOverride)
    , PlatformNotificationService(nullptr)
    , OnApp_EnteringForegroundHandle()
	, OnApp_EnteringBackgroundHandle()
{
	if (GConfig)
	{
		FString ModuleName;
		GConfig->GetString(TEXT("LocalNotification"), TEXT("DefaultPlatformService"), ModuleName, GEngineIni);

		if (ModuleName.Len() > 0)
		{
			// load the module by name from the .ini
			if (ILocalNotificationModule* Module = FModuleManager::LoadModulePtr<ILocalNotificationModule>(*ModuleName))
			{
				PlatformNotificationService = Module->GetLocalNotificationService();
			}
		}
	}

	//Only register with this delegate if we are actually going to monitor background notifications
	if (bOnlySendNotificationInBackground)
	{
		OnApp_EnteringBackgroundHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FBackgroundHttpNotificationObject::OnApp_EnteringBackground);
		OnApp_EnteringForegroundHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FBackgroundHttpNotificationObject::OnApp_EnteringForeground);
	}
}

void FBackgroundHttpNotificationObject::OnApp_EnteringForeground()
{
	bIsInBackground = false;
}

void FBackgroundHttpNotificationObject::OnApp_EnteringBackground()
{
	bIsInBackground = true;
}

FBackgroundHttpNotificationObject::~FBackgroundHttpNotificationObject()
{
	if (bOnlySendNotificationInBackground)
	{
		//These should only be registered if bOnlySendNotificationInBackground is set
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(OnApp_EnteringBackgroundHandle);
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(OnApp_EnteringForegroundHandle);

#if PLATFORM_IOS || PLATFORM_TVOS
		// Temp workaround of ApplicationWillEnterBackgroundDelegate not correctly invoked
		// TODO remove workaround
		bIsInBackground = [UIApplication sharedApplication].applicationState != UIApplicationStateActive;
#endif

		//If we have flagged as only sending notifications when we are in the BG, and we are not in the BG, just early out
		//so we don't send a notification
		if (!bIsInBackground)
		{
			return;
		}
	}

	if (nullptr != PlatformNotificationService)
	{
		if (!bNotifyOnlyOnFullSuccess || (NumFailedDownloads == 0))
		{
			// Schedule notification slightly in the future to pass checks preventing scheduling the past.
			FDateTime TargetTime = FDateTime::Now() + FTimespan::FromSeconds(1);
			if (nullptr != PlatformNotificationService)
			{
				PlatformNotificationService->ScheduleLocalNotificationAtTimeOverrideId(TargetTime, true, NotificationTitle, NotificationBody, NotificationAction, NotificationActivationString, IdOverride);
			}
		}
	}
}

void FBackgroundHttpNotificationObject::NotifyOfDownloadResult(bool bWasSuccess)
{
	if (!bWasSuccess)
	{
		FPlatformAtomics::InterlockedIncrement(&NumFailedDownloads);
	}
}
