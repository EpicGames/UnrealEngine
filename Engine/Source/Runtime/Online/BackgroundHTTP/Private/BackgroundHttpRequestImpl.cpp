// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackgroundHttpRequestImpl.h"

#include "BackgroundHttpModule.h"
#include "Interfaces/IBackgroundHttpResponse.h"

DEFINE_LOG_CATEGORY(LogBackgroundHttpRequest);

FBackgroundHttpRequestImpl::FBackgroundHttpRequestImpl()
	: DownloadCompleteNotificationObject(nullptr)
	, Response(nullptr)
	, NumberOfTotalRetries(0)
	, RequestPriority(EBackgroundHTTPPriority::Normal)
	, ExpectedResultSize{0}
{
}

bool FBackgroundHttpRequestImpl::ProcessRequest()
{
	UE_LOGF(LogBackgroundHttpRequest, Verbose, "Processing Request - RequestID:%ls", *GetRequestID());
	FBackgroundHttpModule::Get().GetBackgroundHttpManager()->AddRequest(SharedThis(this));

	return true;
}

void FBackgroundHttpRequestImpl::CancelRequest()
{
	UE_LOGF(LogBackgroundHttpRequest, Display, "Cancelling Request - RequestID:%ls", *GetRequestID());
	FBackgroundHttpModule::Get().GetBackgroundHttpManager()->RemoveRequest(SharedThis(this));
}

void FBackgroundHttpRequestImpl::RequeueRequest()
{
	UE_LOGF(LogBackgroundHttpRequest, Display, "RequeueRequest - RequestID:%ls", *GetRequestID());
	FBackgroundHttpModule::Get().GetBackgroundHttpManager()->RequeueRequest(SharedThis(this));
}

void FBackgroundHttpRequestImpl::PauseRequest()
{
    //for now a pause is just wrapping a cancel in the general case
    UE_LOGF(LogBackgroundHttpRequest, Display, "Pausing Request (through cancel) - RequestID:%ls", *GetRequestID());
    CancelRequest();
}

void FBackgroundHttpRequestImpl::ResumeRequest()
{
    //for now a resume is just wrapping a restart in the general case
    UE_LOGF(LogBackgroundHttpRequest, Display, "Resuming Request (through restart) - RequestID:%ls", *GetRequestID());
    ProcessRequest();
}

void FBackgroundHttpRequestImpl::OnBackgroundDownloadComplete()
{
	//The complete delegate should only be firing on the game thread 
	//so that requestors don't have to worry about thread safety unexpectedly
	ensureAlwaysMsgf(IsInGameThread(), TEXT("Called from un-expected thread! Potential error in an implementation of background downloads!"));

	FBackgroundHttpModule::Get().GetBackgroundHttpManager()->RemoveRequest(SharedThis(this));

	//Determine if this was a success or not
	FBackgroundHttpResponsePtr SetResponse = GetResponse();
	const bool bWasSuccess = SetResponse.IsValid() ? EHttpResponseCodes::IsOk(SetResponse->GetResponseCode()) : false;
	const FString ResponseTempLocation = Response.IsValid() ? Response->GetTempContentFilePath() : TEXT("None");

	UE_LOGF(LogBackgroundHttpRequest, Display, "Download Complete - RequestID:%ls | bWasSuccess:%d | ResponseTempLocation:%ls", *GetRequestID(), (int)(bWasSuccess), *ResponseTempLocation);

	//First, send a delegate out for this request completing
	OnProcessRequestComplete().ExecuteIfBound(SharedThis(this), bWasSuccess);

	//Second notify our download complete notification of the success/failure of this request. Then remove our reference
	//so a notification will send without having to delete this request. Do this after Complete Delegate so we can ensure
	//whatever kicked this off doesn't want to react to this download complete before we send this notification (IE: Kick off another download
	//using the same NotificationObject, etc.)
	NotifyNotificationObjectOfComplete(bWasSuccess ? EBackgroundHttpRequestCompletionResult::Success : EBackgroundHttpRequestCompletionResult::GenericError);
	
	//Now that we have called our completion delegates and done everything else for this request, allow us to clean up data for it
	FBackgroundHttpModule::Get().GetBackgroundHttpManager()->CleanUpDataAfterCompletingRequest(SharedThis(this));
	
	if (OptionalMetricsInfo.IsSet())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!OnRequestMetrics().IsBound() && !OnRequestMetricsExtended().IsBound())
		{
			UE_LOGF(LogBackgroundHttpRequest, Warning, "OptionalMetricsInfo neither OnRequestMetrics() or OnRequestMetricsExtended() are bound - RequestID:%ls", *GetRequestID());
		}

		OnRequestMetrics().ExecuteIfBound(SharedThis(this), OptionalMetricsInfo->TotalBytesDownloaded, OptionalMetricsInfo->DownloadDuration);
		OnRequestMetricsExtended().ExecuteIfBound(SharedThis(this), OptionalMetricsInfo.GetValue());
		OptionalMetricsInfo.Reset();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		UE_LOGF(LogBackgroundHttpRequest, Warning, "OptionalMetricsInfo missing, possibly iOS not yet didFinishCollectingMetrics: RequestID:%ls", *GetRequestID());
	}
}

void FBackgroundHttpRequestImpl::NotifyNotificationObjectOfComplete(bool bSuccess)
{
	NotifyNotificationObjectOfComplete(bSuccess ? EBackgroundHttpRequestCompletionResult::Success : EBackgroundHttpRequestCompletionResult::GenericError);
}

void FBackgroundHttpRequestImpl::NotifyNotificationObjectOfComplete(EBackgroundHttpRequestCompletionResult Result)
{
	// Get notification object and reset it (swap with nullptr)
	TSharedPtr<FBackgroundHttpNotificationObject, ESPMode::ThreadSafe> LocalDownloadCompleteNotificationObject;
	DownloadCompleteMutex.Lock();
	Swap(DownloadCompleteNotificationObject, LocalDownloadCompleteNotificationObject);
	DownloadCompleteMutex.Unlock();

	// If Result indicates out of diskspace issues, set our flag now
	if (Result == EBackgroundHttpRequestCompletionResult::OutOfDiskSpaceError)
	{
		bHasSeenOutOfDiskSpaceError = true;
	}
	
	if (LocalDownloadCompleteNotificationObject.IsValid())
	{
		LocalDownloadCompleteNotificationObject->NotifyOfDownloadResult(Result == EBackgroundHttpRequestCompletionResult::Success);
	}
}

void FBackgroundHttpRequestImpl::SetMetrics(const int32 TotalBytesDownloaded, const float DownloadDuration)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NotifyRequestMetricsAvailable(TotalBytesDownloaded, DownloadDuration);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FBackgroundHttpRequestImpl::NotifyRequestMetricsAvailable(const int32 TotalBytesDownloaded, const float DownloadDuration)
{
	ensureAlwaysMsgf(!OptionalMetricsInfo.IsSet(), TEXT("Overwriting metrics data!!!"));
	OptionalMetricsInfo = {
		.TotalBytesDownloaded = TotalBytesDownloaded,
		.DownloadDuration = DownloadDuration,
	};
}

void FBackgroundHttpRequestImpl::SetMetricsExtended(const FBackgroundHttpRequestMetricsExtended ExtendedMetrics)
{
	NotifyRequestMetricsExtendedAvailable(ExtendedMetrics);
}

void FBackgroundHttpRequestImpl::NotifyRequestMetricsExtendedAvailable(const FBackgroundHttpRequestMetricsExtended ExtendedMetrics)
{
	ensureAlwaysMsgf(!OptionalMetricsInfo.IsSet(), TEXT("Overwriting metrics data!!!"));
	OptionalMetricsInfo = ExtendedMetrics;
}


void FBackgroundHttpRequestImpl::SetURLAsList(const TArray<FString>& URLs, int NumRetriesIn)
{
	NumberOfTotalRetries = NumRetriesIn;
	for (const FString& URL : URLs)
	{
		URLList.Add(URL);
	}
}

const TArray<FString>& FBackgroundHttpRequestImpl::GetURLList() const
{
	return URLList;
}

void FBackgroundHttpRequestImpl::SetExpectedResultSize(const uint64 ExpectedSize)
{
	ExpectedResultSize = ExpectedSize;
}

uint64 FBackgroundHttpRequestImpl::GetExpectedResultSize() const
{
	return ExpectedResultSize;
}

void FBackgroundHttpRequestImpl::SetCompleteNotification(FBackgroundHttpNotificationObjectPtr DownloadCompleteNotificationObjectIn)
{
	DownloadCompleteNotificationObject = DownloadCompleteNotificationObjectIn;
}

void FBackgroundHttpRequestImpl::CompleteWithExistingResponseData(FBackgroundHttpResponsePtr BackgroundResponse)
{
	Response = BackgroundResponse;

	const bool bHasValidResponse = Response.IsValid();
	const FString ResponseTempLocation = Response.IsValid() ? Response->GetTempContentFilePath() : TEXT("None");

	UE_LOGF(LogBackgroundHttpRequest, Verbose, "Completing Download With Existing Response Data - RequestID:%ls | bHasValidResponse:%d | ResponseTempDownloadLocation:%ls", *GetRequestID(), (int)(bHasValidResponse), *ResponseTempLocation);

	OnBackgroundDownloadComplete();
}

FBackgroundHttpRequestCompleteDelegate& FBackgroundHttpRequestImpl::OnProcessRequestComplete()
{
	return HttpRequestCompleteDelegate;
}

FBackgroundHttpProgressUpdateDelegate& FBackgroundHttpRequestImpl::OnProgressUpdated()
{
	return HttpProgressUpdateDelegate;
}

FBackgroundHttpRequestMetricsDelegate& FBackgroundHttpRequestImpl::OnRequestMetrics()
{
	return HttpRequestMetricsDelegate;
}

FBackgroundHttpRequestMetricsExtendedDelegate& FBackgroundHttpRequestImpl::OnRequestMetricsExtended()
{
	return HttpRequestMetricsExtendedDelegate;
}

const FBackgroundHttpResponsePtr FBackgroundHttpRequestImpl::GetResponse() const
{
	return Response;
}

const FString& FBackgroundHttpRequestImpl::GetRequestID() const
{
	return RequestID;
}

void FBackgroundHttpRequestImpl::SetRequestID(const FString& NewRequestID)
{
	RequestID = NewRequestID;
}

bool FBackgroundHttpRequestImpl::HandleDelayedProcess()
{
	//By default we don't provide an implementation for this. If this is called, it should be overriden by the platform specific
	//BackgroundHttpRequest if the platform expects to make use of this.
	return ensureAlwaysMsgf(false, TEXT("Platform expects an implementation of HandleDelayedProcess on the BackgroundHttpRequest, but none found!"));
}

EBackgroundHTTPPriority FBackgroundHttpRequestImpl::GetRequestPriority() const
{
	return RequestPriority;
}

void FBackgroundHttpRequestImpl::SetRequestPriority(EBackgroundHTTPPriority NewPriority)
{
	RequestPriority = NewPriority;
}
