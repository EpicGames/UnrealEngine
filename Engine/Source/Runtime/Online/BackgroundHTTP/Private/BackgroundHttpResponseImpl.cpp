// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackgroundHttpResponseImpl.h"

#include "Interfaces/IHttpResponse.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Misc/Paths.h"

FBackgroundHttpResponseImpl::FBackgroundHttpResponseImpl()
	: ResponseCode(EHttpResponseCodes::Unknown)
{
}

FBackgroundHttpResponseImpl::FBackgroundHttpResponseImpl(FString InTempContentFilePath, int32 InResponseCode)
	: TempContentFilePath{MoveTemp(InTempContentFilePath)}
	, ResponseCode{InResponseCode}
{
}

int32 FBackgroundHttpResponseImpl::GetResponseCode() const
{
	return ResponseCode;
}

const FString& FBackgroundHttpResponseImpl::GetTempContentFilePath() const
{
	return TempContentFilePath;
}