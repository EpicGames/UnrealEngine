// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformBackgroundHttp.h"
#include "GenericPlatform/GenericPlatformBackgroundHttpManager.h"
#include "GenericPlatform/GenericPlatformBackgroundHttpRequest.h"
#include "GenericPlatform/GenericPlatformBackgroundHttpResponse.h"

#include "Misc/Paths.h"

void FGenericPlatformBackgroundHttp::Initialize()
{
}

void FGenericPlatformBackgroundHttp::Shutdown()
{
}

FBackgroundHttpManagerPtr FGenericPlatformBackgroundHttp::CreatePlatformBackgroundHttpManager()
{
    return MakeShared<FGenericPlatformBackgroundHttpManager, ESPMode::ThreadSafe>();
}

FBackgroundHttpRequestPtr FGenericPlatformBackgroundHttp::ConstructBackgroundRequest()
{
	return MakeShared<FGenericPlatformBackgroundHttpRequest, ESPMode::ThreadSafe>();
}

FBackgroundHttpResponsePtr FGenericPlatformBackgroundHttp::ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath)
{
	return MakeShared<FGenericPlatformBackgroundHttpResponse, ESPMode::ThreadSafe>(ResponseCode, TempFilePath);
}
