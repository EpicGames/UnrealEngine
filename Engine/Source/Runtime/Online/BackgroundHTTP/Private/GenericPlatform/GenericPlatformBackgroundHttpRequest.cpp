// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformBackgroundHttpRequest.h"
#include "GenericPlatform/GenericPlatformBackgroundHttpResponse.h"
#include "Interfaces/IHttpResponse.h"
#include "Interfaces/IHttpRequest.h"
#include "HttpModule.h"
#include "BackgroundHttpModule.h"

FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpRequest()
	: RequestWrapper(nullptr)
{
}

FGenericPlatformBackgroundHttpRequest::~FGenericPlatformBackgroundHttpRequest()
{
	RequestWrapper.Reset();
}

bool FGenericPlatformBackgroundHttpRequest::HandleDelayedProcess()
{
	//Our request may already have a response result from associating with another already completed request. If so, lets not spin up our RequestWrapper.
	if (!GetResponse().IsValid())
	{
		//If we already have a Request Wrapper, lets keep it. Otherwise spin it up now.
		if (!RequestWrapper.IsValid())
		{
			RequestWrapper.Reset(new FGenericPlatformBackgroundHttpWrapper(SharedThis(this), NumberOfTotalRetries));
		}

		if (ensureAlwaysMsgf(RequestWrapper.IsValid(), TEXT("Failure creating FGenericPlatformBackgroundHttpWrapper! Can not process BackgroundHttp request! RequestID:%s"), *GetRequestID()))
		{
			//Start request wrapper
			RequestWrapper->MakeRequest();
		}
	}

	return RequestWrapper.IsValid() || GetResponse().IsValid();
}

void FGenericPlatformBackgroundHttpRequest::CancelRequest()
{
	RequestWrapper.Reset();
	FBackgroundHttpRequestImpl::CancelRequest();
}

void FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpRequest::CompleteWithExistingResponseData(FBackgroundHttpResponsePtr BackgroundResponse)
{
	//Remove our request wrapper immediately so the Response cleans up ASAP instead of the user having to delete this task to clean up the Http Response.
	RequestWrapper.Reset();

	FBackgroundHttpRequestImpl::CompleteWithExistingResponseData(BackgroundResponse);
}

FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::FGenericPlatformBackgroundHttpWrapper(FBackgroundHttpRequestPtr Request, int MaxRetriesToAttempt)
	: OriginalRequest(Request)
	, CurrentRetryNumber(0)
	, MaxRetries(MaxRetriesToAttempt)
	, LastProgressUpdateBytes(0)
{
}

FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::~FGenericPlatformBackgroundHttpWrapper()
{
	CleanUpHttpRequest();
}

void FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::MakeRequest()
{
	if (ensureAlwaysMsgf(OriginalRequest.IsValid(), TEXT("Call to MakeRequest in BackgroundHttpWrapper without associating it to a valid BackgroundHttpRequest! Should never happen!")))
	{
		//If we have an existing HttpRequest, lets clear it out before creating a new one
		CleanUpHttpRequest();

		//Create new Request
		HttpRequest = FHttpModule::Get().CreateRequest();
		HttpRequest->OnProcessRequestComplete().BindRaw(this, &FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::HttpRequestComplete);
		HttpRequest->OnRequestProgress64().BindRaw(this, &FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::UpdateHttpProgress);

		const FString& RequestURL = GetURLForCurrentRetryNumber();
		if (!RequestURL.IsEmpty())
		{
			//Reset this value as we are restarting our progress
			LastProgressUpdateBytes = 0;

			HttpRequest->SetVerb(TEXT("GET"));
			HttpRequest->SetURL(RequestURL);
			HttpRequest->ProcessRequest();
		}
		else
		{
			UE_LOGF(LogBackgroundHttpRequest, Display, "No valid URL for Request. (No valid URL, or out of Retries) - RequestID:%ls | RetryNumber:%d | MaxRetries:%d", *OriginalRequest.Pin()->GetRequestID(), CurrentRetryNumber, MaxRetries);

			//We don't have a valid URL, so just error immediately
			HttpRequestComplete(HttpRequest, nullptr, false);
		}
	}
}

void FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::CleanUpHttpRequest()
{
	if (HttpRequest.IsValid())
	{
		HttpRequest->OnProcessRequestComplete().Unbind();
		HttpRequest->OnRequestProgress64().Unbind();
		HttpRequest->CancelRequest();

		HttpRequest.Reset();
	}
}

const FString FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::GetURLForCurrentRetryNumber()
{
	FString ReturnedURL = FString();

	int URLIndexToUse = CurrentRetryNumber;

	//Don't find a valid URL if we are already passed our retry count
	if (HasRetriesRemaining())
	{
		const TArray<FString>& RequestURLList = OriginalRequest.IsValid() ? OriginalRequest.Pin()->GetURLList() : TArray<FString>();
		if (RequestURLList.Num() > 0)
		{
			//If our number is too high, lets loop back around to the start of the list
			while ((URLIndexToUse >= 0) && (!RequestURLList.IsValidIndex(URLIndexToUse)))
			{
				URLIndexToUse -= RequestURLList.Num();
			}
		}

		if (RequestURLList.IsValidIndex(URLIndexToUse))
		{
			ReturnedURL = RequestURLList[URLIndexToUse];
		}
	}

	return ReturnedURL;
}

void FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::HttpRequestComplete(FHttpRequestPtr HttpRequestIn, FHttpResponsePtr HttpResponse, bool bSuccess)
{
	if (ensureAlwaysMsgf(OriginalRequest.IsValid(), TEXT("Received HttpRequestComplete callback with invalid OriginalRequest pointer! Can not complete request!")))
	{
		//Create a new FGenericPlatformBackgroundHttpRequest from the HttpRequest/Response
		FBackgroundHttpResponsePtr ConstructedResponse = MakeShareable(new FGenericPlatformBackgroundHttpResponse(HttpRequestIn, HttpResponse, bSuccess));

		//Let the FGenericPlatformBackgroundHttpResponse drive if this succeeded or not
		const bool bRequestWasSuccessful = EHttpResponseCodes::IsOk(ConstructedResponse->GetResponseCode());

		UE_LOGF(LogBackgroundHttpRequest, Display, "Underlying HttpRequest complete - RequestID:%ls | bRequestWasSuccessfull:%d | CurrentRetryNumber:%d | MaxRetries:%d", *OriginalRequest.Pin()->GetRequestID(), (int)(bRequestWasSuccessful), CurrentRetryNumber, MaxRetries);

		//Attempt a retry on failure
		if (!bRequestWasSuccessful && HasRetriesRemaining())
		{
			++CurrentRetryNumber;
			MakeRequest();
		}
		else
		{
			//Pass constructed response to our BackgroundRequest so it can complete itself using this response
			OriginalRequest.Pin()->CompleteWithExistingResponseData(ConstructedResponse);
		}
	}
}

bool FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::HasRetriesRemaining() const
{
	return (CurrentRetryNumber <= MaxRetries);
}

void FGenericPlatformBackgroundHttpRequest::FGenericPlatformBackgroundHttpWrapper::UpdateHttpProgress(FHttpRequestPtr UnderlyingHttpRequest, uint64 BytesSent, uint64 BytesReceived)
{
	if (ensureAlwaysMsgf(OriginalRequest.IsValid(), TEXT("Received UpdateHttpProgress callback with invalid OriginalRequest pointer! Can not update request progress!")))
	{
		ensureAlwaysMsgf((BytesReceived >= LastProgressUpdateBytes), TEXT("Invalid Byte Difference in UpdateHttpProgress -- LastProgressUpdateBytes:%llu | BytesSent:%llu | BytesReceived:%llu"), LastProgressUpdateBytes, BytesSent, BytesReceived);
		const uint64 ByteDifference = BytesReceived - LastProgressUpdateBytes;
		LastProgressUpdateBytes = BytesReceived;

		UE_LOGF(LogBackgroundHttpRequest, VeryVerbose, "HttpRequest Progress Update- RequestID:%ls | BytesSent: %llu | BytesReceived:%llu | BytesReceivedSinceLastUpdate:%llu", *OriginalRequest.Pin()->GetRequestID(), BytesSent, BytesReceived, ByteDifference);

		OriginalRequest.Pin()->OnProgressUpdated().ExecuteIfBound(OriginalRequest.Pin(), BytesReceived, ByteDifference);
	}
}
