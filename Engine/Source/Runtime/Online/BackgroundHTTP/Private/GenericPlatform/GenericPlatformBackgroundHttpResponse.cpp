// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformBackgroundHttpResponse.h"

#include "GenericPlatform/GenericPlatformBackgroundHttpManager.h"
#include "GenericPlatform/GenericPlatformBackgroundHttp.h"

#include "BackgroundHttpModule.h"
#include "PlatformBackgroundHttp.h"

#include "Interfaces/IHttpResponse.h"
#include "Misc/FileHelper.h"

FGenericPlatformBackgroundHttpResponse::FGenericPlatformBackgroundHttpResponse(int32 InResponseCode, const FString& InTempContentFilePath)
{
	ResponseCode = InResponseCode;
	TempContentFilePath = InTempContentFilePath;
}

FGenericPlatformBackgroundHttpResponse::FGenericPlatformBackgroundHttpResponse(FHttpRequestPtr HttpRequestIn, FHttpResponsePtr HttpResponse, bool bSuccess)
{
	ResponseCode = EHttpResponseCodes::Unknown;

	//Don't bother making a response out of a failed HttpRequest
	if (bSuccess && HttpResponse.IsValid())
	{
		//Copy HTTP Response Code if we have one, otherwise it will be Unknown
		ResponseCode = HttpResponse->GetResponseCode();

		//Only try to copy results to a temp file if it as an ok result
		if (EHttpResponseCodes::IsOk(ResponseCode))
		{
			const TArray<uint8>& ContentBuffer = HttpResponse->GetContent();
			if (ContentBuffer.Num() > 0)
			{
				const FString RequestURL = HttpRequestIn->GetURL();
				const FString& FileDestination = FBackgroundHttpModule::Get().GetBackgroundHttpManager()->GetTempFileLocationForURL(RequestURL, {});

				FFileHelper::SaveArrayToFile(ContentBuffer, *FileDestination);
				TempContentFilePath = FileDestination;
			}
		}
	}
}
