// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/ApplePlatformBackgroundHttp.h"
#include "IOS/ApplePlatformBackgroundHttpManager.h"
#include "IOS/ApplePlatformBackgroundHttpRequest.h"
#include "IOS/ApplePlatformBackgroundHttpResponse.h"

#include "Misc/Paths.h"

void FApplePlatformBackgroundHttp::Initialize()
{
}

void FApplePlatformBackgroundHttp::Shutdown()
{
}

FBackgroundHttpManagerPtr FApplePlatformBackgroundHttp::CreatePlatformBackgroundHttpManager()
{
    return MakeShared<FApplePlatformBackgroundHttpManager, ESPMode::ThreadSafe>();
}

FBackgroundHttpRequestPtr FApplePlatformBackgroundHttp::ConstructBackgroundRequest()
{
	return MakeShared<FApplePlatformBackgroundHttpRequest, ESPMode::ThreadSafe>();
}

FBackgroundHttpResponsePtr FApplePlatformBackgroundHttp::ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath)
{
	return MakeShared<FApplePlatformBackgroundHttpResponse, ESPMode::ThreadSafe>(ResponseCode, TempFilePath);
}
