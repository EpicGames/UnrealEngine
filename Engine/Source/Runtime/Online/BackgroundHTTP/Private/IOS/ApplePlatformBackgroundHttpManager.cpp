// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/ApplePlatformBackgroundHttpManager.h"
#include "IOS/ApplePlatformBackgroundHttp.h"
#include "IOS/ApplePlatformBackgroundHttpRequest.h"
#include "IOS/ApplePlatformBackgroundHttpResponse.h"

#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformAtomics.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeRWLock.h"

#include "Stats/Stats.h"

#include "BackgroundURLSessionHandler.h"

#include "PlatformBackgroundHttp.h"

FApplePlatformBackgroundHttpManager::FApplePlatformBackgroundHttpManager()
	: FBackgroundHttpManagerImpl()
	, PendingRemoveRequests()
	, PendingRemoveRequestLock()
{
	OnDownloadCompletedHandle = FBackgroundURLSessionHandler::OnDownloadCompletedExtended.AddRaw(this, &FApplePlatformBackgroundHttpManager::OnDownloadCompletedExtended);
	OnDownloadMetricsExtendedHandle = FBackgroundURLSessionHandler::OnDownloadMetricsExtended.AddRaw(this, &FApplePlatformBackgroundHttpManager::OnDownloadMetricsExtended);
	
	FBackgroundURLSessionHandler::CleanupOldTempDownloads();
}

FApplePlatformBackgroundHttpManager::~FApplePlatformBackgroundHttpManager()
{
	FBackgroundURLSessionHandler::OnDownloadCompletedExtended.Remove(OnDownloadCompletedHandle);
	FBackgroundURLSessionHandler::OnDownloadMetricsExtended.Remove(OnDownloadMetricsExtendedHandle);
}

void FApplePlatformBackgroundHttpManager::Initialize(bool bClearPreExistingRequestsAtStartup)
{
	FBackgroundURLSessionHandler::ClearPreExistingRequestsAtStartup(bClearPreExistingRequestsAtStartup);
	FBackgroundHttpManagerImpl::Initialize(bClearPreExistingRequestsAtStartup);
}

void FApplePlatformBackgroundHttpManager::AddRequest(const FBackgroundHttpRequestPtr GenericRequest)
{
	FAppleBackgroundHttpRequestPtr Request = StaticCastSharedPtr<FApplePlatformBackgroundHttpRequest>(GenericRequest);
	if (!ensureAlwaysMsgf(Request.IsValid(), TEXT("Adding a non-Apple background request to our Apple Background Http Manager2! This is not supported or expected!")))
	{
		return;
	}

	if (AssociateWithAnyExistingRequest(Request))
	{
		return;
	}

	const uint64 DownloadId = FBackgroundURLSessionHandler::CreateOrFindDownload(Request->GetURLList(), Request->GetNSURLSessionPriority(), GetFileHashHelper(), Request->GetExpectedResultSize(), Request->GetRequestID());
	Request->SetInternalDownloadId(DownloadId);

	// Should never happen in practice
	if (DownloadId == FBackgroundURLSessionHandler::InvalidDownloadId)
	{
		FBackgroundHttpResponsePtr Response = FPlatformBackgroundHttp::ConstructBackgroundResponse(EHttpResponseCodes::ServerError, FString());
		Request->CompleteWithExistingResponseData(Response);
		return;
	}

	{
		FRWScopeLock ActiveScopeLock(ActiveRequestLock, SLT_Write);
		ActiveRequests.Add(Request);
	}

	++NumCurrentlyActiveRequests;
}

void FApplePlatformBackgroundHttpManager::RemoveRequest(const FBackgroundHttpRequestPtr GenericRequest)
{
	// UpdateProgress might call RemoveRequest, so we could deadlock on ActiveRequestLock if we try to edit ActiveRequests
	FRWScopeLock PendingScopeRemoveRequestLock(PendingRemoveRequestLock, SLT_Write);
	PendingRemoveRequests.Add(GenericRequest);
}

void FApplePlatformBackgroundHttpManager::RequeueRequest(const FBackgroundHttpRequestPtr GenericRequest)
{
	ensureAlwaysMsgf(IsInGameThread(), TEXT("RequeueRequest called from unexpected thread!"));

	FAppleBackgroundHttpRequestPtr Request = StaticCastSharedPtr<FApplePlatformBackgroundHttpRequest>(GenericRequest);
	if (!ensureAlwaysMsgf(Request.IsValid(), TEXT("Re-queuing a non-Apple background request! This is not supported.")))
	{
		return;
	}

	// Cancel the old NSURLSession download immediately (NOT deferred).
	// This avoids the race where deferred removal would cancel the new download instead.
	const uint64 OldDownloadId = Request->GetInternalDownloadId();
	if (OldDownloadId != FBackgroundURLSessionHandler::InvalidDownloadId)
	{
		FBackgroundURLSessionHandler::CancelDownload(OldDownloadId);
	}

	// Rotate URL list so the retry hits a different CDN
	Request->RotateURLList();

	// Create a fresh download with rotated URLs — new DownloadId, fresh TaskData, fresh retry counts
	const uint64 NewDownloadId = FBackgroundURLSessionHandler::CreateOrFindDownload(
		Request->GetURLList(),
		Request->GetNSURLSessionPriority(),
		GetFileHashHelper(),
		Request->GetExpectedResultSize(),
		Request->GetRequestID());
	Request->SetInternalDownloadId(NewDownloadId);

	if (NewDownloadId == FBackgroundURLSessionHandler::InvalidDownloadId)
	{
		UE_LOGF(LogBackgroundHttpManager, Error, "RequeueRequest failed to create new download for RequestID:%ls", *Request->GetRequestID());
		return;
	}

	// Reset progress tracking for the new download
	Request->ResetProgressTracking();

	UE_LOGF(LogBackgroundHttpManager, Display, "RequeueRequest succeeded - RequestID:%ls OldDownloadId:%llu NewDownloadId:%llu", *Request->GetRequestID(), OldDownloadId, NewDownloadId);
}

void FApplePlatformBackgroundHttpManager::GetActiveDownloadRequestIDs(TSet<FString>& OutActiveIDs) const
{
	OutActiveIDs.Reset();

	TSet<uint64> ActiveDownloadIDs;
	FBackgroundURLSessionHandler::GetActiveDownloadIDs(ActiveDownloadIDs);

	FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);
	for (const FBackgroundHttpRequestPtr& GenericRequest : ActiveRequests)
	{
		FAppleBackgroundHttpRequestPtr Request = StaticCastSharedPtr<FApplePlatformBackgroundHttpRequest>(GenericRequest);
		if (Request.IsValid() && ActiveDownloadIDs.Contains(Request->GetInternalDownloadId()))
		{
			OutActiveIDs.Add(Request->GetRequestID());
		}
	}
}

void FApplePlatformBackgroundHttpManager::SetCellularPreference(int32 Value)
{
	FBackgroundURLSessionHandler::AllowCellular(Value > 0);
}

void FApplePlatformBackgroundHttpManager::SetLimitedDataPreference(int32 Value)
{
	FBackgroundURLSessionHandler::AllowConstrained(Value > 0);
}

bool FApplePlatformBackgroundHttpManager::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FApplePlatformBackgroundHttpManager_Tick);

	ensureAlwaysMsgf(IsInGameThread(), TEXT("Called from un-expected thread! Potential error in an implementation of background downloads!"));

	{
		FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);
		for (FBackgroundHttpRequestPtr& GenericRequest : ActiveRequests)
		{
			FAppleBackgroundHttpRequestPtr Request = StaticCastSharedPtr<FApplePlatformBackgroundHttpRequest>(GenericRequest);
			if (ensureAlwaysMsgf(Request.IsValid(), TEXT("Invalid Request Pointer in ActiveRequests list!")))
			{
				Request->UpdateProgress();
			}
		}
	}

	{
		FRWScopeLock PendingScopeRemoveRequestLock(PendingRemoveRequestLock, SLT_Write);
		for (FBackgroundHttpRequestPtr& GenericRequest : PendingRemoveRequests)
		{
			FAppleBackgroundHttpRequestPtr Request = StaticCastSharedPtr<FApplePlatformBackgroundHttpRequest>(GenericRequest);
			if (ensureAlwaysMsgf(Request.IsValid(), TEXT("Adding a non-Apple background request to our Apple Background Http Manager2! This is not supported or expected!")))
			{
				const uint64 DownloadId = Request->GetInternalDownloadId();
				FBackgroundURLSessionHandler::CancelDownload(DownloadId);
				Request->SetInternalDownloadId(FBackgroundURLSessionHandler::InvalidDownloadId);

				FBackgroundHttpManagerImpl::RemoveRequest(Request);
			}
		}

		PendingRemoveRequests.Empty();
	}

	FBackgroundURLSessionHandler::SaveBackgroundHttpFileHashHelperState();

	return true;
}

void FApplePlatformBackgroundHttpManager::OnDownloadCompletedExtended(const uint64 DownloadId, EBackgroundHttpRequestCompletionResult Result)
{
	// Assuming this function will be called rarily (once a few seconds) and overall amount of downloads is limited (<500),
	// it's then faster to just iterate over array to find the corresponding request.

	FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);
	for (FBackgroundHttpRequestPtr& GenericRequest : ActiveRequests)
	{
		FAppleBackgroundHttpRequestPtr Request = StaticCastSharedPtr<FApplePlatformBackgroundHttpRequest>(GenericRequest);
		if (ensureAlwaysMsgf(Request.IsValid(), TEXT("Invalid Request Pointer in ActiveRequests list!")))
		{
			if (Request->GetInternalDownloadId() == DownloadId)
			{
				Request->NotifyNotificationObjectOfComplete(Result);
				return;
			}
		}
	}
}

void FApplePlatformBackgroundHttpManager::OnDownloadMetricsExtended(const uint64 DownloadId, const FBackgroundHttpRequestMetricsExtended MetricsExtended)
{
	FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);
	for (FBackgroundHttpRequestPtr& GenericRequest : ActiveRequests)
	{
		FAppleBackgroundHttpRequestPtr Request = StaticCastSharedPtr<FApplePlatformBackgroundHttpRequest>(GenericRequest);
		if (ensureAlwaysMsgf(Request.IsValid(), TEXT("Invalid Request Pointer in ActiveRequests list!")))
		{
			if (Request->GetInternalDownloadId() == DownloadId)
			{
				Request->NotifyRequestMetricsExtendedAvailable(MetricsExtended);
				return;
			}
		}
	}
	UE_LOGF(LogBackgroundHttpManager, Warning, "Fail to NotifyRequestMetricsExtendedAvailable - RequestID:%llu | TotalBytesDownloaded:%d | DownloadDuration:%f", DownloadId, MetricsExtended.TotalBytesDownloaded, MetricsExtended.DownloadDuration);
}
