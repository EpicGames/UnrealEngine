// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/ApplePlatformBackgroundHttpRequest.h"
#include "IOS/ApplePlatformBackgroundHttpManager.h"
#include "IOS/ApplePlatformBackgroundHttp.h"
#include "BackgroundURLSessionHandler.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"

#import <Foundation/NSURLSession.h>

FApplePlatformBackgroundHttpRequest::FApplePlatformBackgroundHttpRequest()
	: DownloadId(FBackgroundURLSessionHandler::InvalidDownloadId)
{
}

void FApplePlatformBackgroundHttpRequest::PauseRequest()
{
	FBackgroundURLSessionHandler::PauseDownload(DownloadId);
}

void FApplePlatformBackgroundHttpRequest::ResumeRequest()
{
	FBackgroundURLSessionHandler::ResumeDownload(DownloadId);
}

#if !UE_BUILD_SHIPPING
void FApplePlatformBackgroundHttpRequest::GetDebugText(TArray<FString>& Output)
{
	FBackgroundURLSessionHandler::GetDownloadDebugText(DownloadId, Output);
}
#endif

void FApplePlatformBackgroundHttpRequest::UpdateProgress()
{
	if (DownloadId == FBackgroundURLSessionHandler::InvalidDownloadId)
	{
		return;
	}

	const uint64 CurrentDownloadedBytes = FBackgroundURLSessionHandler::GetCurrentDownloadedBytes(DownloadId);
	const uint64 BytesSinceLastReport = CurrentDownloadedBytes > LastReportedDownloadedBytes ? CurrentDownloadedBytes - LastReportedDownloadedBytes : 0;
	LastReportedDownloadedBytes = CurrentDownloadedBytes;

	if (BytesSinceLastReport > 0)
	{
		OnProgressUpdated().ExecuteIfBound(SharedThis(this), CurrentDownloadedBytes, BytesSinceLastReport);
	}

	int32 ResultHTTPCode;
	FString TemporaryFilePath;
	if (FBackgroundURLSessionHandler::IsDownloadFinished(DownloadId, ResultHTTPCode, TemporaryFilePath))
	{
		if (!OptionalMetricsInfo.IsSet())
		{
			// need to wait for metrics collection
			if (MetricsWaitCounter == -1)
			{
				MetricsWaitCounter = 30;	// start a countdown
				return;
			}
			else
			{
				MetricsWaitCounter--;
				if (MetricsWaitCounter == 0)
				{
					// time up, still no metrics? report 0s
					MetricsWaitCounter = -1;
					UE_LOGF(LogBackgroundHttpRequest, Error, "OptionalMetricsInfo still missing after waiting for 30 cycles: RequestID:%ls", *GetRequestID());
					OptionalMetricsInfo = {
						.TotalBytesDownloaded = 0,
						.DownloadDuration = 0.0f
					};
				}
				else
				{
					// count down, try again next cycle
					return;
				}
			}
		}
		else
		{
			MetricsWaitCounter = -1;
		}
		
		const bool bFileExists = !TemporaryFilePath.IsEmpty() && FPlatformFileManager::Get().GetPlatformFile().FileExists(*TemporaryFilePath);

		// Fail request if we can't access the file
		if (!bFileExists)
		{
			ResultHTTPCode = EHttpResponseCodes::ServerError;
		}

		FBackgroundHttpResponsePtr Response = FPlatformBackgroundHttp::ConstructBackgroundResponse(ResultHTTPCode, TemporaryFilePath);
		CompleteWithExistingResponseData(Response); // will internally call RemoveRequest which will cancel download
	}
}

void FApplePlatformBackgroundHttpRequest::ResetProgressTracking()
{
	LastReportedDownloadedBytes = 0;
	MetricsWaitCounter = -1;
}

void FApplePlatformBackgroundHttpRequest::RotateURLList()
{
	if (URLList.Num() > 1)
	{
		FString First = MoveTemp(URLList[0]);
		URLList.RemoveAt(0, EAllowShrinking::No);
		URLList.Add(MoveTemp(First));
	}
}

float FApplePlatformBackgroundHttpRequest::BackgroundRequestPriorityToNSURLSessionPriority(const EBackgroundHTTPPriority Priority)
{
	switch (Priority)
	{
	case EBackgroundHTTPPriority::High:
		return NSURLSessionTaskPriorityHigh;
	case EBackgroundHTTPPriority::Low:
		return NSURLSessionTaskPriorityLow;
	case EBackgroundHTTPPriority::Normal:
	default:
		return NSURLSessionTaskPriorityDefault;
	}
}
