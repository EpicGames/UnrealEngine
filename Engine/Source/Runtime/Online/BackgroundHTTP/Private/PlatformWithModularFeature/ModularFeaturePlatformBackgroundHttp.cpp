// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformWithModularFeature/ModularFeaturePlatformBackgroundHttp.h"

#include "GenericPlatform/GenericPlatformBackgroundHttp.h"

#include "Features/IModularFeatures.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogBackgroundHttpModularFeature);

IBackgroundHttpModularFeature* FModularFeaturePlatformBackgroundHttp::CachedModularFeature = nullptr;
bool FModularFeaturePlatformBackgroundHttp::bHasCheckedForModularFeature = false;

void FModularFeaturePlatformBackgroundHttp::Initialize()
{
	CacheModularFeature();

	if (nullptr != CachedModularFeature)
	{
		CachedModularFeature->Initialize();
	}
	else
	{
		FGenericPlatformBackgroundHttp::Initialize();
	}
}

void FModularFeaturePlatformBackgroundHttp::Shutdown()
{
	if (nullptr != CachedModularFeature)
	{
		CachedModularFeature->Shutdown();
	}
	else
	{
		FGenericPlatformBackgroundHttp::Shutdown();
	}
}

FBackgroundHttpManagerPtr FModularFeaturePlatformBackgroundHttp::CreatePlatformBackgroundHttpManager()
{
	CacheModularFeature();

	if (nullptr != CachedModularFeature)
	{
		return CachedModularFeature->CreatePlatformBackgroundHttpManager();
	}
	else
	{
		return FGenericPlatformBackgroundHttp::CreatePlatformBackgroundHttpManager();
	}
}

FBackgroundHttpRequestPtr FModularFeaturePlatformBackgroundHttp::ConstructBackgroundRequest()
{
	CacheModularFeature();

	if (nullptr != CachedModularFeature)
	{
		return CachedModularFeature->ConstructBackgroundRequest();
	}
	else
	{
		return FGenericPlatformBackgroundHttp::ConstructBackgroundRequest();
	}
}

FBackgroundHttpResponsePtr FModularFeaturePlatformBackgroundHttp::ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath)
{
	CacheModularFeature();

	if (nullptr != CachedModularFeature)
	{
		return CachedModularFeature->ConstructBackgroundResponse(ResponseCode, TempFilePath);
	}
	else
	{
		return FGenericPlatformBackgroundHttp::ConstructBackgroundResponse(ResponseCode, TempFilePath);
	}
}

FName FModularFeaturePlatformBackgroundHttp::GetModularFeatureName()
{
	static FName ModularFeatureName = FName("BackgroundHttpModularFeature");
	return ModularFeatureName;
}

void FModularFeaturePlatformBackgroundHttp::CacheModularFeature()
{
	if (!bHasCheckedForModularFeature)
	{
		bHasCheckedForModularFeature = true;

		FString CommandLineOverrideName;
		const bool bDidCommandLineOverride = FParse::Value(FCommandLine::Get(), TEXT("-BackgroundHttpModularFeatureNameOverride="), CommandLineOverrideName);
		
		//First get name of plugin from the .ini
		FString ModuleName;
		if (bDidCommandLineOverride)
		{
			ModuleName = CommandLineOverrideName;
		}
		else if (!GEngineIni.IsEmpty())
		{
			GConfig->GetString(TEXT("BackgroundHttp"), TEXT("PlatformModularFeatureName"), ModuleName, GEngineIni);
		}
#if PLATFORM_ANDROID
		const FString* BackgroundModuleConfigVar = FAndroidMisc::GetConfigRulesVariable(TEXT("BackgroundHttpModularFeatureNameOverride"));
		if (BackgroundModuleConfigVar)
		{
			if (BackgroundModuleConfigVar->Equals("null", ESearchCase::IgnoreCase))
			{
				ModuleName.Empty();
			}
			else
			{
				ModuleName = *BackgroundModuleConfigVar;
			}
		}
#endif

		//If we don't have any expected module then we don't want to cache any modular features
		if (ModuleName.IsEmpty())
		{
			//Should still be nullptr but explicitly setting it here as a sanity check
			CachedModularFeature = nullptr;
			UE_LOGF(LogBackgroundHttpModularFeature, Display, "BackgroundHttpModularFeature module not set, falling back to generic implementation.");
			return;
		}

		FModuleManager& ModuleManager = FModuleManager::Get();
		const bool bDesiredModuleExists = ModuleName.IsEmpty() ? false : ModuleManager.ModuleExists(*ModuleName);
		if (bDesiredModuleExists)
		{
			if (false == ModuleManager.IsModuleLoaded(*ModuleName))
			{
				ModuleManager.LoadModule(*ModuleName);
			}
		
			if (IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName()))
			{
				CachedModularFeature = &(IModularFeatures::Get().GetModularFeature<IBackgroundHttpModularFeature>(GetModularFeatureName()));
				UE_LOGF(LogBackgroundHttpModularFeature, Display, "Using BackgroundHTTPModularFeature module: %ls for ModularFeatureName: %ls", *CachedModularFeature->GetDebugModuleName(), *(GetModularFeatureName().ToString()));
			}
			else
			{
				UE_LOGF(LogBackgroundHttpModularFeature, Display, "Module %ls exists but is not available/registered for ModularFeatureName: %ls", *ModuleName, *(GetModularFeatureName().ToString()));
			}
		}
		
		if (nullptr == CachedModularFeature)
		{
			UE_LOGF(LogBackgroundHttpModularFeature, Display, "Using Generic Implementation as ModularFeature: %ls is not registered or available", *(GetModularFeatureName().ToString()));
		}
	}
}