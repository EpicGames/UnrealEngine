// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlatformWithModularFeature/ModularFeaturePlatformBackgroundHttp.h"

//Android makes use of a modular game feature for it's BackgroundHttp implementation
typedef FModularFeaturePlatformBackgroundHttp FPlatformBackgroundHttp;