// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Module dependencies

#include "CoreMinimal.h"
#include "BackgroundHttpModule.h"

// Module includes
#include "PlatformBackgroundHttp.h"

BACKGROUNDHTTP_API DECLARE_LOG_CATEGORY_EXTERN(LogBackgroundHttp, Display, All);
