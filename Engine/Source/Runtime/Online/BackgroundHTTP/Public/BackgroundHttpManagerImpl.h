// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackgroundHttpFileHashHelper.h"
#include "Containers/Array.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Interfaces/IBackgroundHttpManager.h"
#include "Interfaces/IBackgroundHttpRequest.h"
#include "Interfaces/IBackgroundHttpResponse.h"
#include "Logging/LogMacros.h"
#include "Templates/Atomic.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBackgroundHttpManager, Log, All)

class FBackgroundHttpFileHashHelper;

/**
 * Contains implementation of some common functions that don't vary between implementation
 */
class FBackgroundHttpManagerImpl 
	: public IBackgroundHttpManager
	, public FTSTickerObjectBase
{
public:
	BACKGROUNDHTTP_API FBackgroundHttpManagerImpl();
	BACKGROUNDHTTP_API virtual ~FBackgroundHttpManagerImpl();

	BACKGROUNDHTTP_API virtual void AddRequest(const FBackgroundHttpRequestPtr Request) override;
	BACKGROUNDHTTP_API virtual void RemoveRequest(const FBackgroundHttpRequestPtr Request) override;

	BACKGROUNDHTTP_API virtual void Initialize(bool bClearPreExistingRequestsAtStartup) override;
	BACKGROUNDHTTP_API virtual void Shutdown() override;

	BACKGROUNDHTTP_API virtual void DeleteAllTemporaryFiles() override;
	
	BACKGROUNDHTTP_API virtual int GetMaxActiveDownloads() const override;
	BACKGROUNDHTTP_API virtual void SetMaxActiveDownloads(int MaxActiveDownloads) override;

	
	BACKGROUNDHTTP_API virtual FString GetTempFileLocationForURL(const FString& URL, const FString& RequestID) override;
	
	BACKGROUNDHTTP_API virtual void CleanUpDataAfterCompletingRequest(const FBackgroundHttpRequestPtr Request) override;

	BACKGROUNDHTTP_API virtual void SetCellularPreference(int32 Value) override;
	BACKGROUNDHTTP_API virtual void SetLimitedDataPreference(int32 Value) override;
	
	//FTSTickerObjectBase implementation
	BACKGROUNDHTTP_API virtual bool Tick(float DeltaTime) override;
	
protected:
	BACKGROUNDHTTP_API virtual bool AssociateWithAnyExistingRequest(const FBackgroundHttpRequestPtr Request) override;

	BACKGROUNDHTTP_API virtual bool CheckForExistingCompletedDownload(const FBackgroundHttpRequestPtr Request, FString& ExistingFilePathOut, int64& ExistingFileSizeOut);
	BACKGROUNDHTTP_API virtual void ActivatePendingRequests();
	
	//Different from DeleteAllTemporaryFiles as this doesn't delete all files but rather cleans up specific bad files that have gone stale
	BACKGROUNDHTTP_API virtual void DeleteStaleTempFiles();
	
	//Gets a list of full filenames for all temp files.
	BACKGROUNDHTTP_API virtual void GatherAllTempFilenames(TArray<FString>& OutAllTempFilenames, bool bOutputAsFullPaths = false) const;
	
	//Helper function that converts the supplied list of Temp folder filenames to a list that is always all full paths.
	BACKGROUNDHTTP_API virtual void ConvertAllTempFilenamesToFullPaths(TArray<FString>& OutFilenamesAsFullPaths, const TArray<FString>& FilenamesToConvertToFullPaths) const;
	
	//Gather a list of any temp files that have timed out of our BackgroundHTTP settings.
	//SecondsToConsiderOld should be a double representing how many seconds old a temp file needs to be to be returned by this function
	//If OptionalFileListToCheck is empty will check all temp files in the backgroundhttp temp file folder. If supplied only the given file paths are checked
	BACKGROUNDHTTP_API virtual void GatherTempFilesOlderThen(TArray<FString>& OutTimedOutTempFilenames,double SecondsToConsiderOld, TArray<FString>* OptionalFileList = nullptr) const;
	
	//Gather a list of any temp files that have no corresponding URL Mapping entry
	//If OptionalFileListToCheck is empty will check all temp files in the backgroundhttp temp file folder. If supplied only the given file paths are checked
	BACKGROUNDHTTP_API virtual void GatherTempFilesWithoutURLMappings(TArray<FString>& OutTempFilesMissingURLMappings, TArray<FString>* OptionalFileList = nullptr) const;
	
	//Gets our FileHashHelper to compute temp file mappings
	virtual BackgroundHttpFileHashHelperRef GetFileHashHelper(){ return FileHashHelper; }
	virtual const BackgroundHttpFileHashHelperRef GetFileHashHelper() const { return FileHashHelper; }

	// Returns number of deleted files
	BACKGROUNDHTTP_API virtual int DeleteAllTemporaryFiles(bool Recursively, const TCHAR* FileExtension);

protected:
	/** List of Background Http requests that we have called AddRequest on, but have not yet started due to platform active download limits **/
	TArray<FBackgroundHttpRequestPtr> PendingStartRequests;
	FRWLock PendingRequestLock;

	/** List of Background Http requests that are actively being processed **/
	TArray<FBackgroundHttpRequestPtr> ActiveRequests;
	mutable FRWLock ActiveRequestLock;

	/** Count of how many requests we have active **/
	volatile int NumCurrentlyActiveRequests;
	TAtomic<int> MaxActiveDownloads;
	
private:
	BackgroundHttpFileHashHelperRef FileHashHelper;
};
