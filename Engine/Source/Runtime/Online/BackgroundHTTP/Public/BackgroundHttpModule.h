// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IBackgroundHttpManager.h"
#include "Interfaces/IBackgroundHttpRequest.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleInterface.h"


/**
 * Module for Http Background request implementations
 */
class FBackgroundHttpModule : 
	public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	BACKGROUNDHTTP_API static FBackgroundHttpModule& Get();

	/**
	* Creates a BackgroundRequest through the appropriate platform-layer.
	*
	* @return SharedRef to the new BackgroundRequest.
	*/
	BACKGROUNDHTTP_API static FBackgroundHttpRequestPtr CreateBackgroundRequest();

	/**
	* Gets the current BackgroundHttpManager from the appropriate platform layer.
	*
	* @return SharedPtr to the BackgroundHttpManager.
	*/
	BACKGROUNDHTTP_API FBackgroundHttpManagerPtr GetBackgroundHttpManager();

	/**
	* Sets whether or not the background downloader's temporary files are to be
	* wiped clean at startup. Can only be set before the first call to
	* GetBackgroundHttpManager().
	*/
	BACKGROUNDHTTP_API static void ClearPreExistingRequestsAtStartup(bool bEnable);

public:
	// IModuleInterface
	/**
	 * Called when Background Http module is loaded
	 */
	virtual void StartupModule() override;

	/**
	 * Called when Background Http module is unloaded
	 */
	virtual void ShutdownModule() override;

private:
	/** Keeps track of Background Http requests while they are being processed. */
	FBackgroundHttpManagerPtr BackgroundHttpManager;

	/** Singleton for the module while loaded and available. */
	static FBackgroundHttpModule* Singleton;

	/** Setting representing whether or not all pre-existing requests are to be removed
	* at startup. False by default. Use ClearPreExistingRequestsAtStartup(bool) to change.
	*/
	static bool bClearPreExistingRequestsAtStartup;
};
