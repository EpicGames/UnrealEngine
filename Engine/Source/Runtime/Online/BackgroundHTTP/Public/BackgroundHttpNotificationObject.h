// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class FBackgroundHttpNotificationObject
	: public TSharedFromThis<FBackgroundHttpNotificationObject, ESPMode::ThreadSafe>
{
public:
	BACKGROUNDHTTP_API FBackgroundHttpNotificationObject(FText InNotificationTitle, FText InNotificationBody, FText InNotificationAction, const FString& InNotificationActivationString, bool InNotifyOnlyOnFullSuccess);
	BACKGROUNDHTTP_API FBackgroundHttpNotificationObject(FText InNotificationTitle, FText InNotificationBody, FText InNotificationAction, const FString& InNotificationActivationString, bool InNotifyOnlyOnFullSuccess, bool bOnlySendNotificationInBackground, int32 InIdOverride, bool bUnused = false);
	BACKGROUNDHTTP_API ~FBackgroundHttpNotificationObject();

	BACKGROUNDHTTP_API void NotifyOfDownloadResult(bool bWasSuccess);

private:
	void OnApp_EnteringBackground();
	void OnApp_EnteringForeground();

private:
	FText NotificationTitle;
	FText NotificationAction;
	FText NotificationBody;
	FString NotificationActivationString;

	bool bOnlySendNotificationInBackground;
	bool bNotifyOnlyOnFullSuccess;

	volatile bool bIsInBackground;
	volatile int32 NumFailedDownloads;

	int32 IdOverride;

	class ILocalNotificationService* PlatformNotificationService;

	FDelegateHandle OnApp_EnteringForegroundHandle;
	FDelegateHandle OnApp_EnteringBackgroundHandle;

	//No default constructor
	FBackgroundHttpNotificationObject() {}
};

typedef TSharedPtr<FBackgroundHttpNotificationObject, ESPMode::ThreadSafe> FBackgroundHttpNotificationObjectPtr;
