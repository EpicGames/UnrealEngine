// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackgroundHttpNotificationObject.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IBackgroundHttpRequest.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "Async/Mutex.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBackgroundHttpRequest, Log, All)

enum class EBackgroundHttpRequestCompletionResult
{
	Success,
	GenericError, 
	OutOfDiskSpaceError,
};

/**
 * Contains implementation of some common functions that don't have to vary between implementation
 */
class FBackgroundHttpRequestImpl 
	: public IBackgroundHttpRequest
{
public:
	BACKGROUNDHTTP_API FBackgroundHttpRequestImpl();
	virtual ~FBackgroundHttpRequestImpl() = default;

	//This should be called from the platform level when a BG download finishes.
	BACKGROUNDHTTP_API virtual void OnBackgroundDownloadComplete();

	// IBackgroundHttpRequest
	BACKGROUNDHTTP_API virtual bool ProcessRequest() override;
	BACKGROUNDHTTP_API virtual void CancelRequest() override;
	BACKGROUNDHTTP_API virtual void RequeueRequest() override;
    BACKGROUNDHTTP_API virtual void PauseRequest() override;
    BACKGROUNDHTTP_API virtual void ResumeRequest() override;
    BACKGROUNDHTTP_API virtual void SetURLAsList(const TArray<FString>& URLs, int NumRetriesToAttempt) override;
	BACKGROUNDHTTP_API virtual const TArray<FString>& GetURLList() const override;
	BACKGROUNDHTTP_API virtual void SetExpectedResultSize(const uint64 ExpectedSize) override;
	BACKGROUNDHTTP_API virtual uint64 GetExpectedResultSize() const override;
	BACKGROUNDHTTP_API virtual void SetCompleteNotification(FBackgroundHttpNotificationObjectPtr DownloadCompleteNotificationObjectIn) override;
	BACKGROUNDHTTP_API virtual void CompleteWithExistingResponseData(FBackgroundHttpResponsePtr BackgroundResponse) override;
	BACKGROUNDHTTP_API virtual FBackgroundHttpRequestCompleteDelegate& OnProcessRequestComplete() override;
	BACKGROUNDHTTP_API virtual FBackgroundHttpProgressUpdateDelegate& OnProgressUpdated() override;
	UE_DEPRECATED(5.7, "FBackgroundHttpRequestImpl::OnRequestMetrics is deprecated, please use OnRequestMetricsExtended instead.")
	BACKGROUNDHTTP_API virtual FBackgroundHttpRequestMetricsDelegate& OnRequestMetrics() override;
	BACKGROUNDHTTP_API virtual FBackgroundHttpRequestMetricsExtendedDelegate& OnRequestMetricsExtended() override;
	BACKGROUNDHTTP_API virtual const FBackgroundHttpResponsePtr GetResponse() const override;
	BACKGROUNDHTTP_API virtual const FString& GetRequestID() const override;
	BACKGROUNDHTTP_API virtual void SetRequestID(const FString& NewRequestID) override;
	BACKGROUNDHTTP_API virtual bool HandleDelayedProcess() override;
	BACKGROUNDHTTP_API virtual EBackgroundHTTPPriority GetRequestPriority() const override;
	BACKGROUNDHTTP_API virtual void SetRequestPriority(EBackgroundHTTPPriority NewPriority) override;
	BACKGROUNDHTTP_API virtual bool HasSeenOutOfDiskSpaceError() const override { return bHasSeenOutOfDiskSpaceError; }
	
	UE_DEPRECATED(5.8, "FBackgroundHttpRequestImpl::NotifyNotificationObjectOfComplete(bool) is deprecated, please use NotifyNotificationObjectOfComplete(EBackgroundHttpRequestCompletionResult) instead.")
	BACKGROUNDHTTP_API virtual void NotifyNotificationObjectOfComplete(const bool bWasSuccess);
	BACKGROUNDHTTP_API virtual void NotifyNotificationObjectOfComplete(EBackgroundHttpRequestCompletionResult Result);

	UE_DEPRECATED(5.7, "FBackgroundHttpRequestImpl::NotifyRequestMetricsAvailable is deprecated, please use NotifyRequestMetricsExtendedAvailable instead.")
	BACKGROUNDHTTP_API virtual void NotifyRequestMetricsAvailable(const int32 TotalBytesDownloaded, const float DownloadDuration);
	BACKGROUNDHTTP_API virtual void NotifyRequestMetricsExtendedAvailable(const FBackgroundHttpRequestMetricsExtended ExtendedMetrics);

	UE_DEPRECATED(5.7, "FBackgroundHttpRequestImpl::SetMetrics is deprecated, please use SetMetricsExtended instead.")
	BACKGROUNDHTTP_API virtual void SetMetrics(const int32 TotalBytesDownloaded, const float DownloadDuration) override;
	BACKGROUNDHTTP_API virtual void SetMetricsExtended(const FBackgroundHttpRequestMetricsExtended ExtendedMetrics) override;
	
protected:
	UE::FMutex DownloadCompleteMutex;
	TSharedPtr<FBackgroundHttpNotificationObject, ESPMode::ThreadSafe> DownloadCompleteNotificationObject;
	FBackgroundHttpResponsePtr Response;
	TArray<FString> URLList;
	FString RequestID;
	int NumberOfTotalRetries;
	EBackgroundHTTPPriority RequestPriority;
	uint64 ExpectedResultSize;
	
	bool bHasSeenOutOfDiskSpaceError = false;
	
	using FDownloadMetricsInfo = FBackgroundHttpRequestMetricsExtended;
	
	TOptional<FDownloadMetricsInfo> OptionalMetricsInfo;
	
	FBackgroundHttpRequestCompleteDelegate HttpRequestCompleteDelegate;
	FBackgroundHttpProgressUpdateDelegate HttpProgressUpdateDelegate;
	FBackgroundHttpRequestMetricsDelegate HttpRequestMetricsDelegate;
	FBackgroundHttpRequestMetricsExtendedDelegate HttpRequestMetricsExtendedDelegate;
};
