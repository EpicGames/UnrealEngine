// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Interfaces/IBackgroundHttpResponse.h"

/**
 * Contains implementation of some common functions that don't vary between implementation
 */
class FBackgroundHttpResponseImpl 
	: public IBackgroundHttpResponse
{
public:
	FBackgroundHttpResponseImpl();
	FBackgroundHttpResponseImpl(FString InTempContentFilePath, int32 InResponseCode);
	virtual ~FBackgroundHttpResponseImpl() override = default;

	//IHttpBackgroundResponse
	virtual int32 GetResponseCode() const override;
	virtual const FString& GetTempContentFilePath() const override;

protected:	
	FString TempContentFilePath;
	int32 ResponseCode;
};
