// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IBackgroundHttpRequest.h"

class IBackgroundHttpManager;

/**
 * Generic version of Background Http implementations for platforms that don't need a special implementation
 * Intended usage is to use FPlatformBackgroundHttp instead of FGenericPlatformHttp
 * On platforms without a specific implementation, you should still use FPlatformBackgroundHttp and it will call into these functions
 */
class FGenericPlatformBackgroundHttp
{
public:
	/**
	 * Platform initialization step
	 */
	static BACKGROUNDHTTP_API void Initialize();

	/**
	 * Platform shutdown step
	 */
	static BACKGROUNDHTTP_API void Shutdown();

	/**
	 * Creates a platform-specific Background HTTP manager.
	 * Un-implemented platforms should create a FGenericPlatformBackgroundHttpManager
	 */
	static BACKGROUNDHTTP_API FBackgroundHttpManagerPtr CreatePlatformBackgroundHttpManager();

	/**
	 * Creates a new Background Http request instance for the current platform
	 * that will continue to download when the application is in the background
	 *
	 * @return request object
	 */
	static BACKGROUNDHTTP_API FBackgroundHttpRequestPtr ConstructBackgroundRequest();

	/**
	 * Creates a new Background Http Response instance for the current platform
	 * This normally is called by the request and associated with itself.
	 *
	 * @return response object
	 */
	static BACKGROUNDHTTP_API FBackgroundHttpResponsePtr ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath);
};
