// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackgroundHttpManagerImpl.h"
#include "Containers/Ticker.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformBackgroundHttpRequest.h"

/**
 * Manages Background Http request that are currently being processed if no platform specific implementation has been made. 
 */
class FGenericPlatformBackgroundHttpManager
	: public FBackgroundHttpManagerImpl
{
public:
	virtual bool IsGenericImplementation() const override { return true; }
	BACKGROUNDHTTP_API virtual ~FGenericPlatformBackgroundHttpManager();
};
