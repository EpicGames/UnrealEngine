// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackgroundHttpRequestImpl.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Interfaces/IBackgroundHttpRequest.h"
#include "Interfaces/IHttpRequest.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"


/**
 * Contains implementation of some common functions that don't vary between implementation
 */
class FGenericPlatformBackgroundHttpRequest 
	: public FBackgroundHttpRequestImpl
{
public:

	BACKGROUNDHTTP_API FGenericPlatformBackgroundHttpRequest();
	BACKGROUNDHTTP_API virtual ~FGenericPlatformBackgroundHttpRequest();

	//IHttpBackgroundHttpRequest
	BACKGROUNDHTTP_API virtual bool HandleDelayedProcess() override;
	BACKGROUNDHTTP_API virtual void CompleteWithExistingResponseData(FBackgroundHttpResponsePtr BackgroundResponse) override;
	BACKGROUNDHTTP_API virtual void CancelRequest() override;

protected:
	//In the default implementation we actually use HTTPRequests instead of HttpBackgroundRequests.
	//This class handles wrapping some of the HttpBackgroundRequest only functionality (such as request lists
	//and notification objects) so they work with non-background HttpRequests
	class FGenericPlatformBackgroundHttpWrapper
	{
	public:
		FGenericPlatformBackgroundHttpWrapper(FBackgroundHttpRequestPtr Request, int MaxRetriesToAttempt);
		~FGenericPlatformBackgroundHttpWrapper();

		void MakeRequest();
		void HttpRequestComplete(FHttpRequestPtr HttpRequestIn, FHttpResponsePtr HttpResponse, bool bSuccess);

	private:
		
		bool HasRetriesRemaining() const;
		void UpdateHttpProgress(FHttpRequestPtr UnderlyingHttpRequest, uint64 BytesSent, uint64 BytesReceived);

		void CleanUpHttpRequest();

		// Gets the current URL the HTTP Request should use by comparing the URLList in the background request to the current retry number.
		// Returns empty FString if we are "out of" retries (CurrentRetryNumber > MaxRetries) or if no valid URL.
		const FString GetURLForCurrentRetryNumber();

		//Don't want to provide default implementation. Must provide an IHttpBackgroundRequest to wrap in public constructor
		FGenericPlatformBackgroundHttpWrapper() {}

		//Reference to the creating request
		TWeakPtr<class IBackgroundHttpRequest, ESPMode::ThreadSafe> OriginalRequest;

		//Current Http Request being processed by this wrapper
		TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> HttpRequest;

		//Tracking retry number we are currently on
		int32 CurrentRetryNumber;

		//Max number of retries we should do
		int32 MaxRetries;

		//How many bytes we had last time we sent progress updates
		uint64 LastProgressUpdateBytes;
	};

	TUniquePtr<FGenericPlatformBackgroundHttpWrapper> RequestWrapper;
};
