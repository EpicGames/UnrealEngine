// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackgroundHttpResponseImpl.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Interfaces/IHttpRequest.h"

/**
 * Contains implementation of some common functions that don't vary between implementation
 */
class FGenericPlatformBackgroundHttpResponse
	: public FBackgroundHttpResponseImpl
{
public:
	//Provide constructor to create an already 
	FGenericPlatformBackgroundHttpResponse(int32 ResponseCode, const FString& TempContentFilePath);

	//Provide constuctor to create a request from underlying HttpRequest information
	FGenericPlatformBackgroundHttpResponse(FHttpRequestPtr HttpRequestIn, FHttpResponsePtr HttpResponse, bool bSuccess);
};