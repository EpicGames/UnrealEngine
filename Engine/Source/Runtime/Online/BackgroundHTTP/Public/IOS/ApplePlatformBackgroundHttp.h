// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

#include "Interfaces/IBackgroundHttpRequest.h"
#include "Interfaces/IBackgroundHttpManager.h"
#include "Interfaces/IBackgroundHttpResponse.h"

/**
 * Apple specific Background Http implementations
 * Intended usage is to use FPlatformBackgroundHttp
 */
class BACKGROUNDHTTP_API FApplePlatformBackgroundHttp
{
public:
	/**
	 * Platform initialization step
	 */
	static void Initialize();

	/**
	 * Platform shutdown step
	 */
	static void Shutdown();

	/**
	 * Creates a platform-specific Background HTTP manager.
	 * Un-implemented platforms should create a FGenericPlatformBackgroundHttpManager
	 */
	static FBackgroundHttpManagerPtr CreatePlatformBackgroundHttpManager();

	/**
	 * Creates a new Background Http request instance for the current platform
	 * that will continue to download when the application is in the background
	 *
	 * @return request object
	 */
	static FBackgroundHttpRequestPtr ConstructBackgroundRequest();

	/**
	 * Creates a new Background Http Response instance for the current platform
	 * This normally is called by the request and associated with itself.
	 *
	 * @return response object
	 */
	static FBackgroundHttpResponsePtr ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath);
};

//Setup Platform Implementation calls to point here at the Apple Implementation
typedef FApplePlatformBackgroundHttp FPlatformBackgroundHttp;
