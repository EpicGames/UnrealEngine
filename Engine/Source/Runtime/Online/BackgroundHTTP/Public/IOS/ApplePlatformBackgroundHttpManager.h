// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BackgroundHttpManagerImpl.h"
#include "Interfaces/IBackgroundHttpRequest.h"
#include "IOS/ApplePlatformBackgroundHttpRequest.h"

typedef TWeakPtr<FApplePlatformBackgroundHttpRequest,ESPMode::ThreadSafe> FBackgroundHttpURLMappedRequestPtr;

DECLARE_DELEGATE(FIOSBackgroundHttpPostSessionWorkCallback);

/**
 * Manages Background Http request that are currently being processed if we are on an Apple Platform
 */
class BACKGROUNDHTTP_API FApplePlatformBackgroundHttpManager
	: public FBackgroundHttpManagerImpl
{
public:
	FApplePlatformBackgroundHttpManager();
	virtual ~FApplePlatformBackgroundHttpManager();

	virtual void Initialize(bool bClearPreExistingRequestsAtStartup) override;
	virtual void AddRequest(const FBackgroundHttpRequestPtr Request) override;
	virtual void RemoveRequest(const FBackgroundHttpRequestPtr Request) override;
	virtual void RequeueRequest(const FBackgroundHttpRequestPtr Request) override;
	virtual void GetActiveDownloadRequestIDs(TSet<FString>& OutActiveIDs) const override;
	virtual void SetCellularPreference(int32 Value) override;
	virtual void SetLimitedDataPreference(int32 Value) override;
	virtual bool IsGenericImplementation() const override { return false; }
	virtual bool Tick(float DeltaTime) override;

private:
	TArray<FBackgroundHttpRequestPtr> PendingRemoveRequests;
	FRWLock PendingRemoveRequestLock;
	FDelegateHandle OnDownloadCompletedHandle;
	FDelegateHandle OnDownloadMetricsExtendedHandle;
	void OnDownloadCompletedExtended(const uint64 DownloadId, EBackgroundHttpRequestCompletionResult Result);
	void OnDownloadMetricsExtended(const uint64 DownloadId, const FBackgroundHttpRequestMetricsExtended MetricsExtended);
};
