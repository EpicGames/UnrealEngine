// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IBackgroundHttpRequest.h"
#include "BackgroundHttpRequestImpl.h"

/**
 * Contains implementation of Apple specific background http requests
 */
class BACKGROUNDHTTP_API FApplePlatformBackgroundHttpRequest 
	: public FBackgroundHttpRequestImpl
{
public:
	FApplePlatformBackgroundHttpRequest();
	virtual ~FApplePlatformBackgroundHttpRequest() = default;

	virtual void PauseRequest() override;
	virtual void ResumeRequest() override;

#if !UE_BUILD_SHIPPING
	virtual void GetDebugText(TArray<FString>& Output) override;
#endif

protected:
	void SetInternalDownloadId(uint64 Id) {DownloadId = Id;}
	uint64 GetInternalDownloadId() const {return DownloadId;}

	void UpdateProgress();
	void ResetProgressTracking();
	void RotateURLList();

	static float BackgroundRequestPriorityToNSURLSessionPriority(const EBackgroundHTTPPriority Priority);
	float GetNSURLSessionPriority() const
	{
		return BackgroundRequestPriorityToNSURLSessionPriority(GetRequestPriority());
	}

	friend class FApplePlatformBackgroundHttpManager;
private:
	uint64 DownloadId;

	uint64 LastReportedDownloadedBytes = 0;
	
	int MetricsWaitCounter = -1;
};

typedef TSharedPtr<class FApplePlatformBackgroundHttpRequest, ESPMode::ThreadSafe> FAppleBackgroundHttpRequestPtr;
