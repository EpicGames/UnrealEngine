// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackgroundHttpResponseImpl.h"
#include "Interfaces/IBackgroundHttpRequest.h"

/**
 * Contains implementation of some common functions that don't vary between implementation
 */
class FApplePlatformBackgroundHttpResponse
	: public FBackgroundHttpResponseImpl
{
public:
    //Provide constructor to create an already
    FApplePlatformBackgroundHttpResponse(int32 ResponseCode, const FString& TempContentFilePath);
};
