// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IBackgroundHttpRequest.h"

typedef TSharedPtr<class IBackgroundHttpManager, ESPMode::ThreadSafe> FBackgroundHttpManagerPtr;

class IBackgroundHttpManager : public TSharedFromThis<IBackgroundHttpManager>
{
public:

	virtual ~IBackgroundHttpManager() = default;

	/**
	 * Initialize
	 */
	virtual void Initialize(bool bClearPreExistingRequestsAtStartup) = 0;

	/**
	 * Shutdown
	 */
	virtual void Shutdown() = 0;

	/**
	 * Adds a Background Http request instance to the manager for tracking
	 * Manager should always have a list of requests currently being processed
	 *
	 * @param Request - the request object to add
	 */
	virtual void AddRequest(const FBackgroundHttpRequestPtr Request) = 0;

	/**
	 * Removes a Background Http request instance from the manager
	 * Presumably it is done being processed
	 *
	 * @param Request - the request object to remove
	 */
	virtual void RemoveRequest(const FBackgroundHttpRequestPtr Request) = 0;

	/**
	 * Re-queues an active request by canceling the current platform download and creating a fresh one.
	 * Unlike CancelRequest+ProcessRequest, this is safe on all platforms as it avoids deferred removal races.
	 * The URL list is rotated so the retry hits a different CDN.
	 *
	 * @param Request - the request object to re-queue
	 */
	virtual void RequeueRequest(const FBackgroundHttpRequestPtr Request) {}

	/**
	* Function to remove all temporary files used by the system to store completed downloads.
	* This should be called when there are no active background downloads that we care
	* about to regain disk space from background downloads that may no longer be relevant (IE: downloads that were never moved after completion)
	* but are still on disk.
	*
	*/
	virtual void DeleteAllTemporaryFiles() = 0;

	/** Returns whether or not this is a platform specific implementation */
	virtual bool IsGenericImplementation() const = 0;

	/**
	* Function that returns how many active BackgroundHttpRequests we should have actively downloading at once.
	*
	* @return int Number of downloads we should have active at once.
	*/
	virtual int GetMaxActiveDownloads() const = 0;

	/**
	* Function that sets how many active BackgroundHttpRequests we should have actively downloading at once.
	*
	* @param MaxActiveDownloads the maximum number of downloads that should be active at once
	*/
	virtual void SetMaxActiveDownloads(int MaxActiveDownloads) = 0;

	/**
	 * Function that returns an FString fullpath where we would expect the given URL's temp part of the given RequestID to be located
	 *
	 */
	virtual FString GetTempFileLocationForURL(const FString& URL, const FString& RequestID) = 0;
	UE_DEPRECATED(5.8, "Please use GetTempFileLocationForURL with added RequestID argument instead")
	FString GetTempFileLocationForURL(const FString& URL) { return GetTempFileLocationForURL(URL, {}); }
	
	/**
	 * Function that cleans up any persistent data after we have completed a reqeust.
	 * Should really never be calling this outside of a BackgroundHTTP class
	 */
	virtual void CleanUpDataAfterCompletingRequest(const FBackgroundHttpRequestPtr Request) = 0;

	/**
	 * Setting cellular network preference
	 */
	virtual void SetCellularPreference(int32 Value) = 0;

	/**
	 * Setting limited data network preference
	 */
	virtual void SetLimitedDataPreference(int32 Value) = 0;

	/**
	 * Returns the set of RequestIDs that are actively being downloaded by the platform.
	 * Requests queued by the platform (waiting for a concurrency slot) are excluded.
	 * Default implementation returns empty set (treats no requests as confirmed active).
	 */
	virtual void GetActiveDownloadRequestIDs(TSet<FString>& OutActiveIDs) const {}

protected:
	/**
	* Designed to be called internally by AddRequest to associate our incoming request with any 
	* previously completed background downloads (might have completed before this app launch, or 
	* carried over from a previous application launch and be running without any other information.
	*
	* @param Request - The request to check for any existing request matches on.
	*
	* @return bool - true if we found an existing request and associated with it. false otherwise.
	*/
	virtual bool AssociateWithAnyExistingRequest(const FBackgroundHttpRequestPtr Request) = 0;
};
