// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

#include "Features/IModularFeature.h"

#include "Interfaces/IBackgroundHttpRequest.h"

#include "Logging/LogMacros.h"

class IBackgroundHttpManager;

DECLARE_LOG_CATEGORY_EXTERN(LogBackgroundHttpModularFeature, Log, All)

/**
 * This version of BackgroundHttp is designed to be used by any platform that relies on a ModularFeature to override the BackgroundHttp behavior. 
 * If no modular feature is found, we fall back on the GenericPlatform implementation to provide functionality
 */
class IBackgroundHttpModularFeature : public IModularFeature
{
public:
	//Virtual methods for all PlatformBackgroundHttp required functions
	virtual void Initialize() = 0;
	virtual void Shutdown() = 0;
	virtual FBackgroundHttpManagerPtr CreatePlatformBackgroundHttpManager() = 0;
	virtual FBackgroundHttpRequestPtr ConstructBackgroundRequest() = 0;
	virtual FBackgroundHttpResponsePtr ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath) = 0;
	
	//Should return the name of the Plugin/Module that causes this ModularFeature to be registered.
	virtual FString GetDebugModuleName() const = 0;
};
