// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BackgroundHttpNotificationObject.h"

//Included as we use EHttpResponseCode
#include "Interfaces/IHttpResponse.h"

//Included for FBackgroundHttpRequestMetricsExtended
#include "BackgroundHttpMetrics.h"

typedef TSharedPtr<class IBackgroundHttpRequest, ESPMode::ThreadSafe> FBackgroundHttpRequestPtr;
typedef TSharedPtr<class IBackgroundHttpResponse, ESPMode::ThreadSafe> FBackgroundHttpResponsePtr;
typedef TSharedPtr<class IBackgroundHttpManager, ESPMode::ThreadSafe> FBackgroundHttpManagerPtr;



/**
 * Delegate called when a Background Http request completes
 *
 * @param Request original Background Http request that started things
 * @param Response response received from the server if a successful connection was established
 */
DECLARE_DELEGATE_TwoParams(FBackgroundHttpRequestCompleteDelegate, FBackgroundHttpRequestPtr /*Request*/, bool /*bWasSuccessful*/);

/**
* Delegate Called when a Background Http request updates its progress
*
* @param Request Background Http Request that is updating its progress
* @param TotalBytesWritten Amount of data we have written so far. Total Figure.
* @param BytesWrittenSinceLastUpdate Amount of data we have written since our last call to this delegate.
*/
DECLARE_DELEGATE_ThreeParams(FBackgroundHttpProgressUpdateDelegate, FBackgroundHttpRequestPtr /*Request*/, int32 /*TotalBytesWritten*/, int32 /*BytesWrittenSinceLastUpdate*/);

/**
 * Delegate called when a Background Http request metrics are ready
 * @param Request Background Http Request for which the metrics were calculated
 * @param TotalBytesDownloaded The count of bytes downloaded in this request
 * @param DownloadDuration The duration of the download in seconds
 */
DECLARE_DELEGATE_ThreeParams(FBackgroundHttpRequestMetricsDelegate, FBackgroundHttpRequestPtr /* Request */, int32 /* TotalBytesDownloaded */, float /* DownloadDuration */);

/**
 * Delegate called when a Background Http request extended metrics are ready
 * @param Request Background Http Request for which the metrics were calculated
 * @param BackgroundHttpRequestMetricsExtended struct containing all the metrics
 */
DECLARE_DELEGATE_TwoParams(FBackgroundHttpRequestMetricsExtendedDelegate, FBackgroundHttpRequestPtr /* Request */, FBackgroundHttpRequestMetricsExtended /* All the metrics on the request */);

/**
 * Enum used to describe download priority. Higher priorities will be downloaded first.
 * Note: Should always be kept in High -> Low priority order if adding more Priorities!
 */
enum class EBackgroundHTTPPriority : uint8
{
	  High
	, Normal
	, Low
	, Num
};

inline const TCHAR* LexToString(EBackgroundHTTPPriority InType)
{
	switch (InType)
	{
	case EBackgroundHTTPPriority::High:
		return TEXT("High");
	case EBackgroundHTTPPriority::Normal:
		return TEXT("Normal");
	case EBackgroundHTTPPriority::Low:
		return TEXT("Low");
	case EBackgroundHTTPPriority::Num:
		return TEXT("INVALID(Num)");

	default:
		break;
	}
	return TEXT("<Unknown EBackgroundHTTPPriority>");
}

inline bool LexTryParseString(EBackgroundHTTPPriority& OutMode, const TCHAR* InBuffer)
{
	if (FCString::Stricmp(InBuffer, TEXT("High")) == 0)
	{
		OutMode = EBackgroundHTTPPriority::High;
		return true;
	}
	if (FCString::Stricmp(InBuffer, TEXT("Normal")) == 0)
	{
		OutMode = EBackgroundHTTPPriority::Normal;
		return true;
	}
	if (FCString::Stricmp(InBuffer, TEXT("Low")) == 0)
	{
		OutMode = EBackgroundHTTPPriority::Low;
		return true;
	}
	return false;
}

/**
 * Interface for Http requests (created using FHttpFactory)
 */
class IBackgroundHttpRequest 
	: public TSharedFromThis<IBackgroundHttpRequest, ESPMode::ThreadSafe>
{
public:
	/**
	* Sets up a list of URLs to automatically fall through as each one fails.
	* Must be called before ProcessRequest.
	*
	* @param URLs list of URLs. Eg: download.epicgames.com/downloadfilehere.txt, download2.epicgames.com/downloadfilehere.txt, download3.epicgames/downloadfilehere.txt
	* @param NumRetriesToAttempt How many times we want to fall through and try different URLS. Loops back to the beginning of the list if NumRetries > URLs.Num()
	*
	*/
	virtual void SetURLAsList(const TArray<FString>& URLs, int NumRetriesToAttempt) = 0;

	/**
	* Gets the current URL List that this background request is currently processing.
	*
	* @return List of URLs as FStrings
	*
	*/
	virtual const TArray<FString>& GetURLList() const = 0;

	/**
	* Sets expected result size for the request, might be used by the implementation
	* to check file size before invoking completion notification. 
	* Must be called before ProcessRequest.
	*
	* @param ExpectedSize expected file size in bytes, 0 to disable the check. Default value is 0.
	*
	*/
	virtual void SetExpectedResultSize(const uint64 ExpectedSize) {}

	/**
	* Returns expected result size for the request, set by SetExpectedResultSize.
	*
	* @return Expected result size in bytes.
	*
	*/
	virtual uint64 GetExpectedResultSize() const {return 0;}

	/**
	* Sets an FHTTPRequestDownloadNotificationObject to be mapped to this HTTPRequest.
	* This request will keep the particular FHttpRequestDownloadNotificationObject reference until it completes.
	* To use this function make sure you keep a reference to the supplied DownloadNotificationObject until you have created all the different
	* IHTTPRequest that you would like to use it. Then delete your reference. Once all references are removed (by each Request completing)
	* a callback set on the DownloadCompleteDelegateObject will be called
	*
	* @param DownloadCompleteDelegateObject The particular download complete delegate object
	*
	*/
	virtual void SetCompleteNotification(FBackgroundHttpNotificationObjectPtr DownloadCompleteNotificationObject) = 0;

	/**
	* Function used to complete an IHttpBackgroundRequest from an external source, passing it in a pre-existing response data.
	*
	* @param BackgroundReponse, A SharedRef to the already existing IHttpBackgroundResponse we want to base this tasks' Response off of.
	*/
	virtual void CompleteWithExistingResponseData(FBackgroundHttpResponsePtr BackgroundResponse) = 0;

	/**
	* Delegate called when the request is complete. See FBackgroundHttpRequestCompleteDelegate
	*/
	virtual FBackgroundHttpRequestCompleteDelegate& OnProcessRequestComplete() = 0;

	/**
	* Delegate called when the request has a progress update.
	*/
	virtual FBackgroundHttpProgressUpdateDelegate& OnProgressUpdated() = 0;
	
	/**
	 * Delegate called when the request's metrics were calculated
	 */
	UE_DEPRECATED(5.7, "IBackgroundHttpRequest::OnRequestMetrics is deprecated, please use IBackgroundHttpRequest::OnRequestMetricsExtended instead.")
	virtual FBackgroundHttpRequestMetricsDelegate& OnRequestMetrics() = 0;

	/**
	 * Delegate called when the request's extended metrics were calculated
	 */
	virtual FBackgroundHttpRequestMetricsExtendedDelegate& OnRequestMetricsExtended() = 0;

	
	/**
	 * Set metrics for analytics event
	 */
	UE_DEPRECATED(5.7, "IBackgroundHttpRequest::SetMetrics is deprecated, please use IBackgroundHttpRequest::SetMetricsExtended instead.")
	virtual void SetMetrics(const int32 TotalBytesDownloaded, const float DownloadDuration) {}

	/**
	 * Set extended metrics for analytics event
	 */
	virtual void SetMetricsExtended(const FBackgroundHttpRequestMetricsExtended ExtendedMetrics) {}


	/**
	* Called to begin processing the request.
	* OnProcessRequestComplete delegate is always called when the request completes or on error if it is bound.
	* A request can be re-used but not while still being processed.
	*
	* @return if the request was successfully started.
	*/
	virtual bool ProcessRequest() = 0;

	/**
	* Called by certain platform's implementation when we have to wait for the BackgroundHttpManager / PlatformBackgroundHttp to do some work
	* before we can finish our ProcessRequest call. Should only be called by different platform layers.
	* 
	* NOTE: Should really only be called by the BackgroundHttpManager! You are probably looking for ProcessRequest.
	*
	* @return if the request was successfully handled.
	*/
	virtual bool HandleDelayedProcess() = 0;

	/**
	 * Called to cancel a request that is still being processed
	 */
	virtual void CancelRequest() = 0;

	/**
	 * Re-queues a request by canceling the current platform download and creating a fresh one.
	 * Rotates the URL list so the retry hits a different CDN. Safe to call on active requests.
	 */
	virtual void RequeueRequest() {}

    /**
    * Called to pause a request that is still being processed
    */
    virtual void PauseRequest() = 0;
    
    /**
    * Called to resume a request that was previously paused
    */
    virtual void ResumeRequest() = 0;
    
	/**
	* Get the associated Response
	*
	* @return the response
	*/
	virtual const FBackgroundHttpResponsePtr GetResponse() const = 0;

	/**
	* Gets the associated RequestID for this BackgroundDownload.
	*
	* @return The RequestID
	*/
	virtual const FString& GetRequestID() const = 0;

	/**
	* Sets the associated RequestID for this BackgroundDownload. Useful as we associate a background downlaod with multiple
	* URLs, so this provides an easier way to identify the download with 1 string instead of checking multiple URLs.
	*
	* @param NewRequestID FString to set the Request ID to.
	*/
	virtual void SetRequestID(const FString& NewRequestID) = 0;

	/**
	* Gets the associated Requeset's Priority for Background Downloadings. Where possible, we attempt to finish downloads in lowest-priority first order.
	*
	* @return The request Priority
	*/
	virtual EBackgroundHTTPPriority GetRequestPriority() const = 0;
	
	/**
	* Sets the associated Requeset's Priority for Background Downloadings. Where possible, we attempt to finish downloads in lowest-priority first order.
	*
	* @param uint32 describing priority of this download. Lower happens first and thus 0 is the highest priority.
	*/
	virtual void SetRequestPriority(EBackgroundHTTPPriority NewPriority) = 0;
	
	/** 
	* Destructor for overrides 
	*/
	virtual ~IBackgroundHttpRequest() = default;
	
	/**
	 * Returns whether or not this request has seen an out of disk space error at least once.
	 */
	virtual bool HasSeenOutOfDiskSpaceError() const { return false; }

#if !UE_BUILD_SHIPPING
	/**
	 * Returns debug text for download screen to help troubleshoot download issues
	 */
	virtual void GetDebugText(TArray<FString>& Output) {}
#endif
};

