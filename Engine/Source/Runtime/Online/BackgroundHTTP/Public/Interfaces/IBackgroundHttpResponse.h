// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Interface for Http Background Responses
 */
class IBackgroundHttpResponse
{
public:
	/**
	 * Gets the response code returned by the requested server.
	 * See EHttpResponseCodes for known response codes. Same as HttpRequest (non-background)
	 *
	 * @return the response code.
	 */
	virtual int32 GetResponseCode() const = 0;

	/**
	* Gets the file name that our temporary content file will be stored in. HttpBackgroundRequests
	* Download their files to a temp location and this can be used to get at the temporary file on disk and move it to your desired location.
	* 
	* @return FString representing the FilePath to the temporary file.
	*/
	virtual const FString& GetTempContentFilePath() const = 0;

	/** 
	 * Destructor for overrides 
	 */
	virtual ~IBackgroundHttpResponse() = default;
};

