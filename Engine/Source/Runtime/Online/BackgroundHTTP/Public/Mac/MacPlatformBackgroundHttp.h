// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformBackgroundHttp.h"

//Currently no implementation on Mac
typedef FGenericPlatformBackgroundHttp FPlatformBackgroundHttp;