// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include COMPILED_PLATFORM_HEADER(PlatformBackgroundHttp.h)
