// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IBackgroundHttpModularFeature.h"
#include "Interfaces/IBackgroundHttpRequest.h"
#include "UObject/NameTypes.h"

class IBackgroundHttpManager;
class IBackgroundHttpModularFeature;

/**
 * This version of BackgroundHttp is designed to be used by any platform that relies on a ModularFeature to override the BackgroundHttp behavior. 
 * If no modular feature is found, we fall back on the GenericPlatform implementation to provide functionality
 */
class FModularFeaturePlatformBackgroundHttp
{
public:
	/**
	 * Platform initialization step
	 */
	static BACKGROUNDHTTP_API void Initialize();

	/**
	 * Platform shutdown step
	 */
	static BACKGROUNDHTTP_API void Shutdown();

	/**
	 * Creates a platform-specific Background HTTP manager.
	 * Un-implemented platforms should create a FGenericPlatformBackgroundHttpManager
	 */
	static BACKGROUNDHTTP_API FBackgroundHttpManagerPtr CreatePlatformBackgroundHttpManager();

	/**
	 * Creates a new Background Http request instance for the current platform
	 * that will continue to download when the application is in the background
	 *
	 * @return request object
	 */
	static BACKGROUNDHTTP_API FBackgroundHttpRequestPtr ConstructBackgroundRequest();

	/**
	 * Creates a new Background Http Response instance for the current platform
	 * This normally is called by the request and associated with itself.
	 *
	 * @return response object
	 */
	static BACKGROUNDHTTP_API FBackgroundHttpResponsePtr ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath);

public:
	static BACKGROUNDHTTP_API FName GetModularFeatureName();

protected:
	static BACKGROUNDHTTP_API void CacheModularFeature();

private:
	static IBackgroundHttpModularFeature* CachedModularFeature;
	static bool bHasCheckedForModularFeature;
};
