// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BackgroundHTTPFileHash : ModuleRules
{
    public BackgroundHTTPFileHash(ReadOnlyTargetRules Target) : base(Target)
    {
        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
            }
        );
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BuildSettings",
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
    }
}
