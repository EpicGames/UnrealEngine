// Copyright Epic Games, Inc. All Rights Reserved.
#include "BackgroundHttpFileHashHelper.h"
#include "BackgroundHttpFileHashPrivate.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"

#include "Hash/CityHash.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include "BuildSettings.h"

DEFINE_LOG_CATEGORY(LogBackgroundHttpFileHash);

FString FURLTempFileMapping::ToString() const
{
	// if changed make sure to update the format expected in InitFromString!
	return FString::Printf(TEXT("-URL=%s -TempFilename=%s"), *URL, *TempFilename);
}

bool FURLTempFileMapping::InitFromString(const FString& StringIn)
{
	FURLTempFileMapping ReturnedMapping;

	FString URLIn;
	FString TempFilenameIn;

	const bool bDidSuccessfullyParse = FParse::Value(*StringIn, TEXT("-URL="), URLIn)
								&& FParse::Value(*StringIn, TEXT("-TempFilename="), TempFilenameIn);

	// Only set our values if all values parsed successfully
	if (bDidSuccessfullyParse)
	{
		URL = URLIn;
		TempFilename = TempFilenameIn;
	}

	return bDidSuccessfullyParse;
}

void FBackgroundHttpFileHashHelper::LoadData()
{
	FWriteScopeLock _(URLFileMappingsLock);

	// No reason to load this data more then once
	if (!bHasLoadedURLData)
	{
		FString FileContents;
		const bool bLoadedFile = FFileHelper::LoadFileToString(FileContents, *GetURLMappingFilePathNoLock());
		if (bLoadedFile)
		{
			InitFromStringNoLock(FileContents);
		}
	}

	bHasLoadedURLData = true;
}

void FBackgroundHttpFileHashHelper::SaveData()
{
	FWriteScopeLock _(URLFileMappingsLock); // write due to bIsDirty

	if (!bIsDirty.exchange(false))
	{
		return;
	}

	// Only save to disk if we have already loaded the URLData or we will overwrite it with empty data!
	if (ensureAlwaysMsgf(bHasLoadedURLData, TEXT("Attempt to Save URL Mapping Data before calling LoadData! Skipping as otherwise we would overwrite our persistent data!")))
	{
		FFileHelper::SaveStringToFile(ToStringNoLock(), *GetURLMappingFilePathNoLock());
	}
}

const FString& FBackgroundHttpFileHashHelper::GetTemporaryRootPath()
{
	static FString BackgroundHttpDir = FPaths::Combine(FPlatformMisc::GamePersistentDownloadDir(), TEXT("BackgroundHttp"));
	return BackgroundHttpDir;
}

const FString& FBackgroundHttpFileHashHelper::GetTemporaryDownloadPath()
{
#if PLATFORM_ANDROID || PLATFORM_IOS
    static FString DownloadDir = FPaths::Combine(GetTemporaryRootPath(), BuildSettings::GetBranchName());
    return DownloadDir;
#else
    return GetTemporaryRootPath();
#endif
}

const FString& FBackgroundHttpFileHashHelper::GetTempFileExtension()
{
	static FString TempExtension = ".tmp";
	return TempExtension;
}

const FString& FBackgroundHttpFileHashHelper::GetURLMappingFilePathNoLock()
{
	static FString URLMappingFilePath = FPaths::Combine(GetTemporaryRootPath(), TEXT("URLMap"), TEXT("TempFileURLMappings.urlmap"));
	return URLMappingFilePath;
}

FString FBackgroundHttpFileHashHelper::GetFullPathOfTempFilename(const FString& TempFilename)
{
	// don't take lock here, as other methods taking lock call it
	FString FullFileName = FPaths::SetExtension(FPaths::GetCleanFilename(TempFilename), GetTempFileExtension());
	return FPaths::Combine(GetTemporaryDownloadPath(), FullFileName);
}

FString FBackgroundHttpFileHashHelper::ToString() const
{
	FReadScopeLock _(URLFileMappingsLock);
	return ToStringNoLock();
}

FString FBackgroundHttpFileHashHelper::ToStringNoLock() const
{
	FString ReturnedString;
	for(const TPair<FString, FURLTempFileMapping>& URLMapPair : URLFileMappings)
	{
		const FURLTempFileMapping& URLMapping = URLMapPair.Value;
		ReturnedString.Append(URLMapping.ToString());

		// Add a newline between each entry.
		// This needs to match what we are looking for between entries in InitFromString if changed!
		ReturnedString.Append(TEXT("\r\n"));
	}

	return MoveTemp(ReturnedString);
}

bool FBackgroundHttpFileHashHelper::InitFromString(const FString& StringIn)
{
	FWriteScopeLock _(URLFileMappingsLock);
	return InitFromStringNoLock(StringIn);
}

bool FBackgroundHttpFileHashHelper::InitFromStringNoLock(const FString& StringIn)
{
	// Grab a single line into it's own string
	FString CurrentLine;
	FString RestOfString = StringIn;

	bool bInitSuccess = true;

	// Parse each line of the string until we run out of data
	bool bStillHasMoreToParse = !RestOfString.IsEmpty();
	while (bStillHasMoreToParse)
	{
		// Entries are assumed to be split by a newline. Should be the same as what is setup in the ToString function!
		RestOfString.Split("\r\n", &CurrentLine, &RestOfString);

		// We still have something to parse, so lets go ahead and create an entry in the new mapp
		if (!CurrentLine.IsEmpty())
		{
			FURLTempFileMapping NewMapping;
			const bool bParsedSuccessfully = NewMapping.InitFromString(CurrentLine);

			if (ensureAlwaysMsgf(bParsedSuccessfully, TEXT("Invalid data found in supplied string! Invalid Line: %s"), *CurrentLine))
			{
				// This isn't saved separately as its just always the TempFilename
				FString HashForMap = NewMapping.TempFilename;

				// Actually add an entry for the URLFileMapping
				URLFileMappings.Add(HashForMap, MoveTemp(NewMapping));
			}
			else
			{
				bInitSuccess = false;
			}
		}

		bStillHasMoreToParse = !RestOfString.IsEmpty();
	}

	bIsDirty = true;
	return bInitSuccess;
}

bool FBackgroundHttpFileHashHelper::FindTempFilenameMappingForURL(const FString& URL, const FString& RequestID, FString& OutTempFilename) const
{
	FReadScopeLock _(URLFileMappingsLock);

	const FURLTempFileMapping* FoundMapping = FindMappingForURLNoLock(URL, RequestID);
	if (nullptr != FoundMapping)
	{
		OutTempFilename = FoundMapping->TempFilename;
		return true;
	}

	return false;
}

const FString* FBackgroundHttpFileHashHelper::FindTempFilenameMappingForURL(const FString& URL) const
{
	FReadScopeLock _(URLFileMappingsLock);

	const FURLTempFileMapping* FoundMapping = FindMappingForURLNoLock(URL, {});
	if (nullptr != FoundMapping)
	{
		return &FoundMapping->TempFilename;
	}

	return nullptr;
}

const FString& FBackgroundHttpFileHashHelper::FindOrAddTempFilenameMappingForURL(const FString& URL, const FString& RequestID)
{
	FWriteScopeLock _(URLFileMappingsLock);

	const FURLTempFileMapping* FoundMapping = FindMappingForURLNoLock(URL, RequestID);
	if (nullptr != FoundMapping)
	{
		return (FoundMapping->TempFilename);
	}

	// Didn't find a valid old one, so lets add it
	FString Hash = FindValidFilenameHashForURLNoLock(URL, RequestID);
	FURLTempFileMapping& NewMapping = URLFileMappings.FindOrAdd(Hash);
	NewMapping.URL = URL;
	NewMapping.TempFilename = Hash;
	
	bIsDirty = true;
	return NewMapping.TempFilename;
}

bool FBackgroundHttpFileHashHelper::FindMappedURLForTempFilename(const FString& TempFilename, FString& OutMappedURL) const
{
	// Make sure we are only looking for the base temp filename instead of the full path
	FString BaseFilename = FPaths::GetBaseFilename(TempFilename);

	FReadScopeLock _(URLFileMappingsLock);

	const FURLTempFileMapping* FoundMapping = URLFileMappings.Find(BaseFilename);
	if (nullptr != FoundMapping)
	{
		OutMappedURL = FoundMapping->URL;
		return true;
	}
	else
	{
		return false;
	}
}

const FString* FBackgroundHttpFileHashHelper::FindMappedURLForTempFilename(const FString& TempFilename) const
{
	// Make sure we are only looking for the base temp filename instead of the full path
	FString BaseFilename = FPaths::GetBaseFilename(TempFilename);

	FReadScopeLock _(URLFileMappingsLock);

	const FURLTempFileMapping* FoundMapping = URLFileMappings.Find(BaseFilename);
	return (nullptr != FoundMapping) ? &(FoundMapping->URL) : nullptr;
}

const FURLTempFileMapping* FBackgroundHttpFileHashHelper::FindMappingForURLNoLock(const FString& URL, const FString& RequestID) const
{
	FString Hash = FindValidFilenameHashForURLNoLock(URL, RequestID);
	const FURLTempFileMapping* FoundMapping = URLFileMappings.Find(Hash);

	// We found an already existing entry for the given Hash, so return that temp file mapping
	if (nullptr != FoundMapping)
	{
		//Sanity check that these things match, shouldn't be required as we should always find a valid hash from FindValidFilenameHashForURL
		if (ensureAlwaysMsgf(URL.Equals(FoundMapping->URL, ESearchCase::IgnoreCase), TEXT("Unexpected result from FindValidFilenameHashForURL! Did not match the existing URL! Found:%s , Expected:%s!"), *URL, *(FoundMapping->URL)))
		{
			return FoundMapping;
		}
	}

	return nullptr;
}

FString FBackgroundHttpFileHashHelper::FindValidFilenameHashForURLNoLock(const FString& URL, const FString& RequestID) const
{
	FString HashToReturn;

	// Using a simple counter here that we should never hit in practice, but want to protect against
	// and infinite loop in the case of a logic error here
	const uint32 MaxCollisionsToTry = 200;
	uint32 CollisionCount = 0;

	// Put a simple counter here that we should NEVER hit, but wanted to protect against an infinite loop
	// in case there is a serious logic error here
	while (CollisionCount < MaxCollisionsToTry)
	{
		const FString Hash = GenerateHashedFilenameForURLNoLock(URL, RequestID, CollisionCount);
		const FURLTempFileMapping* FoundMapping = URLFileMappings.Find(Hash);
		const bool bFoundValidMapping = (nullptr != FoundMapping);

		// Data already in our mapping
		if (bFoundValidMapping)
		{
			// Existing entry was ours so use it
			if (FoundMapping->URL.Equals(URL, ESearchCase::IgnoreCase))
			{
				UE_LOGF(LogBackgroundHttpFileHash, VeryVerbose, "Found valid existing URL Temp File Mapping for %ls and %ls at %ls", *URL, *RequestID, *Hash);
				
				HashToReturn = Hash;
				break;
			}
			// The existing entry was not ours, we have a collision and need to move on
			else
			{
				UE_LOGF(LogBackgroundHttpFileHash, Log, "Collision for URL Mapping for %ls and %ls with %ls with hash %ls", *URL, *RequestID, *(FoundMapping->URL), *Hash);
			}
		}
		// Data not in our mapping, so its definitely safe to use for this URL
		else
		{
			UE_LOGF(LogBackgroundHttpFileHash, Verbose, "Found valid unused URL Temp File Mapping for %ls and %ls at %ls", *URL, *RequestID, *Hash);
			HashToReturn = Hash;

			// No need to keep searching as we found a valid result
			break;
		}

		++CollisionCount;
	}

	// If we didn't manage to find a valid hash above, fallback to our sanitized URL
	if (HashToReturn.IsEmpty())
	{
		const FString TempFilename = FPaths::MakeValidFileName(URL + RequestID);
		UE_LOGF(LogBackgroundHttpFileHash, Error, "Unable to find valid temp filename after max collisions for %ls and %ls! Falling back to using sanitized URL %ls. This shouldn't ever be required!", *URL, *RequestID, *TempFilename);
		
		HashToReturn = TempFilename;
	}

	return MoveTemp(HashToReturn);
}

FString FBackgroundHttpFileHashHelper::GenerateHashedFilenameForURLNoLock(const FString& URL, const FString& RequestID, uint32 CollisionCount)
{
	const FString CompleteString = FString::Printf(TEXT("%s|%s"), *URL, *RequestID);
	const TArray<TCHAR, FString::AllocatorType>& UnderlyingCharData = CompleteString.GetCharArray();
	const uint32 Hash = CityHash32(reinterpret_cast<const char*>(UnderlyingCharData.GetData()), (UnderlyingCharData.Num() * sizeof(TCHAR)));
	return FPaths::MakeValidFileName(FString::Printf(TEXT("%d%d"), Hash, CollisionCount));
}

void FBackgroundHttpFileHashHelper::DeleteURLMappingsWithoutTempFiles()
{
	FWriteScopeLock _(URLFileMappingsLock);

	TArray<FString> MappingsToRemove;

	for (const TPair<FString, FURLTempFileMapping>& Entry : URLFileMappings)
	{
		const FString FullFilePathForEntry = GetFullPathOfTempFilename(Entry.Value.TempFilename);
		if (!FPaths::FileExists(FullFilePathForEntry))
		{
			MappingsToRemove.Add(Entry.Key);
		}
	}

	for (const FString& Key : MappingsToRemove)
	{
		URLFileMappings.Remove(Key);
	}
	
	bIsDirty = true;
}

void FBackgroundHttpFileHashHelper::RemoveURLMapping(const FString& URL, const FString& RequestID)
{
	FWriteScopeLock _(URLFileMappingsLock);

	const FString Hash = FindValidFilenameHashForURLNoLock(URL, RequestID);
	URLFileMappings.Remove(Hash);
	
	bIsDirty = true;
}
