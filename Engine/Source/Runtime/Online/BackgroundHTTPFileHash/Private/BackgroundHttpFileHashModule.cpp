// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackgroundHttpFileHashPrivate.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, BackgroundHTTPFileHash);
