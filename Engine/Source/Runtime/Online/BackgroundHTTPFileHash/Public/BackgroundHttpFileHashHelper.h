// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

//Simple data wrapper used to hold a mapping of a URL to the temp file holding the finished download of that URL in the BackgroundHTTP Temp folder
struct FURLTempFileMapping
{
public:
	FString URL;
	FString TempFilename;
	
	FURLTempFileMapping()
		: URL()
		, TempFilename()
	{
	}
	
	FURLTempFileMapping(const FString& URLIn, const FString& TempFilenameIn)
		: URL(URLIn)
		, TempFilename(TempFilenameIn)
	{
	}
	
	//Creates an FString representation of this FURLTempFileMapping
	FString ToString() const;
	
	//Sets the values of this FURLTempFileMapping from the string representation
	//Returns if the parse was successful or not
	bool InitFromString(const FString& StringIn);
};

/**
*Helper that handles generating unique hashed file paths to use store to store BackgroundHTTP work once completed.
*Can be saved and loaded to/from disk to handle persistence across session when background work is compelted without any
*usefull program state.
*NOTE: This is in it's own separate module as some platforms (IOS, etc.) need to include this functionality in ApplicationCore
*and thus fully adding a dependency to those modules on BackgroundHTTP would not work
*/
class FBackgroundHttpFileHashHelper
{
public:
	FBackgroundHttpFileHashHelper()
		: bHasLoadedURLData(false)
		, bIsDirty(false)
		, URLFileMappings()
		, URLFileMappingsLock()
	{
	}

	// Handles loading URL Mapping data from disk. No operation if data has previously been loaded.
	BACKGROUNDHTTPFILEHASH_API void LoadData();

	// Handles saving URL Mapping data to disk. No operation if data isn't dirty
	BACKGROUNDHTTPFILEHASH_API void SaveData();

	// Creates an FString representation of this FBackgroundHttpFileHashHelper's URL mappings. Used for serializing to/from disk
	BACKGROUNDHTTPFILEHASH_API FString ToString() const;

	// Sets the values of this FBackgroundHttpFileHashHelper from the string representation
	// Returns if the parse was successful or not
	BACKGROUNDHTTPFILEHASH_API bool InitFromString(const FString& StringIn);

	// Looks for a temp filename mapping for the given URL. If one isn't found returns false and does NOT generate one.
	BACKGROUNDHTTPFILEHASH_API bool FindTempFilenameMappingForURL(const FString& URL, const FString& RequestID, FString& OutTempFilename) const;
	UE_DEPRECATED(5.8, "Please use FindTempFilenameMappingForURL with RequestID and reference out arguments instead")
	BACKGROUNDHTTPFILEHASH_API const FString* FindTempFilenameMappingForURL(const FString& URL) const;

	// Looks for a temp filename mapping for the given URL. If one isn't found a new one is generated and returned.
	BACKGROUNDHTTPFILEHASH_API const FString& FindOrAddTempFilenameMappingForURL(const FString& URL, const FString& RequestID);

	// Removes URL mapping
	BACKGROUNDHTTPFILEHASH_API void RemoveURLMapping(const FString& URL, const FString& RequestID);
	UE_DEPRECATED(5.8, "Please use RemoveURLMapping with RequestID argument instead")
	void RemoveURLMapping(const FString& URL) { RemoveURLMapping(URL, {}); }

	// Looks for a URL mapped to the supplied TempFilename. Nullptr returned if one isn't found
	BACKGROUNDHTTPFILEHASH_API bool FindMappedURLForTempFilename(const FString& TempFilename, FString& OutMappedURL) const;

	UE_DEPRECATED(5.7, "Please use FindMappedURLForTempFilename with reference out argument instead")
	BACKGROUNDHTTPFILEHASH_API const FString* FindMappedURLForTempFilename(const FString& TempFilename) const;

	// Deletes any URLMapping that doesn't have a corresponding Temp file actually on disk.
	BACKGROUNDHTTPFILEHASH_API void DeleteURLMappingsWithoutTempFiles();

	// Get our base directory used to store Temp files
	static BACKGROUNDHTTPFILEHASH_API const FString& GetTemporaryRootPath();

    // Get temporary download path
    static BACKGROUNDHTTPFILEHASH_API const FString& GetTemporaryDownloadPath();
    
	// Helper function that returns the file extension used by our BackgroundHTTP temp files
	static BACKGROUNDHTTPFILEHASH_API const FString& GetTempFileExtension();

	// Helper function that returns a full path representation of the given TempFilename
	static BACKGROUNDHTTPFILEHASH_API FString GetFullPathOfTempFilename(const FString& TempFilename);

private:
	// Same as ToString but doesn't take a lock 
	BACKGROUNDHTTPFILEHASH_API FString ToStringNoLock() const;

	// Same as InitFromString but doesn't take a lock 
	BACKGROUNDHTTPFILEHASH_API bool InitFromStringNoLock(const FString& StringIn);

	// Helper function that just returns the filepath we use to save/load our URL mapping data
	static BACKGROUNDHTTPFILEHASH_API const FString& GetURLMappingFilePathNoLock();

	// Looks for a temp filename mapping for the given URL. If one isn't found returns nullptr an does NOT generate one.
	BACKGROUNDHTTPFILEHASH_API const FURLTempFileMapping* FindMappingForURLNoLock(const FString& URL, const FString& RequestID) const;

	// Finds the hash that should be used to refer to this URL+RequestID. Either finds the hash used in the map to reference the
	// URLFileMappings entry for this URL+RequestID or returns the hash we should use (the next valid hash entry) without inserting it into the map.
	BACKGROUNDHTTPFILEHASH_API FString FindValidFilenameHashForURLNoLock(const FString& URL, const FString& RequestID) const;

	// Creates a hashed Filename with a modifier for the given collision count.
	static BACKGROUNDHTTPFILEHASH_API FString GenerateHashedFilenameForURLNoLock(const FString& URL, const FString& RequestID, uint32 CollisionCount);

	// Used to make sure that we have called LoadData before any operations that depend on it
	std::atomic<bool> bHasLoadedURLData;

	// Tracks if we have changed data since our last call to SaveData()
	std::atomic<bool> bIsDirty;

	// Stores our FURLTempFileMapping mapped to a Hashed filename
	TMap<FString, FURLTempFileMapping> URLFileMappings;

	// Access lock for URLFileMappings
	mutable FRWLock URLFileMappingsLock;
};

typedef TSharedRef<FBackgroundHttpFileHashHelper, ESPMode::ThreadSafe> BackgroundHttpFileHashHelperRef;
typedef TSharedPtr<FBackgroundHttpFileHashHelper, ESPMode::ThreadSafe> BackgroundHttpFileHashHelperPtr;
