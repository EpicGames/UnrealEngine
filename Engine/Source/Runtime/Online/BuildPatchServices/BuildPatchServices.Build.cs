// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

public class BuildPatchServices : ModuleRules
{
	[ConfigFile(ConfigHierarchyType.Engine, "BuildPatchServices")]
	bool bEnableDiskOverflowStore = true;

	public BuildPatchServices(ReadOnlyTargetRules Target) : base(Target)
	{
		StaticAnalyzerDisabledCheckers.Add("core.uninitialized.ArraySubscript");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject"
			}
		);
		PrivateDependencyModuleNames.AddRange(
		new string[] {
				"Analytics",
				"AnalyticsET",
				"HTTP",
				"Json",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"HTTP"
			}
		);
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"HTTP"
			}
		);

		// OpenSSL is needed on desktop platforms to support calculating SHA 256
		bool bWithOpenSsl = false;
		if (Target.Platform == UnrealTargetPlatform.Win64
		 || Target.Platform == UnrealTargetPlatform.Mac
		 || Target.Platform == UnrealTargetPlatform.Linux)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
			bWithOpenSsl = true;
		}

		// Add a definition to allow BPS source to switch openssl code in and out
		if (bWithOpenSsl)
		{
			PublicDefinitions.Add("BPS_WITH_OPENSSL=1");
		}
		else
		{
			PublicDefinitions.Add("BPS_WITH_OPENSSL=0");
		}

		if (EnableDiskOverflowStore)
		{
			PublicDefinitions.Add("ENABLE_PATCH_DISK_OVERFLOW_STORE=1");
		}
		else
		{
			PublicDefinitions.Add("ENABLE_PATCH_DISK_OVERFLOW_STORE=0");
		}

		if (EnableDiskOverflowStore)
		{
			PublicDefinitions.Add("ENABLE_PATCH_DISK_OVERFLOW_STORE=1");
		}
		else
		{
			PublicDefinitions.Add("ENABLE_PATCH_DISK_OVERFLOW_STORE=0");
		}

	}

	protected bool EnableDiskOverflowStore
	{
		get
		{
			ConfigCache.ReadSettings(DirectoryReference.FromFile(Target.ProjectFile), Target.Platform, this);
			return bEnableDiskOverflowStore;
		}
	}
}
