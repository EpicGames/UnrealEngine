// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchDelta.h"
#include "BuildPatchUtil.h"
#include "BuildPatchMergeManifests.h"

BUILDPATCHSERVICES_API FString BuildPatchServices::GetChunkDeltaFilename(const IBuildManifestRef& SourceManifest, const IBuildManifestRef& DestinationManifest, const FString& FilenameTrailer)
{
	return FBuildPatchUtils::GetChunkDeltaFilename(StaticCastSharedRef<FBuildPatchAppManifest>(SourceManifest).Get(), StaticCastSharedRef<FBuildPatchAppManifest>(DestinationManifest).Get(), FilenameTrailer);
}

BUILDPATCHSERVICES_API IBuildManifestPtr BuildPatchServices::MergeDeltaManifest(const IBuildManifestRef& Manifest, const IBuildManifestRef& Delta)
{
	return FBuildMergeManifests::MergeDeltaManifest(StaticCastSharedRef<FBuildPatchAppManifest>(Manifest).Get(), StaticCastSharedRef<FBuildPatchAppManifest>(Delta).Get());
}
