// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchFileConstructor.h"
#include "IBuildManifestSet.h"
#include "Algo/Transform.h"
#include "HAL/UESemaphore.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "HAL/RunnableThread.h"
#include "Hash/xxhash.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ConfigCacheIni.h"
#include "BuildPatchServicesPrivate.h"
#include "Interfaces/IBuildInstaller.h"
#include "Data/ChunkData.h"
#include "Common/StatsCollector.h"
#include "Common/SpeedRecorder.h"
#include "Common/FileSystem.h"
#include "Compression/CompressionUtil.h"
#include "Installer/ChunkSource.h"
#include "Installer/ChunkDbChunkSource.h"
#include "Installer/InstallChunkSource.h"
#include "Installer/CloudChunkSource.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/InstallerError.h"
#include "Installer/InstallerAnalytics.h"
#include "Installer/InstallerSharedContext.h"
#include "Installer/MessagePump.h"
#include "Templates/Greater.h"
#include "BuildPatchUtil.h"
#include "ProfilingDebugging/CountersTrace.h"

#if PLATFORM_MAC
#include <sys/stat.h>
#include <unistd.h>
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogBuildPatchFileConstructor, Log, All);
DEFINE_LOG_CATEGORY(LogBuildPatchFileConstructor);

using namespace BuildPatchServices;

static int32 SleepTimeWhenFileSystemThrottledSeconds = 1;
static FAutoConsoleVariableRef CVarSleepTimeWhenFileSystemThrottledSeconds(
	TEXT("BuildPatchFileConstructor.SleepTimeWhenFileSystemThrottledSeconds"),
	SleepTimeWhenFileSystemThrottledSeconds,
	TEXT("The amount of time to sleep if the destination filesystem is throttled."),
	ECVF_Default);

// This can be overridden by the installation parameters.
static bool bCVarStallWhenFileSystemThrottled = false;
static FAutoConsoleVariableRef CVarStallWhenFileSystemThrottled(
	TEXT("BuildPatchFileConstructor.bStallWhenFileSystemThrottled"),
	bCVarStallWhenFileSystemThrottled,
	TEXT("Whether to stall if the file system is throttled"),
	ECVF_Default);

static bool bCVarAllowMultipleFilesInFlight = true;
static FAutoConsoleVariableRef CVarAllowMultipleFilesInFlight(
	TEXT("BuildPatchFileConstructor.bCVarAllowMultipleFilesInFlight"),
	bCVarAllowMultipleFilesInFlight,
	TEXT("Whether to allow multiple files to be constructed at the same time, though still sequentially."),
	ECVF_Default);

static int32 CVarDisableResumeBelowMB = 0;
static FAutoConsoleVariableRef CVarRefDisableResumeBelowMB(
	TEXT("BuildPatchFileConstructor.DisableResumeBelowMB"),
	CVarDisableResumeBelowMB,
	TEXT("If nonzero, installs (not patches) below this size will not create or check any resume data."),
	ECVF_Default);

// Helper functions wrapping common code.
namespace FileConstructorHelpers
{
	void WaitWhilePaused(FThreadSafeBool& bIsPaused, FThreadSafeBool& bShouldAbort)
	{
		// Wait while paused
		while (bIsPaused && !bShouldAbort)
		{
			FPlatformProcess::Sleep(0.5f);
		}
	}

	TOptional<FDiskSpaceInfo> GetRemainingDiskSpace(const FString& InstallDirectory, uint64 RemainingBytesRequired)
	{
		bool bContinueConstruction = true;
		uint64 TotalSize = 0;
		uint64 AvailableDiskSpace = 0;
		if (!FPlatformMisc::GetDiskTotalAndFreeSpace(InstallDirectory, TotalSize, AvailableDiskSpace))
		{
			return TOptional<FDiskSpaceInfo>{};
		}
		return TOptional<FDiskSpaceInfo>{FDiskSpaceInfo{ AvailableDiskSpace, AvailableDiskSpace >= RemainingBytesRequired }};
	}

	uint64 CalculateRequiredDiskSpace(const FBuildPatchAppManifestPtr& CurrentManifest, const FBuildPatchAppManifestRef& BuildManifest, const EInstallMode& InstallMode, const TSet<FString>& InInstallTags)
	{
		// Make tags expected
		TSet<FString> InstallTags = InInstallTags;
		if (InstallTags.Num() == 0)
		{
			BuildManifest->GetFileTagList(InstallTags);
		}
		InstallTags.Add(TEXT(""));
		// Calculate the files that need constructing.
		TSet<FString> TaggedFiles;
		BuildManifest->GetTaggedFileList(InstallTags, TaggedFiles);
		FString DummyString;
		TSet<FString> FilesToConstruct;
		BuildManifest->GetOutdatedFiles(CurrentManifest.Get(), DummyString, TaggedFiles, FilesToConstruct);
		// Count disk space needed by each operation.
		int64 DiskSpaceDeltaPeak = 0;
		if (InstallMode == EInstallMode::DestructiveInstall && CurrentManifest.IsValid())
		{
			// The simplest method will be to run through each high level file operation, tracking peak disk usage delta.
			int64 DiskSpaceDelta = 0;

			// Loop through all files to be made next, in order.
			// This is sorted coming in and needs to stay in that order to pass BPT test suite
			//FilesToConstruct.Sort(TLess<FString>());
			for (const FString& FileToConstruct : FilesToConstruct)
			{
				// First we would need to make the new file.
				DiskSpaceDelta += BuildManifest->GetFileSize(FileToConstruct);
				if (DiskSpaceDeltaPeak < DiskSpaceDelta)
				{
					DiskSpaceDeltaPeak = DiskSpaceDelta;
				}
				// Then we can remove the current existing file.
				DiskSpaceDelta -= CurrentManifest->GetFileSize(FileToConstruct);
			}
		}
		else
		{
			// When not destructive, or no CurrentManifest, we always stage all new and changed files.
			DiskSpaceDeltaPeak = BuildManifest->GetFileSize(FilesToConstruct);
		}
		return FMath::Max<int64>(DiskSpaceDeltaPeak, 0);
	}

	static float ReadRWDurationThresholdSec()
	{
		float RWDurationThresholdSec = 4.0;
		GConfig->GetFloat(TEXT("BuildPatchServices"), TEXT("ConstructionRWDurationThresholdSec"), RWDurationThresholdSec, GEngineIni);
		return RWDurationThresholdSec;
	}

	static void LogLowRWDurationTime(const ISpeedRecorder::FRecord& ActivityRecord, const TCHAR* ProcedureType)
	{
		// We've seen reports of abnormally poor HDD response times so we log when things are truly bad (e.g. multiple seconds for an access).
		static const double RWDurationThresholdSec = ReadRWDurationThresholdSec();
		const double Duration = FStatsCollector::CyclesToSeconds(ActivityRecord.CyclesEnd - ActivityRecord.CyclesStart);
		if (Duration > RWDurationThresholdSec)
		{
			UE_LOGF(LogBuildPatchServices, Warning, "FBuildPatchFileConstructor: %ls time of %.3f seconds (%lld bytes)", ProcedureType, Duration, ActivityRecord.Size);
		}
	}
}


struct FAdministrationScope
{
	ISpeedRecorder::FRecord ActivityRecord;
	IFileConstructorStat* FileConstructorStat;
	FAdministrationScope(IFileConstructorStat* InFileConstructorStat)
	{
		FileConstructorStat = InFileConstructorStat;
		FileConstructorStat->OnBeforeAdminister();
		ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
	}
	~FAdministrationScope()
	{
		ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
		ActivityRecord.Size = 0;
		FileConstructorStat->OnAfterAdminister(ActivityRecord);
	}
};

struct FReadScope
{
	ISpeedRecorder::FRecord ActivityRecord;
	IFileConstructorStat* FileConstructorStat;
	const TCHAR* LogDetails = nullptr;
	FReadScope(IFileConstructorStat* InFileConstructorStat, int64 Size, const TCHAR* InLogDetails)
	{
		LogDetails = InLogDetails;
		FileConstructorStat = InFileConstructorStat;
		ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
		ActivityRecord.Size = Size;
		FileConstructorStat->OnBeforeRead();
	}
	~FReadScope()
	{
		ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
		FileConstructorHelpers::LogLowRWDurationTime(ActivityRecord, LogDetails);
		FileConstructorStat->OnAfterRead(ActivityRecord);
	}
};

enum class EConstructionError : uint8
{
	None = 0,
	CannotCreateFile,
	FailedWrite,
	FailedInitialSizeCheck,
	MissingChunk,
	SerializeError,
	TrackingError,
	OutboundDataError,
	InternalConsistencyError,
	Aborted,
	MissingFileInfo,
	InvalidSymlink,
	CloseError
};

enum class EFileStage : uint8
{
	Unknown,	 // The file stage is unknown due to a failure (FFileConstructionState::bSuccess is false).
	Complete,    // The file is complete; it exists and contains the expected data size.
	Incomplete,  // The file is incomplete; it exists but does not contain the expected data size. Resume the operation.
	New          // The file is missing; a new file should be created and populated from the start.
};


// Since we can have more than one file in flight, store state here.
struct FFileConstructionState
{
	FGuid ErrorContextGuid;
	EConstructionError ConstructionError = EConstructionError::None;
	int32 CreateFilePlatformLastError = 0;

	FSHA1 HashState;

	// If this is true then we didn't actually have to make the file, it was already done or a symlink or something.
	bool bSkippedConstruction = false;
	bool bSuccess = true;
	bool bIsResumedFile = false;

	// We track how far we are in the file when we write into the write buffer so that
	// we advance progress bars smoothly instead of in huge writebuffer sized chunks.
	UE::FMutex ProgressLock;
	uint64 Progress = 0;
	uint64 LastSeenProgress = 0; // not locked.	

	// Where we started constructing the file. Can be non zero due to resume.
	int64 StartPosition = 0;

	int32 OutstandingBatches = 0;

	// Can be nonzero in the first batch due to resume.
	int32 NextChunkPartToRead = 0;

	// This is null if we are constructing/install in memory.
	TUniquePtr<FArchive> NewFile;

	// Which index in the chunk reference tracker this file starts at.
	int32 BaseReferenceIndex = 0;

	int32 ConstructionIndex = -1;
	const FFileManifest* FileManifest;
	const FString& BuildFilename; // name as it is in the manifest, references the ConstructList in the configuration.
	const FString NewFilename; // with output path and such

	EFileStage Stage = EFileStage::Unknown;

	FFileConstructionState(const FFileManifest* InFileManifest, const FString& InBuildFilename, FString&& InNewFilename) 
	: FileManifest(InFileManifest)
	, BuildFilename(InBuildFilename)
	, NewFilename(InNewFilename)
	{
		if (!InFileManifest)
		{
			bSuccess = false;
			ConstructionError = EConstructionError::MissingFileInfo;
		}
	}

	void SetAttributes()
	{
#if PLATFORM_MAC || PLATFORM_LINUX
		if (bSuccess && 
			EnumHasAllFlags(FileManifest->FileMetaFlags, EFileMetaFlags::UnixExecutable))
		{
			// Enable executable permission bit
			struct stat FileInfo;
			if (stat(TCHAR_TO_UTF8(*NewFilename), &FileInfo) == 0)
			{
				bSuccess = chmod(TCHAR_TO_UTF8(*NewFilename), FileInfo.st_mode | S_IXUSR | S_IXGRP | S_IXOTH) == 0;
				if (!bSuccess)
				{
					UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Failed to set exec bit %ls", *BuildFilename);
				}
			}
		}
#endif

#if PLATFORM_ANDROID
		if (bSuccess)
		{
			IFileManager::Get().SetTimeStamp(*NewFilename, FDateTime::UtcNow());
		}
#endif
	}
};


/**
 * This struct handles loading and saving of simple resume information, that will allow us to decide which
 * files should be resumed from. It will also check that we are creating the same version and app as we expect to be.
 */
struct FResumeData
{
public:
	// File system dependency
	const IFileSystem* const FileSystem;

	// The manifests for the app we are installing
	const IBuildManifestSet* const ManifestSet;

	// Save the staging directory
	const FString StagingDir;

	// The filename to the resume data information
	const FString ResumeDataFilename;

	// The resume ids that we loaded from disk
	TSet<FString> LoadedResumeIds;

	// Maps each resumed file to its current stage: Complete or Incomplete. Files not found on disk or treated as New are not present in this map.
	TMap<FString, EFileStage> FilesStage;

	// Whether we have any resume data for this install
	bool bHasResumeData = false;

	// For small installs we may disable resume entirely to mitigate the number of file operations.
	bool bResumeEnabled = false;

public:

	FResumeData(IFileSystem* InFileSystem, IBuildManifestSet* InManifestSet, const FString& InStagingDir, const FString& InResumeDataFilename)
		: FileSystem(InFileSystem)
		, ManifestSet(InManifestSet)
		, StagingDir(InStagingDir)
		, ResumeDataFilename(InResumeDataFilename)
	{
		// Leave resume disabled until initialized;
	}

	void InitResume()
	{
		bResumeEnabled = true;

		// Load data from previous resume file
		bHasResumeData = FileSystem->FileExists(*ResumeDataFilename);
		GLog->Logf(TEXT("BuildPatchResumeData file found: %s"), bHasResumeData ? TEXT("true") : TEXT("false"));
		if (bHasResumeData)
		{
			// Grab existing resume metadata.
			const bool bCullEmptyLines = true;
			FString PrevResumeData;
			TArray<FString> PrevResumeDataLines;
			FileSystem->LoadFileToString(*ResumeDataFilename, PrevResumeData);
			PrevResumeData.ParseIntoArrayLines(PrevResumeDataLines, bCullEmptyLines);
			// Grab current resume ids
			const bool bCheckLegacyIds = true;
			TSet<FString> NewResumeIds;
			ManifestSet->GetInstallResumeIds(NewResumeIds, bCheckLegacyIds);
			LoadedResumeIds.Reserve(PrevResumeDataLines.Num());
			// Check if any builds we are installing are a resume from previous run.
			for (FString& PrevResumeDataLine : PrevResumeDataLines)
			{
				PrevResumeDataLine.TrimStartAndEndInline();
				LoadedResumeIds.Add(PrevResumeDataLine);
				if (NewResumeIds.Contains(PrevResumeDataLine))
				{
					bHasResumeData = true;
					GLog->Logf(TEXT("BuildPatchResumeData version matched %s"), *PrevResumeDataLine);
				}
			}
		}
	}

	/**
	 * Saves out the resume data
	 */
	void SaveOut(const TSet<FString>& ResumeIds)
	{
		// Save out the patch versions
		if (bResumeEnabled)
		{
			FileSystem->SaveStringToFile(*ResumeDataFilename, FString::Join(ResumeIds, TEXT("\n")));
		}
	}

	/**
	 * Deletes the resume data
	 */
	void RemoveResumeData()
	{
		if (bResumeEnabled)
		{
			bHasResumeData = false;
			FileSystem->DeleteFile(*ResumeDataFilename);
		}
	}

	/**
	 * Checks whether the file was completed during last install attempt and adds it to FilesCompleted if so
	 * @param Filename    The filename to check
	 * @return A flag indicating that the file is incompatible and should be removed and constracted from scratch.
	 */
	bool CheckFile(const FString& Filename)
	{
		// If we had resume data, check if this file might have been resumable
		if (bHasResumeData)
		{
			int64 DiskFileSize;
			const FString FullFilename = StagingDir / Filename;
			const bool bFileExists = FileSystem->GetFileSize(*FullFilename, DiskFileSize);
			const bool bCheckLegacyIds = true;
			TSet<FString> FileResumeIds;
			ManifestSet->GetInstallResumeIdsForFile(Filename, FileResumeIds, bCheckLegacyIds);
			if (LoadedResumeIds.Intersect(FileResumeIds).Num() > 0)
			{
				const FFileManifest* NewFileManifest = ManifestSet->GetNewFileManifest(Filename);
				if (NewFileManifest && bFileExists)
				{
					const uint64 UnsignedDiskFileSize = DiskFileSize;
					if (UnsignedDiskFileSize == NewFileManifest->FileSize)
					{
						FilesStage.Add(Filename, EFileStage::Complete);
					}
					else if (UnsignedDiskFileSize > 0 && UnsignedDiskFileSize < NewFileManifest->FileSize)
					{
						FilesStage.Add(Filename, EFileStage::Incomplete);
					}
					else if (UnsignedDiskFileSize > NewFileManifest->FileSize)
					{
						return true;
					}
					// If the file size is zero, we just treat it as EFileStage::New
				}
			}
			else if (bFileExists)
			{
				return true;
			}
		}
		return false;
	}
};

static FString FormatNumber(uint64 Value) { return FText::AsNumber(Value).ToString(); }


// We need a place to put chunks if we know we're going to need them again after
// their source retires. We don't want to route everything through here. We also
// want to be able to optionally overflow to disk, and ideally this would be persistent
// across resumes so that we don't have to re-download a huge amount of harvested chunks
// from install files.
class FChunkBackingStore : IConstructorChunkSource
{
	FBuildPatchFileConstructor* ParentConstructor = nullptr;
	const FString& InstallDirectory;
	FBuildPatchFileConstructor::FBackingStoreStats& Stats;

	struct FStoredChunk
	{
		TArray<uint8> ChunkData;
		uint32 ChunkSize = 0;
		int32 NextUsageIndex;
		int32 LastUsageIndex;

		// Has the data made it into memory yet?
		bool bCommitted = false;

		// Have we been evicted for memory concerns? We are in UsedEntries if so.
		bool bBackedByDisk = false;

		// When something is reading/writing to us we mark that so we don't
		// evict during async operations.
		uint16 LockCount = 0;
	};

	TMap<FGuid, FStoredChunk> StoredChunks;
	uint64 CurrentMemoryLoad = 0;
	uint64 PeakMemoryLoad = 0;

	int32 ReadCount = 0;
	int32 WriteCount = 0;

	static constexpr uint64 ChunkStoreMemoryLimitDisabledSentinel = TNumericLimits<uint64>::Max();
	uint64 MaxMemoryBytes = 1 << 30; // 1GB.


	// 0 disables
	uint64 MaxDiskSpaceBytes = 1 << 30; // 1GB.
	uint64 AdditionalDiskSpaceHeadroomBytes = 0;
	uint64 InstallationFreeSpaceRequired = 0;

	// The backing store allocates in 128kb chunks.
	static constexpr uint32 BitsPerEntry = 17;
	struct FBackingStoreFreeSpan
	{
		uint32 StartEntryIndex = 0;
		uint32 EndEntryIndex = 0;
		uint64 Size() const { return EntryCount() << BitsPerEntry; }
		uint64 Offset() const { return (uint64)StartEntryIndex << BitsPerEntry; }
		uint64 EntryCount() const { return EndEntryIndex - StartEntryIndex; }
	};
	struct FBackingStoreUsedSpan
	{
		uint32 StartEntryIndex = 0;
		uint32 EndEntryIndex = 0;
		uint32 UsedBytes = 0;
		FXxHash64 Hash;
		uint64 ReservedSize() const { return EntryCount() << BitsPerEntry; }
		uint64 Offset() const { return (uint64)StartEntryIndex << BitsPerEntry; }
		uint64 EntryCount() const { return EndEntryIndex - StartEntryIndex; }
	};

	uint32 BackingStoreMaxEntries = 0;
	uint32 BackingStoreEntryCount = 0;
	uint64 BackingStoreUsedSpace = 0;
	uint64 BackingStoreWastedSpace = 0;
	TMap<FGuid, FBackingStoreUsedSpan> UsedDiskSpans;
	TArray<FBackingStoreFreeSpan> FreeDiskSpans;
	uint64 CurrentDiskLoad() const { return (uint64)(BackingStoreEntryCount) << BitsPerEntry; }

	TUniquePtr<IFileHandle> BackingStoreFileHandle;
	FString BackingStoreFileName;

	void DumpFreeDiskSpans()
	{
		UE_LOGF(LogBuildPatchServices, Display, "Backing Store (Entries / Max): %d / %d", BackingStoreEntryCount, BackingStoreMaxEntries);
		UE_LOGF(LogBuildPatchServices, Display, "Dumping Free Disk Spans...");		

		for (int32 FreeIndex = 0; FreeIndex < FreeDiskSpans.Num(); FreeIndex++)
		{
			UE_LOGF(LogBuildPatchServices, Display, "   %d/%d: %d - %d", FreeIndex, FreeDiskSpans.Num(), FreeDiskSpans[FreeIndex].StartEntryIndex, FreeDiskSpans[FreeIndex].EndEntryIndex);
		}
	}
	// false means consistency failure
	[[nodiscard]] bool ConsistencyCheck()
	{
		bool bSuccess = true;

		uint64 UsedEntryReservedBytes = 0;
		uint64 UsedEntryUsedBytes = 0;
		for (TPair<FGuid, FBackingStoreUsedSpan>& Pair : UsedDiskSpans)
		{
			FBackingStoreUsedSpan& Span = Pair.Value;
			UsedEntryReservedBytes += Span.ReservedSize();
			UsedEntryUsedBytes += Span.UsedBytes;
		}

		uint64 FreeBytes = 0;

		for (int32 FreeIndex = 0; FreeIndex < FreeDiskSpans.Num(); FreeIndex++)
		{
			FBackingStoreFreeSpan& Span = FreeDiskSpans[FreeIndex];

			FreeBytes += Span.Size();

			if (Span.Size() == 0)
			{
				UE_LOGF(LogBuildPatchServices, Error, "BackingStore FreeList Merge Fail: empty entry failed to get deleted");
				bSuccess = false;
			}
			if (Span.EndEntryIndex > BackingStoreEntryCount)
			{
				UE_LOGF(LogBuildPatchServices, Error, "BackingStore FreeList Merge Fail: entry exceeds backing store size");
				bSuccess = false;
			}
			if (Span.StartEntryIndex > Span.EndEntryIndex)
			{
				UE_LOGF(LogBuildPatchServices, Error, "BackingStore FreeList Merge Fail: negative sized entry");
				bSuccess = false;
			}

			if (FreeIndex)
			{
				if (Span.StartEntryIndex == FreeDiskSpans[FreeIndex-1].EndEntryIndex)
				{
					UE_LOGF(LogBuildPatchServices, Error, "BackingStore FreeList Merge Fail: adjacent entries failed to merge");
					bSuccess = false;
				}
				if (Span.StartEntryIndex < FreeDiskSpans[FreeIndex-1].EndEntryIndex)
				{
					UE_LOGF(LogBuildPatchServices, Error, "BackingStore FreeList Merge Fail: adjacent entries overlap or are out of order");
					bSuccess = false;
				}
			}
		}

		if (!bSuccess)
		{
			DumpFreeDiskSpans();
		}

		if (UsedEntryReservedBytes != (BackingStoreUsedSpace + BackingStoreWastedSpace) ||
			UsedEntryUsedBytes != BackingStoreUsedSpace ||
			CurrentDiskLoad() - UsedEntryReservedBytes != FreeBytes)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Disk Backing Store Consistency Fail:");
			UE_LOGF(LogBuildPatchServices, Error, "Actual:");
			UE_LOGF(LogBuildPatchServices, Error, "    ReservedBytes: %ls", *FormatNumber(UsedEntryReservedBytes));
			UE_LOGF(LogBuildPatchServices, Error, "    WastedBytes: %ls", *FormatNumber(UsedEntryReservedBytes - UsedEntryUsedBytes));
			UE_LOGF(LogBuildPatchServices, Error, "    FreeBytes: %ls", *FormatNumber(FreeBytes));
			UE_LOGF(LogBuildPatchServices, Error, "Expected:");
			UE_LOGF(LogBuildPatchServices, Error, "    ReservedBytes: %ls", *FormatNumber(BackingStoreUsedSpace + BackingStoreWastedSpace));
			UE_LOGF(LogBuildPatchServices, Error, "    WastedBytes: %ls", *FormatNumber(BackingStoreWastedSpace));
			UE_LOGF(LogBuildPatchServices, Error, "    FreeBytes: %ls", *FormatNumber(CurrentDiskLoad() - BackingStoreUsedSpace - BackingStoreWastedSpace));
			bSuccess = false;
		}

		return bSuccess;
	}

	// false means consistency or write failure
	// constructor thread
	[[nodiscard]] bool PageOut(const FGuid& InGuid)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BackingStore_PageOut);

		// InChunk must be valid for paging out at this point!
		FStoredChunk& InChunk = StoredChunks[InGuid];

		// It's possible we loaded back into memory but are already backed by disk so we can
		// just free.
		if (InChunk.bBackedByDisk)
		{
			InChunk.ChunkData.Empty();
			CurrentMemoryLoad -= InChunk.ChunkSize;
			ParentConstructor->SetChunkLocation(InGuid, EConstructorChunkLocation::DiskOverflow);
			return true;
		}

		UE_LOGF(LogBuildPatchServices, VeryVerbose, "Paging out: %ls", *WriteToString<40>(InGuid));

		uint32 EntriesRequired = Align(InChunk.ChunkSize, (1 << BitsPerEntry)) >> BitsPerEntry;

		int32 SpanIndex = 0;
		while (SpanIndex < FreeDiskSpans.Num() && 
			FreeDiskSpans[SpanIndex].EntryCount() < EntriesRequired)
		{
			SpanIndex++;
		}

		bool bAppendingToFile = false;
		if (SpanIndex == FreeDiskSpans.Num())
		{
			// Check our disk space limitations.
			uint64 UseMaxDiskSpaceBytes = MaxDiskSpaceBytes;

			// If we have a headroom value, we want to dynamically adjust
			// our max disk space so that we always leave that amount of disk space free.
			// This is expected to almost always be enabled in order to prevent the backing store from
			// eating into space reserved for the actual installation. 
			if (InstallationFreeSpaceRequired || AdditionalDiskSpaceHeadroomBytes)
			{
				uint64 TotalDiskBytes, FreeDiskBytes;
				if (!FPlatformMisc::GetDiskTotalAndFreeSpace(InstallDirectory, TotalDiskBytes, FreeDiskBytes))
				{
					// If we fail to get disk space then disable it since we don't really know what we're doing
					// at that point.
					InstallationFreeSpaceRequired = 0;
					AdditionalDiskSpaceHeadroomBytes = 0;
				}
				else
				{
					// The free space we got is counting any bytes we've already written to disk, so adjust for that.
					uint64 FreeSizeBytesWithoutBackingStore = FreeDiskBytes + CurrentDiskLoad();

					uint64 HeadroomRequiredBytes = InstallationFreeSpaceRequired + AdditionalDiskSpaceHeadroomBytes;

					// By default we aren't allowed any space due to headroom limitations.
					uint64 HeadroomRestrictedMaxDiskSpace = 0;

					// If we have enough space above the headroom, then we can talk.
					if (FreeSizeBytesWithoutBackingStore > HeadroomRequiredBytes)
					{
						HeadroomRestrictedMaxDiskSpace = FreeSizeBytesWithoutBackingStore - HeadroomRequiredBytes;
						if (UseMaxDiskSpaceBytes == 0 || // Note if UseMaxDiskSpaceBytes is 0 then we they have no other limitation.
							HeadroomRestrictedMaxDiskSpace < UseMaxDiskSpaceBytes)
						{
							UseMaxDiskSpaceBytes = HeadroomRestrictedMaxDiskSpace;
						}
					}
				}
			}

			// Need to expand the backing store.
			if (BackingStoreFileHandle == nullptr ||
				(UseMaxDiskSpaceBytes && (((uint64)BackingStoreEntryCount + EntriesRequired) << BitsPerEntry) > UseMaxDiskSpaceBytes))
			{
				// We can't expand - this fails to page out and gets evicted from the backing store.
				Stats.DiskLostChunkCount++;
				ParentConstructor->SetChunkLocation(InGuid, EConstructorChunkLocation::Cloud);
				CurrentMemoryLoad -= InChunk.ChunkSize;
				InChunk.ChunkData.Empty();
				StoredChunks.Remove(InGuid);
				return true;
			}

			FBackingStoreFreeSpan& NewSpan = FreeDiskSpans.AddDefaulted_GetRef();
			NewSpan.StartEntryIndex = BackingStoreEntryCount;
			BackingStoreEntryCount += EntriesRequired;
			if (BackingStoreEntryCount > BackingStoreMaxEntries)
			{
				BackingStoreMaxEntries = BackingStoreEntryCount;
			}
			NewSpan.EndEntryIndex = NewSpan.StartEntryIndex + EntriesRequired;
			bAppendingToFile = true;

			Stats.DiskPeakUsageBytes = CurrentDiskLoad();
		}

		// SpanIndex is the one we just added or found to reuse
		FBackingStoreFreeSpan& FreeSpan = FreeDiskSpans[SpanIndex];
		FBackingStoreUsedSpan& UsedSpan = UsedDiskSpans.FindOrAdd(InGuid);
		if (!UsedSpan.Hash.IsZero())
		{
			// Can't be paging out twice!
			UE_LOGF(LogBuildPatchServices, Error, "Consistency failure: Backing store used entry already existed for %ls", *WriteToString<40>(InGuid));
			return false;
		}

		UsedSpan.StartEntryIndex = FreeSpan.StartEntryIndex;
		UsedSpan.EndEntryIndex = UsedSpan.StartEntryIndex + EntriesRequired;
		UsedSpan.UsedBytes = InChunk.ChunkSize;

		FreeSpan.StartEntryIndex += EntriesRequired;

		// LONGTERM - we should be using XX has for all chunk consistency checking, so we would have this value already.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BSPO_Hash);
			UsedSpan.Hash = FXxHash64::HashBuffer(InChunk.ChunkData.GetData(), InChunk.ChunkData.Num());
		}

		if (FreeSpan.EndEntryIndex == FreeSpan.StartEntryIndex)
		{
			FreeDiskSpans.RemoveAt(SpanIndex);
		}

		// Write
		{
			WriteCount++;

			TRACE_CPUPROFILER_EVENT_SCOPE(BSPO_Write);
			if (!BackingStoreFileHandle->Seek(UsedSpan.Offset()))
			{
				UE_LOGF(LogBuildPatchServices, Error, "Failed to seek disk backing store to %llu", UsedSpan.Offset());
				return false;
			}

			if (!BackingStoreFileHandle->Write(InChunk.ChunkData.GetData(), InChunk.ChunkData.Num()))
			{
				UE_LOGF(LogBuildPatchServices, Error, "Failed to write %u bytes to disk backing store at %llu", InChunk.ChunkData.Num(), UsedSpan.Offset());
				return false;
			}
		}

		uint64 Wastage = UsedSpan.ReservedSize() - InChunk.ChunkData.Num();

		// If we just added to the end of the file, we only wrote the size of the chunk data, not necessarily
		// the size of our reservation, so top off with zeroes.
		if (bAppendingToFile)
		{
			// This is at most a 128kb allocation (1 << BitsPerEntry), but the chunk size currently in use
			// is 3 bytes shy of being perfectly aligned to a multiple of that meaning we only expect to need to write
			// 3 bytes of zeroes. So we make sure we have enough space on the stack for that.
			TArray<uint8, TInlineAllocator<128>> Zeroes;
			Zeroes.AddZeroed(Wastage);
			if (!BackingStoreFileHandle->Write(Zeroes.GetData(), Zeroes.Num()))
			{
				UE_LOGF(LogBuildPatchServices, Error, "Failed to write %u bytes of zeroes to disk backing store at %llu", Zeroes.Num(), UsedSpan.Offset());
				return false;
			}
		}

		// Done.
		CurrentMemoryLoad -= InChunk.ChunkSize;
		BackingStoreUsedSpace += InChunk.ChunkSize;
		BackingStoreWastedSpace += Wastage;
		InChunk.ChunkData.Empty();
		InChunk.bBackedByDisk = true;
		ParentConstructor->SetChunkLocation(InGuid, EConstructorChunkLocation::DiskOverflow);

		return ConsistencyCheck();
	}

	// false means consistency failure
	[[nodiscard]] bool ReleaseBackingStoreEntry(const FGuid& InGuid, FStoredChunk& InChunk)
	{
		// Return the bits back to the free list.
		FBackingStoreFreeSpan NewFreeSpan;
		int32 FreeSpanIndex;
		{
			FBackingStoreUsedSpan& UsedEntry = UsedDiskSpans[InGuid];

			BackingStoreUsedSpace -= InChunk.ChunkSize;
			BackingStoreWastedSpace -= UsedEntry.ReservedSize() - InChunk.ChunkSize;

			FreeSpanIndex = Algo::LowerBoundBy(FreeDiskSpans, UsedEntry.StartEntryIndex, &FBackingStoreFreeSpan::StartEntryIndex);
			
			NewFreeSpan.StartEntryIndex = UsedEntry.StartEntryIndex;
			NewFreeSpan.EndEntryIndex = UsedEntry.EndEntryIndex;
		
			InChunk.bBackedByDisk = false;

			UsedDiskSpans.Remove(InGuid);
		}

		// Merge into an adjacent entry without adding and having to do a linear
		// pass to coalesce. 
		bool bMerged = false;
		if (FreeSpanIndex < FreeDiskSpans.Num())
		{
			if (NewFreeSpan.EndEntryIndex == FreeDiskSpans[FreeSpanIndex].StartEntryIndex)
			{
				// We are right before to the one after us - just extend them lower
				FreeDiskSpans[FreeSpanIndex].StartEntryIndex = NewFreeSpan.StartEntryIndex;
				bMerged = true;

				// OK they got merged down, see if they can connect with the one below
				if (FreeSpanIndex > 0)
				{
					if (FreeDiskSpans[FreeSpanIndex - 1].EndEntryIndex == FreeDiskSpans[FreeSpanIndex].StartEntryIndex)
					{
						// We fill a gap, we can connect and remove.
						FreeDiskSpans[FreeSpanIndex - 1].EndEntryIndex = FreeDiskSpans[FreeSpanIndex].EndEntryIndex;
						FreeDiskSpans.RemoveAt(FreeSpanIndex);
					}
				}
			}
		}
		
		if (!bMerged && FreeSpanIndex > 0) // if we missed, this will be Num(). If we have nothing yet it'll fail this and be fine.
		{
			if (FreeDiskSpans[FreeSpanIndex - 1].EndEntryIndex == NewFreeSpan.StartEntryIndex)
			{
				// We are right after the one before us, extend them farther.
				FreeDiskSpans[FreeSpanIndex - 1].EndEntryIndex = NewFreeSpan.EndEntryIndex;
				bMerged = true;

				// They got merged up, see if we filled a gap.
				if (FreeSpanIndex < FreeDiskSpans.Num())
				{
					if (FreeDiskSpans[FreeSpanIndex - 1].EndEntryIndex == FreeDiskSpans[FreeSpanIndex].StartEntryIndex)
					{
						FreeDiskSpans[FreeSpanIndex - 1].EndEntryIndex = FreeDiskSpans[FreeSpanIndex].EndEntryIndex;
						FreeDiskSpans.RemoveAt(FreeSpanIndex);
					}
				}
			}
		}

		// If we didn't merge, we need to insert
		if (!bMerged)
		{
			FreeDiskSpans.Insert(NewFreeSpan, FreeSpanIndex);
		}

		// Check and see if the free space is at the end of the file. If so, we can truncate
		// and free up disk space. 
		if (FreeDiskSpans.Top().EndEntryIndex == BackingStoreEntryCount)
		{
			uint64 TruncateToSize = (uint64)FreeDiskSpans.Top().StartEntryIndex << BitsPerEntry;
			if (BackingStoreFileHandle->Truncate(TruncateToSize)) // truncate can fail... don't update structures if we can't!
			{
				BackingStoreEntryCount = FreeDiskSpans.Top().StartEntryIndex;
				FreeDiskSpans.Pop();
			}
		}

		return ConsistencyCheck();
	}

	[[nodiscard]] bool ReleaseEntryInternal(const FGuid& InGuid, FStoredChunk* InStoredChunk)
	{
		if (InStoredChunk->LockCount == 0)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Releasing memory entry that isn't locked! %ls", *WriteToString<40>(InGuid));
			return false;
		}
		else
		{
			InStoredChunk->LockCount--;
			return true;
		}
	}

public:

	FChunkBackingStore(FBuildPatchFileConstructor* InParentConstructor, const FString& InInstallDirectory, FBuildPatchFileConstructor::FBackingStoreStats& InStats) 
	: ParentConstructor(InParentConstructor)
	, InstallDirectory(InInstallDirectory)
	, Stats(InStats)
	{
		BackingStoreFileName = InParentConstructor->Configuration.BackingStoreDirectory / TEXT("backingstore");

		bool bUseDiskOverflowStore = true;
		GConfig->GetBool(TEXT("BuildPatchServices"), TEXT("bEnableDiskOverflowStore"), bUseDiskOverflowStore, GEngineIni);

		if (ParentConstructor->Configuration.bInstallToMemory)
		{
			UE_LOGF(LogBuildPatchServices, Display, "Disabling backing store due to InstallToMemory");
			bUseDiskOverflowStore = false;
		}
		
		// Is there a hard limit on how much disk space we can use?
		// negative = no
		// 0 = disable disk overflow
		int64 MaxDiskMB = 0;
		GConfig->GetInt64(TEXT("BuildPatchServices"), TEXT("DiskOverflowStoreLimitMB"), MaxDiskMB, GEngineIni);
		if (MaxDiskMB < 0)
		{
			MaxDiskSpaceBytes = TNumericLimits<int64>::Max();
		}
		else
		{
			MaxDiskSpaceBytes = MaxDiskMB << 20;
		}

		// Do we want to always try and keep some disk space available, no matter what our limit is?
		// note that independent of this we try and prevent the disk backing store from eating into space
		// we have reserved for the actual install so they don't compete.
		//
		// this checks the free space after each file and updates the space limit correspondingly.
		int64 AdditionalDiskSpaceHeadroomMB = 0;
		GConfig->GetInt64(TEXT("BuildPatchServices"), TEXT("DiskOverflowStoreAdditionalHeadroomMB"), AdditionalDiskSpaceHeadroomMB, GEngineIni);
		if (AdditionalDiskSpaceHeadroomMB >= 0)
		{
			AdditionalDiskSpaceHeadroomBytes = AdditionalDiskSpaceHeadroomMB << 20;
		}

		if (bUseDiskOverflowStore)
		{
			IFileManager::Get().MakeDirectory(*InParentConstructor->Configuration.BackingStoreDirectory, true);
			BackingStoreFileHandle.Reset(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*BackingStoreFileName, false, true));
		}

		if (!BackingStoreFileHandle)
		{
			// Prevent any pageouts.
			UE_CLOGF(bUseDiskOverflowStore, LogBuildPatchServices, Warning, "Unable to open disk backing store at %ls", *BackingStoreFileName);
			UE_LOGF(LogBuildPatchServices, Warning, "Disk backing store will be disabled");
			MaxDiskSpaceBytes = 0;
		}

		UE_LOGF(LogBuildPatchServices, Display, "DiskOverflowStore is: %ls - MaxSize = %ls, Additional Headroom = %ls", 
			BackingStoreFileHandle ? TEXT("Enabled") : TEXT("Disabled"),
			*FormatNumber(MaxDiskSpaceBytes),
			*FormatNumber(AdditionalDiskSpaceHeadroomBytes)
			);

		
		// Now memory limits.
		uint64 ChunkStoreMemoryLimit = 0;

		// Check old values and warn/assume
		{
			int32 ChunkStoreMemorySizeChunks;
			const bool bLoadedStoreSize = GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkStoreMemorySize"), ChunkStoreMemorySizeChunks, GEngineIni);
			if (bLoadedStoreSize)
			{
				UE_LOGF(LogBuildPatchServices, Warning, "Outdated memory size limitation found: ChunkStoreMemorySize. Assuming chunk size is 1MB, use ChunkStoreMemorySizeMB instead.");
				ChunkStoreMemoryLimit = (uint64)FMath::Max(ChunkStoreMemorySizeChunks, 0) << 20;
			}

			int32 CloudChunkStoreMemorySizeChunks = 0;
			int32 InstallChunkStoreMemorySizeChunks = 0;
			const bool bLoadedCloudSize = GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("CloudChunkStoreMemorySize"), CloudChunkStoreMemorySizeChunks, GEngineIni);
			const bool bLoadedInstallSize = GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("InstallChunkStoreMemorySize"), InstallChunkStoreMemorySizeChunks, GEngineIni);
			if (bLoadedCloudSize || bLoadedInstallSize)
			{
				UE_LOGF(LogBuildPatchServices, Warning, "Outdated memory size limitations found: CloudChunkStoreMemorySize or InstallChunkStoreMemorySize. Assuming chunk size is 1MB.");
				UE_LOGF(LogBuildPatchServices, Warning, "Use ChunkStoreMemorySizeMB and/or ChunkStoreMemoryHeadRoomMB.");
				uint64 OldMemoryLimit = FMath::Max(CloudChunkStoreMemorySizeChunks + InstallChunkStoreMemorySizeChunks, 0);
				OldMemoryLimit <<= 20;
				ChunkStoreMemoryLimit = OldMemoryLimit;
			}
		}

		

		// check current values. We expect this to override anything from above.
		int32 ChunkStoreMemoryLimitMB = 0;
		if (GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkStoreMemorySizeMB"), ChunkStoreMemoryLimitMB, GEngineIni))
		{
			// To be consistent with other limitations, negative disables it
			// 0 is OK - we require locked data to be in memory so we'll go over the limit but it'll be a minimum.
			if (ChunkStoreMemoryLimitMB < 0)
			{
				ChunkStoreMemoryLimit = ChunkStoreMemoryLimitDisabledSentinel;
			}
			else
			{
				ChunkStoreMemoryLimit = (uint64)ChunkStoreMemoryLimitMB << 20;
			}
		}
		

		// Get headroom. Default to 2GB of headroom. If no config is entered then we expect a 0 chunk limit that
		// gets updated off of this default headroom.
		int32 ChunkStoreMemoryHeadRoomMB = 2000;
		if (GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkStoreMemoryHeadRoomMB"), ChunkStoreMemoryHeadRoomMB, GEngineIni))
		{
			if (ChunkStoreMemoryHeadRoomMB < 0)
			{
				// negative disables
				ChunkStoreMemoryHeadRoomMB = -1;
			}
			else if (ChunkStoreMemoryHeadRoomMB < 500)
			{
				UE_LOGF(LogBuildPatchServices, Warning, "ChunkStoreMemoryHeadRoomMB too low (%d), using min (500)", ChunkStoreMemoryHeadRoomMB);
				ChunkStoreMemoryHeadRoomMB = 500;
			}
		}		



		FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
		uint64 AvailableMem = MemoryStats.AvailablePhysical;
		if (ChunkStoreMemoryHeadRoomMB >= 0)
		{
			uint64 RequestedHeadRoom = ((uint64)ChunkStoreMemoryHeadRoomMB << 20);

			uint64 ProposedChunkStoreMemoryLimit = ChunkStoreMemoryLimit;

			if (RequestedHeadRoom < AvailableMem)
			{
				uint64 MemoryStoreMem = AvailableMem - RequestedHeadRoom;
				ProposedChunkStoreMemoryLimit = MemoryStoreMem;
			}
			else
			{
				// Cap at available.
				if (ProposedChunkStoreMemoryLimit > AvailableMem)
				{
					ProposedChunkStoreMemoryLimit = AvailableMem;
				}
			}

			// If there's already a limit requested by the inis, we don't want to make it _smaller_. If there's no
			// limit specified, then use the proposed limit.
			if (ChunkStoreMemoryLimit != ChunkStoreMemoryLimitDisabledSentinel)
			{
				ChunkStoreMemoryLimit = FMath::Max(ChunkStoreMemoryLimit, ProposedChunkStoreMemoryLimit);
			}
			else
			{
				ChunkStoreMemoryLimit = ProposedChunkStoreMemoryLimit;
			}
		}

		if (ChunkStoreMemoryLimit == ChunkStoreMemoryLimitDisabledSentinel)
		{
			UE_LOGF(LogBuildPatchServices, Display, "ChunkStoreMemoryLimits are disabled");
		}
		else
		{
			UE_LOGF(LogBuildPatchServices, Display, "ChunkStoreMemoryLimits: %ls using headroom of %ls (%ls available memory, %ls used physical, %ls used virtual)", 
				*FormatNumber(ChunkStoreMemoryLimit),
				ChunkStoreMemoryHeadRoomMB >= 0 ? *FormatNumber((uint64)ChunkStoreMemoryHeadRoomMB << 20) : TEXT("<disabled>"),
				*FormatNumber(AvailableMem),
				*FormatNumber(MemoryStats.UsedPhysical),
				*FormatNumber(MemoryStats.UsedVirtual)
			);
		}

		MaxMemoryBytes = ChunkStoreMemoryLimit;
		Stats.MemoryLimitBytes = MaxMemoryBytes;
	}

	~FChunkBackingStore()
	{
		if (BackingStoreFileHandle.IsValid())
		{
			BackingStoreFileHandle.Reset(); // have to close the file before we try to delete it.
			if (!FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*BackingStoreFileName))
			{
				UE_LOGF(LogBuildPatchServices, Warning, "Unable to delete disk backing store: %ls", *BackingStoreFileName);
			}
			UE_LOGF(LogBuildPatchServices, Verbose, "BackingStore Reads: %d Writes: %d", ReadCount, WriteCount);
		}
	}

	// Set the amount of disk space the installation needs to we can ensure that we don't 
	// expand into that space no matter what our config disk space limits are.
	void SetDynamicDiskSpaceHeadroom(uint64 InInstallationFreeSpaceRequired)
	{
		InstallationFreeSpaceRequired = InInstallationFreeSpaceRequired;
	}

	// false means consistency failure
	// Constructor thread
	[[nodiscard]] bool DereserveHarvestingEntry(const FGuid& InGuid)
	{
		FStoredChunk* StoredChunk = StoredChunks.Find(InGuid);
		if (StoredChunk)
		{
			if (StoredChunk->bCommitted || StoredChunk->bBackedByDisk)
			{
				UE_LOGF(LogBuildPatchServices, Error, 
					"Consistency Failure: deserve memory entry that's uncommitted or paged out! %ls, paged out = %d committed = %d", 
					*WriteToString<40>(InGuid), StoredChunk->bBackedByDisk, StoredChunk->bCommitted);
				return false;
			}

			CurrentMemoryLoad -= StoredChunk->ChunkData.Num();
			StoredChunks.Remove(InGuid);
		}
		else
		{
			UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Cleared memory entry that doesn't exist! %ls", *WriteToString<40>(InGuid));
			return false;
		}
		return true;
	}

	// false means consistency failure
	// Constructor thread
	[[nodiscard]] bool ReleaseEntry(const FGuid& InGuid)
	{
		FStoredChunk* StoredChunk = StoredChunks.Find(InGuid);
		if (StoredChunk)
		{
			return ReleaseEntryInternal(InGuid, StoredChunk);
		}
		else
		{
			UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Releasing memory entry that doesn't exist! %ls", *WriteToString<40>(InGuid));
			return false;
		}
	}

	// false means consistency failure
	// Constructor thread
	[[nodiscard]] bool CommitAndReleaseEntry(const FGuid& InGuid)
	{
		FStoredChunk* StoredChunk = StoredChunks.Find(InGuid);
		if (StoredChunk)
		{
			if (StoredChunk->bCommitted)
			{
				UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Committing memory entry that is already committed! %ls", *WriteToString<40>(InGuid));
				return false;
			}
			StoredChunk->bCommitted = true;
			return ReleaseEntryInternal(InGuid, StoredChunk);
		}
		else
		{
			UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Committed memory entry that doesn't exist! %ls", *WriteToString<40>(InGuid));
			return false;
		}
	}

	// false means consistency failure
	// Constructor thread
	[[nodiscard]] bool LockEntry(const FGuid& InGuid)
	{
		FStoredChunk* StoredChunk = StoredChunks.Find(InGuid);
		if (!StoredChunk)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: locking memory entry that doesn't exist! %ls", *WriteToString<40>(InGuid));
			return false;
		}
		else
		{
			StoredChunk->LockCount++;
		}
		return true;
	}

	// Returns an empty view on consistency failure
	// Constructor thread.
	[[nodiscard]] FMutableMemoryView ReserveAndLockEntry(const FGuid& InGuid, uint32 InChunkSize, int32 LastUsageIndex)
	{
		// Note that this function can be called to reserve an entry already in the backing store
		// because we need to read from the disk to memory for a sub-chunk (or otherwise). So it needs
		// to be able to handle reserving on top of paged out chunks.

		int32 CurrentUsageIndex = ParentConstructor->ChunkReferenceTracker->GetCurrentUsageIndex();

		if (MaxMemoryBytes != ChunkStoreMemoryLimitDisabledSentinel)
		{
			while (CurrentMemoryLoad + InChunkSize > MaxMemoryBytes)
			{
				// Gotta dump stuff to disk. If we fail to dump to disk we mark the chunk
				// as only available via the cloud source.

				// Evict the one that is the longest until we use it. Chunks are almost always the same size
				// so we expect this to run once.
				int32 FarthestNextUsage = -1;
				FGuid EvictGuid;
				for (TPair<FGuid, FStoredChunk>& Pair : StoredChunks)
				{
					FStoredChunk& Chunk = Pair.Value;

					if (Chunk.LockCount || // Actively in use - don't evict.
						!Chunk.ChunkData.Num()) // Not in memory - can't evict.
					{					
						continue;
					}

					if (Chunk.NextUsageIndex < CurrentUsageIndex)
					{
						Chunk.NextUsageIndex = ParentConstructor->ChunkReferenceTracker->GetNextUsageForChunk(Pair.Key, Chunk.LastUsageIndex);
					}

					if (Chunk.NextUsageIndex > FarthestNextUsage)
					{
						FarthestNextUsage = Pair.Value.NextUsageIndex;
						EvictGuid = Pair.Key;
					}
				}

				if (FarthestNextUsage == -1)
				{
					// This means we can't reserve and also keep our memory requirements. 
					//
					// We currently have a minimum memory requirement for construction: Partial or reused chunks
					// are routed through the backing store for holding. Partial because we need a full allocation to decompress
					// the chunk and reuse so we don't have to re-read from disk.
					//
					// For unpatched installs this is minimal as most chunks are able to write directly to
					// the destination buffer. For optimized delta manifests this is not the case as BPT assembles
					// chunks from all over the place - I've seen 80+ chunk references to assemble 16MB of data,
					// resulting in total buffer usage of 80MB + 16MB > 100MB of memory use per batch. With two
					// batches in flight this pushes 200MB total buffer allocation.
					//
					// This is not likely an issue in real life as 200MB isn't much, but it does mean if we have a 
					// low memory usage limit we can hit this legitimately.
					//
					// So...we let the reservation continue in violation of our memory constraints and hope we don't OOM.
					//
					// Future work could be:
					// 1. Limit batch creation based on inflight chunk in addition to write buffer size
					// 2. Only allow 1 batch in flight when using an optimized delta + low memory constraints.
					break;
				}
				else
				{
					if (!PageOut(EvictGuid))
					{
						return FMutableMemoryView();
					}
				}
			} // end while over limits
		} // end if memory limits enabled
		FStoredChunk& StoredChunk = StoredChunks.FindOrAdd(InGuid);

		//
		// It's possible to request a reservation for a chunk already in the backing store when reading from the
		// disk into memory (i.e. with a non-direct read). However it can't already be in memory.
		//
		if (StoredChunk.ChunkData.Num())
		{
			UE_LOGF(LogBuildPatchServices, Error, "Consistency failure: Reserving read space for a chunk already in memory: %ls", *WriteToString<40>(InGuid));
			return FMutableMemoryView();
		}
		else
		{
			StoredChunk.LastUsageIndex = LastUsageIndex;
			StoredChunk.bCommitted = false;
			StoredChunk.ChunkData.AddUninitialized(InChunkSize);
			StoredChunk.ChunkSize = InChunkSize;
			StoredChunk.LockCount = 1;
			CurrentMemoryLoad += InChunkSize;
			if (CurrentMemoryLoad > PeakMemoryLoad)
			{
				PeakMemoryLoad = CurrentMemoryLoad;
				Stats.MemoryPeakUsageBytes = PeakMemoryLoad;
			}
		}

		FMutableMemoryView ChunkBuffer(StoredChunk.ChunkData.GetData(), StoredChunk.ChunkData.Num());
		return ChunkBuffer;
	}
	
	// false means consistency failure
	// Constructor thread
	[[nodiscard]] bool CheckRetirements(int32 CurrentUsageIndex)
	{
		// 6 just chosen because we wouldn't expect a ton of things at once
		// but we could get several. shrug.
		TArray<FGuid, TInlineAllocator<6>> GuidsToDelete;

		for (TPair<FGuid, FStoredChunk>& Pair : StoredChunks)
		{
			if (Pair.Value.LastUsageIndex < CurrentUsageIndex)
			{
				GuidsToDelete.Add(Pair.Key);
			}
		}

		for (FGuid& ToDelete : GuidsToDelete)
		{
			FStoredChunk* Chunk = StoredChunks.Find(ToDelete);
			if (Chunk->bBackedByDisk)
			{
				if (!ReleaseBackingStoreEntry(ToDelete, *Chunk))
				{
					return false;
				}
			}

			if (Chunk->LockCount)
			{
				UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Retiring memory entry with lock count! %ls %d", *WriteToString<40>(ToDelete), Chunk->LockCount);
				return false;
			}
			
			if (Chunk->ChunkData.Num())
			{
				if (!Chunk->bCommitted)
				{
					UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Retiring memory entry that never got committed! %ls", *WriteToString<40>(ToDelete));
					return false;
				}

				CurrentMemoryLoad -= Chunk->ChunkSize;
			}

			StoredChunks.Remove(ToDelete);

			ParentConstructor->SetChunkLocation(ToDelete, EConstructorChunkLocation::Retired);
		}

		return true;
	}
	
	// This should only be for paged out chunks... in-memory chunks should be handled directly.
	// Consistency failures in this will pass as data read fails, which will end up redirecting to
	// the cloud. However the post-file consistency check will catch it and fail the install.
	// Constructor thread only for this chunk source
	[[nodiscard]] FRequestProcessFn CreateRequest(const FGuid& DataId, FMutableMemoryView DestinationBuffer, void* UserPtr, FChunkRequestCompleteDelegate CompleteFn)
	{
		// LONGTERM - can we make this fully async once we have a pipe we can prevent reentrancy on or ReadAtOffset API?
		TRACE_CPUPROFILER_EVENT_SCOPE(BackingStoreRead);

		bool bSuccess = true;
		// This should go to a generate IO dispatch with completion function I think? idk...
		// This acts as any other IO source - we don't know where it's going, it might be going back into memory.
		FStoredChunk* StoredChunk = StoredChunks.Find(DataId);
		bSuccess = StoredChunk != nullptr;

		if (bSuccess && !StoredChunk->bBackedByDisk)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Trying to page in a chunk that isn't paged out! %ls", *WriteToString<40>(DataId));
			bSuccess = false;
		}

		FBackingStoreUsedSpan* UsedEntry = nullptr;
		if (bSuccess)
		{
			UsedEntry = UsedDiskSpans.Find(DataId);
			if (!UsedEntry)
			{
				UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Backing store entry not found for paged out chunk! %ls", *WriteToString<40>(DataId));
				bSuccess = false;
			}
		}

		if (bSuccess)
		{
			ReadCount++;
			BackingStoreFileHandle->Seek(UsedEntry->Offset());
			if (!BackingStoreFileHandle->Read((uint8*)DestinationBuffer.GetData(), DestinationBuffer.GetSize()))
			{
				UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Backing store page-in failed read! %ls", *WriteToString<40>(DataId));
				bSuccess = false;
			}
		}

		if (bSuccess)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BSR_Hash);
			if (FXxHash64::HashBuffer(DestinationBuffer) != UsedEntry->Hash)
			{
				UE_LOGF(LogBuildPatchServices, Error, "Consistency Failure: Backing store page-in failed hash check! %ls", *WriteToString<40>(DataId));
				bSuccess = false;
			}
		}

		if (!bSuccess)
		{
			Stats.DiskLoadFailureCount++;
		}

		Stats.DiskChunkLoadCount++;
		CompleteFn.Execute(DataId, false, !bSuccess, UserPtr);
		return [](bool) { return; };
	}

	// false means consistency failure
	// Constructor thread
	[[nodiscard]] bool CheckNoLocks(bool bIsHarvest)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BackingStore_CheckNoLocks);
		// After a file there should be no locked chunks since there are no reads.
		bool bSuccess = true;
		for (TPair<FGuid, FStoredChunk>& Chunk : StoredChunks)
		{
			if (Chunk.Value.LockCount)
			{
				UE_LOGF(LogBuildPatchServices, Error, "Chunk %ls locked with count %d after %ls!", *WriteToString<40>(Chunk.Key), Chunk.Value.LockCount, bIsHarvest ? TEXT("harvest") : TEXT("file completion"));
				bSuccess = false;
			}
		}

		if (bSuccess)
		{
			bSuccess = ConsistencyCheck();
		}

		return bSuccess;
	}

	// This is for the disk store - we don't free the page entry until it retires, so it's always available and can
	// be read direct.
	virtual int32 GetChunkUnavailableAt(const FGuid& DataId) const override { return TNumericLimits<int32>::Max(); }

	FMemoryView GetViewForChunk(const FGuid& DataId) const
	{
		FStoredChunk const* StoredChunk = StoredChunks.Find(DataId);
		if (!StoredChunk)
		{
			return FMemoryView();
		}

		return FMemoryView(StoredChunk->ChunkData.GetData(), StoredChunk->ChunkData.Num());
	}
}; // FChunkBackingStore

/* FBuildPatchFileConstructor implementation
 *****************************************************************************/
FBuildPatchFileConstructor::FBuildPatchFileConstructor(
	FFileConstructorConfig InConfiguration, IFileSystem* InFileSystem, 
	IConstructorChunkDbChunkSource* InChunkDbChunkSource, IConstructorCloudChunkSource* InCloudChunkSource, IConstructorInstallChunkSource* InInstallChunkSource, 
	IChunkReferenceTracker* InChunkReferenceTracker, IInstallerError* InInstallerError, IInstallerAnalytics* InInstallerAnalytics, IMessagePump* InMessagePump,
	IFileConstructorStat* InFileConstructorStat, TMap<FGuid, EConstructorChunkLocation>&& InChunkLocations)
	: Configuration(MoveTemp(InConfiguration))
	, bIsDownloadStarted(false)
	, bInitialDiskSizeCheck(false)
	, bIsPaused(false)
	, bShouldAbort(false)
	, FileSystem(InFileSystem)
	, ChunkDbSource(InChunkDbChunkSource)
	, InstallSource(InInstallChunkSource)
	, CloudSource(InCloudChunkSource)
	, ChunkLocations(MoveTemp(InChunkLocations))
	, ChunkReferenceTracker(InChunkReferenceTracker)
	, InstallerError(InInstallerError)
	, InstallerAnalytics(InInstallerAnalytics)
	, MessagePump(InMessagePump)
	, FileConstructorStat(InFileConstructorStat)
	, TotalJobSize(0)
	, ByteProcessed(0)
	, RequiredDiskSpace(0)
	, AvailableDiskSpace(0)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FileConstructor_ctor);

	bStallWhenFileSystemThrottled = bCVarStallWhenFileSystemThrottled;
	if (Configuration.StallWhenFileSystemThrottled.IsSet())
	{
		UE_LOGF(LogBuildPatchServices, Display, "Overridding StallWhenFileSystemThrottled to: %d, cvar was %d", Configuration.StallWhenFileSystemThrottled.GetValue(), bStallWhenFileSystemThrottled);
		bStallWhenFileSystemThrottled = Configuration.StallWhenFileSystemThrottled.GetValue();
	}

	bAllowMultipleFilesInFlight = bCVarAllowMultipleFilesInFlight;

	BackingStore.Reset(new FChunkBackingStore(this, Configuration.InstallDirectory, BackingStoreStats));

	// Count initial job size
	ConstructionList.Reserve(Configuration.ConstructList.Num());
	
	// Track when we will complete files in the reference chain.
	int32 CurrentPosition = 0;
	FileCompletionPositions.Reserve(Configuration.ConstructList.Num());

	// The first index after the file is complete.
	// \todo with the construction list now stable across the install, we could
	// key this off a StringView and save the allocs. For another CL...
	TMap<FString, int32> FileRetirementPositions;

	for (const FString& FileToConstructName : Configuration.ConstructList)
	{
		const FFileManifest* FileManifest = Configuration.ManifestSet->GetNewFileManifest(FileToConstructName);

		FFileToConstruct FileToConstruct;
		FileToConstruct.FileManifest = FileManifest;

		// If we are missing the file manifest, we will fail to install when we get to the file. However,
		// we guarantee a 1:1 mapping with the arrays we are filling here so we use invalid data for those
		// slots (which won't get used).
		// Maybe we should fail immediately? Need to review whether we can fail in the constructor or we need
		// to delay until Run().
		if (FileManifest)
		{
			if (Configuration.bUseSHA1StageFilenames)
			{
				FBuildPatchUtils::SHAToBase32(FileManifest->SHA1Hash, FileToConstruct.StagingFilename);
			}
			else
			{
				FileToConstruct.StagingFilename = FileToConstructName;
			}
			TotalJobSize += FileManifest->FileSize;

			// We will be advancing the chunk reference tracker by this many chunks.
			int32 AdvanceCount = FileManifest->ChunkParts.Num();
			CurrentPosition += AdvanceCount;
		}

		FileCompletionPositions.Add(CurrentPosition);
		FileRetirementPositions.Add(FileToConstructName, CurrentPosition);
		ConstructionList.Add(MoveTemp(FileToConstruct));
	}

	// Let the install source know when we're going to be deleting their sources.
	bool bHasInstallSource = false;
	if (InstallSource)
	{
		InstallSource->SetFileRetirementPositions(MoveTemp(FileRetirementPositions));	

		if (InstallSource->GetAvailableChunks().Num() != 0)
		{
			bHasInstallSource = true;

			// We need to set up a dependency chain so that files can know when they can start. We can't start constructing a file
			// that needs parts from a file that isn't done as the patch authoring tool expects the file to be done.
			//
			// There are a couple major concerns here:
			//	1. If a later file requires a chunk from an earlier file, the file will have been deleted, so we Harvest the
			//		chunks. We can't start the file until that harvest completes. This dependency checking prevents this.
			//	2. When harvesting chunks from a single file, the chunk we harvest could come from multiple files, including ones
			//		in flight. Rather than track such dependencies here we provide file access locking inside the install source.
			TMap<FStringView, int32> FileToIndexMap;
			FileToIndexMap.Reserve(Configuration.ConstructList.Num());
			for (int32 FileConstructIndex = 0; FileConstructIndex < Configuration.ConstructList.Num(); FileConstructIndex++)
			{
				// The construct list has filenames that match the manifest, and afaict manifest filenames are already normalized.
				FileToIndexMap.Add(Configuration.ConstructList[FileConstructIndex], FileConstructIndex);
			}

			for (int32 FileConstructIndex = 0; FileConstructIndex < Configuration.ConstructList.Num(); FileConstructIndex++)
			{
				FFileToConstruct& FileToConstruct = ConstructionList[FileConstructIndex];
				if (!FileToConstruct.FileManifest)
				{
					continue;
				}

				for (const FChunkPart& ChunkPart : FileToConstruct.FileManifest->ChunkParts)
				{
					EConstructorChunkLocation ChunkLocation = ChunkLocations[ChunkPart.Guid];
					if (ChunkLocation == EConstructorChunkLocation::Install)
					{
						InstallSource->EnumerateFilesForChunk(ChunkPart.Guid, [FileConstructIndex, &FileToIndexMap, &FileToConstruct, NormalizedInstallDirectory = &Configuration.InstallDirectory](const FString& ChunkNormalizedInstallDirectory, const FString& NormalizedFilenameContainingChunk)
							{
								if (*NormalizedInstallDirectory != ChunkNormalizedInstallDirectory)
								{
									// We aren't affecting that install source so it's not a dependency.
									return;
								}

								const int32* DependentFileIndexPtr = FileToIndexMap.Find(NormalizedFilenameContainingChunk);
								if (DependentFileIndexPtr)
								{
									int32 DependentFileIndex = *DependentFileIndexPtr;

									// If the file is constructed before us then we can't start until it's ready.
									// We only care about the latest file.
									if (DependentFileIndex < FileConstructIndex)
									{
										FileToConstruct.LatestDependentInstallSource = FMath::Max(FileToConstruct.LatestDependentInstallSource, DependentFileIndex);
									}
								}
							}
						);
					}
				}

				if (FileToConstruct.LatestDependentInstallSource >= 0)
				{
					UE_LOGF(LogBuildPatchServices, Display, "File: %ls can't start until %ls finishes", 
						*Configuration.ConstructList[FileConstructIndex],
						*Configuration.ConstructList[FileToConstruct.LatestDependentInstallSource]);
				}
			}
		}
	}

	//
	// Create the threads we are allowed to.
	//
	bHasChunkDbSource = ChunkDbSource->GetAvailableChunks().Num() != 0;
	
	// Default everything to running synchronously.
	for (int8& ThreadAssignment : ThreadAssignments)
	{
		ThreadAssignment = -1;
	}
	WriteThreadIndex = -1;

	bool bSpawnAdditionalIOThreads = Configuration.bDefaultSpawnAdditionalIOThreads;
	if (GConfig->GetBool(TEXT("Portal.BuildPatch"), TEXT("ConstructorSpawnAdditionalIOThreads"), bSpawnAdditionalIOThreads, GEngineIni))
	{
		UE_LOGF(LogBuildPatchServices, Verbose, "Got INI ConstructorSpawnAdditionalIOThreads = %d", bSpawnAdditionalIOThreads);
	}

	if (Configuration.SpawnAdditionalIOThreads.IsSet())
	{
		bSpawnAdditionalIOThreads = Configuration.SpawnAdditionalIOThreads.Get(bSpawnAdditionalIOThreads);
		UE_LOGF(LogBuildPatchServices, Verbose, "Got override ConstructorSpawnAdditionalIOThreads = %d", bSpawnAdditionalIOThreads);
	}

	// For now we have to strictly assign jobs to threads so that we don't accidentally
	// hit the same file handle on multiple threads. Once we have proper ReadAtOffset support
	// we can go nuts (and just use UE::Tasks)
	// LONGTERM try using UE::Pipe and just blasting everything on tasks?
	int32 ThreadCount = 0;
	if (bSpawnAdditionalIOThreads)
	{
		if (!Configuration.bInstallToMemory && !Configuration.bConstructInMemory)
		{
			WriteThreadIndex = 0;
			ThreadCount++;
		}

		if (bHasInstallSource)
		{
			ThreadAssignments[EConstructorChunkLocation::Install] = ThreadCount;
			ThreadCount++;
		}
		if (bHasChunkDbSource)
		{
			ThreadAssignments[EConstructorChunkLocation::ChunkDb] = ThreadCount;
			ThreadCount++;
		}
	}
	
	WakeUpDispatchThreadEvent = FPlatformProcess::GetSynchEventFromPool();
	CloudSource->SetWakeupFunction([WakeUpDispatchThreadEvent = WakeUpDispatchThreadEvent]()
		{
			WakeUpDispatchThreadEvent->Trigger();
		}
	);

	// Preallocate the arrays so we don't get any movement.
	Threads.SetNum(ThreadCount);
	ThreadWakeups.SetNum(ThreadCount);
	ThreadCompleteEvents.SetNum(ThreadCount);
	ThreadJobPostingLocks.SetNum(ThreadCount); // Default init is fine for these.
	ThreadJobPostings.SetNum(ThreadCount);  // Default init is fine for these.

	for (int32 ThreadIndex = 0; ThreadIndex < ThreadCount; ThreadIndex++)
	{
		Threads[ThreadIndex] = Configuration.SharedContext->CreateThread();
		ThreadWakeups[ThreadIndex] = FPlatformProcess::GetSynchEventFromPool();
		ThreadCompleteEvents[ThreadIndex] = FPlatformProcess::GetSynchEventFromPool();

		Threads[ThreadIndex]->RunTask([this, ThreadIndex]() { GenericThreadFn(ThreadIndex); });
	}
}

FBuildPatchFileConstructor::~FBuildPatchFileConstructor()
{
	// Wait for threads to shut down.
	Abort();
	
	
	for (int32 ThreadIndex = 0; ThreadIndex < Threads.Num(); ThreadIndex++)
	{
		ThreadCompleteEvents[ThreadIndex]->Wait();
	}

	for (int32 ThreadIndex = 0; ThreadIndex < Threads.Num(); ThreadIndex++)
	{
		FPlatformProcess::ReturnSynchEventToPool(ThreadWakeups[ThreadIndex]);
		FPlatformProcess::ReturnSynchEventToPool(ThreadCompleteEvents[ThreadIndex]);
		Configuration.SharedContext->ReleaseThread(Threads[ThreadIndex]);
	}

	FPlatformProcess::ReturnSynchEventToPool(WakeUpDispatchThreadEvent);
	WakeUpDispatchThreadEvent = nullptr;
}

void FBuildPatchFileConstructor::SetPaused(bool bInIsPaused)
{
	bool bWasPaused = bIsPaused.AtomicSet(bInIsPaused);
	if (bWasPaused && !bInIsPaused)
	{
		// If we unpaused, the dispatch thread might be waiting in an event for us
		// to tell it to unpark.
		WakeUpDispatchThreadEvent->Trigger();
	}
}

void FBuildPatchFileConstructor::Abort()
{
	bool bAlreadyAborted = bShouldAbort.AtomicSet(true);

	UE_LOGF(LogBuildPatchServices, Verbose, "Issuing abort (previously aborted: %d)", bAlreadyAborted)

	if (bAlreadyAborted)
	{
		return;
	}

	// Make sure to wake up any threads that might be parked so they can bail.
	WakeUpDispatchThreadEvent->Trigger();
	
	for (int32 ThreadIndex = 0; ThreadIndex < Threads.Num(); ThreadIndex++)
	{
		ThreadWakeups[ThreadIndex]->Trigger();
	}

}

void FBuildPatchFileConstructor::SetChunkLocation(const FGuid& InGuid, EConstructorChunkLocation InNewLocation)
{
	// This is almost always called on the constructor thread, but on error conditions gets
	// called in completion handlers to set chunk locations to the cloud.
	// We assume a) that ChunkLocations is filled before threading
	// and that b) no guids are used across concurrent actions.
	//
	// This might be too complicated for its own good - if the only reason we touch this on other threads
	// is for error handling, maybe we move errors to sideband info and only touch ChunkLocations on the
	// constructor thread.
	FRWScopeLock ChunkLock(ChunkLocationsLock, SLT_Write);
	EConstructorChunkLocation* Location = ChunkLocations.Find(InGuid);
	if (!Location)
	{
		UE_LOGF(LogBuildPatchServices, Error, "Consistency failure: setting chunk location for non existent chunk %ls", *WriteToString<40>(InGuid));
	}
	else
	{
		if (*Location != EConstructorChunkLocation::Cloud &&
			InNewLocation == EConstructorChunkLocation::Cloud)
		{
			uint64 ChunkSize = Configuration.ManifestSet->GetDownloadSize(InGuid);

			UE_LOGF(LogBuildPatchServices, VeryVerbose, "Migrating chunk to cloud: %ls, %llu bytes", *WriteToString<40>(InGuid), ChunkSize);

			DownloadRequirement += ChunkSize;
			CloudSource->PostRequiredByteCount(DownloadRequirement);
		}

		*Location = InNewLocation;
	}
}

void FBuildPatchFileConstructor::QueueGenericThreadTask(int32 ThreadIndex, IConstructorChunkSource::FRequestProcessFn&& Task)
{
	// No thread for this task - run synchronously
	if (ThreadIndex == -1 || ThreadIndex >= Threads.Num())
	{
		Task(false);
		WakeUpDispatchThreadEvent->Trigger();
		return;
	}

	bool bPosted = false;
	ThreadJobPostingLocks[ThreadIndex].Lock();
	if (!bShouldAbort)
	{
		ThreadJobPostings[ThreadIndex].Add(MoveTemp(Task));
		bPosted = true;
	}
	ThreadJobPostingLocks[ThreadIndex].Unlock();
	ThreadWakeups[ThreadIndex]->Trigger();

	if (!bPosted)
	{
		// This means we aborted during the queue - make sure to run
		Task(true);
	}
}


void FBuildPatchFileConstructor::GenericThreadFn(int32 ThreadIndex)
{
	bool bLastShouldAbort = false;
	do
	{
		ThreadWakeups[ThreadIndex]->Wait();

		TArray<IConstructorChunkSource::FRequestProcessFn> GrabbedJobs;
		{
			const UE::TScopeLock Lock(ThreadJobPostingLocks[ThreadIndex]);
			GrabbedJobs = MoveTemp(ThreadJobPostings[ThreadIndex]);
		}

		bLastShouldAbort = bShouldAbort;
		for (const IConstructorChunkSource::FRequestProcessFn& Job : GrabbedJobs)
		{
			Job(bLastShouldAbort);
		}

		WakeUpDispatchThreadEvent->Trigger();
	} while (!bLastShouldAbort);

	ThreadCompleteEvents[ThreadIndex]->Trigger();
}

struct FChunkHarvestContext
{
	// This gets unlocked when all harvests are done to signal the main thread
	UE::FMutex HarvestsCompleteLock;

	std::atomic_int32_t PendingHarvestRequests = 0;
};

// Called from basically any thread.
void FBuildPatchFileConstructor::ChunkHarvestCompletedFn(const FGuid& Guid, bool bAborted, bool bFailedToRead, void* UserPtr)
{
	if (bFailedToRead)
	{
		// We tell the main thread this failed by setting the location since that's thread
		// safe.
		SetChunkLocation(Guid, EConstructorChunkLocation::Cloud);
	}

	FChunkHarvestContext* HarvestContext = (FChunkHarvestContext*)UserPtr;
	if (HarvestContext->PendingHarvestRequests.fetch_add(-1, std::memory_order_acq_rel) == 1)
	{
		HarvestContext->HarvestsCompleteLock.Unlock();
	}
}

bool FBuildPatchFileConstructor::HarvestChunksForCompletedFile(const FString& InstallationDirectory, const FString& FilenameInManifest)
{
	UE_LOGF(LogBuildPatchServices, Verbose, "Harvesting source: %ls / %ls", *InstallationDirectory, *FilenameInManifest);

	TRACE_CPUPROFILER_EVENT_SCOPE(Harvest);
	// We need to grab any chunks from install sources that are no longer available.
	// Anything that's already been loaded is already placed into the memory store appropriately,
	// but anything that _hasn't_ needs to be pulled out.	
	TSet<FGuid> FileChunks;
	InstallSource->GetChunksForFile(InstallationDirectory, FilenameInManifest, FileChunks);

	struct FNeededChunk
	{
		FGuid Id;
		int32 LastUsageIndex;
		int32 NextUsageIndex;
		int32 ChunkSize;
	};

	TArray<FNeededChunk> ChunksFromFileWeNeed;

	{
		FRWScopeLock ChunkLock(ChunkLocationsLock, SLT_ReadOnly);

		
		int32 CurrentUsageIndex = ChunkReferenceTracker->GetCurrentUsageIndex();
		for (const FGuid& FileChunk : FileChunks)
		{
			EConstructorChunkLocation* Location = ChunkLocations.Find(FileChunk);
			if (Location && *Location == EConstructorChunkLocation::Install)
			{
				int32 LastUsageIndex = 0;
				int32 NextUsageIndex = ChunkReferenceTracker->GetNextUsageForChunk(FileChunk, LastUsageIndex);

				if (NextUsageIndex == -1 ||
					LastUsageIndex < CurrentUsageIndex)
				{
					// The chunk is no longer needed
					*Location = EConstructorChunkLocation::Retired;
					continue;
				}

				int32 ChunkSize = Configuration.ManifestSet->GetChunkInfo(FileChunk)->DataSizeUncompressed;

				ChunksFromFileWeNeed.Add({FileChunk, LastUsageIndex, NextUsageIndex, ChunkSize});
			}
		}
	}

	UE_LOGF(LogBuildPatchServices, Verbose, "    Chunks Needed: %d", ChunksFromFileWeNeed.Num());
	if (ChunksFromFileWeNeed.Num() == 0)
	{
		return true;
	}

	// Try to load all the chunks that are about to go away. If it fails we don't particularly
	// care since we would have fallen back to cloud anyway.

	// There's some care here - if we just kick off a ton of reads, all those backing store
	// entries are locked during the reads so we have to allocate space and can't page anything
	// out. This is fine if we can load the whole file, but under constrained memory we want to
	// only keep stuff that's going to get used soon - so we load the stuff we _aren't_ going to use
	// soon so it can get paged out. Then we load in batches so we can release the locks and let them
	// page out. 
	// LONGTERM - detect this condition and write directly to the disk backing store? Ideally this
	// would be something we can retain across restarts as right now any harvested chunks get lost
	// on abort and cause a download from the cloud source (not chunkdb!)
	Algo::SortBy(ChunksFromFileWeNeed, &FNeededChunk::NextUsageIndex, TGreater<int32>());

	constexpr int32 HarvestBatchSize = 16 << 20; // 16 MB

	bool bHarvestSuccess = true;
	for (int32 ChunkIndex = 0; ChunkIndex < ChunksFromFileWeNeed.Num();)
	{
		int32 BatchSize = 0;
		int32 ChunkEndIndex = ChunkIndex + 1;
		for (; ChunkEndIndex < ChunksFromFileWeNeed.Num(); ChunkEndIndex++)
		{
			if (BatchSize + ChunksFromFileWeNeed[ChunkEndIndex].ChunkSize > HarvestBatchSize)
			{
				break;
			}
			BatchSize += ChunksFromFileWeNeed[ChunkEndIndex].ChunkSize;
		}

		FChunkHarvestContext ChunkHarvestContext;
		ChunkHarvestContext.HarvestsCompleteLock.Lock();
		ChunkHarvestContext.PendingHarvestRequests.store(ChunkEndIndex - ChunkIndex, std::memory_order_release);
		UE_LOGF(LogBuildPatchServices, Verbose, "    Harvesting %d chunks", ChunkEndIndex - ChunkIndex);
		for (int32 DispatchIndex = ChunkIndex; DispatchIndex < ChunkEndIndex; DispatchIndex++)
		{
			FNeededChunk& Chunk = ChunksFromFileWeNeed[DispatchIndex];

			SetChunkLocation(Chunk.Id, EConstructorChunkLocation::Memory);

			FMutableMemoryView Destination = BackingStore->ReserveAndLockEntry(Chunk.Id, Chunk.ChunkSize, Chunk.LastUsageIndex);
			if (Destination.GetSize() == 0)
			{
				// Call the completion function so we decrement the request count,
				// but this is a consistency failure so we can't use the results.
				ChunkHarvestCompletedFn(Chunk.Id, false, false, &ChunkHarvestContext);
				bHarvestSuccess = false;
			}

			IConstructorChunkSource::FRequestProcessFn HarvestFn = InstallSource->CreateRequest(Chunk.Id, Destination, &ChunkHarvestContext, 
				IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateRaw(this, &FBuildPatchFileConstructor::ChunkHarvestCompletedFn));

			HarvestFn(false);
		}

		// The read is synchronous but the verification is not, so we still need to do the wait.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(HarvestWait);
			UE_LOGF(LogBuildPatchServices, Verbose, "    Waiting...");
			ChunkHarvestContext.HarvestsCompleteLock.Lock();
			UE_LOGF(LogBuildPatchServices, Verbose, "    Done.");
		}

		if (bHarvestSuccess)
		{
			// Unlock any memory store entries
			FRWScopeLock ChunkLock(ChunkLocationsLock, SLT_ReadOnly);

			for (int32 DispatchIndex = ChunkIndex; DispatchIndex < ChunkEndIndex; DispatchIndex++)
			{
				// The read could have failed - in which case the location was switched to cloud from Memory.
				// We handle this here so we don't have to deal with synchronization in the completion function.
				if (ChunkLocations[ChunksFromFileWeNeed[DispatchIndex].Id] == EConstructorChunkLocation::Cloud)
				{
					if (!BackingStore->DereserveHarvestingEntry(ChunksFromFileWeNeed[DispatchIndex].Id))
					{
						bHarvestSuccess = false;
					}
				}
				else
				{
					if (!BackingStore->CommitAndReleaseEntry(ChunksFromFileWeNeed[DispatchIndex].Id))
					{
						bHarvestSuccess = false;
					}
				}
			}
		}

		if (bHarvestSuccess && !bAllowMultipleFilesInFlight)
		{
			bHarvestSuccess = BackingStore->CheckNoLocks(true);
		}

		ChunkIndex = ChunkEndIndex;

		if (bShouldAbort || !bHarvestSuccess)
		{
			break;
		}
	}

	return bHarvestSuccess;
}

void FBuildPatchFileConstructor::Run()
{
	FileConstructorStat->OnTotalRequiredUpdated(TotalJobSize);

	
	// We'd really like to have a sense of what each chunk looks like, size-wise, so that
	// we know things like how many downloads we expect per batch.
	// Map of window sizes to counts of that size.
	TMap<uint32, int32> WindowSizes;

	// lock not requried - no threads yet.
	DownloadRequirement = 0;
	for (const TPair<FGuid, EConstructorChunkLocation>& ChunkLocation : ChunkLocations)
	{
		FChunkInfo const* ChunkInfo = Configuration.ManifestSet->GetChunkInfo(ChunkLocation.Key);
		WindowSizes.FindOrAdd(ChunkInfo->DataSizeUncompressed)++;

		if (ChunkLocation.Value == EConstructorChunkLocation::Cloud)
		{
			DownloadRequirement += ChunkInfo->FileSize;
		}
	}
	CloudSource->PostRequiredByteCount(DownloadRequirement);

	{
		ExpectedChunkSize = 0;
		int32 LargestWindowCount = 0;
		for (TPair<uint32, int32>& Entry : WindowSizes)
		{
			if (Entry.Value > LargestWindowCount)
			{
				LargestWindowCount = Entry.Value;
				ExpectedChunkSize = Entry.Key;
			}
		}

		if (ExpectedChunkSize)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Expected chunk size: %d count %d", ExpectedChunkSize, LargestWindowCount);
		}
		else
		{
			UE_LOGF(LogBuildPatchServices, Log, "Can't find largest chunk size, using 1MB");
			ExpectedChunkSize = 1 << 20;
		}
	}

	// We disable resume if we are a fresh install that's below a certain threshold in order to minimize
	// io operations.
	bool bResumeEnabled = true;
	if (!Configuration.ManifestSet->ContainsUpdate())
	{
		int64 DisableResumeBelowBytes = ((int64)Configuration.DisableResumeBelowMB.Get(CVarDisableResumeBelowMB)) << 20;
		if (DisableResumeBelowBytes > TotalJobSize)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Disabling resume: JobSize = %ls, Disable = %ls, from %ls", *FormatNumber(TotalJobSize), *FormatNumber(DisableResumeBelowBytes), Configuration.DisableResumeBelowMB.IsSet() ? TEXT("config") : TEXT("cvar"));
			bResumeEnabled = false;
		}
	}

	if (Configuration.bInstallToMemory)
	{
		UE_LOGF(LogBuildPatchServices, Log, "Disabling resume: install to memory");
		bResumeEnabled = false;
	}

	const FString ResumeDataFilename = Configuration.MetaDirectory / TEXT("$resumeData");
	FResumeData ResumeData(FileSystem, Configuration.ManifestSet, Configuration.StagingDirectory, ResumeDataFilename);

	if (bResumeEnabled)
	{
		IFileManager::Get().MakeDirectory(*Configuration.MetaDirectory);

		// Check for resume data, we need to also look for a legacy resume file to use instead in case we are resuming from an install of previous code version.
		const FString LegacyResumeDataFilename = Configuration.StagingDirectory / TEXT("$resumeData");		
		const bool bHasLegacyResumeData = FileSystem->FileExists(*LegacyResumeDataFilename);
		// If we find a legacy resume data file, lets move it first.
		if (bHasLegacyResumeData)
		{
			FileSystem->MoveFile(*ResumeDataFilename, *LegacyResumeDataFilename);
		}

		ResumeData.InitResume();

		// Check the stage of resumed files and remove any incompatible files.
		if (ResumeData.bHasResumeData)
		{
			if (!Configuration.bUseSHA1StageFilenames)
			{
				for (const FString& FileToConstruct : Configuration.ConstructList)
				{
					const bool bFileIncompatible = ResumeData.CheckFile(FileToConstruct);
					if (bFileIncompatible)
					{
						UE_LOGF(LogBuildPatchFileConstructor, Log, "Deleting incompatible stage file %ls", *FileToConstruct);
						FileSystem->DeleteFile(*(Configuration.StagingDirectory / FileToConstruct));
					}
				}
			}

			if (Configuration.bConstructInMemory)
			{
				UE_LOGF(LogBuildPatchServices, Log, "Removing incomplete entries from resume data: construct in memory");
				ResumeData.FilesStage = ResumeData.FilesStage.FilterByPredicate([](const TPair<FString, EFileStage>& Entry) { return Entry.Value != EFileStage::Incomplete; });
			}

			// Uninstall should not have any files to construct
			const bool bIsUninstall = Configuration.ConstructList.IsEmpty();
			if (!bIsUninstall)
			{
				// For uninstall, we will remove staged files at an upper level
				RemoveUnnecessaryStagedFiles(Configuration.StagingDirectory);
			}
		}

		// Save for started versions
		TSet<FString> ResumeIds;
		const bool bCheckLegacyIds = false;

		Configuration.ManifestSet->GetInstallResumeIds(ResumeIds, bCheckLegacyIds);
		ResumeData.SaveOut(ResumeIds);
	}

	// Start resume progress at zero or one.
	FileConstructorStat->OnResumeStarted();

	// While we have files to construct, run.
	ConstructFiles(ResumeData);

	// Mark resume complete if we didn't have work to do.
	if (!bIsDownloadStarted)
	{
		FileConstructorStat->OnResumeCompleted();
	}

	if (InstallerError->HasError() == false)
	{
		ResumeData.RemoveResumeData();
	}

	FileConstructorStat->OnConstructionCompleted();
}

uint64 FBuildPatchFileConstructor::GetRequiredDiskSpace()
{
	return RequiredDiskSpace.load(std::memory_order_relaxed);
}

uint64 FBuildPatchFileConstructor::GetAvailableDiskSpace()
{
	return AvailableDiskSpace.load(std::memory_order_relaxed);
}

bool FBuildPatchFileConstructor::HasCorruptFiles() const
{
	return !CorruptFiles.IsEmpty();
}

FBuildPatchFileConstructor::FOnBeforeDeleteFile& FBuildPatchFileConstructor::OnBeforeDeleteFile()
{
	return BeforeDeleteFileEvent;
}

FBuildPatchFileConstructor::FOnFileIsStaged& FBuildPatchFileConstructor::OnFileIsStaged()
{
	return FileIsStagedEvent;
}

void FBuildPatchFileConstructor::CountBytesProcessed( const int64& ByteCount )
{
	ByteProcessed += ByteCount;
	FileConstructorStat->OnProcessedDataUpdated(ByteProcessed);
}

int64 FBuildPatchFileConstructor::GetRemainingBytes()
{
	// Need the sum of the output sizes of files not yet started.
	// Since this gets called from any thread, construction will continue
	// as we calculate this, but all the structures are stable as long as
	// the constructor is still valid memory.

	int32 LocalNextIndexToConstruct = NextIndexToConstruct.load(std::memory_order_acquire);

	uint64 RemainingBytes = 0;
	for (int32 ConstructIndex = LocalNextIndexToConstruct; ConstructIndex < ConstructionList.Num(); ConstructIndex++)
	{
		if (ConstructionList[ConstructIndex].FileManifest)
		{
			RemainingBytes += ConstructionList[ConstructIndex].FileManifest->FileSize;
		}
	}

	return RemainingBytes;
}

uint64 FBuildPatchFileConstructor::CalculateInProgressDiskSpaceRequired(const FFileManifest& InProgressFileManifest, uint64 InProgressFileAmountWritten)
{
	if (Configuration.InstallMode == EInstallMode::DestructiveInstall)
	{
		TSet<FString> MissedCurrentFiles; // Files that were not installed, possibly due to filtering by tags.
		Configuration.ManifestSet->GetMissedFiles(Configuration.InstallDirectory, MissedCurrentFiles);

		// The simplest method will be to run through each high level file operation, tracking peak disk usage delta.

		// We know we need enough space to finish writing this file
		uint64 RemainingThisFileSpace = InProgressFileManifest.FileSize - InProgressFileAmountWritten;
		
		int64 DiskSpaceDeltaPeak = RemainingThisFileSpace;
		int64 DiskSpaceDelta = RemainingThisFileSpace;

		// Then we move this file over.
		{
			const FFileManifest* OldFileManifest = Configuration.ManifestSet->GetCurrentFileManifest(InProgressFileManifest.Filename);
			if (OldFileManifest)
			{
				DiskSpaceDelta -= OldFileManifest->FileSize;
			}

			// We've already accounted for the new file above, so we could be pretty negative if we resumed the file
			// almost at the end and had an existing file we're deleting.
		}

		// Loop through all files to be made next, in order.
		int32 LocalNextIndexToConstruct = NextIndexToConstruct.load(std::memory_order_acquire);
		for (int32 ConstructionIndex = LocalNextIndexToConstruct; ConstructionIndex < ConstructionList.Num(); ConstructionIndex++)
		{
			const FFileManifest* NewFileManifest = ConstructionList[ConstructionIndex].FileManifest;
			if (!NewFileManifest)
			{
				continue;
			}

			const FFileManifest* OldFileManifest = Configuration.ManifestSet->GetCurrentFileManifest(Configuration.ConstructList[ConstructionIndex]);

			// First we would need to make the new file.
			DiskSpaceDelta += NewFileManifest->FileSize;
			if (DiskSpaceDeltaPeak < DiskSpaceDelta)
			{
				DiskSpaceDeltaPeak = DiskSpaceDelta;
			}
			// Then we can remove the currently existing file if it was previously created.
			if (OldFileManifest && !MissedCurrentFiles.Contains(Configuration.ConstructList[ConstructionIndex]))
			{
				DiskSpaceDelta -= OldFileManifest->FileSize;
			}
		}
		return DiskSpaceDeltaPeak;
	}
	else
	{
		// When not destructive, we always stage all new and changed files.
		uint64 RemainingFilesSpace = GetRemainingBytes();
		uint64 RemainingThisFileSpace = InProgressFileManifest.FileSize - InProgressFileAmountWritten;
		return RemainingFilesSpace + RemainingThisFileSpace;
	}
}

uint64 FBuildPatchFileConstructor::CalculateDiskSpaceRequirementsWithDeleteDuringInstall(int64 InProgressFileSize)
{
	checkf(ChunkDbSource, TEXT("CalculateDiskSpaceRequirementsWithDeleteDuringInstall can't be called without valid ChunkDbSource"));

	// These are the sizes at after each file that we _started_ with. This is the size after retirement for the
	// file at those positions.
	TArray<uint64> ChunkDbSizesAtPosition;
	ChunkDbSizesAtPosition.Reserve(FileCompletionPositions.Num());
	const uint64 TotalChunkDbSize = ChunkDbSource->GetChunkDbSizesAtIndexes(FileCompletionPositions, ChunkDbSizesAtPosition);
	// Strip off the files we've completed.
	int32 CompletedFileCount = NextIndexToConstruct.load(std::memory_order_acquire);

	// Since we are called after the first file is popped (but before it's actually done), we have one less completed.
	check(CompletedFileCount > 0); // should ALWAYS be at least 1!
	if (CompletedFileCount > 0)
	{
		CompletedFileCount--;
	}
	const uint64 MaxDiskSize = FBuildPatchUtils::CalculateDiskSpaceRequirementsWithDeleteDuringInstall(
		Configuration.ConstructList, CompletedFileCount, InProgressFileSize, Configuration.ManifestSet, ChunkDbSizesAtPosition,
		Configuration.InstallDirectory, Configuration.InstallMode);

	// Strip off the data we already have on disk.
	uint64 PostDlSize = 0; // Zero means no additional free space is needed for action, as it will be freed by removing chunks.
	if (MaxDiskSize > TotalChunkDbSize)
	{
		PostDlSize = MaxDiskSize - TotalChunkDbSize;
	}

	return PostDlSize;
}

struct FBatchState;
struct FRequestInfo
{
	struct FRequestSplat
	{
		uint64 DestinationOffset;
		uint32 OffsetInChunk;
		uint32 BytesToCopy;
	};

	FGuid Guid;

	// We can do a lot of shortcuts if we are working with the entire chunk
	uint32 ChunkSize = 0;

	// We could request the same guid multiple times
	// for the same buffer... in this case we want one
	// request but we need to remember to splat it
	// afterwards.
	TArray<FRequestSplat, TInlineAllocator<1>> Splats;
	
	// The read goes here - this is usually directly into the write buffer,
	// but we might need to duplicate out of this (and this might be memory
	// store owned if we don't use the whole chunk)
	FMutableMemoryView ReadBuffer;

	FBatchState* Batch = nullptr;

	// Splats offset into this.
	FMutableMemoryView DestinationBuffer;

	FFileConstructionState* File = nullptr;

	// We can only read direct in some cases.
	bool bReadIntoMemoryStore = false;

	// Never save back to memory store if it came from it.
	bool bSourceIsMemoryStore = false;
	bool bAborted = false;
	bool bFailedToRead = false;
	bool bLaunchedFallback = false;

	// Which index in the chunk reference tracker we are requesting.
	int32 ChunkIndexInFile;
	int32 ChunkUnavailableAt = TNumericLimits<int32>::Max();
};


struct FBatchState
{
	int32_t BatchId = 0;

	TMap<FGuid, FRequestInfo> Requests;
	std::atomic_int32_t PendingRequestCount = 0;
	std::atomic_int32_t FailedRequestCount = 0;

	// Reads for the batch end up here, and will be written to the target
	// file in order.
	FMutableMemoryView BatchBuffer;

	EConstructionError Error = EConstructionError::None;
	FGuid ErrorContextGuid;

	FFileConstructionState* OwningFile = nullptr;

	int32 StartChunkPartIndex = 0;
	int32 ChunkCount = 0;

	// If true, this batch never reads or writes, it exists to complete the empty file in order.
	bool bIsEmptyFileSentinel = false;
	bool bNeedsWrite = false;
	bool bIsReading = false;
	bool bIsWriting = false;	

	// Set by the completing threads when the batch is done.
	std::atomic<bool> bIsFinished = true;

	// Number of chunk parts remaining in the tracker at the moment of batch read completion.
	std::atomic<int32> RemainingChunkCount = TNumericLimits<int32>::Max();
	
	FBatchState()
	{
		static std::atomic_int32_t UniqueBatchId = 1;
		BatchId = UniqueBatchId.fetch_add(1);
	}
};

// Called from basically any thread.
void FBuildPatchFileConstructor::RequestCompletedFn(const FGuid& Guid, bool bAborted, bool bFailedToRead, void* UserPtr)
{
	FRequestInfo* Request = (FRequestInfo*)UserPtr;
	Request->bAborted = bAborted;
	Request->bFailedToRead = bFailedToRead;

	// If failed but didn't abort and we haven't already kicked the fallback, kick the fallback
	if (!bAborted && bFailedToRead && !Request->bLaunchedFallback)
	{		
		Request->bLaunchedFallback = true;
		Request->bFailedToRead = false;

		// We couldn't read the expected source. This means we need to fall back to the cloud source.

		// This should be safe because the values already exist in the map and we only
		// ever have 1 request for a Guid active at one time. However if we've already read into the
		// memory store then it's already updated to memory which is where the cloud source will read
		// it to. If it's not reading into the memory store, then we need to remember to grab it
		// from the cloud next time.
		if (!Request->bReadIntoMemoryStore)
		{
			SetChunkLocation(Request->Guid, EConstructorChunkLocation::Cloud);
		}
		else
		{
			// Normally SetChunkLocation will update the download amount, but we aren't actually
			// changing the chunk's location since it's going into the memory store. We do still
			// need to tell the user about the download requirement though:
			uint64 ChunkSize = Configuration.ManifestSet->GetDownloadSize(Guid);

			ChunkLocationsLock.WriteLock();
			DownloadRequirement += ChunkSize;
			CloudSource->PostRequiredByteCount(DownloadRequirement);
			ChunkLocationsLock.WriteUnlock();

		}

		if (bHasChunkDbSource)
		{
			// Only send this message if we have chunk dbs. The theory is if they don't have chunkdbs then they are expecting
			// chunks to download from the cloud. If they provide chunkdbs then they are surprised when chunks come from the cloud.
			MessagePump->SendMessage({FGenericMessage::EType::CloudSourceUsed, Guid});
		}

		QueueGenericThreadTask(ThreadAssignments[EConstructorChunkLocation::Cloud], CloudSource->CreateRequest(
			Request->Guid, 
			Request->ReadBuffer, 
			Request,
			IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateRaw(this, &FBuildPatchFileConstructor::RequestCompletedFn)));
	}
	else
	{
		if (bFailedToRead)
		{
			Request->Batch->ErrorContextGuid = Guid;
			Request->Batch->FailedRequestCount.fetch_add(1, std::memory_order_relaxed);
		}
		else if (!bAborted)
		{
			uint64 TotalToDestinationBuffer = 0;

			if (Request->bReadIntoMemoryStore)
			{
				// If the read went to memory, we need to copy splats. Otherwise it was a single
				// direct read so do nothing.
				TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_Splats);
				for (const FRequestInfo::FRequestSplat& Splat : Request->Splats)
				{
					FMemory::Memcpy(
						(uint8*)Request->DestinationBuffer.GetData() + Splat.DestinationOffset,
						(uint8*)Request->ReadBuffer.GetData() + Splat.OffsetInChunk,
						Splat.BytesToCopy);
					TotalToDestinationBuffer += Splat.BytesToCopy;
				}
			}
			else
			{
				// Direct means 1 splat
				TotalToDestinationBuffer += Request->Splats[0].BytesToCopy;
			}

			Request->File->ProgressLock.Lock();
			Request->File->Progress += TotalToDestinationBuffer;
			Request->File->ProgressLock.Unlock();
		}
		Request->Batch->RemainingChunkCount.store(RemainingChunkCount, std::memory_order_relaxed);
		if (Request->Batch->PendingRequestCount.fetch_add(-1, std::memory_order_relaxed) == 1)
		{
			Request->Batch->bIsFinished.store(true, std::memory_order_release);
			WakeUpDispatchThreadEvent->Trigger();
		}
	}
}

// Return a function that writes the data for the batch to the given file.
IConstructorChunkSource::FRequestProcessFn FBuildPatchFileConstructor::CreateWriteRequest(FArchive* File, FBatchState& Batch)
{
	return [this, File, Batch = &Batch](bool bIsAbort)
	{
		if (bIsAbort)
		{
			// Report the batch finished otherwise we hard lock. This happens if we abort after the write thread grabs the jobs
			// but before we pass the abort check in the generic thread dispatch.
			Batch->bIsFinished.store(true, std::memory_order_release);
			return;
		}

		// Has to be mutable because of the serialize call.
		FMutableMemoryView WriteBuffer = Batch->BatchBuffer;

		// Manage write limits.
		if (bStallWhenFileSystemThrottled)
		{
			uint64 AvailableBytes = FileSystem->GetAllowedBytesToWriteThrottledStorage(*File->GetArchiveName());
			while (WriteBuffer.GetSize() > AvailableBytes)
			{
				UE_LOGF(LogBuildPatchServices, Display, "Avaliable write bytes to write throttled storage exhausted (%ls).  Sleeping %ds.  Bytes needed: %llu, bytes available: %llu"
					, *File->GetArchiveName(), SleepTimeWhenFileSystemThrottledSeconds, WriteBuffer.GetSize(), AvailableBytes);
				FPlatformProcess::Sleep(SleepTimeWhenFileSystemThrottledSeconds);
				AvailableBytes = FileSystem->GetAllowedBytesToWriteThrottledStorage(*File->GetArchiveName());

				if (bShouldAbort)
				{
					return;
				}
			}
		}

		FileConstructorStat->OnBeforeWrite();
		ISpeedRecorder::FRecord ActivityRecord;
		ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
		
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WriteThread_Serialize)
			File->Serialize(WriteBuffer.GetData(), WriteBuffer.GetSize());
		}

		ActivityRecord.Size = WriteBuffer.GetSize();
		ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
		FileConstructorHelpers::LogLowRWDurationTime(ActivityRecord, TEXT("Write"));

		FileConstructorStat->OnAfterWrite(ActivityRecord);
		
		Batch->bIsFinished.store(true, std::memory_order_release);
	};
}
					

// Always called on Constructor thread.
void FBuildPatchFileConstructor::CompleteReadBatch(const FFileManifest& FileManifest, FBatchState& Batch)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_CompleteRead);

	if (Batch.Error == EConstructionError::None && Batch.FailedRequestCount.load(std::memory_order_acquire))
	{
		Batch.Error = EConstructionError::MissingChunk;
	}

	UE_LOGF(LogBuildPatchServices, Verbose, "Completing ReadBatch: %d", Batch.BatchId);

	// We have to copy the memory source chunks after the reads are done
	// because if we have two buffer's reads queued, the first one could be
	// filling the memory source. If we copy these after we are done, we guarantee
	// that the previous buffer has completed its reads so we know we are
	// working with valid memory.
	if (Batch.Error == EConstructionError::None)
	{
		for (TPair<FGuid, FRequestInfo>& RequestPair : Batch.Requests)
		{					
			FRequestInfo& Request = RequestPair.Value;

			if (Request.bSourceIsMemoryStore)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_DirectCopy);
				// Just copy what we need directly.
				FMemoryView ChunkData = BackingStore->GetViewForChunk(Request.Guid);
				bool bFailedToGetChunk = ChunkData.GetSize() == 0;

				if (!bFailedToGetChunk)
				{
					uint64 TotalToDestinationBuffer = 0;
					for (const FRequestInfo::FRequestSplat& Splat : Request.Splats)
					{
						FMemory::Memcpy(
							(uint8*)Request.DestinationBuffer.GetData() + Splat.DestinationOffset,
							(uint8*)ChunkData.GetData() + Splat.OffsetInChunk,
							Splat.BytesToCopy);
						TotalToDestinationBuffer += Splat.BytesToCopy;
					}

					Request.File->ProgressLock.Lock();
					Request.File->Progress += TotalToDestinationBuffer;
					Request.File->ProgressLock.Unlock();
				}
				else
				{
					Batch.Error = EConstructionError::MissingChunk;
					Batch.ErrorContextGuid = Request.Guid;
				}

				// Mark that we're done with the memory so it can get evicted if necessary.
				if (!BackingStore->ReleaseEntry(Request.Guid))
				{
					Batch.Error = EConstructionError::InternalConsistencyError;
					Batch.ErrorContextGuid = Request.Guid;
				}
			}
			else if (Request.bReadIntoMemoryStore)
			{
				// Commit the read so the memory store knows its safe to evict if necessary.
				if (!BackingStore->CommitAndReleaseEntry(Request.Guid))
				{
					Batch.Error = EConstructionError::InternalConsistencyError;
					Batch.ErrorContextGuid = Request.Guid;
				}
			}
		}
	}

	if (Batch.Error == EConstructionError::None)
	{
		// Retire the chunks we've used. This has to be in order 
		for (int32 i = 0; i < Batch.ChunkCount; i++)
		{
			if (!ChunkReferenceTracker->PopReference(FileManifest.ChunkParts[Batch.StartChunkPartIndex + i].Guid))
			{
				Batch.Error = EConstructionError::TrackingError;
				Batch.ErrorContextGuid = FileManifest.ChunkParts[Batch.StartChunkPartIndex + i].Guid;
			}
		}
		RemainingChunkCount = ChunkReferenceTracker->GetRemainingChunkCount();
	}

	if (Batch.Error == EConstructionError::None)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_MemRetire);
		// Has to be after the splats because this might free the memory we need!				
		if (!BackingStore->CheckRetirements(ChunkReferenceTracker->GetCurrentUsageIndex()))
		{
			Batch.Error = EConstructionError::InternalConsistencyError;
		}
	}
}

// Always called from Constructor thread
// This return -1 when we run into an error that requires stopping the installation.
// Note that there could be outstanding reads and we can no longer rely on the completion functions
// to wake up the dispatch thread (which we must run on), so we can't ever wait if this returns -1.
// Check Batch.Error for the error on -1 return.
void FBuildPatchFileConstructor::StartReadBatch(FFileConstructionState& CurrentFile, FBatchState& Batch)
{
	CurrentFile.OutstandingBatches++;
	Batch.Requests.Reset();
	Batch.StartChunkPartIndex = CurrentFile.NextChunkPartToRead;

	int32 EndChunkIdx = Batch.StartChunkPartIndex;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_Count);
		uint64 BufferFillLevel = 0;
		for (; EndChunkIdx < CurrentFile.FileManifest->ChunkParts.Num(); ++EndChunkIdx)
		{
			const FChunkPart& ChunkPart = CurrentFile.FileManifest->ChunkParts[EndChunkIdx];
			if (ChunkPart.Size + BufferFillLevel > Batch.BatchBuffer.GetSize())
			{
				break;
			}

			FRequestInfo& Request = Batch.Requests.FindOrAdd(ChunkPart.Guid);
			Request.Guid = ChunkPart.Guid;
			Request.Batch = &Batch;
			Request.ChunkSize = Configuration.ManifestSet->GetChunkInfo(ChunkPart.Guid)->DataSizeUncompressed;
			Request.File = &CurrentFile;
			Request.ChunkIndexInFile = EndChunkIdx;
			
			FRequestInfo::FRequestSplat& Splat = Request.Splats.AddDefaulted_GetRef();
			Splat.DestinationOffset = BufferFillLevel;
			Splat.OffsetInChunk = ChunkPart.Offset;
			Splat.BytesToCopy = ChunkPart.Size;
				
			BufferFillLevel += ChunkPart.Size;
		}

		// Trim our view to the amount we actually used. The calling code will reclaim the rest.
		Batch.BatchBuffer.LeftInline(BufferFillLevel);
	}

	Batch.ChunkCount = EndChunkIdx - Batch.StartChunkPartIndex;
	Batch.PendingRequestCount = Batch.Requests.Num();
	Batch.FailedRequestCount = 0;
	
	//
	// IMPORTANT!!!
	//
	// We MUST call the completion routine for each request here so that we can know
	// when all outstanding requests are done. If we bail early then we can't know when
	// all requests are done during cancelation/abort. If we hit a consistency error during then
	// we just need to call it with a failure.
	//
	for (TPair<FGuid, FRequestInfo>& RequestPair : Batch.Requests)
	{
		// Can come from:
		// -- memory. This chunk had already been loaded and we needed it again _after it's source expired_.
		// -- chunkdb. async IO + decompress
		// -- cloud. async download + decompress
		// -- disk. we ran out of memory store and had to dump to disk. async IO + maybe decompress
		// -- install. async IO

		// Note -- most chunks will be used more than once.

		// sources that expire: note that expired chunks can always be redownloaded via the cloud source
		//  -- cloud (they expire immediately, but can be redownloaded)
		//  -- install

		// We aren't necessarily using the whole chunk - e.g. if we are a small file, we'll only
		// be a tiny part of the chunk and the rest will need to be used by the next file. In this case
		// we have to read into a memory store destination so that it can be carried over into the
		// next file.


		FRequestInfo& Request = RequestPair.Value;
		Request.DestinationBuffer = Batch.BatchBuffer;

		
		const EConstructorChunkLocation* ChunkLocationPtr = ChunkLocations.Find(Request.Guid);
		if (ChunkLocationPtr == nullptr)
		{
			Batch.Error = EConstructionError::InternalConsistencyError;
			RequestCompletedFn(Request.Guid, true, false, &Request);
			continue;
		}
		EConstructorChunkLocation ChunkLocation = *ChunkLocationPtr;

		Request.bSourceIsMemoryStore = ChunkLocation == EConstructorChunkLocation::Memory;
		if (Request.bSourceIsMemoryStore)
		{
			// We copy after the reads are done since the memory might not be ready.
			// Make sure we don't evict it in the meantime
			if (!BackingStore->LockEntry(Request.Guid))
			{
				Batch.Error = EConstructionError::InternalConsistencyError;
				RequestCompletedFn(Request.Guid, true, false, &Request);
				continue;
			}
			RequestCompletedFn(Request.Guid, false, false, &Request);
		}
		else
		{
			IConstructorChunkSource* ThisChunkSource = nullptr;
			switch (ChunkLocation)
			{
				case EConstructorChunkLocation::Install: ThisChunkSource = InstallSource; break;
				case EConstructorChunkLocation::Cloud: ThisChunkSource = CloudSource; break;
				case EConstructorChunkLocation::ChunkDb: ThisChunkSource = ChunkDbSource; break;
				case EConstructorChunkLocation::Memory: /* already handled above */ break;
				case EConstructorChunkLocation::DiskOverflow: ThisChunkSource = (IConstructorChunkSource*)BackingStore.Get(); break;
			}

			if (ThisChunkSource == CloudSource)
			{
				// If we are already downloading from the cloud, then failures shouldn't try to 
				// fall back to the cloud.
				Request.bLaunchedFallback = true;

				if (bHasChunkDbSource)
				{
					// Only send this message if we have chunk dbs. The theory is if they don't have chunkdbs then they are expecting
					// chunks to download from the cloud. If they provide chunkdbs then they are surprised when chunks come from the cloud.
					MessagePump->SendMessage({FGenericMessage::EType::CloudSourceUsed, RequestPair.Value.Guid});
				}
			}

			// We need to kick a request. The question is whether we can request direct
			// or need to route through the memory store.
			int32 LastUsageIndex = 0;
			ChunkReferenceTracker->GetNextUsageForChunk(Request.Guid, LastUsageIndex);
			Request.ChunkUnavailableAt = ThisChunkSource->GetChunkUnavailableAt(Request.Guid);
				
			const bool bNeedsEntireChunk = Request.Splats.Num() == 1 && Request.Splats[0].BytesToCopy == Request.ChunkSize && Request.Splats[0].OffsetInChunk == 0;

			int32 ThisUsageIndex = CurrentFile.BaseReferenceIndex + Request.ChunkIndexInFile;

			// Unavailable might be reported before the current usage which would force this to route through
			// memory store even though it could read direct, so only check needed after retirement if we need it again.
			const bool bNeededAfterRetirement = (LastUsageIndex > ThisUsageIndex) && LastUsageIndex >= Request.ChunkUnavailableAt;

			if (bNeedsEntireChunk && !bNeededAfterRetirement)
			{
				// Read direct.
				Request.ReadBuffer = MakeMemoryView((uint8*)Request.DestinationBuffer.GetData() + Request.Splats[0].DestinationOffset, Request.Splats[0].BytesToCopy);
			}
			else
			{
				// Route through memory store.
				Request.ReadBuffer = BackingStore->ReserveAndLockEntry(Request.Guid, Request.ChunkSize, LastUsageIndex);
				if (Request.ReadBuffer.GetSize() == 0)
				{
					Batch.Error = EConstructionError::InternalConsistencyError;
					RequestCompletedFn(Request.Guid, true, false, &Request);
					continue;
				}
				Request.bReadIntoMemoryStore = true;

				// Note that when we set this, the next batch read could want this chunk before its read is done.
				// Hence reads for memory sources are done _after_ reads are done, because we retire reads in order,
				// we then know this memory is populated.
				SetChunkLocation(RequestPair.Value.Guid, EConstructorChunkLocation::Memory);
			}
			

			IConstructorChunkSource::FRequestProcessFn RequestProcess = ThisChunkSource->CreateRequest(
				Request.Guid, 
				Request.ReadBuffer,
				(void*)&Request,
				IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateRaw(this, &FBuildPatchFileConstructor::RequestCompletedFn));
						
			QueueGenericThreadTask(ThreadAssignments[ChunkLocation], MoveTemp(RequestProcess));

			if (ThisChunkSource == ChunkDbSource)
			{
				IConstructorChunkSource::FRequestProcessFn TruncateTask = [this](bool bIsAborted) {
					if (!bIsAborted)
					{
						ChunkDbSource->TryTruncateFiles(RemainingChunkCount);
					}
				};
				// Execute the truncating on the same thread that reads chunks from chunkdb to make archive access lock-free.
				QueueGenericThreadTask(ThreadAssignments[ChunkLocation], MoveTemp(TruncateTask));

			}
		}
	}

	CurrentFile.NextChunkPartToRead = EndChunkIdx;
}


void FBuildPatchFileConstructor::StartTruncateChunkDbForCompletedBatch(const FBatchState& Batch)
{
	UE_LOGF(LogBuildPatchServices, Verbose, "Starting async truncating request");
	checkf(Batch.bIsWriting && Batch.bIsFinished || Batch.bIsEmptyFileSentinel, TEXT("Batch must be finished or be an empty file sentinel before truncating chunkdb"));
	for (const TPair<FGuid, FRequestInfo>& RequestPair : Batch.Requests)
	{
		const FRequestInfo& Request = RequestPair.Value;
		if (Request.bSourceIsMemoryStore)
		{
			// Doesn't make sense to continue for data cached in memory.
			continue;
		}

		const EConstructorChunkLocation* ChunkLocationPtr = ChunkLocations.Find(Request.Guid);
		if (ChunkLocationPtr == nullptr)
		{
			continue;
		}

		EConstructorChunkLocation ChunkLocation = *ChunkLocationPtr;
		if (*ChunkLocationPtr != EConstructorChunkLocation::ChunkDb)
		{
			continue; // Only chunk db sources can be truncated.
		}


		IConstructorChunkSource::FRequestProcessFn TruncateTask = [this, BatchRemainingChunkCount = Batch.RemainingChunkCount.load(std::memory_order_acquire)](bool bIsAborted) {
			if (!bIsAborted)
			{
				ChunkDbSource->TryTruncateFiles(BatchRemainingChunkCount);
			}
		};
		// Execute the truncating on the same thread that reads chunks from chunkdb to make archive access lock-free.
		QueueGenericThreadTask(ThreadAssignments[ChunkLocation], MoveTemp(TruncateTask));
	}
}

void FBuildPatchFileConstructor::WakeUpDispatch()
{
	WakeUpDispatchThreadEvent->Trigger();
}

EFileStage FBuildPatchFileConstructor::GetFileStage(const FFileConstructionState& CurrentFile, const FResumeData& ResumeData) const
{
	if (!CurrentFile.bSuccess)
	{
		return EFileStage::Unknown;
	}

	if (Configuration.bUseSHA1StageFilenames)
	{
		// We have to check file existence here
		int64 DiskFileSize;
		if (const bool bFileExists = FileSystem->GetFileSize(*CurrentFile.NewFilename, DiskFileSize))
		{
			const uint64 UnsignedDiskFileSize = DiskFileSize;
			if (UnsignedDiskFileSize == CurrentFile.FileManifest->FileSize)
			{
				return EFileStage::Complete;
			}
			
			if (UnsignedDiskFileSize > 0 && UnsignedDiskFileSize < CurrentFile.FileManifest->FileSize) // we only want to resume if the file hasn't been completed.
			{
				return EFileStage::Incomplete;
			}
			
			if (UnsignedDiskFileSize > CurrentFile.FileManifest->FileSize)
			{
				UE_LOGF(LogBuildPatchFileConstructor, Log, "Deleting incompatible stage file %ls (%ls)", *CurrentFile.BuildFilename, *CurrentFile.NewFilename);
				FileSystem->DeleteFile(*CurrentFile.NewFilename);
			}
			// A zero-size file is treated as EFileStage::New.
		}
	}
	else if(const EFileStage * StagePtr = ResumeData.FilesStage.Find(CurrentFile.BuildFilename))
	{
		// Can only be EFileStage::Complete or EFileStage::Incomplete.
		const EFileStage Stage = *StagePtr; // Avoid SA C6269
		ensure((Stage == EFileStage::Complete) || (Stage == EFileStage::Incomplete));
		return Stage;
	}

	// Treat the file as new.
	return EFileStage::New;
}

void FBuildPatchFileConstructor::InitFile(FFileConstructionState& CurrentFile, const FResumeData& ResumeData)
{
	if (!CurrentFile.bSuccess)
	{
		return;
	}

	// Construct or skip the file.
	if (CurrentFile.Stage == EFileStage::Complete)
	{
		const int64 FileSize = CurrentFile.FileManifest->FileSize;
		CountBytesProcessed(FileSize);

		UE_LOGF(LogBuildPatchServices, Display, "Skipping completed file %ls", *CurrentFile.BuildFilename);
		// Go through each chunk part, and dereference it from the reference tracker.
		for (const FChunkPart& ChunkPart : CurrentFile.FileManifest->ChunkParts)
		{
			if (!ChunkReferenceTracker->PopReference(ChunkPart.Guid))
			{
				CurrentFile.bSuccess = false;
				CurrentFile.ConstructionError = EConstructionError::TrackingError;
				break;
			}
		}
		RemainingChunkCount = ChunkReferenceTracker->GetRemainingChunkCount();

		// Can't skip if we are failing.
		CurrentFile.bSkippedConstruction = CurrentFile.bSuccess;
		return;
	}

	if (CurrentFile.bSuccess &&
		!CurrentFile.FileManifest->SymlinkTarget.IsEmpty())
	{
		// Symlinks are handled separately from file construction
		if (Configuration.bRejectSymlinks)
		{
			CurrentFile.bSuccess = false;
			CurrentFile.ConstructionError = EConstructionError::InvalidSymlink;
		}

		// Can't skip if we are failing.
		CurrentFile.bSkippedConstruction = CurrentFile.bSuccess;
		return;
	}

	if (Configuration.bInstallToMemory || Configuration.bConstructInMemory)
	{
		// Allocate the destination buffer. This is most likely where we would run out of memory, but UE doesn't
		// really have any way to gracefully handle OOM the way we might handle running out of disk space.
		TArray64<uint8> OutputFile;
		OutputFile.SetNumUninitialized(CurrentFile.FileManifest->FileSize);

		MemoryOutputFiles.Add(CurrentFile.BuildFilename, MoveTemp(OutputFile));
	}
}

bool FBuildPatchFileConstructor::HandleInitialDiskSizeCheck(const FFileManifest& FileManifest, int64 StartPosition)
{
	if (Configuration.bInstallToMemory || Configuration.bSkipInitialDiskSizeCheck)
	{
		bInitialDiskSizeCheck = true;
		return true;
	}

	// Returns false if we failed the disk space check.
	if (!bInitialDiskSizeCheck)
	{
		bInitialDiskSizeCheck = true;

		if (Configuration.bInstallToMemory || Configuration.bSkipInitialDiskSizeCheck)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Initial Disk Size check skipped due to the flags: bInstallToMemory: %ls, bSkipInitialDiskSizeCheck: %ls", *LexToString(Configuration.bInstallToMemory), *LexToString(Configuration.bSkipInitialDiskSizeCheck));
			return true;
		}

		// Normal operation can just use the classic calculation
		uint64 LocalDiskSpaceRequired = 0;

		// If we are delete-during-install this gets more complicated because we'll be freeing up
		// space as we add.
		if (Configuration.bDeleteChunkDBFilesAfterUse && ChunkDbSource)
		{
			LocalDiskSpaceRequired = CalculateDiskSpaceRequirementsWithDeleteDuringInstall(StartPosition);
		}
		else
		{
			// Normal operation can just use the classic calculation
			LocalDiskSpaceRequired = CalculateInProgressDiskSpaceRequired(FileManifest, StartPosition);
		}

		const TOptional<FileConstructorHelpers::FDiskSpaceInfo> DiskSpaceInfo = FileConstructorHelpers::GetRemainingDiskSpace(Configuration.InstallDirectory, LocalDiskSpaceRequired);
		if (DiskSpaceInfo.IsSet())
		{
			BackingStore->SetDynamicDiskSpaceHeadroom(LocalDiskSpaceRequired);
		}
		UE_LOGF(LogBuildPatchServices, Display, "Initial Disk Sizes: Required: %ls Available: %ls", *FormatNumber(LocalDiskSpaceRequired), DiskSpaceInfo.IsSet() ? *FormatNumber(DiskSpaceInfo->AvailableDiskSpace) : TEXT("<failed>"));

		const uint64 LocalDiskSpaceAvailable = DiskSpaceInfo.IsSet() ? DiskSpaceInfo->AvailableDiskSpace : 0;
		AvailableDiskSpace.store(LocalDiskSpaceAvailable, std::memory_order_release);
		RequiredDiskSpace.store(LocalDiskSpaceRequired, std::memory_order_release);	

		if (!DiskSpaceInfo.IsSet() || !DiskSpaceInfo->bHaveSufficientFreeSpace)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Insufficient disk space on initial check.");
			InstallerError->SetError(
				EBuildPatchInstallError::OutOfDiskSpace,
				StartPosition != 0 ? DiskSpaceErrorCodes::DuringInstallation : DiskSpaceErrorCodes::InitialSpaceCheck,
				0,
				BuildPatchServices::GetDiskSpaceMessage(Configuration.InstallDirectory, LocalDiskSpaceRequired, LocalDiskSpaceAvailable));
			return false;
		}
	}

	return true;
}

void FBuildPatchFileConstructor::ResumeFile(FFileConstructionState& FileToResume)
{
	if (!FileToResume.bSuccess || FileToResume.bSkippedConstruction)
	{
		return;
	}

	check(!Configuration.bInstallToMemory);
	check(!Configuration.bConstructInMemory);

	// We have to read in the existing file so that the hash check can still be done.
	TUniquePtr<FArchive> NewFileReader(IFileManager::Get().CreateFileReader(*FileToResume.NewFilename));
	if (!NewFileReader.IsValid())
	{
		// We don't fail if we can't read in the previous file - we try and rebuild it from scratch.
		// (Note that the likely outcome here is we can't open the file for write either and fail to
		// install - we're only here if we were supposed to be resuming!)
		return;
	}

	const int32 ReadBufferSize = 4 * 1024 * 1024;
	// Read buffer
	TArray<uint8> ReadBuffer;
	ReadBuffer.SetNumUninitialized(ReadBufferSize);

	// We don't allow resuming mid-chunk for simplicity, so eat entire chunks until
	// we can't anymore.
	int64 OnDiskSize = NewFileReader->TotalSize();
	int64 ByteCounter = 0;
	for (int32 ChunkPartIdx = 0; ChunkPartIdx < FileToResume.FileManifest->ChunkParts.Num() && !bShouldAbort; ++ChunkPartIdx)
	{
		const FChunkPart& ChunkPart = FileToResume.FileManifest->ChunkParts[ChunkPartIdx];
		const int64 NextBytePosition = ByteCounter + ChunkPart.Size;
		ByteCounter = NextBytePosition;
		if (NextBytePosition <= OnDiskSize)
		{
			// Ensure buffer is large enough
			ReadBuffer.SetNumUninitialized(ChunkPart.Size, EAllowShrinking::No);

			{
				FReadScope _(FileConstructorStat, ChunkPart.Size, TEXT("ReadResume"));
				NewFileReader->Serialize(ReadBuffer.GetData(), ChunkPart.Size);
			}

			FileToResume.HashState.Update(ReadBuffer.GetData(), ChunkPart.Size);

			// Update resume start position
			FileToResume.StartPosition = NextBytePosition;
			FileToResume.NextChunkPartToRead = ChunkPartIdx + 1;

			// Inform the reference tracker of the chunk part skip
			if (!ChunkReferenceTracker->PopReference(ChunkPart.Guid))
			{
				FileToResume.bSuccess = false;
				FileToResume.ConstructionError = EConstructionError::TrackingError;
				FileToResume.ErrorContextGuid = ChunkPart.Guid;
				break;
			}

			CountBytesProcessed(ChunkPart.Size);
			FileConstructorStat->OnFileProgress(FileToResume.BuildFilename, NewFileReader->Tell());
			// Wait if paused
			FileConstructorHelpers::WaitWhilePaused(bIsPaused, bShouldAbort);
		}
		else
		{
			// We can't consume any more full chunks from the part list, bail.
			break;
		}
	}
	RemainingChunkCount = ChunkReferenceTracker->GetRemainingChunkCount();

	NewFileReader->Close();
	FileToResume.bIsResumedFile = true;
}

void FBuildPatchFileConstructor::OpenFileToConstruct(FFileConstructionState& CurrentFile)
{
	if (!CurrentFile.bSuccess || CurrentFile.bSkippedConstruction || Configuration.bInstallToMemory || Configuration.bConstructInMemory)
	{
		return;
	}

	// Attempt to create the file
	{
		FAdministrationScope _(FileConstructorStat);
		CurrentFile.NewFile = FileSystem->CreateFileWriter(*CurrentFile.NewFilename, CurrentFile.bIsResumedFile ? EWriteFlags::Append : EWriteFlags::None);
		CurrentFile.CreateFilePlatformLastError = FPlatformMisc::GetLastError();
	}

	CurrentFile.bSuccess = CurrentFile.NewFile != nullptr;
	if (!CurrentFile.bSuccess)
	{
		CurrentFile.ConstructionError = EConstructionError::CannotCreateFile;
		return;
	}

	// Seek to file write position
	if (CurrentFile.NewFile->Tell() != CurrentFile.StartPosition)
	{
		FAdministrationScope _(FileConstructorStat);

		// Currently no way of checking if the seek succeeded. If it didn't and further reads succeed, then
		// we can end up with a bad file on disk and not know it as the hash is assuming this worked - requires
		// full load-and-hash verification to find.
		CurrentFile.NewFile->Seek(CurrentFile.StartPosition);
	}

	CurrentFile.Progress = CurrentFile.StartPosition;
	CurrentFile.LastSeenProgress = CurrentFile.Progress;
}


void FBuildPatchFileConstructor::CompleteConstructedFile(FFileConstructionState& CurrentFile, FResumeData& ResumeData)
{
	UE_LOGF(LogBuildPatchServices, Display, "Completing: %ls", *CurrentFile.BuildFilename);
	bShouldLogNextDependencyWait = true;

	TRACE_CPUPROFILER_EVENT_SCOPE(Constructor_CompleteFile);
	if (!CurrentFile.bSkippedConstruction)
	{
		if (CurrentFile.NewFile) // we don't have a file when constructing/installing in memory.
		{
			if (CurrentFile.bSuccess && CurrentFile.NewFile->IsError())
			{
				// This should aleady have been caught during the write!
				UE_LOGF(LogBuildPatchServices, Warning, "Got serialize error during CompleteConstructedFile! Should have already been caught.");
				CurrentFile.ConstructionError = EConstructionError::SerializeError;
				CurrentFile.bSuccess = false;
			}

			if (CurrentFile.bSuccess)
			{
				// Close the file writer
				bool bArchiveSuccess = false;
				{
					FAdministrationScope _(FileConstructorStat);
					bArchiveSuccess = CurrentFile.NewFile->Close();
					CurrentFile.NewFile.Reset();
				}

				// Check for final success
				if (CurrentFile.bSuccess && !bArchiveSuccess)
				{
					CurrentFile.ConstructionError = EConstructionError::CloseError;
					CurrentFile.bSuccess = false;
				}
			}
		}

		// We can't check for zero locks if we have multiple files in flight because the other
		// files hold locks.
		if (CurrentFile.bSuccess && !bAllowMultipleFilesInFlight)
		{
			if (!BackingStore->CheckNoLocks(false))
			{
				CurrentFile.bSuccess = false;
				CurrentFile.ConstructionError = EConstructionError::InternalConsistencyError;
			}
		}
	
		// Verify the hash for the file that we created
		if (CurrentFile.bSuccess)
		{
			CurrentFile.HashState.Final();

			FSHAHash HashValue;
			CurrentFile.HashState.GetHash(HashValue.Hash);
			CurrentFile.bSuccess = HashValue == CurrentFile.FileManifest->SHA1Hash;
			if (!CurrentFile.bSuccess)
			{
				CurrentFile.ConstructionError = EConstructionError::OutboundDataError;
			}
		}

		if (!Configuration.bInstallToMemory && !Configuration.bConstructInMemory)
		{
			CurrentFile.SetAttributes();
		}

	} // end if we did actual construction work for this file.

	// If we are destructive, remove the old file.
	if (CurrentFile.bSuccess &&
		Configuration.InstallMode == EInstallMode::DestructiveInstall &&
		!Configuration.bInstallToMemory) // Install to memory never patches/has an existing installation dir.
	{
		TMap<FString, FString> EmptyPerFileSubdirectories;
		FString FileToDelete = FBuildPatchUtils::ResolveInstallationFileName(Configuration.InstallDirectory, Configuration.PerFileSubdirectories ? *Configuration.PerFileSubdirectories : EmptyPerFileSubdirectories, CurrentFile.BuildFilename);

		FPaths::NormalizeFilename(FileToDelete);
		FPaths::CollapseRelativeDirectories(FileToDelete);
		if (FileSystem->FileExists(*FileToDelete))
		{
			if (InstallSource && !HarvestChunksForCompletedFile(Configuration.InstallDirectory, CurrentFile.BuildFilename))
			{
				CurrentFile.bSuccess = false;
				CurrentFile.ConstructionError = EConstructionError::InternalConsistencyError;
			} // end if install source exists.

			OnBeforeDeleteFile().Broadcast(FileToDelete);
			{
				// This can take forever due to file system filters. If we throw this on an async
				// job then we can go over our calculated disk space.
				TRACE_CPUPROFILER_EVENT_SCOPE(Delete);
				const bool bRequireExists = false;
				const bool bEvenReadOnly = true;
				IFileManager::Get().Delete(*FileToDelete, bRequireExists, bEvenReadOnly);
			}
		}
	}

	if (CurrentFile.bSuccess && Configuration.bConstructInMemory)
	{
		// Now that we have the entire file ready, we write it in one big pass. This
		// is synchronous to avoid complexity, but we do eat some performance since we
		// aren't overlapping work with these writes.
		
		// This is specifically after the deletion of the old installation file in order to
		// manage disk space requirements - we never want to have two copies of the same file (independent
		// of version) on the disk at the same time - avoiding that is the entire reason we added bConstructInMemory!

		{
			FAdministrationScope _(FileConstructorStat);
			CurrentFile.NewFile = FileSystem->CreateFileWriter(*CurrentFile.NewFilename, EWriteFlags::None);
			CurrentFile.CreateFilePlatformLastError = FPlatformMisc::GetLastError();
		}
		
		CurrentFile.bSuccess = CurrentFile.NewFile != nullptr;
		if (!CurrentFile.bSuccess)
		{
			CurrentFile.ConstructionError = EConstructionError::CannotCreateFile;
		}
		else
		{
			TArray64<uint8>& Data = MemoryOutputFiles[CurrentFile.BuildFilename];

			
			FileConstructorStat->OnBeforeWrite();
			ISpeedRecorder::FRecord ActivityRecord;
			ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
		
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MemoryConstruction_Write)
				int64 WritePosition = 0;
				for (;;)
				{
					if (WritePosition == Data.Num())
					{
						break;
					}
					int64 WriteAmount = 4 << 20; // 4mb write chunks.
					if (WritePosition + WriteAmount > Data.Num())
					{
						WriteAmount = Data.Num() - WritePosition;
					}

					CurrentFile.NewFile->Serialize(Data.GetData() + WritePosition, WriteAmount);

					WritePosition += WriteAmount;
				}
			}

			ActivityRecord.Size = Data.Num();
			ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
			FileConstructorStat->OnAfterWrite(ActivityRecord);

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MemoryConstruction_WriteFree)
				Data.Empty();
			}
			
			if (CurrentFile.NewFile->IsError())
			{
				CurrentFile.bSuccess = false;
				CurrentFile.ConstructionError = EConstructionError::SerializeError;
			}

			bool bArchiveSuccess = true;
			{
				FAdministrationScope _(FileConstructorStat);
				bArchiveSuccess = CurrentFile.NewFile->Close();
				CurrentFile.NewFile.Reset();
			}

			// Check for final success
			if (CurrentFile.bSuccess && !bArchiveSuccess)
			{
				CurrentFile.ConstructionError = EConstructionError::SerializeError;
				CurrentFile.bSuccess = false;
			}
		}

		CurrentFile.SetAttributes();
	}

	RemainingChunkCount = ChunkReferenceTracker->GetRemainingChunkCount();
	if (CurrentFile.bSuccess && ChunkDbSource)
	{
		ChunkDbSource->ReportFileCompletion(RemainingChunkCount);
	}

	FileConstructorStat->OnFileCompleted(CurrentFile.BuildFilename, CurrentFile.bSuccess);
	
	// Report errors.
	if (!CurrentFile.bSuccess)
	{
		const bool bReportAnalytic = InstallerError->HasError() == false;
		switch (CurrentFile.ConstructionError)
		{
		case EConstructionError::OutboundDataError:
			{
				UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Verify failed after constructing %ls", *CurrentFile.BuildFilename);
				CorruptFiles.Add(CurrentFile.BuildFilename);
				break;
			}
		case EConstructionError::FailedWrite:
			{
				// We get this when we fail during write. The problem is our cached values are out of date:
				// AvailableDiskSpace is either 0, because we ran out (likely), or non zero, because of a write error (unlikely).
				// RequiredDiskSpace is not accurate because presumably we've written something. So instead we
				// tell them we failed, and to verify disk space. It's not useful to share the actual value we
				// get back from the operating system since we are either in a write media failure mode, or so
				// close to the edge of free space that values can look like we have enough, but don't.

				uint64 LocalDiskSpaceRequired;
				if (Configuration.bDeleteChunkDBFilesAfterUse && ChunkDbSource)
				{
					LocalDiskSpaceRequired = CalculateDiskSpaceRequirementsWithDeleteDuringInstall(CurrentFile.Progress);
				}
				else
				{
					LocalDiskSpaceRequired = CalculateInProgressDiskSpaceRequired(*CurrentFile.FileManifest, CurrentFile.Progress);
				}

				const TOptional<FileConstructorHelpers::FDiskSpaceInfo> DiskSpaceInfo = FileConstructorHelpers::GetRemainingDiskSpace(Configuration.InstallDirectory, LocalDiskSpaceRequired);
				UE_LOGF(LogBuildPatchServices, Error, "Write failure during installation. Required: %ls Available: %ls", *FormatNumber(LocalDiskSpaceRequired), DiskSpaceInfo.IsSet() ? *FormatNumber(DiskSpaceInfo->AvailableDiskSpace) : TEXT("<failed>"));

				const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
					.SetMinimumFractionalDigits(2)
					.SetMaximumFractionalDigits(2);
				FFormatNamedArguments Arguments;
				Arguments.Emplace(TEXT("RequiredBytes"), FText::AsMemory(LocalDiskSpaceRequired, &FormatOptions, nullptr, EMemoryUnitStandard::SI));
				FText OutOfDiskSpaceDuringInstall = FText::Format(NSLOCTEXT("BuildPatchInstallError", "InstallationWriteFailed", "Low-level failure to write files to disk during installation. Verify there is sufficient disk space. {RequiredBytes} more is required."), Arguments);

				// Note that because it's almost certainly because of disk space, we report it as disk space.
				// We can't be more specific because UE doesn't surface the exact reason for the write failure.
				InstallerError->SetError(
					EBuildPatchInstallError::OutOfDiskSpace,
					DiskSpaceErrorCodes::DuringInstallation,
					0,
					MoveTemp(OutOfDiskSpaceDuringInstall));
				break;
			}
		case EConstructionError::CannotCreateFile:
			{
				if (bReportAnalytic)
				{
					InstallerAnalytics->RecordConstructionError(CurrentFile.BuildFilename, CurrentFile.CreateFilePlatformLastError, TEXT("Could Not Create File"));
				}
				UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Could not create %ls (LastError=%d)", *CurrentFile.BuildFilename, CurrentFile.CreateFilePlatformLastError);
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::FileCreateFail, CurrentFile.CreateFilePlatformLastError);
				break;
			}
		case EConstructionError::CloseError:
			{
				if (bReportAnalytic)
				{
					InstallerAnalytics->RecordConstructionError(CurrentFile.BuildFilename, INDEX_NONE, TEXT("Could Not Close File"));
				}
				UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Could not close %ls", *CurrentFile.BuildFilename);
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::CloseFileError);
				break;
			}
		case EConstructionError::MissingChunk:
			{
				if (bReportAnalytic)
				{
					InstallerAnalytics->RecordConstructionError(CurrentFile.BuildFilename, INDEX_NONE, TEXT("Missing Chunk"));
				}

				UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Failed %ls due to missing chunk %ls (can be 0000 if unknown)", *CurrentFile.BuildFilename, *WriteToString<64>(CurrentFile.ErrorContextGuid));
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::MissingChunkData);
				break;
			}
		case EConstructionError::SerializeError:
			{
				if (bReportAnalytic)
				{
					InstallerAnalytics->RecordConstructionError(CurrentFile.BuildFilename, INDEX_NONE, TEXT("Serialization Error"));
				}
				UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Failed %ls due to serialization error", *CurrentFile.BuildFilename);
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::SerializationError);
				break;
			}
		case EConstructionError::TrackingError:
			{
				if (bReportAnalytic)
				{
					InstallerAnalytics->RecordConstructionError(CurrentFile.BuildFilename, INDEX_NONE, TEXT("Tracking Error"));
				}
				UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Failed %ls due to untracked chunk %ls (can be 0000 if unknown)", *CurrentFile.BuildFilename, *WriteToString<64>(CurrentFile.ErrorContextGuid));
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::TrackingError);
				break;
			}
		case EConstructionError::InvalidSymlink:
			{
				if (bReportAnalytic)
				{
					InstallerAnalytics->RecordConstructionError(CurrentFile.BuildFilename, INDEX_NONE, TEXT("Symlink present when not allowed"));
				}
				UE_LOGF(LogBuildPatchFileConstructor, Error, "File manifest includes symlinks when they are forbidden %ls", *CurrentFile.BuildFilename);
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::SymlinkCreateForbidden);
				break;
			}
		case EConstructionError::InternalConsistencyError:
			{
				if (bReportAnalytic)
				{
					InstallerAnalytics->RecordConstructionError(CurrentFile.BuildFilename, INDEX_NONE, TEXT("Internal Consistency Error"));
				}
				UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Failed %ls due to internal consistency checking failure", *CurrentFile.BuildFilename);
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::InternalConsistencyFailure);
				break;
			}
		case EConstructionError::MissingFileInfo:
			{
				if (bReportAnalytic)
				{
					InstallerAnalytics->RecordConstructionError(CurrentFile.BuildFilename, INDEX_NONE, TEXT("Missing File Manifest"));					
				}
				UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: Missing file manifest for %ls", *CurrentFile.BuildFilename);
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::MissingFileInfo);
				break;
			}
		case EConstructionError::FailedInitialSizeCheck:
			{
				// Error already set back when we had the info.

				// If we failed the initial disk space check before we did anything,
				// delete the resume data so that we don't leave any trash. We know there wasn't any existing resume
				// data because we failed on the first file at the beginning of the file.
				if (CurrentFile.ConstructionIndex == 0 &&
					CurrentFile.StartPosition == 0)
				{
					ResumeData.RemoveResumeData();
				}
				break;
			}
		case EConstructionError::Aborted:
			{
				// We don't set errors on abort.
				break;
			}
		}

		// Delete the staging file if unsuccessful by means of any failure that could leave the file in unknown state.
		switch (CurrentFile.ConstructionError)
		{
			// Errors we expect that we can conceptually resume:
			case EConstructionError::FailedWrite:
			case EConstructionError::MissingChunk:
			case EConstructionError::Aborted:
			{
				break;
			}

			// Errors we expect there to be issues with the outbound file:
			case EConstructionError::CannotCreateFile:
			case EConstructionError::CloseError:
			case EConstructionError::SerializeError:
			case EConstructionError::TrackingError:
			case EConstructionError::OutboundDataError:
			case EConstructionError::InternalConsistencyError:
			{
				if (!FileSystem->DeleteFile(*CurrentFile.NewFilename))
				{
					UE_LOGF(LogBuildPatchServices, Warning, "FBuildPatchFileConstructor: Error deleting file: %ls (Error Code %i)", *CurrentFile.NewFilename, FPlatformMisc::GetLastError());
				}
				break;
			}
		}

		// Before aborting the installation, we need to identify and process all corrupted files.
		if (CurrentFile.ConstructionError != EConstructionError::OutboundDataError)
		{
			// Stop trying to install further files.
			Abort();
		}
	}
	else
	{
		OnFileIsStaged().Broadcast(CurrentFile.BuildFilename);
	}
}

void FBuildPatchFileConstructor::StartFile(FFileConstructionState& CurrentFile, const FResumeData& ResumeData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Constructor_StartFile);

	InitFile(CurrentFile, ResumeData);

	if (CurrentFile.bSkippedConstruction)
	{
		return;
	}

	if (CurrentFile.Stage == EFileStage::Incomplete && CurrentFile.bSuccess)
	{
		ResumeFile(CurrentFile);

		// Generally speaking we only expect there to be one file to resume (as of this writing there
		// is no way for the FilesStarted set to have more than one file), so we update the download requirements
		// after the first one.

		// Need to sum up the size of all remaining chunks that we need. We can't just look at the
		// chunk locations because we don't know if it's still needed or not.
		TSet<FGuid> RemainingNeededChunks = ChunkReferenceTracker->GetReferencedChunks();

		uint64 NewDownloadRequirement = 0;
		FRWScopeLock ChunkLock(ChunkLocationsLock, SLT_Write); // write because we're updating the download requirement
		for (FGuid& NeededChunk : RemainingNeededChunks)
		{
			EConstructorChunkLocation* LocationPtr = ChunkLocations.Find(NeededChunk);
			if (LocationPtr && *LocationPtr == EConstructorChunkLocation::Cloud)
			{
				NewDownloadRequirement += Configuration.ManifestSet->GetChunkInfo(NeededChunk)->FileSize;
			}
		}

		DownloadRequirement = NewDownloadRequirement;
		CloudSource->PostRequiredByteCount(DownloadRequirement);
	}

	// If we haven't done so yet, make the initial disk space check. We do this after resume
	// so that we know how much to discount from our current file size.
	if (!HandleInitialDiskSizeCheck(*CurrentFile.FileManifest, CurrentFile.StartPosition))
	{
		CurrentFile.bSuccess = false;
		CurrentFile.ConstructionError = EConstructionError::FailedInitialSizeCheck;
	}

	if (!bIsDownloadStarted)
	{
		bIsDownloadStarted = true;
		FileConstructorStat->OnResumeCompleted();
	}

	OpenFileToConstruct(CurrentFile);

}

void FBuildPatchFileConstructor::ConstructFiles(FResumeData& ResumeData)
{
	MaxWriteBatchSize = FFileConstructorConfig::DefaultIOBatchSizeMB;
	if (GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ConstructorIOBatchSizeMB"), (int32&)MaxWriteBatchSize, GEngineIni))
	{
		UE_LOGF(LogBuildPatchServices, Verbose, "Got INI ConstructorIOBatchSizeMB = %d", MaxWriteBatchSize);
	}
	if (Configuration.IOBatchSizeMB.IsSet())
	{
		MaxWriteBatchSize = FMath::Max(1, Configuration.IOBatchSizeMB.Get(MaxWriteBatchSize));
		UE_LOGF(LogBuildPatchServices, Verbose, "Got override ConstructorIOBatchSizeMB = %d", MaxWriteBatchSize);
	}
	MaxWriteBatchSize <<= 20; // to MB;

	IOBufferSize = FFileConstructorConfig::DefaultIOBufferSizeMB;
	if (GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ConstructorIOBufferSizeMB"), (int32&)IOBufferSize, GEngineIni))
	{
		UE_LOGF(LogBuildPatchServices, Verbose, "Got INI ConstructorIOBufferSizeMB = %d", IOBufferSize);
	}
	if (Configuration.IOBufferSizeMB.IsSet())
	{
		IOBufferSize = FMath::Max(1, Configuration.IOBufferSizeMB.Get(IOBufferSize));
		UE_LOGF(LogBuildPatchServices, Verbose, "Got override ConstructorIOBufferSizeMB = %d", IOBufferSize);
	}
	IOBufferSize <<= 20; // to MB;

	// This is the buffer we issue reads into and writes out of. We segment it up
	// based on batch sizing.	
	TArray<uint8> IOBuffer;

	// Ensure that our batch size can always make progress (all chunks can fit)
	if (!Configuration.bInstallToMemory &&
		!Configuration.bConstructInMemory)
	{
		uint32 LargestChunkSize = 0;
		for (const FFileToConstruct& FileToConstruct : ConstructionList)
		{
			if (FileToConstruct.FileManifest)
			{
				for (const FChunkPart& ChunkPart : FileToConstruct.FileManifest->ChunkParts)
				{
					if (ChunkPart.Size > LargestChunkSize)
					{
						LargestChunkSize = ChunkPart.Size;
					}
				}
			}
		}

		if (LargestChunkSize > MaxWriteBatchSize)
		{
			MaxWriteBatchSize = LargestChunkSize;
			UE_LOGF(LogBuildPatchServices, Display, "Increasing batch size to fit any chunk size: %u bytes", MaxWriteBatchSize);

			if (MaxWriteBatchSize > IOBufferSize)
			{
				IOBufferSize = MaxWriteBatchSize;
				UE_LOGF(LogBuildPatchServices, Display, "Increasing IO buffer size to fit batch size: %u bytes", IOBufferSize);
			}
		}

		IOBuffer.SetNumUninitialized(IOBufferSize);
	}

	uint64 ConstructStartCycles = FPlatformTime::Cycles64();
	uint64 WriteDoneCycles = 0;
	uint64 ReadDoneCycles = 0;
	uint64 ReadCheckCycles = 0;
	uint64 CloudTickCycles = 0;
	uint64 HashCycles = 0;
	uint64 WaitCycles = 0;
	uint64 WriteStartCycles = 0;

	// List of files that are currently opened. The last one is the one we are starting reads on,
	// the first one is the one we are currently writing.
	TArray<TUniquePtr<FFileConstructionState>> ActiveFiles;

	// List of batches in flight. These must be dispatched in order.
	TArray<TUniquePtr<FBatchState>> Batches;

	struct Range
	{
		uint64 Start=0, End=0;
	};
	TArray<Range, TInlineAllocator<3>> IOBufferFreeBlockList;
	IOBufferFreeBlockList.Add({0, (uint64)IOBuffer.Num()});

	auto SortAndCoalesceFreeList = [](TArray<Range, TInlineAllocator<3>>& FreeList)
	{
		// Sort the list on Start.
		Algo::SortBy(FreeList, &Range::Start, TLess<uint64>());

		// go over everything and coalesce anything that's adjacent.
		for (int32 i=0; i<FreeList.Num() - 1; i++)
		{
			if (FreeList[i].End == FreeList[i+1].Start)
			{
				// Extend us.
				FreeList[i].End = FreeList[i+1].End;

				// Remove them.
				FreeList.RemoveAt(i+1);

				// Recheck us.
				i--;
			}
		}
	};

	//
	// The abort handling with this loop is:
	// During the loop, if we see the abort signal, we mark all active files as failed.
	// That prevents any new work from starting, but we have to have the outstanding work
	// complete in order to be thread safe. Once that work completes, we then break out of the loop
	// (bDoneWithFiles will still be true).
	//
	bool bStillProcessingFiles = true;
	for (;bStillProcessingFiles;)
	{		
		bool bHasAnyFileFailed = false;
		// Check state. We can continue to run this after a failure has been encountered in order
		// to drain any async tasks so we know we can shut down safely.
		{
			// Is our next write done?
			if (Batches.Num() &&
				((Batches[0]->bIsWriting && Batches[0]->bIsFinished.load(std::memory_order_acquire)) || Batches[0]->bIsEmptyFileSentinel))
			{
				uint64 WriteDoneStartCycles = FPlatformTime::Cycles64();

				FFileConstructionState* File = Batches[0]->OwningFile;
				File->OutstandingBatches--;

				if (!Configuration.bInstallToMemory && 
					!Configuration.bConstructInMemory)
				{
					// Could be empty buffer if sentinel
					if (Batches[0]->BatchBuffer.GetSize())
					{
						uint64 BufferBase = (uint64)((uint8*)Batches[0]->BatchBuffer.GetData() - (uint8*)IOBuffer.GetData());

						IOBufferFreeBlockList.Add({BufferBase, BufferBase + Batches[0]->BatchBuffer.GetSize()});
						SortAndCoalesceFreeList(IOBufferFreeBlockList);
					}
				
					if (File->NewFile->IsError())
					{
						File->bSuccess = false;

						// This is not great, but right now the FArchive can't report what the error was. Without any
						// way to tease it apart, we run with the most likely issue: we ran out of disk space. Unfortunately
						// it's unreliable to ask the file system what is available and check if it's low and use that for the 
						// conversion.
						File->ConstructionError = EConstructionError::FailedWrite;
					}
					else
					{
						StartTruncateChunkDbForCompletedBatch(*Batches[0]);
					}
				}

				Batches.RemoveAt(0);
				
				// If we have nothing further to read and all the file's reads are done,
				// since we only ever have 1 write batch we know it must be done.
				if (File->NextChunkPartToRead == File->FileManifest->ChunkParts.Num() &&
					File->OutstandingBatches == 0)
				{
					CompleteConstructedFile(*File, ResumeData);

					UE_LOGF(LogBuildPatchServices, Verbose, "Completed file: %ls, New ActiveFileCount = %d", *File->BuildFilename, ActiveFiles.Num() - 1);

					// Since we write in order, we must be the first in the queue.
					check (ActiveFiles[0].Get() == File);

					// This frees the file
					ActiveFiles.RemoveAt(0);

					// We completed the previous file, if there's another active file we need to tell
					// the stats that it has started. If there isn't, we'll fire it off when we start it.
					if (ActiveFiles.Num())
					{
						FileConstructorStat->OnFileStarted(ActiveFiles[0]->BuildFilename, ActiveFiles[0]->FileManifest->FileSize);
					}
				}

				WriteDoneCycles += FPlatformTime::Cycles64() - WriteDoneStartCycles;
			}

			// Check for completed reads.
			if (Batches.Num())
			{
				uint64 ReadDoneStartCycles = FPlatformTime::Cycles64();

				// We have to retire reads in order because later reads could be waiting for data to
				// get placed correctly by the earlier one. e.g. read 1 could be placing in the memory store
				// and read 2 could be wanting to read from that when it retires.
				// Since we queue the batches in order, we can look at the first reading batch and know it's the next one.
				for (TUniquePtr<FBatchState>& Batch : Batches)
				{
					if (Batch->bIsReading)
					{
						if (Batch->bIsFinished.load(std::memory_order_acquire))
						{
							FFileConstructionState* File = Batch->OwningFile;

							CompleteReadBatch(*File->FileManifest, *Batch.Get());
							Batch->bIsReading = false;
							Batch->bNeedsWrite = true;

							// Only update the file construction error if it's the first one
							// since errors can cascade and make it not clear what the original problem was.
							if (File->ConstructionError == EConstructionError::None &&
								Batch->Error != EConstructionError::None)
							{
								File->ErrorContextGuid = Batch->ErrorContextGuid;
								File->ConstructionError = Batch->Error;
								File->bSuccess = false;
							}
						}
						else
						{
							// We have to retire reads in order, so once we hit one that isn't finished we stop.
							break;
						}
					}
					// else - might be a writing batch before the first read.
				}

				ReadDoneCycles += FPlatformTime::Cycles64() - ReadDoneStartCycles;
			}

			// If we've aborted, fail the files.
			for (TUniquePtr<FFileConstructionState>& File : ActiveFiles)
			{
				if (File->bSuccess &&
					bShouldAbort)
				{
					File->ConstructionError = EConstructionError::Aborted;
					File->bSuccess = false;
				}

				bHasAnyFileFailed |= !File->bSuccess;
			}

			// If we are in a failure case, we want to clear out anything happening with the cloud source.
			// We'll hit this over and over as we "drain", but we need to do that anyway because further failures
			// might try to queue more cloud reads.
			if (bHasAnyFileFailed)
			{
				CloudSource->Abort();
			}

			if (!bIsPaused && !bHasAnyFileFailed)
			{
				// Can we start another read?
				// LONGTERM - if we are memory constrained and have an install source we can consider only dispatching
				// single reads. This dramatically lowers our memory requirements as install sources often use a lot
				// of small pieces of chunks - and we have to load the entire chunk into memory.
				bool bCheckForAnotherRead = true;
				while (bCheckForAnotherRead)
				{
					double ReadCheckStartCycles = FPlatformTime::Cycles64();

					TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_ReadCheck);

					// We only check for another if we got one queued so we fill up the buffer space asap.
					bCheckForAnotherRead = false;

					bool bNeedsIoBufferSpace = !Configuration.bInstallToMemory && !Configuration.bConstructInMemory;
					
					int32 BiggestFreeBlockSize = 0;
					int32 BiggestFreeBlockSlotIndex = 0;
					if (bNeedsIoBufferSpace)
					{
						for (int32 Slot = 0; Slot < IOBufferFreeBlockList.Num(); Slot++)
						{
							if (IOBufferFreeBlockList[Slot].End - IOBufferFreeBlockList[Slot].Start > BiggestFreeBlockSize)
							{
								BiggestFreeBlockSize = IOBufferFreeBlockList[Slot].End - IOBufferFreeBlockList[Slot].Start;
								BiggestFreeBlockSlotIndex = Slot;
							}
						}
					}

					if (!bNeedsIoBufferSpace || BiggestFreeBlockSize > 0)
					{
						// Default to continue to work on the last file.
						FFileConstructionState* FileToStart = nullptr;
						int32 ActiveFileCount = ActiveFiles.Num();
						if (ActiveFileCount && ActiveFiles[ActiveFileCount-1]->NextChunkPartToRead < ActiveFiles[ActiveFileCount-1]->FileManifest->ChunkParts.Num())
						{
							FileToStart = ActiveFiles[ActiveFileCount-1].Get();
						}

						// No more work do to on active files - is there another to start?
						if (!FileToStart)
						{
							// This is not a race because the only place we ever increment is this thread, this function right
							// below.
							int32 IndexToConstruct = NextIndexToConstruct.load(std::memory_order_acquire);
							bool bAnotherFileExists = IndexToConstruct < ConstructionList.Num();

							bool bAllowAnotherFile = ActiveFiles.Num() == 0;
							if (!bAllowAnotherFile && bAnotherFileExists)
							{
								if (bAllowMultipleFilesInFlight)
								{
									bAllowAnotherFile = true;

									// The IOBuffer size (i.e. memory) is what limits how many files can be in flight. 
									// For normal operation, this happens implicitly below when we look for a read destination,
									// but for situations where we are writing directly to the destination, we need to explictly
									// govern this.
									if (Configuration.bInstallToMemory || Configuration.bConstructInMemory)
									{
										// For install to memory, we are going to eventually have anything in memory _anyway_,
										// but this also limits how many requests get queued up so we still limit.
										// For construct in memory, if we don't limit here it'll freely allocate the entire
										// installation when we almost certainly don't want that.
										//
										// Note that the IOBuffer isn't actually getting allocated in these situations - but
										// we can use the user provided limitation as a limit on how much to have in flight.
										uint64 TotalActiveData = 0;
										for (const TUniquePtr<FFileConstructionState>& ActiveFile : ActiveFiles)
										{
											const TArray64<uint8>& Data = MemoryOutputFiles[ActiveFile->BuildFilename];
											TotalActiveData += Data.Num();
										}

										// Be sure to include the size of the next file. We don't get here at all
										// if there are no active files.
										uint64 NeededData = ConstructionList[IndexToConstruct].FileManifest->FileSize;

										bAllowAnotherFile = (TotalActiveData + NeededData) < IOBufferSize;
									}
								}
							}

							
							if (bAllowAnotherFile && bAnotherFileExists)
							{
								// Even though another file exists, we might not be able to start it if its dependent
								// files aren't done.
								bool bDelayForDependencies = false;
								if (ConstructionList[IndexToConstruct].LatestDependentInstallSource != -1)
								{
									// Files are in construct order - if the first one is after our last dependency then we know we
									// are safe.
									if (ActiveFileCount && ActiveFiles[0]->ConstructionIndex <= ConstructionList[IndexToConstruct].LatestDependentInstallSource)
									{
										if (bShouldLogNextDependencyWait)
										{
											bShouldLogNextDependencyWait = false;

											UE_LOGF(LogBuildPatchServices, Display, "Delaying %ls, active head is %ls, waiting for %ls %d/%d", 
												*Configuration.ConstructList[IndexToConstruct],
												*Configuration.ConstructList[ActiveFiles[0]->ConstructionIndex],
												*Configuration.ConstructList[ConstructionList[IndexToConstruct].LatestDependentInstallSource],
												ActiveFiles[0]->ConstructionIndex,
												ConstructionList[IndexToConstruct].LatestDependentInstallSource
											);
										}

										bDelayForDependencies = true;
									}
								}

								if (!bDelayForDependencies)
								{
									const FString &StagingFileName = ConstructionList[IndexToConstruct].StagingFilename;
									TUniquePtr<FFileConstructionState> AnotherFileCandidate = MakeUnique<FFileConstructionState>(
										ConstructionList[IndexToConstruct].FileManifest,
										Configuration.ConstructList[IndexToConstruct],
										Configuration.StagingDirectory / StagingFileName);

									AnotherFileCandidate->Stage = GetFileStage(*AnotherFileCandidate, ResumeData);
									
									// The complete file will modify ChunkReferenceTracker, which should be done sequentially after processing active files.
									const bool bDelayForCompleteFile = AnotherFileCandidate->Stage == EFileStage::Complete && ActiveFiles.Num() > 0;

									if (!bDelayForCompleteFile)
									{
										// Now commit this file since we are starting it.
										NextIndexToConstruct.fetch_add(1, std::memory_order_acq_rel);

										// This log has to be here for tests to succeed, but we don't want spam so we make it Verbose.
										UE_LOGF(LogBuildPatchFileConstructor, Log, "Constructing file %ls (%ls)", *Configuration.ConstructList[IndexToConstruct], *StagingFileName);

										TUniquePtr<FFileConstructionState> AnotherFile = MoveTemp(AnotherFileCandidate);

										AnotherFile->BaseReferenceIndex = 0;
										if (IndexToConstruct)
										{
											AnotherFile->BaseReferenceIndex = FileCompletionPositions[IndexToConstruct - 1];
										}

										AnotherFile->ConstructionIndex = IndexToConstruct;

										TRACE_BOOKMARK(TEXT("Starting File: %s [%u]"), *AnotherFile->BuildFilename, AnotherFile->FileManifest->ChunkParts.Num());

										UE_LOGF(LogBuildPatchServices, Display, "Starting File: %ls [%ls bytes, %d chunks], New ActiveFileCount = %d",
											*AnotherFile->BuildFilename,
											*FormatNumber(AnotherFile->FileManifest->FileSize),
											AnotherFile->FileManifest->ChunkParts.Num(),
											ActiveFiles.Num() + 1);

										StartFile(*AnotherFile.Get(), ResumeData);

										if (AnotherFile->bSkippedConstruction)
										{
											// Nothing else needs to happen with this file - we'll loop around to try for another.
											bCheckForAnotherRead = true;
										}
										else
										{
											// Only start the file if there's not currently an active file - otherwise we
											// do it when the current one finishes.
											if (ActiveFileCount == 0)
											{
												FileConstructorStat->OnFileStarted(AnotherFile->BuildFilename, AnotherFile->FileManifest->FileSize);
											}

											FileToStart = AnotherFile.Get();
											ActiveFiles.Push(MoveTemp(AnotherFile));
										}
									}
									else
									{
										UE_LOGF(LogBuildPatchServices, Verbose, "Delaying %ls due to it complete and should be processed after active files.", *Configuration.ConstructList[IndexToConstruct]);
									}
								} // end if not delaying for dependencies

								// If we are delaying for dependencies we fall through here with FileToStart = nullptr
								// and nothing happens until we check again when the construct thread is woken up again.
							}
							else
							{
								// If we don't have any files to construct, then we can't start any more reads at all.
								// If there are no active files, then we are done.
								if (ActiveFiles.Num() == 0)
								{
									bStillProcessingFiles = false;
								}
							}
						} // end if we are starting a new file.

						// It's possible the file failed during creation and we need to start the failure process. We need to
						// set this since we did the scan before here.
						if (FileToStart && !FileToStart->bSuccess)
						{
							bHasAnyFileFailed = true;
						}

						if (FileToStart && FileToStart->bSuccess)
						{
							if (FileToStart->FileManifest->ChunkParts.Num() == 0)
							{
								// We have a file that will never launch any batches, which means it'll never hit the finalization
								// logic. We can't complete it here because then we are out of order. So we need to inject a placeholder
								// batch that will auto pass the write check and also prevent us from sleeping on an event.
								TUniquePtr<FBatchState> Batch = MakeUnique<FBatchState>();
								Batch->bNeedsWrite = false;
								Batch->bIsReading = false;
								Batch->OwningFile = FileToStart;
								Batch->bIsEmptyFileSentinel = true;
								Batch->bIsFinished.store(false, std::memory_order_release);
								FileToStart->OutstandingBatches++;

								Batches.Add(MoveTemp(Batch));

								bCheckForAnotherRead = true;
							}
							else if (Configuration.bInstallToMemory || Configuration.bConstructInMemory)
							{
								// When installing to memory we can start a batch for the entirety of the file.
								TArray64<uint8>& Output = MemoryOutputFiles[FileToStart->BuildFilename];

								// \todo break up the batches a bit so that the hash overlaps with other
								// work. Right now its a giant hit on the constructor thread.

								TUniquePtr<FBatchState> Batch = MakeUnique<FBatchState>();
								Batch->bNeedsWrite = true;
								Batch->bIsReading = true;
								Batch->OwningFile = FileToStart;
								Batch->bIsFinished.store(false, std::memory_order_release);

								Batch->BatchBuffer = FMutableMemoryView(Output.GetData(), FileToStart->FileManifest->FileSize);

								StartReadBatch(*FileToStart, *Batch.Get());
	
								// This should use the entire buffer. If it doesn't we've violated some assumptions here.
								if (Batch->BatchBuffer.GetSize() != FileToStart->FileManifest->FileSize)
								{
									FileToStart->bSuccess = false;
									FileToStart->ConstructionError = EConstructionError::InternalConsistencyError;
									bHasAnyFileFailed = true;
																	
									UE_LOGF(LogBuildPatchServices, Error, "Memory construction setup failed - batch buffer didn't fill entirely: Expected %llu, got %llu. File %ls",
										FileToStart->FileManifest->FileSize,
										Batch->BatchBuffer.GetSize(),
										*FileToStart->BuildFilename);
								}
								else
								{
									bCheckForAnotherRead = true;								
									Batches.Push(MoveTemp(Batch));
								}
							}
							else
							{
								uint32 MaxBufferSize = BiggestFreeBlockSize;
								if (MaxBufferSize > MaxWriteBatchSize)
								{
									MaxBufferSize = MaxWriteBatchSize;
								}

								// We want to do big batches as much as possible. If we only have space for a single chunk,
								// that's fine if it's a single chunk file, but generally we want to try and favor large batches.
								// If we're at the point where we are worried about this, we have enough outstanding work to keep
								// the pipelines full so we can afford to wait for room.
								uint32 MaxFileBatchSize = 0;
								const TArray<FChunkPart>& FileChunkParts = FileToStart->FileManifest->ChunkParts;
								uint32 NextChunkSize = FileChunkParts[FileToStart->NextChunkPartToRead].Size;

								for (int32 ChunkPartIndex = FileToStart->NextChunkPartToRead;
									ChunkPartIndex < FileChunkParts.Num();
									ChunkPartIndex++)
								{
									MaxFileBatchSize += FileChunkParts[ChunkPartIndex].Size;
									if (MaxFileBatchSize >= MaxWriteBatchSize)
									{
										// If it's big enough for the max batch size we no longer care.
										MaxFileBatchSize = MaxWriteBatchSize;
										break;
									}
								}

								// If the file can support a large batch, we want to wait until we have room for a reasonable size.
								// We know we can eventually read the next chunk because during init we sized the buffers such that
								// we could.
								if (NextChunkSize > MaxBufferSize &&
									Batches.Num() == 0)
								{
									FileToStart->bSuccess = false;
									FileToStart->ConstructionError = EConstructionError::InternalConsistencyError;
									UE_LOGF(LogBuildPatchServices, Error, "Chunk size encountered larger than batch buffer size! %u vs %u", NextChunkSize, MaxBufferSize);
									bHasAnyFileFailed = true;

									// We'll fail the next conditional below and then start the failure process on the next loop.
								}

								if (MaxBufferSize >= MaxFileBatchSize)
								{
									TUniquePtr<FBatchState> Batch = MakeUnique<FBatchState>();
									Batch->bNeedsWrite = true;
									Batch->bIsReading = true;
									Batch->OwningFile = FileToStart;
									Batch->BatchBuffer = FMutableMemoryView(IOBuffer.GetData() + IOBufferFreeBlockList[BiggestFreeBlockSlotIndex].Start, MaxBufferSize);
									Batch->bIsFinished.store(false, std::memory_order_release);
									StartReadBatch(*FileToStart, *Batch.Get());

									// The read might not have used the whole thing, so only consume as much
									// as it needed.
									IOBufferFreeBlockList[BiggestFreeBlockSlotIndex].Start += Batch->BatchBuffer.GetSize();
									if (IOBufferFreeBlockList[BiggestFreeBlockSlotIndex].Start == IOBufferFreeBlockList[BiggestFreeBlockSlotIndex].End)
									{
										// Ate whole thing
										IOBufferFreeBlockList.RemoveAt(BiggestFreeBlockSlotIndex);
										SortAndCoalesceFreeList(IOBufferFreeBlockList);
									}

									bCheckForAnotherRead = true;
								
									UE_LOGF(LogBuildPatchServices, VeryVerbose, "Starting ReadBatch: %d, Chunks=%d, Bytes=%ls, Batches=%d", 
										Batch->BatchId, 
										Batch->ChunkCount, 
										*FormatNumber(Batch->BatchBuffer.GetSize()),
										Batches.Num() + 1
									);

									Batches.Push(MoveTemp(Batch));
								}
							} // end if file has parts
						} // end if we have a file to read from.
					} // end if free block exists.

					ReadCheckCycles += FPlatformTime::Cycles64() - ReadCheckStartCycles;
				} // end looping on whether we should start a read.

				// Can we start a write? 
				// We always have to issue writes in order and there can only be one, so it must be the
				// first active file, and the first batch. 
				if (Batches.Num() && Batches[0]->bNeedsWrite && !Batches[0]->bIsReading)
				{
					uint64 WriteStartStartCycles = FPlatformTime::Cycles64();

					FFileConstructionState* FirstFile = ActiveFiles[0].Get();
					if (Configuration.bInstallToMemory || Configuration.bConstructInMemory)
					{
						// We read directly into the output - nothing needs to be done. However we need to
						// catch the write completion logic so we mark bIsFinished and signal our wakeup
						// event so we immediately do another pass
						Batches[0]->bNeedsWrite = false;
						Batches[0]->bIsWriting = true;
						Batches[0]->bIsFinished.store(true, std::memory_order_release);
						WakeUpDispatchThreadEvent->Trigger();
					}
					else
					{
						Batches[0]->bNeedsWrite = false;
						Batches[0]->bIsWriting = true;
						Batches[0]->bIsFinished.store(false, std::memory_order_release);

						// Launch the write and hash the buffer on this thread.
						UE_LOGF(LogBuildPatchServices, VeryVerbose, "Writing Batch: %d, file %ls [%d - %d]",
							Batches[0]->BatchId, 
							*FirstFile->BuildFilename,
							Batches[0]->StartChunkPartIndex,
							Batches[0]->StartChunkPartIndex + Batches[0]->ChunkCount
						);

						IConstructorChunkSource::FRequestProcessFn WriteFn = CreateWriteRequest(FirstFile->NewFile.Get(), *Batches[0].Get());
						QueueGenericThreadTask(WriteThreadIndex, MoveTemp(WriteFn));
					}
					
					uint64 HashStartCycles = FPlatformTime::Cycles64();
					WriteStartCycles += HashStartCycles - WriteStartStartCycles;

					{
						TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_Hash);
						FirstFile->HashState.Update((const uint8*)Batches[0]->BatchBuffer.GetData(), Batches[0]->BatchBuffer.GetSize());
					}
					HashCycles += FPlatformTime::Cycles64() - HashStartCycles;

				} // end if checking for write
			} // end not paused
		} // end state check

		// If the file progress changed since we last saw it, post the update.
		// Note that we want this to update reasonably often but we're about to wait
		// potentially until all reads complete - however the only time things actually
		// take a long time wall-clock wise is when we are downloading, and the cloud
		// source will then prevent us from sleeping too long, so we actually catch these
		// updates.
		// We do this from here to ensure we always increase rather than risk multi thread
		// races.
		if (ActiveFiles.Num())
		{
			// We only post the progress for the first file in the active list - this means
			// that when we finish that file we'll likely jump to the middle progress for the next
			// file, but we don't have a way to post the progress per file.
			FFileConstructionState* File = ActiveFiles[0].Get();

			uint64 CurrentFileProgress;
			File->ProgressLock.Lock();
			CurrentFileProgress = File->Progress;
			File->ProgressLock.Unlock();
					
			if (CurrentFileProgress != File->LastSeenProgress)
			{
				// this updates the overall install progress.
				CountBytesProcessed(CurrentFileProgress - File->LastSeenProgress);
				File->LastSeenProgress = CurrentFileProgress;
				FileConstructorStat->OnFileProgress(File->BuildFilename, CurrentFileProgress);
			}
		}

		uint32 WaitTimeMs = TNumericLimits<uint32>::Max();
		{
			uint64 CloudTickStartCycles = FPlatformTime::Cycles64();

			TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_TickCloud);

			// Max downloads is tricky - the internet makes no guarantees about which of our downloads finishes first. So while
			// we want to have as many outstanding as possible to cover up connection overhead / resends and all that, if we enqueue 
			// downloads from several batches, we can end up where the first batch can't complete because it's waiting on a download 
			// that isn't finishing due to congestion from the next batch's download. Then we counterintuitively end up with FEWER 
			// outstanding downloads because we can't launch more batches due to waiting on the front of this long chain. This is easily reproducible
			// where Insights will show highly out of order completion. This ordering unfortunately scales with the number of
			// outstanding downloads to a certain extent: if you allow 16 downloads then you'll end up waiting on a download
			// 16 issues old - 32 downloads and you'll wait on one 32 issues old. We try to bound this by capping the issued
			// downloads here, and in the cloud source we prevent queues if we get too far ahead of the last download.
			// Note this also gets adjusted by the connection health stuff internal to the cloud source.
			uint32 MaxDownloads = FMath::Max(1U, FMath::DivideAndRoundUp(MaxWriteBatchSize, ExpectedChunkSize));

			// If we are reading directly, then we don't have to worry about individual batches anymore - but we don't want to wait
			// too long at the end of each file either. We expose IoBuffer as a conceptual limit to how much the user wants going
			// on in these cases, so we use it here as well.
			if (Configuration.bInstallToMemory || Configuration.bConstructInMemory)
			{
				MaxDownloads = FMath::Max(1U, FMath::DivideAndRoundUp(IOBufferSize, ExpectedChunkSize));
			}

			
			// WaitTimeMs is an OUT param
			CloudSource->Tick(!bIsPaused && !bHasAnyFileFailed, WaitTimeMs, MaxDownloads);

			CloudTickCycles += FPlatformTime::Cycles64() - CloudTickStartCycles;
		}

		int32 ActiveReadBatches = 0;
		int32 ActiveWriteBatches = 0;
		int32 EmptyFileBatches = 0;
		for (const TUniquePtr<FBatchState>& _Batch : Batches)
		{
			EmptyFileBatches += _Batch->bIsEmptyFileSentinel ? 1 : 0;
			ActiveReadBatches += _Batch->bIsReading ? 1 : 0;
			ActiveWriteBatches += _Batch->bIsWriting ? 1 : 0;
		}

		// Empty files don't have async jobs.
		bool bAsyncJobExists = ActiveReadBatches || ActiveWriteBatches;

		if (bHasAnyFileFailed)
		{
			// We can only bail when all our async jobs have completed.
			if (!bAsyncJobExists)
			{
				break;
			}
		}

		TRACE_INT_VALUE(TEXT("BPS.FC.ActiveFiles"), ActiveFiles.Num());
		TRACE_INT_VALUE(TEXT("BPS.FC.ActiveReadBatches"), ActiveReadBatches);

		if (bStillProcessingFiles && 
			bAsyncJobExists)
		{
			if (WaitTimeMs == TNumericLimits<uint32>::Max())
			{
				WaitTimeMs = 15*1000;
			}

			uint64 WaitStartCycles = FPlatformTime::Cycles64();
			TRACE_CPUPROFILER_EVENT_SCOPE(CFFC_Waiting);
			// We have a bunch of stuff outstanding that will wake us up if something happens.
			WakeUpDispatchThreadEvent->Wait(WaitTimeMs);

			WaitCycles += FPlatformTime::Cycles64() - WaitStartCycles;
		}
	} // end loop until we complete all the files.

	// Any remaining active files (due to abort/failure) need to be failed and completed
	// so that errors get reported. We want to report the non-abort failures first, because
	// anything else that was in flight gets reported as an abort and the first error is the
	// one we actually care about.
	if (ActiveFiles.Num())
	{
		for (TUniquePtr<FFileConstructionState>& File : ActiveFiles)
		{
			if (!File->bSuccess && File->ConstructionError != EConstructionError::Aborted)
			{
				CompleteConstructedFile(*File.Get(), ResumeData);
			}
		}

		// Now handle everything else.
		for (TUniquePtr<FFileConstructionState>& File : ActiveFiles)
		{
			if (!File->bSuccess && File->ConstructionError != EConstructionError::Aborted)
			{
				continue; // handled in previous loop.
			}

			if (File->bSuccess)
			{
				File->bSuccess = false;
				File->ConstructionError = EConstructionError::Aborted;
			}
			CompleteConstructedFile(*File.Get(), ResumeData);
		}
	}

	if (CorruptFiles.Num() > 0)
	{
		Abort();
		UE_LOGF(LogBuildPatchServices, Error, "FBuildPatchFileConstructor: The %d files are corrupt.", CorruptFiles.Num());
		InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::OutboundCorrupt);
	}

	uint64 ConstructCycles = FPlatformTime::Cycles64() - ConstructStartCycles;
	uint64 UnaccountedForCycles = ConstructCycles - HashCycles - WaitCycles - ReadCheckCycles - ReadDoneCycles - WriteStartCycles - WriteDoneCycles - CloudTickCycles;

	double ConstructSec = FPlatformTime::ToSeconds64(ConstructCycles);
	UE_LOGF(LogBuildPatchServices, Display, "Construction done: %.2f sec. Hash %.1f%% Wait %.1f%% ReadCheck %.1f%% WriteStart %.1f%% ReadDone %.1f%% WriteDone %.1f%% CloudTick %.1f%% Unaccounted %.1f%%",
		ConstructSec,
		100.0 * HashCycles / ConstructCycles,
		100.0 * WaitCycles / ConstructCycles,
		100.0 * ReadCheckCycles / ConstructCycles,
		100.0 * WriteStartCycles / ConstructCycles,
		100.0 * ReadDoneCycles / ConstructCycles,
		100.0 * WriteDoneCycles / ConstructCycles,
		100.0 * CloudTickCycles / ConstructCycles,
		100.0 * UnaccountedForCycles / ConstructCycles
		);
}

void FBuildPatchFileConstructor::RemoveUnnecessaryStagedFiles(const FString& StagingDirectory) const
{
	// Read staging folder to get the list of existing staged files
	TSet<FString> AllStagedFiles;
	IFileManager::Get().IterateDirectory(*StagingDirectory, [&AllStagedFiles, StagingDirectory](const TCHAR* Filename, bool) {
			AllStagedFiles.Add(FString(Filename).Mid(StagingDirectory.Len() + 1)); // get relative path.
			return true;
		});

	// Get a set of potentially staged files based on ConstructionList
	TSet<FString> NecessaryStagedFiles;
	Algo::Transform(ConstructionList, NecessaryStagedFiles, [](const FFileToConstruct& FileToConstruct)
		{
			return FileToConstruct.StagingFilename;
		});

	const TSet<FString> UnnecessaryFiles = AllStagedFiles.Difference(NecessaryStagedFiles);
	for (const FString& Filename : UnnecessaryFiles)
	{
		UE_LOGF(LogBuildPatchFileConstructor, Log, "RemoveUnnecessaryStagedFiles: Deleting unnecessary staged file %ls", *Filename);
		if (!FileSystem->DeleteFile(*(StagingDirectory / Filename)))
		{
			UE_LOGF(LogBuildPatchServices, Error, "RemoveUnnecessaryStagedFiles: Error deleting file: %ls (Error Code %i)", *Filename, FPlatformMisc::GetLastError());
		}
	}
}

