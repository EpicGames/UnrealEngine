// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Mutex.h"
#include "Misc/Guid.h"
#include "HAL/Runnable.h"
#include "BuildPatchProgress.h"
#include "BuildPatchManifest.h"
#include "Installer/ChunkSource.h"
#include "Installer/Controllable.h"
#include "Common/SpeedRecorder.h"
#include "BuildPatchInstall.h"
#include "Async/Future.h"


// Forward declarations
struct FBatchState;
struct FResumeData;
struct FFileConstructionState;
class FBuildPatchAppManifest;
enum class EConstructionError : uint8;
enum class EFileStage : uint8;

class IBuildInstallerSharedContext;
class FChunkBackingStore;

namespace BuildPatchServices
{
	enum EConstructorChunkLocation : uint8
	{
		Install,
		ChunkDb,
		Memory,
		DiskOverflow,
		Cloud,
		Retired,
		COUNT
	};

	struct FChunkPart;
	class IFileSystem;
	class IChunkSource;
	class IChunkReferenceTracker;
	class IInstallerError;
	class IInstallerAnalytics;
	class IFileConstructorStat;
	class IMessagePump;
	class IBuildManifestSet;
	class IBuildInstallerThread;
	class IConstructorChunkDbChunkSource;
	class IConstructorInstallChunkSource;
	class IConstructorCloudChunkSource;
	

	/**
	 * A struct containing the configuration values for a file constructor.
	 */
	struct FFileConstructorConfig
	{
		// The manifest set class for details on the installation files.
		IBuildManifestSet* ManifestSet;

		// The location for the installation.
		FString InstallDirectory;

		// The location where new installation files will be constructed.
		FString StagingDirectory;

		// The location where temporary files for tracking can be stored.
		FString MetaDirectory;

		// The list of files to be constructed, filename paths should match those contained in manifest.
		TArray<FString> ConstructList;

		// The install mode used for this installation.
		EInstallMode InstallMode;
		// Whether we are needing to use SHA1 hash string value as filename for temp staging files.
		bool bUseSHA1StageFilenames;

		// If true, fail the installation if the manifest contains a symlink.
		bool bRejectSymlinks = false;

		// The location where memory overflow will get written to.
		FString BackingStoreDirectory;

		// Overrides for file locations.
		const TMap<FString, FString>* PerFileSubdirectories = nullptr;

		IBuildInstallerSharedContext* SharedContext = nullptr;

		// See comments in installer config (FBuildInstallerConfiguration)
		bool bInstallToMemory = false;
		bool bConstructInMemory = false;
		bool bSkipInitialDiskSizeCheck = false;

		bool bDeleteChunkDBFilesAfterUse = false;

		static const bool bDefaultSpawnAdditionalIOThreads = true;
		static const int32 DefaultIOBatchSizeMB = 10;
		static const int32 DefaultIOBufferSizeMB = 64;

		TOptional<bool> SpawnAdditionalIOThreads;
		TOptional<int32> IOBatchSizeMB;
		TOptional<int32> IOBufferSizeMB;
		TOptional<bool> StallWhenFileSystemThrottled;
		TOptional<int32> DisableResumeBelowMB;
	};

	/**
	 * FBuildPatchFileConstructor
	 * This class controls a thread that constructs files from a file list, given install details, and chunk availability notifications
	 */
	class FBuildPatchFileConstructor : public IControllable
	{
		friend FChunkBackingStore;
	public:

		/**
		 * Constructor
		 * @param Configuration             The configuration for the constructor.
		 * @param FileSystem                The service used to open files.
		 * @param ChunkSource               Pointer to the chunk source.
		 * @param ChunkReferenceTracker     Pointer to the chunk reference tracker.
		 * @param InstallerError            Pointer to the installer error class for reporting fatal errors.
		 * @param InstallerAnalytics        Pointer to the installer analytics handler for reporting events.
		 * @param FileConstructorStat       Pointer to the stat class for receiving updates.
		 */
		FBuildPatchFileConstructor(
			FFileConstructorConfig Configuration, IFileSystem* FileSystem, 
			IConstructorChunkDbChunkSource* ChunkDbChunkSource, IConstructorCloudChunkSource* CloudChunkSource, IConstructorInstallChunkSource* InstallChunkSource, 
			IChunkReferenceTracker* ChunkReferenceTracker, IInstallerError* InstallerError, IInstallerAnalytics* InstallerAnalytics, IMessagePump* MessagePump,
			IFileConstructorStat* FileConstructorStat, TMap<FGuid, EConstructorChunkLocation>&& ChunkLocations);

		/**
		 * Default Destructor, will delete the allocated Thread
		 */
		~FBuildPatchFileConstructor();

		void Run();

		// IControllable interface begin.
		virtual void SetPaused(bool bInIsPaused) override;
		virtual void Abort() override;

		virtual void Reset() override
		{
			// This should _never_ be called. I'm not sure how you can Reset() a constructor..
			check(false);
		}
		// IControllable interface end.

		void WakeUpDispatch();
		
		/**
		 * Get the disk space that was required to perform the installation. This can change over time and indicates the required
		 * space to _finish_ the installation from the current state. It is not initialized until after resume is processed and returns
		 * zero until that time. Note that since this and GetAvailableDiskSpace are separate accessors there's no guarantee that they
		 * match - e.g. if you call GetRequiredDiskSpace and then GetAvailableDiskSpace immediately afterwards, it's possible the Available
		 * Disk Space value is from a later call. This is highly unlikely due to how rare these updates are, but it's possible. Use these
		 * for UI purposes only.
		 */
		uint64 GetRequiredDiskSpace();

		/**
		 * Get the disk space that was available when last updating RequiredDiskSpace. See notes with GetRequiredDiskSpace.
		 * It's possible for this to return 0 due to the underlying operating system being unable to report a value in cases of
		 * e.g. the drive being disconnected.
		 */
		uint64 GetAvailableDiskSpace();

		/**
		 * Gets flag indicating that at least one file is corrupt due to invalid hash sum.
		 * @return	true if at least one file is corrupt.
		 */
		bool HasCorruptFiles() const;

		/**
		 * Broadcasts with full filepath to file that the constructor is about to delete in order to free up space.
		 * @return	Reference to the event object.
		 */
		DECLARE_EVENT_OneParam(FBuildPatchFileConstructor, FOnBeforeDeleteFile, const FString& /*FullFilePath*/);
		FOnBeforeDeleteFile& OnBeforeDeleteFile();

		/**
		 * Broadcasts with the build relative path for a file that has finished being staged (including by a previous incomplete installation).
		 * @return	Reference to the event object.
		 */
		DECLARE_EVENT_OneParam(FBuildPatchFileConstructor, FOnFileIsStaged, const FString& /*BuildFilename*/);
		FOnFileIsStaged& OnFileIsStaged();

		void GrabFilesInstalledToMemory(TMap<FString, TArray64<uint8>>& OutFilesInstalledToMemory)
		{
			OutFilesInstalledToMemory = MoveTemp(MemoryOutputFiles);
		}

	private:

		void SetChunkLocation(const FGuid& InGuid, EConstructorChunkLocation InNewLocation);

		/**
		 * Count additional bytes processed, and set new install progress value
		 * @param ByteCount		Number of bytes to increment by
		 */
		void CountBytesProcessed(const int64& ByteCount);

		/**
		 * @return the total bytes size of files not yet started construction
		 */
		int64 GetRemainingBytes();

		/**
		 * Calculates the minimum required disk space for the remaining work to be completed, based on a current file, and the list of files left in ConstructionStack.
		 * @param InProgressFileManifest	The manifest for the file currently being constructed.
		 * @param InProgressFileSize		The remaining size required for the file currently being constructed.
		 * @return the number of bytes required on disk to complete the installation.
		 */
		uint64 CalculateInProgressDiskSpaceRequired(const FFileManifest& InProgressFileManifest, uint64 InProgressFileSize);

		/**
		 * Calculates the amount of disk space we need to finish the install, needs to be called on file boundaries on the construct thread.
		 * @param InProgressFileSize		The remaining size required for the file currently being constructed.
		 * @return Number of bytes needed to complete the installation, or zero if space will be freed by removing chunks.
		 */
		uint64 CalculateDiskSpaceRequirementsWithDeleteDuringInstall(int64 InProgressFileSize);

		/**
		 * Sequentially build the files required, potentially skipping already created files or resuming partial
		 * files.
		 */
		void ConstructFiles(FResumeData& ResumeData);

	private:
		// The configuration for the constructor.
		const FFileConstructorConfig Configuration;

		// A flag marking that we a running.
		bool bIsRunning;

		// A flag marking that we initialized correctly.
		bool bIsInited;

		// A flag marking that our init returned a failure (true means failed).
		bool bInitFailed;

		// A flag marking that we told the chunk cache to queue required downloads.
		bool bIsDownloadStarted;

		// A flag marking that we have made the initial disk space check following resume logic complete.
		bool bInitialDiskSizeCheck;

		// If true, the chunkdb source has chunks to provide
		bool bHasChunkDbSource = false;

		// Our local resolved copy of the cvar with overrides applied.
		bool bStallWhenFileSystemThrottled = false;

		// A flag marking whether we should be paused.
		FThreadSafeBool bIsPaused;

		// A flag marking whether we should abort operations and exit. Always call Abort() to set this.
		FThreadSafeBool bShouldAbort;

		// Indexes in to ConstructionList and associated parallel arrays. This is the next file that will
		// start construction when dependencies are met.
		std::atomic_int32_t NextIndexToConstruct = 0;

		struct FFileToConstruct
		{
			const FFileManifest* FileManifest = nullptr;

			// When using an install source with multiple files in flight, we can't start this 
			// file until all of the install sources it needs have been harvested. Since files are
			// constructed in order, we only track the file that will be harvested last. If this
			// file has no dependencies this is -1
			int32 LatestDependentInstallSource = -1;
			// Name of the staging file. It may be obfuscated if bUseSHA1StageFilenames is true.
			FString StagingFilename;
		};

		// The in-oder list of files to construct. The array is parallel with Configuration.ConstructList.
		TArray<FFileToConstruct> ConstructionList;

		// Pointer to the file system.
		IFileSystem* FileSystem;

		IConstructorChunkDbChunkSource* ChunkDbSource; // can be null if not using.
		IConstructorInstallChunkSource* InstallSource;
		IConstructorCloudChunkSource* CloudSource;

		// Keyed off of the filename relative to the install directory.
		TMap<FString, TArray64<uint8>> MemoryOutputFiles;

		// We always want to know exactly where we think a chunk should be. If it's not there,
		// we update this list to where it can be found (i.e. cloud)
		// This is almost always read only after initialization, but in rare situations can be updated
		// (chunk failures, file resume) and is multi threaded access.
		FRWLock ChunkLocationsLock;
		TMap<FGuid, EConstructorChunkLocation> ChunkLocations;
		// Track how much data we expect to have to download. This is protected by the ChunkLocationsLock since they are in sync.
		uint64 DownloadRequirement = 0;

		// Pointer to the chunk reference tracker.
		IChunkReferenceTracker* ChunkReferenceTracker;

		std::atomic<int32> RemainingChunkCount = TNumericLimits<int32>::Max();

		// Pointer to the installer error class.
		IInstallerError* InstallerError;

		// Pointer to the installer analytics handler.
		IInstallerAnalytics* InstallerAnalytics;

		IMessagePump* MessagePump = nullptr;

		// Pointer to the stat class.
		IFileConstructorStat* FileConstructorStat;

		bool bAllowMultipleFilesInFlight = true;

		bool bShouldLogNextDependencyWait = true;

		// The size we expect for chunks. This should be used for estimation purposes, not anything requiring hard limits.
		uint32 ExpectedChunkSize = 0;

		// Total job size for tracking progress.
		int64 TotalJobSize;

		// Byte processed so far for tracking progress.
		int64 ByteProcessed;

		uint32 MaxWriteBatchSize = 0;
		uint32 IOBufferSize = 0;

		int32 WriteCount = 0;

		// The amount of disk space requirement that was calculated when beginning the process. 0 if the install process was not started, or no additional space was needed.
		std::atomic_uint64_t RequiredDiskSpace;

		// The amount of disk space available when beginning the process. 0 if the install process was not started.
		std::atomic_uint64_t AvailableDiskSpace;

		// Event executed before deleting an old installation file.
		FOnBeforeDeleteFile BeforeDeleteFileEvent;
		// Event executed when a file has now been staged or was previously staged.
		FOnFileIsStaged FileIsStagedEvent;
		

		TArray<FEvent*> ThreadWakeups;
		TArray<TArray<IConstructorChunkSource::FRequestProcessFn>> ThreadJobPostings;
		TArray<UE::FMutex> ThreadJobPostingLocks;
		TArray<FEvent*> ThreadCompleteEvents;
		TArray<IBuildInstallerThread*> Threads;

		void QueueGenericThreadTask(int32 ThreadIndex, TUniqueFunction<void(bool)>&& Task);
		void GenericThreadFn(int32 ThreadIndex);
		int8 ThreadAssignments[EConstructorChunkLocation::COUNT];
		int8 WriteThreadIndex = -1;

		TUniquePtr<FChunkBackingStore> BackingStore;

		// Fire this to wake up the main thread to process completed tasks.
		FEvent* WakeUpDispatchThreadEvent = nullptr;

		IConstructorChunkSource::FRequestProcessFn CreateWriteRequest(FArchive* File, FBatchState& Batch);

		// Where we are in the chunk consumption list after each file.
		TArray<int32> FileCompletionPositions;

		void StartReadBatch(FFileConstructionState& CurrentFile, FBatchState& Buffer);
		void CompleteReadBatch(const FFileManifest& FileManifest, FBatchState& Buffer);
		void RequestCompletedFn(const FGuid& Guid, bool bAborted, bool bFailedToRead, void* UserPtr);

		void ChunkHarvestCompletedFn(const FGuid& Guid, bool bAborted, bool bFailedToRead, void* UserPtr);

		void StartFile(FFileConstructionState& CurrentFile, const FResumeData& ResumeData);
		void ResumeFile(FFileConstructionState& FileToResume);
		void OpenFileToConstruct(FFileConstructionState& CurrentFile);
		bool HandleInitialDiskSizeCheck(const FFileManifest& CurrentFileManifest, int64 BytesInToCurrentFile);
		bool HarvestChunksForCompletedFile(const FString& InstallationDirectory, const FString& FilenameInManifest);
		// Starts an asynchronous truncation task in the chunk reading thread, which guarantees single-threaded access to chunkdb.
		void StartTruncateChunkDbForCompletedBatch(const FBatchState& Batch);

		void CompleteConstructedFile(FFileConstructionState& CurrentFile, FResumeData& ResumeData);

		// Determine and return the file stage based on its existence and file size.
		EFileStage GetFileStage(const FFileConstructionState& CurrentFile, const FResumeData& ResumeData) const;

		// Initialize and validate.
		void InitFile(FFileConstructionState& CurrentFile, const FResumeData& ResumeData);
		
		// Removes staged files that are no longer required for the current operation.
		void RemoveUnnecessaryStagedFiles(const FString& StagingDirectory) const;

		// Set of files considered corrupt due to an invalid hash sum.
		TSet<FString> CorruptFiles;

	public:
		struct FBackingStoreStats
		{
			uint64 DiskPeakUsageBytes=0;
			uint64 MemoryPeakUsageBytes=0;
			uint64 MemoryLimitBytes=0;
			uint32 DiskLoadFailureCount=0;
			uint32 DiskLostChunkCount=0;
			uint32 DiskChunkLoadCount=0;
		};

		// This isn't safe to call during operations as values are changing on other threads.
		FBackingStoreStats GetBackingStoreStats()
		{
			return BackingStoreStats;
		}
	private:

		FBackingStoreStats BackingStoreStats;
	};

	/**
	 * This interface defines the statistics class required by the file constructor. It should be implemented in order to collect
	 * desired information which is being broadcast by the system.
	 */
	class IFileConstructorStat
	{
	public:
		virtual ~IFileConstructorStat() = default;

		/**
		 * Called when the resume process begins.
		 */
		virtual void OnResumeStarted() = 0;

		/**
		 * Called when the resume process completes.
		 */
		virtual void OnResumeCompleted() = 0;

		/**
		 * Called for each Get made to the chunk source.
		 * @param ChunkId       The id for the chunk required.
		 */
		virtual void OnChunkGet(const FGuid& ChunkId) = 0;

		/**
		 * Called when a file construction has started.
		 * @param Filename      The filename of the file.
		 * @param FileSize      The size of the file being constructed.
		 */
		virtual void OnFileStarted(const FString& Filename, int64 FileSize) = 0;

		/**
		 * Called during a file construction with the current progress.
		 * @param Filename      The filename of the file.
		 * @param TotalBytes    The number of bytes processed so far.
		 */
		virtual void OnFileProgress(const FString& Filename, int64 TotalBytes) = 0;

		/**
		 * Called when a file construction has completed.
		 * @param Filename      The filename of the file.
		 * @param bSuccess      True if the file construction succeeded.
		 */
		virtual void OnFileCompleted(const FString& Filename, bool bSuccess) = 0;

		/**
		 * Called when the construction process completes.
		 */
		virtual void OnConstructionCompleted() = 0;

		/**
		 * Called to update the total amount of bytes which have been constructed.
		 * @param TotalBytes    The number of bytes constructed so far.
		 */
		virtual void OnProcessedDataUpdated(int64 TotalBytes) = 0;

		/**
		 * Called to update the total number of bytes to be constructed.
		 * @param TotalBytes    The total number of bytes to be constructed.
		 */
		virtual void OnTotalRequiredUpdated(int64 TotalBytes) = 0;

		/**
		 * Called when we are beginning a file administration, such as open, close, seek.
		 */
		virtual void OnBeforeAdminister() = 0;

		/**
		 * Called upon completing an admin operation, with activity recording.
		 * @param Record        The activity record.
		 */
		virtual void OnAfterAdminister(const ISpeedRecorder::FRecord& Record) = 0;

		/**
		 * Called when we are beginning a read operation.
		 */
		virtual void OnBeforeRead() = 0;

		/**
		 * Called upon completing a read operation, with activity recording.
		 * @param Record        The activity record.
		 */
		virtual void OnAfterRead(const ISpeedRecorder::FRecord& Record) = 0;

		/**
		 * Called when we are beginning a write operation.
		 */
		virtual void OnBeforeWrite() = 0;

		/**
		 * Called upon completing a write operation, with activity recording.
		 * @param Record        The activity record.
		 */
		virtual void OnAfterWrite(const ISpeedRecorder::FRecord& Record) = 0;
	};
}

/**
 * Helpers for calculations that are useful for other classes or operations.
 */
namespace FileConstructorHelpers
{
	uint64 CalculateRequiredDiskSpace(const FBuildPatchAppManifestPtr& CurrentManifest, const FBuildPatchAppManifestRef& BuildManifest, const BuildPatchServices::EInstallMode& InstallMode, const TSet<FString>& InstallTags);
	struct FDiskSpaceInfo
	{ 
		uint64 AvailableDiskSpace = 0;
		bool bHaveSufficientFreeSpace = false;
	};
	TOptional<FDiskSpaceInfo> GetRemainingDiskSpace(const FString& InstallDirectory, uint64 RemainingBytesRequired);
}
