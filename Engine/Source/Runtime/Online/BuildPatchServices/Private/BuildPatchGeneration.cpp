// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchGeneration.h"
#include "Templates/UniquePtr.h"
#include "Templates/Greater.h"
#include "Templates/Tuple.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/SecureHash.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Containers/Ticker.h"
#include "Containers/Queue.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "BuildPatchManifest.h"
#include "BuildPatchServicesModule.h"
#include "BuildPatchHash.h"
#include "Core/BlockStructure.h"
#include "Core/BlockData.h"
#include "Common/Crypto.h"
#include "Common/FileSystem.h"
#include "Data/ChunkData.h"
#include "Generation/DataScanner.h"
#include "Generation/BuildStreamer.h"
#include "Generation/CloudEnumeration.h"
#include "Generation/ManifestBuilder.h"
#include "Generation/FileAttributesParser.h"
#include "Generation/ChunkWriter.h"
#include "Generation/ChunkMatchProcessor.h"
#include "Installer/Statistics/CloudChunkSourceStat.h"
#include "BuildPatchUtil.h"

using namespace BuildPatchServices;

DECLARE_LOG_CATEGORY_EXTERN(LogPatchGeneration, Log, All);
DEFINE_LOG_CATEGORY(LogPatchGeneration);

namespace BuildPatchServices
{
	struct FScannerDetails
	{
	public:
		FScannerDetails(int32 InLayer, uint64 InLayerOffset, bool bInIsFinalScanner, uint64 InPaddingSize, TArray<uint8> InData, FBlockStructure InStructure, const TArray<uint32>& ChunkWindowSizes, const ICloudEnumeration* CloudEnumeration, FStatsCollector* StatsCollector)
			: Layer(InLayer)
			, LayerOffset(InLayerOffset)
			, bIsFinalScanner(bInIsFinalScanner)
			, PaddingSize(InPaddingSize)
			, Data(MoveTemp(InData))
			, Structure(MoveTemp(InStructure))
			, Scanner(FDataScannerFactory::Create(ChunkWindowSizes, Data, CloudEnumeration, StatsCollector, Structure.GetHead()->GetOffset()))
		{}

	public:
		const int32 Layer;
		const uint64 LayerOffset;
		const bool bIsFinalScanner;
		const uint64 PaddingSize;
		const TArray<uint8> Data;
		const FBlockStructure Structure;
		const TUniquePtr<IDataScanner> Scanner;
	};
}

namespace PatchGenerationHelpers
{
	FDirectoryChunkerConfiguration& Normalize(FDirectoryChunkerConfiguration& Configuration)
	{
		FPaths::NormalizeDirectoryName(Configuration.RootDirectory);
		return Configuration;
	}
	int32 GetMaxScannerBacklogCount()
	{
		int32 MaxScannerBacklogCount = 75;
		GConfig->GetInt(TEXT("BuildPatchServices"), TEXT("MaxScannerBacklog"), MaxScannerBacklogCount, GEngineIni);
		MaxScannerBacklogCount = FMath::Clamp<int32>(MaxScannerBacklogCount, 5, 500);
		return MaxScannerBacklogCount;
	}

	bool ScannerArrayFull(const TArray<TUniquePtr<FScannerDetails>>& Scanners)
	{
		static int32 MaxScannerBacklogCount = GetMaxScannerBacklogCount();
		return (FDataScannerCounter::GetNumIncompleteScanners() > FDataScannerCounter::GetNumRunningScanners()) || (Scanners.Num() >= MaxScannerBacklogCount);
	}

	FSHAHash GetShaForDataSet(const uint8* DataSet, uint32 DataSize)
	{
		FSHAHash SHAHash;
		FSHA1::HashBuffer(DataSet, DataSize, SHAHash.Hash);
		return SHAHash;
	}

	TArray<FString> ReadFileLines(const FString& Filename)
	{
		TArray<FString> Result;
		FString FileData = TEXT("");
		if (FFileHelper::LoadFileToString(FileData, *Filename))
		{
			FileData.ParseIntoArrayLines(Result, true);
		}
		return Result;
	}

	void ReadInputFileList(const FString& InputListFile, const FString& BuildRoot, TArray<FString>& EnumeratedFiles)
	{
		TArray<FString> InputFiles = ReadFileLines(InputListFile);
		for (FString& InputFile : InputFiles)
		{
			InputFile = InputFile.TrimStartAndEnd().TrimQuotes();
			if (InputFile.Len() > 0)
			{
				FString FullInputFile = BuildRoot / InputFile;
				FPaths::NormalizeFilename(FullInputFile);
				EnumeratedFiles.Add(FullInputFile);
			}
		}
	}

	void ReadIgnoreFileList(const FString& IgnoreFile, const FString& BuildRoot, TSet<FString>& IgnoreFiles, TSet<FString>& IgnorePatterns)
	{
		TArray<FString> LinesIgnoreFiles = ReadFileLines(IgnoreFile);
		for (FString& Filename : LinesIgnoreFiles)
		{
			int32 TabLocation = Filename.Find(TEXT("\t"));
			if (TabLocation != INDEX_NONE)
			{
				// Strip tab separated timestamp if it exists
				Filename = Filename.Left(TabLocation);
			}
			Filename = Filename.TrimStartAndEnd();
			if (FRegexFinder::StartsLikePattern(Filename))
			{
				if (FRegexFinder::EndsWithQuote(Filename))
				{
					IgnorePatterns.Add(MoveTemp(Filename));
					continue;
				}
				else
				{
					UE_LOGF(LogPatchGeneration, Warning, "Did not find an ending quote for a regex. Regexes must be quoted: R\"<>\"");
					UE_LOGF(LogPatchGeneration, Warning, "Item is considered as a filename %ls.", *Filename);
				}
			}

			Filename = BuildRoot / Filename.TrimQuotes();
			FPaths::NormalizeFilename(Filename);
			IgnoreFiles.Add(MoveTemp(Filename));
		}
	}

	void NormalizeFilenames(TArray<FString>& AllFiles)
	{
		for (FString& Filename : AllFiles)
		{
			FPaths::NormalizeFilename(Filename);
		}
	}

	void StripIgnoredFiles(const TSet<FString>& IgnoreFiles, const TSet<FString>& IgnorePatterns, TArray<FString>& AllFiles)
	{
		struct FRemoveMatchingStrings
		{
			const TSet<FString>& IgnoreFiles;
			const TSet<FString>& IgnorePatterns;

			bool operator()(const FString& RemovalCandidate) const
			{
				const bool bRemove = IgnoreFiles.Contains(RemovalCandidate) || FRegexFinder::Match(RemovalCandidate, IgnorePatterns);
				if (bRemove)
				{
					UE_LOGF(LogPatchGeneration, Log, "    - %ls", *RemovalCandidate);
				}
				return bRemove;
			}
		};

		UE_LOGF(LogPatchGeneration, Log, "Stripping ignorable files");
		const int32 OriginalNumFiles = AllFiles.Num();
		
		// Filter file list
		FRemoveMatchingStrings FileFilter{ IgnoreFiles, IgnorePatterns };
		AllFiles.RemoveAll(FileFilter);

		const int32 NewNumFiles = AllFiles.Num();
		UE_LOGF(LogPatchGeneration, Log, "Stripped %d ignorable file(s)", (OriginalNumFiles - NewNumFiles));
	}

	class FChunkWriterEventsHandler
	{
	public:
		FChunkWriterEventsHandler(IParallelChunkWriter* InChunkWriter)
			: ChunkWriter(InChunkWriter)
		{
			check(ChunkWriter);
		}

		~FChunkWriterEventsHandler()
		{
			RemoveTicker();
		}

		void RegisterTicker()
		{
			if (!ChunkWriterPumpHandle.IsValid())
			{
				AsyncHelpers::ExecuteOnGameThread<void>([this]()
				{
					ChunkWriterPumpHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([&](float)
					{
						ChunkWriter->PumpEvents();
						return true;
					}));
				}).Wait();
			}
		}

		void RemoveTicker()
		{
			if (ChunkWriterPumpHandle.IsValid())
			{
				// Remove pump from ChunkWriter, making sure a final pump is called.
				AsyncHelpers::ExecuteOnGameThread<void>([this]()
				{
					FTSTicker::GetCoreTicker().RemoveTicker(ChunkWriterPumpHandle);
					ChunkWriter->PumpEvents();
				}).Wait();

				ChunkWriterPumpHandle.Reset();
			}
		}

	private:
		FTSTicker::FDelegateHandle ChunkWriterPumpHandle;
		IParallelChunkWriter* ChunkWriter;
	};
}

namespace BuildPatchServices
{
	class FDirectoryChunker
		: public IDirectoryChunker
	{
	public:
		FDirectoryChunker(FDirectoryChunkerConfiguration Configuration);
		~FDirectoryChunker();

		// IChunkDeltaOptimiser interface begin.
		virtual FOnBuildFilesEnumerated& OnBuildFilesEnumerated() override;
		virtual FOnChunkFileWritten& OnChunkFileWritten() override;
		virtual FOnChunkMatch& OnChunkMatch() override;
		virtual FOnManifestGenerated& OnManifestGenerated() override;
		virtual FOnManifestGenerated& OnManifestGeneratedDecrypted() override;
		virtual FOnManifestFileWritten& OnManifestFileWritten() override;
		virtual bool Run() override;
		virtual void Abort() override;
		// IChunkDeltaOptimiser interface end.

	private:
		bool RunAsync();

	private:
		const FDirectoryChunkerConfiguration Configuration;
		IDirectoryChunker::FOnBuildFilesEnumerated BuildFilesEnumeratedEvent;
		IDirectoryChunker::FOnChunkFileWritten ChunkFileWrittenEvent;
		IDirectoryChunker::FOnChunkMatch ChunkMatchEvent;
		IDirectoryChunker::FOnManifestGenerated ManifestGeneratedEvent;
		IDirectoryChunker::FOnManifestGenerated ManifestGeneratedDecryptedEvent;
		IDirectoryChunker::FOnManifestFileWritten ManifestFileWrittenEvent;
		FThreadSafeBool bShouldAbort;
	};

	FDirectoryChunker::FDirectoryChunker(FDirectoryChunkerConfiguration InConfiguration)
		: Configuration(MoveTemp(PatchGenerationHelpers::Normalize(InConfiguration)))
		, bShouldAbort(false)
	{
	}

	FDirectoryChunker::~FDirectoryChunker()
	{
	}

	IDirectoryChunker::FOnBuildFilesEnumerated& FDirectoryChunker::OnBuildFilesEnumerated()
	{
		return BuildFilesEnumeratedEvent;
	}

	IDirectoryChunker::FOnChunkFileWritten& FDirectoryChunker::OnChunkFileWritten()
	{
		return ChunkFileWrittenEvent;
	}

	IDirectoryChunker::FOnChunkMatch& FDirectoryChunker::OnChunkMatch()
	{
		return ChunkMatchEvent;
	}

	IDirectoryChunker::FOnManifestGenerated& FDirectoryChunker::OnManifestGenerated()
	{
		return ManifestGeneratedEvent;
	}

	IDirectoryChunker::FOnManifestGenerated& FDirectoryChunker::OnManifestGeneratedDecrypted()
	{
		return ManifestGeneratedDecryptedEvent;
	}

	IDirectoryChunker::FOnManifestFileWritten& FDirectoryChunker::OnManifestFileWritten()
	{
		return ManifestFileWrittenEvent;
	}

	bool FDirectoryChunker::Run()
	{
		if (!IsInGameThread())
		{
			return RunAsync();
		}

		// Kick off async task
		TFuture<bool> Future = Async(EAsyncExecution::Thread, [this]() { return RunAsync(); });

		float MainsFramerate = 60.0f;
		float BatteryFramerate = 30.0f;
		const float MainsFrameTime = 1.0f / MainsFramerate;
		const float BatteryFrameTime = 1.0f / BatteryFramerate;

		double DeltaTime = 0.0;
		double LastTime = FPlatformTime::Seconds();

		while (!Future.IsReady())
		{
			// Increment global frame counter once for each app tick.
			GFrameCounter++;

			// Tick our sub-systems
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(DeltaTime);

			GLog->FlushThreadedLogs();

			// Throttle frame rate
			const bool bIsOnBattery = FPlatformMisc::IsRunningOnBattery();
			const float FrameTime = bIsOnBattery ? BatteryFrameTime : MainsFrameTime;
			FPlatformProcess::Sleep(FMath::Max<float>(0.0f, FrameTime - (FPlatformTime::Seconds() - LastTime)));

			// Calculate deltas
			const double AppTime = FPlatformTime::Seconds();
			DeltaTime = AppTime - LastTime;
			LastTime = AppTime;
		}

		return Future.Get();
	}

	bool FDirectoryChunker::RunAsync()
	{
		const uint64 StartTime = FStatsCollector::GetCycles();

		// Check for the required output filename.
		if (Configuration.OutputFilename.IsEmpty())
		{
			UE_LOGF(LogPatchGeneration, Error, "Manifest OutputFilename was not provided");
			return false;
		}

		// Check for featurelevel support
#if !BPS_WITH_OPENSSL
		if (Configuration.FeatureLevel >= EFeatureLevel::StoresFileSHA256Hashes)
		{
			UE_LOGF(LogPatchGeneration, Error, "Platforms without OPENSSL support cannot generate output as %ls. Please use feature level EFeatureLevel::StoresFileMIMEType or lower.", LexToString(Configuration.FeatureLevel));
			return false;
		}
#endif

		// Ensure that cloud directory exists, and create it if not.
		IFileManager::Get().MakeDirectory(*Configuration.CloudDirectory, true);
		if (!IFileManager::Get().DirectoryExists(*Configuration.CloudDirectory))
		{
			UE_LOGF(LogPatchGeneration, Error, "Unable to create specified cloud directory %ls", *Configuration.CloudDirectory);
			return false;
		}

		// The last time we logged out data processed.
		double LastProgressLog = FPlatformTime::Seconds();
		const double TimeGenStarted = LastProgressLog;

		// Load settings from config.
		float GenerationScannerSizeMegabytes = 32.5f;
		float StatsLoggerTimeSeconds = 10.0f;
		GConfig->GetFloat(TEXT("BuildPatchServices"), TEXT("GenerationScannerSizeMegabytes"), GenerationScannerSizeMegabytes, GEngineIni);
		GConfig->GetFloat(TEXT("BuildPatchServices"), TEXT("StatsLoggerTimeSeconds"), StatsLoggerTimeSeconds, GEngineIni);
		GenerationScannerSizeMegabytes = FMath::Clamp<float>(GenerationScannerSizeMegabytes, 10.0f, 500.0f);
		StatsLoggerTimeSeconds = FMath::Clamp<float>(StatsLoggerTimeSeconds, 1.0f, 60.0f);
		const uint64 ScannerDataSize = GenerationScannerSizeMegabytes * 1048576;

		// Create stat collector.
		TUniquePtr<FStatsCollector> StatsCollector(FStatsCollectorFactory::Create());

		// Setup Generation stats.
		volatile int64* StatTotalTime = StatsCollector->CreateStat(TEXT("Generation: Total Time"), EStatFormat::Timer);
		volatile int64* StatLayers = StatsCollector->CreateStat(TEXT("Generation: Layers"), EStatFormat::Value);
		volatile int64* StatNumScanners = StatsCollector->CreateStat(TEXT("Generation: Scanner Backlog"), EStatFormat::Value);
		volatile int64* StatUnknownDataAlloc = StatsCollector->CreateStat(TEXT("Generation: Unmatched Buffers Allocation"), EStatFormat::DataSize);
		volatile int64* StatUnknownDataNum = StatsCollector->CreateStat(TEXT("Generation: Unmatched Buffers Use"), EStatFormat::DataSize);
		int64 MaxLayer = 0;

		// Create a chunk writer.
		TUniquePtr<IFileSystem> FileSystem(FFileSystemFactory::Create());
		TUniquePtr<ICrypto> Crypto(FCryptoFactory::Create());
		TMap<FGuid, TArray<uint8>> EncryptionSecrets;
		const bool bEncryptNewData = Configuration.FeatureLevel >= EFeatureLevel::ChunkEncryptionSupport && Configuration.EncryptionSecretId.IsValid();
		if (bEncryptNewData)
		{
			EncryptionSecrets.Add(Configuration.EncryptionSecretId, Configuration.EncryptionSecretKey);
		}
		TUniquePtr<ICloudChunkSourceStat> CloudChunkSourceStat(FCloudChunkSourceStatFactory::Create(StatsCollector.Get()));
		TUniquePtr<IChunkDataSerialization> ChunkDataSerialization(FChunkDataSerializationFactory::Create(FileSystem.Get(), Crypto.Get(), { Configuration.FeatureLevel, MoveTemp(EncryptionSecrets), Configuration.EncryptionSecretId /* UE5 MERGE TODO : , CloudChunkSourceStatistics.Get()*/ }));
		TUniquePtr<IParallelChunkWriter> ChunkWriter(FParallelChunkWriterFactory::Create({ 5, 5, 50, 8, Configuration.bResaveExistingChunks, Configuration.CloudDirectory, Configuration.FeatureLevel }, FileSystem.Get(), ChunkDataSerialization.Get(), StatsCollector.Get()));
		// Register chunk writer events and pump from main thread.
		PatchGenerationHelpers::FChunkWriterEventsHandler ChunkWriterEventsHandler(ChunkWriter.Get());
		ChunkWriterEventsHandler.RegisterTicker();
		if (OnChunkFileWritten().IsBound())
		{
			ChunkWriter->OnChunkFileWritten().AddLambda([this](const FString& FullFilePath, const FMD5Hash& MD5Hash, const FSHAHash& SHA1Hash)
			{
				OnChunkFileWritten().Broadcast(FullFilePath, MD5Hash);
			});
		}

		ChunkWriter->OnChunkFileWriteFailed().AddLambda([this, ChunkWriterPtr = ChunkWriter.Get()](const FString& FullFilePath) {
			ChunkWriterPtr->Abort();
			bShouldAbort = true;
			UE_LOGF(LogPatchGeneration, Error, "Chunk writer aborted");
		});

		// Create a manifest details.
		FManifestDetails ManifestDetails;
		ManifestDetails.FeatureLevel = Configuration.FeatureLevel;
		ManifestDetails.AppId = Configuration.AppId;
		ManifestDetails.AppName = Configuration.AppName;
		ManifestDetails.BuildVersion = Configuration.BuildVersion;
		ManifestDetails.LaunchExe = Configuration.LaunchExe;
		ManifestDetails.LaunchCommand = Configuration.LaunchCommand;
		ManifestDetails.PrereqIds = Configuration.PrereqIds;
		ManifestDetails.PrereqName = Configuration.PrereqName;
		ManifestDetails.PrereqPath = Configuration.PrereqPath;
		ManifestDetails.PrereqArgs = Configuration.PrereqArgs;
		ManifestDetails.UninstallActionPath = Configuration.UninstallActionPath;
		ManifestDetails.UninstallActionArgs = Configuration.UninstallActionArgs;
		ManifestDetails.CustomFields = Configuration.CustomFields;
		ManifestDetails.EncryptionSecretKey = Configuration.EncryptionSecretKey;
		ManifestDetails.EncryptionSecretId = Configuration.EncryptionSecretId;

		// Load the required file attributes.
		if (!Configuration.AttributeListFile.IsEmpty())
		{
			FFileAttributesParserRef FileAttributesParser = FFileAttributesParserFactory::Create();
			TMap<FString, FFileAttributes> FileAttributes;
			if (!FileAttributesParser->ParseFileAttributes(Configuration.AttributeListFile, FileAttributes))
			{
				UE_LOGF(LogPatchGeneration, Error, "Attributes list file did not parse %ls", *Configuration.AttributeListFile);
				return false;
			}
			ManifestDetails.FileAttributesMap.Emplace(MoveTemp(FileAttributes));
		}

		// This lambda ensures that the given exe have the Unix executable bit set.
		auto EnsureExecutableBitSetLambda = [&ManifestDetails](const FString& ExecutablePath)
		{
			if (!ExecutablePath.IsEmpty())
			{
				ManifestDetails.FileAttributesMap.FindExactOrAdd(ExecutablePath).bUnixExecutable = true;
			}
		};

		// Ensure the launch, prereqs, and uninstall action executables have the executable bit set
		EnsureExecutableBitSetLambda(Configuration.LaunchExe);
		EnsureExecutableBitSetLambda(Configuration.PrereqPath);
		EnsureExecutableBitSetLambda(Configuration.UninstallActionPath);

		// Enumerate build files.
		TArray<FString> EnumeratedFiles;
		uint32 FileEnumerationStart = FStatsCollector::GetCycles();
		if (Configuration.InputListFile.IsEmpty())
		{
			// We won't filter the output.
			const TCHAR* const FileExtension = nullptr;
			FileSystem->FindFilesRecursively(EnumeratedFiles, *Configuration.RootDirectory, FileExtension, Configuration.bFollowSymlinks);
		}
		else
		{
			PatchGenerationHelpers::ReadInputFileList(Configuration.InputListFile, Configuration.RootDirectory, EnumeratedFiles);
		}

		PatchGenerationHelpers::NormalizeFilenames(EnumeratedFiles);
		uint32 FileEnumerationEnd = FStatsCollector::GetCycles();
		uint32 FileEnumerationTime = FileEnumerationEnd - FileEnumerationStart;
		UE_LOGF(LogPatchGeneration, Log, "Enumerated %d files in %ls", EnumeratedFiles.Num(), *FPlatformTime::PrettyTime(FStatsCollector::CyclesToSeconds(FileEnumerationTime)));

		// Remove the files that appear in the ignore list.
		if (!Configuration.IgnoreListFile.IsEmpty())
		{
			TSet<FString> IgnoreFiles, IgnorePatterns;
			PatchGenerationHelpers::ReadIgnoreFileList(Configuration.IgnoreListFile, Configuration.RootDirectory, IgnoreFiles, IgnorePatterns);
			PatchGenerationHelpers::StripIgnoredFiles(IgnoreFiles, IgnorePatterns, EnumeratedFiles);
		}

		// Convert final file list back to root relative paths.
		TArray<FString> BuildFiles;
		const int32 RootDirectoryLength = Configuration.RootDirectory.Len();
		Algo::Transform(EnumeratedFiles, BuildFiles, [RootDirectoryLength](const FString& EnumeratedFile) { FString Result = EnumeratedFile.RightChop(RootDirectoryLength); Result.RemoveFromStart(TEXT("/"), ESearchCase::CaseSensitive); return Result; });

		// Check existence of launch exe, if specified.
		if (!Configuration.LaunchExe.IsEmpty() && !BuildFiles.Contains(Configuration.LaunchExe))
		{
			UE_LOGF(LogPatchGeneration, Error, "Provided launch executable file was not found within the build root. %ls", *Configuration.LaunchExe);
			return false;
		}

		// Check existence of prereq exe, if specified.
		if (!Configuration.PrereqPath.IsEmpty() && !BuildFiles.Contains(Configuration.PrereqPath))
		{
			UE_LOGF(LogPatchGeneration, Error, "Provided prerequisite executable file was not found within the build root. %ls", *Configuration.PrereqPath);
			return false;
		}

		// Check existence of the uninstall action exe, if specified.
		if (!Configuration.UninstallActionPath.IsEmpty() && !BuildFiles.Contains(Configuration.UninstallActionPath))
		{
			UE_LOGF(LogPatchGeneration, Error, "Provided uninstall action executable file was not found within the build root. %ls", *Configuration.UninstallActionPath);
			return false;
		}

		// Enumerate Chunks.
		const FDateTime Cutoff = Configuration.bShouldHonorReuseThreshold ? FDateTime::UtcNow() - FTimespan::FromDays(Configuration.DataAgeThreshold) : FDateTime::MinValue();
		TUniquePtr<ICloudEnumeration> CloudEnumeration(FCloudEnumerationFactory::Create(Configuration.CloudDirectory, Cutoff, Configuration.ReuseManifestPredicate, Configuration.FeatureLevel, StatsCollector.Get()));

		// Sort the files accordingly.
		static_assert((int32)EFileSortOrder::InvalidOrMax == 2, "Please implement additional file sort orders.");
		switch (Configuration.FileSortOrder)
		{
		case EFileSortOrder::AlphabeticalFilename:
			BuildFiles.Sort();
			break;
		case EFileSortOrder::AlphabeticalTagThenFilename:
			Algo::Sort(BuildFiles, [&ManifestDetails](const FString& FileA, const FString& FileB)
			{
				FFileAttributes FileAttributesA = ManifestDetails.FileAttributesMap.Find(FileA);
				FFileAttributes FileAttributesB = ManifestDetails.FileAttributesMap.Find(FileB);
				const FString& FirstTagA = FileAttributesA.InstallTags.Num() > 0 ? *FileAttributesA.InstallTags.CreateConstIterator() : FString();
				const FString& FirstTagB = FileAttributesB.InstallTags.Num() > 0 ? *FileAttributesB.InstallTags.CreateConstIterator() : FString();
				// Sort by first tag if not the same.
				if (FirstTagA != FirstTagB)
				{
					return FirstTagA < FirstTagB;
				}
				// Otherwise alphabetical filename.
				return FileA < FileB;
			});
			break;
		}

		// Broadcast the build files enumeration.
		AsyncHelpers::ExecuteOnGameThread<void>([this, EnumeratedFiles]()
		{
			OnBuildFilesEnumerated().Broadcast(EnumeratedFiles);
		}).Wait();

		// Start the build stream.
		const bool bDetectMimeType = Configuration.FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType;
		const bool bCalculateMD5 = !Configuration.bSHA1Only && Configuration.FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType;
		const bool bCalculateSHA256 = !Configuration.bSHA1Only && Configuration.FeatureLevel >= BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes;
		FDirectoryBuildStreamerConfig BuildStreamConfig({ Configuration.RootDirectory, BuildFiles, Configuration.bFollowSymlinks, bDetectMimeType, bCalculateMD5, bCalculateSHA256 });
		FDirectoryBuildStreamerDependencies BuildStreamDependencies({ StatsCollector.Get(), FileSystem.Get() });
		TUniquePtr<IDirectoryBuildStreamer> BuildStream(FBuildStreamerFactory::Create(MoveTemp(BuildStreamConfig), MoveTemp(BuildStreamDependencies)));

		// We've got to wait for enumeration to complete as that shares a thread pool.
		while (!bShouldAbort && CloudEnumeration->IsComplete() == false)
		{
			FStatsCollector::Set(StatTotalTime, FStatsCollector::GetCycles() - StartTime);
			StatsCollector->LogStats(StatsLoggerTimeSeconds);

			// Sleep to allow other threads.
			FPlatformProcess::Sleep(0.01f);
		}

		if (bShouldAbort)
		{
			UE_LOGF(LogPatchGeneration, Error, "Aborting chunk generation process.");
			return false;
		}

		if (CloudEnumeration->GetUnsupportedFeatureLevelBuilds().Num() > 0)
		{
			const TArray<FString>& IncompartibleBuilds = CloudEnumeration->GetUnsupportedFeatureLevelBuilds();
			UE_LOGF(LogPatchGeneration, Error, "Incompatible build(s) detected. Please download and use the latest version of BPT to ensure data consistency. The following %d build(s) were generated by a newer version of BPT: \n\t\"%ls\"", IncompartibleBuilds.Num(), *FString::Join(IncompartibleBuilds, TEXT("\"\n\t\"")));
			return false;
		}

		// Grab the window sizes we are trying to match against.
		TArray<uint32> WindowSizes;
		if (Configuration.bShouldMatchAnyWindowSize)
		{
			WindowSizes.Append(CloudEnumeration->GetUniqueWindowSizes().Array());
		}
		else
		{
			WindowSizes.Add(Configuration.OutputChunkWindowSize);
		}
		Algo::Sort(WindowSizes, TGreater<uint32>());
		const uint32 LargestWindowSize = FMath::Max<uint32>(Configuration.OutputChunkWindowSize, WindowSizes.Num() > 0 ? WindowSizes[0] : 0);
		const uint64 ScannerOverlapSize = LargestWindowSize - 1;

		// Construct the chunk match processor.
		TUniquePtr<IChunkMatchProcessor> ChunkMatchProcessor(FChunkMatchProcessorFactory::Create());

		// Keep a record of the new chunk inventory.
		TMap<uint64, TSet<FGuid>> ChunkInventory;
		TMap<FGuid, FSHAHash> ChunkShaHashes;
		TMap<FGuid, FAESAuthTag> ChunkAuthTags;

		// Tracking info per layer for rescanning.
		TMap<int32, uint64> LayerToScannerCount;
		TMap<int32, FBlockStructure> LayerToBuildSpaceStructure;
		TMap<int32, uint64> LayerToCreatedScannerOffset;
		TMap<int32, uint64> LayerToScannedSize;
		TMap<int32, uint64> LayerToTotalDataSize;
		TMap<int32, FBlockStructure> LayerToUnknownLayerSpaceStructure;
		TMap<int32, FBlockStructure> LayerToUnknownBuildSpaceStructure;
		TMap<int32, TBlockData<uint8>> LayerToLayerSpaceBlockData;

		// This is a blatant hack :(
		TMap<FGuid, uint32> OriginalWindowSizes;

		// Create the manifest builder.
		FManifestBuilderConfig ManifestBuilderConfig;
		ManifestBuilderConfig.bAllowEmptyBuilds = Configuration.bAllowEmptyBuild;
		IManifestBuilderRef ManifestBuilder = FManifestBuilderFactory::Create(FileSystem.Get(), ManifestBuilderConfig, ManifestDetails);

		FBlockStructure AcceptedBuildSpaceMatches;
		FBlockStructure CreatedBuildSpaceMatches;
		TSet<FGuid> NewCreatedChunks;
		TMap<int32, FBlockStructure> LayerCreatingScannersTest;
		TMap<int32, FBlockStructure> LayerCreatingScannersLayerSpaceTest;
		TMap<int32, FBlockStructure> LayerCreatingChunksTest;
		TMap<int32, bool> LayerCreatingFinalTest;

		// Run the main loop.
		TArray<uint8> DataBuffer;
		uint64 DataBufferFirstIdx = 0;
		uint32 ReadLen = 0;
		TArray<TUniquePtr<FScannerDetails>> Scanners;
		bool bHasUnknownData = true;
		while (!bShouldAbort && !BuildStream->HasAborted() && (!BuildStream->IsEndOfData() || Scanners.Num() > 0 || bHasUnknownData))
		{
			// Grab a scanner result.
			if (Scanners.Num() > 0 && Scanners[0]->Scanner->IsComplete())
			{
				FScannerDetails& ScannerDetails = *Scanners[0];
				TArray<FChunkMatch> ChunkMatches = ScannerDetails.Scanner->GetResultWhenComplete();
				for (FChunkMatch& ChunkMatch : ChunkMatches)
				{
					// Translate to build space.
					FBlockStructure BuildSpaceChunkStructure;
					const uint64 BytesFound = ScannerDetails.Structure.SelectSerialBytes(ChunkMatch.DataOffset, ChunkMatch.WindowSize, BuildSpaceChunkStructure);
					const bool bFoundOk = ScannerDetails.bIsFinalScanner || BytesFound == ChunkMatch.WindowSize;
					if (!bFoundOk)
					{
						// Fatal error if the scanner returned a matched range that doesn't fit inside it's data.
						UE_LOGF(LogPatchGeneration, Error, "Chunk match was not within scanner's data structure.");
						return false;
					}

					ChunkMatch.DataOffset += ScannerDetails.LayerOffset;
					if (ChunkMatch.WindowSize != BytesFound)
					{
						OriginalWindowSizes.Add(ChunkMatch.ChunkGuid, ChunkMatch.WindowSize);
					}
					ChunkMatch.WindowSize = BytesFound;

					ChunkMatchProcessor->ProcessMatch(ScannerDetails.Layer, ChunkMatch, MoveTemp(BuildSpaceChunkStructure));
				}

				FBlockStructure OverlapStructure = LayerToBuildSpaceStructure.FindOrAdd(ScannerDetails.Layer).Intersect(ScannerDetails.Structure);
				uint64 OverlapBytes = BlockStructureHelpers::CountSize(OverlapStructure);
				check(OverlapBytes == ScannerOverlapSize || ScannerDetails.LayerOffset == 0);

				// Store the layer build space.
				LayerToBuildSpaceStructure.FindOrAdd(ScannerDetails.Layer).Add(ScannerDetails.Structure);

				// Add to layer space block data. We include padding that comes at the end of any layer as that may be included.
				const uint64 LayerDataStartIndex = ScannerDetails.LayerOffset == 0 ? 0 : ScannerOverlapSize;
				const uint64 LayerDataSize = ScannerDetails.LayerOffset == 0 ? ScannerDetails.Data.Num() : ScannerDetails.Data.Num() - ScannerOverlapSize;
				TBlockData<uint8>& LayerSpaceBlockData = LayerToLayerSpaceBlockData.FindOrAdd(ScannerDetails.Layer);
				LayerSpaceBlockData.AddData(FBlockStructure(ScannerDetails.LayerOffset + LayerDataStartIndex, LayerDataSize), ScannerDetails.Data.GetData() + LayerDataStartIndex, LayerDataSize);

				// Give some flush time to the processor.
				check(ScannerDetails.PaddingSize == 0 || ScannerDetails.bIsFinalScanner);
				const FBlockRange ScannerRange = FBlockRange::FromFirstAndSize(ScannerDetails.LayerOffset, ScannerDetails.Data.Num() - ScannerDetails.PaddingSize);
				const uint64 SafeFlushSize = ScannerDetails.bIsFinalScanner ? ScannerRange.GetLast() + 1 : ScannerRange.GetLast() - ScannerOverlapSize;
				ChunkMatchProcessor->FlushLayer(ScannerDetails.Layer, SafeFlushSize);
				if (ScannerDetails.bIsFinalScanner)
				{
					LayerToTotalDataSize.Add(ScannerDetails.Layer, SafeFlushSize);
				}

				// Remove scanner from list.
				LayerToScannerCount.FindOrAdd(ScannerDetails.Layer)--;
				Scanners.RemoveAt(0);
			}

			// Handle accepted chunk matches, and unknown data tracking.
			for (int32 LayerIdx = 0; LayerIdx <= MaxLayer; ++LayerIdx)
			{
				TArray<FMatchEntry> AcceptedChunkMatches;
				const FBlockStructure& LayerBuildSpaceStructure = LayerToBuildSpaceStructure.FindOrAdd(LayerIdx);
				uint64& LayerScannedSize = LayerToScannedSize.FindOrAdd(LayerIdx);
				const FBlockRange CollectionRange = ChunkMatchProcessor->CollectLayer(LayerIdx, AcceptedChunkMatches);
				if (CollectionRange.GetSize() > 0)
				{
					// Add new chunk matches to the manifest builder, and track new unknown data.
					TBlockData<uint8>& LayerSpaceBlockData = LayerToLayerSpaceBlockData.FindOrAdd(LayerIdx);
					FBlockStructure BlockDataToRemove;
					FBlockStructure NewUnknownLayerSpaceStructure(CollectionRange.GetFirst(), CollectionRange.GetSize());
					FBlockStructure NewUnknownBuildSpaceStructure;
					const uint64 BytesFound = LayerBuildSpaceStructure.SelectSerialBytes(CollectionRange.GetFirst(), CollectionRange.GetSize(), NewUnknownBuildSpaceStructure);
					checkSlow(BytesFound == CollectionRange.GetSize());
					const uint64 BeforeDataCount = LayerSpaceBlockData.GetDataCount();
					const uint64 BeforeStructureCount = CollectionRange.GetSize();
					for (FMatchEntry& AcceptedChunkMatch : AcceptedChunkMatches)
					{
						const FChunkMatch& ChunkMatch = AcceptedChunkMatch.ChunkMatch;
						const FBlockStructure& BlockStructure = AcceptedChunkMatch.BlockStructure;
						const FBlockStructure LayerSpaceStructure(ChunkMatch.DataOffset, ChunkMatch.WindowSize);

						check(BlockStructureHelpers::CountSize(LayerSpaceStructure.Intersect(NewUnknownLayerSpaceStructure)) == ChunkMatch.WindowSize);
						check(BlockStructureHelpers::CountSize(LayerSpaceStructure.Intersect(BlockDataToRemove)) == 0);
						check(BlockStructureHelpers::CountSize(NewUnknownBuildSpaceStructure.Intersect(BlockStructure)) == ChunkMatch.WindowSize);
						checkf(CreatedBuildSpaceMatches.Intersect(BlockStructure).GetHead() == nullptr, TEXT("ACCEPTEDCHUNK Overlap %llu bytes with created struct!"), BlockStructureHelpers::CountSize(CreatedBuildSpaceMatches.Intersect(BlockStructure)));
						checkf(AcceptedBuildSpaceMatches.Intersect(BlockStructure).GetHead() == nullptr, TEXT("ACCEPTEDCHUNK Overlap %llu bytes with accepted struct!"), BlockStructureHelpers::CountSize(AcceptedBuildSpaceMatches.Intersect(BlockStructure)));

						AcceptedBuildSpaceMatches.Add(BlockStructure);
						NewUnknownBuildSpaceStructure.Remove(BlockStructure);
						ManifestBuilder->AddChunkMatch(ChunkMatch.ChunkGuid, BlockStructure);
						// Do we need to re-save the chunk at current feature level?
						const bool bNeedsFeatureLevelUpgrade = !CloudEnumeration->IsChunkFeatureLevelMatch(ChunkMatch.ChunkGuid);
						if (Configuration.bResaveKnownChunks || bNeedsFeatureLevelUpgrade)
						{
							// Grab the data.
							TArray<uint8> ChunkDataArray;
							LayerSpaceBlockData.CopyTo(ChunkDataArray, LayerSpaceStructure);
							check(ChunkDataArray.Num() == ChunkMatch.WindowSize);
							// Ensure padding if necessary.
							const uint32 TrueWindowSize = OriginalWindowSizes.Contains(ChunkMatch.ChunkGuid) ? OriginalWindowSizes[ChunkMatch.ChunkGuid] : ChunkMatch.WindowSize;
							ChunkDataArray.SetNumZeroed(TrueWindowSize);

							// Save it out.
							const uint64& ChunkHash = CloudEnumeration->GetChunkHash(ChunkMatch.ChunkGuid);
							const FSHAHash& ChunkSha = CloudEnumeration->GetChunkShaHash(ChunkMatch.ChunkGuid);
							check(ChunkHash == FRollingHash::GetHashForDataSet(ChunkDataArray.GetData(), TrueWindowSize));
							check(ChunkSha == PatchGenerationHelpers::GetShaForDataSet(ChunkDataArray.GetData(), TrueWindowSize));
							if (!ChunkWriter->AddChunkData(MoveTemp(ChunkDataArray), ChunkMatch.ChunkGuid, ChunkHash, ChunkSha))
							{
								return false;
							}
						}
						if(!bNeedsFeatureLevelUpgrade)
						{
							// Broadcast the chunk match.
							AsyncHelpers::ExecuteOnGameThread<void>([this, ChunkGuid = ChunkMatch.ChunkGuid]()
							{
								OnChunkMatch().Broadcast(ChunkGuid);
							});
						}
						NewUnknownLayerSpaceStructure.Remove(LayerSpaceStructure);
						BlockDataToRemove.Add(LayerSpaceStructure);
					}
					const uint64 BlockDataToRemoveSize = BlockStructureHelpers::CountSize(BlockDataToRemove);
					LayerSpaceBlockData.RemoveData(BlockDataToRemove);
					const uint64 AfterDataCount = LayerSpaceBlockData.GetDataCount();
					const uint64 AfterStructureCount = BlockStructureHelpers::CountSize(NewUnknownLayerSpaceStructure);
					const uint64 RemovedDataCount = BeforeDataCount - AfterDataCount;
					const uint64 RemovedStructureCount = BeforeStructureCount - AfterStructureCount;
					check(BeforeDataCount >= AfterDataCount);
					check(BeforeStructureCount >= AfterStructureCount);
					check(RemovedDataCount == RemovedStructureCount);
					check(RemovedDataCount == BlockDataToRemoveSize);

					check(BlockStructureHelpers::CountSize(NewUnknownLayerSpaceStructure) == BlockStructureHelpers::CountSize(NewUnknownBuildSpaceStructure));

					// Grab layer tracking.
					FBlockStructure& UnknownLayerSpaceStructure = LayerToUnknownLayerSpaceStructure.FindOrAdd(LayerIdx);
					FBlockStructure& UnknownBuildSpaceStructure = LayerToUnknownBuildSpaceStructure.FindOrAdd(LayerIdx);

					// Expect to never get overlap with this new system.
					check(!BlockStructureHelpers::HasIntersection(UnknownLayerSpaceStructure, NewUnknownLayerSpaceStructure));
					check(!BlockStructureHelpers::HasIntersection(UnknownBuildSpaceStructure, NewUnknownBuildSpaceStructure));

					// Add unknown tracking to the structures.
					UnknownLayerSpaceStructure.Add(NewUnknownLayerSpaceStructure);
					UnknownBuildSpaceStructure.Add(NewUnknownBuildSpaceStructure);
					check(BlockStructureHelpers::CountSize(UnknownLayerSpaceStructure) == BlockStructureHelpers::CountSize(UnknownBuildSpaceStructure));

					// Count processed data
					LayerScannedSize = CollectionRange.GetLast() + 1;
				}
			}

			// Collect unknown data into new chunks.
			for (int32 LayerIdx = 0; LayerIdx <= MaxLayer; ++LayerIdx)
			{
				FBlockStructure& UnknownLayerSpaceStructure = LayerToUnknownLayerSpaceStructure.FindOrAdd(LayerIdx);
				FBlockStructure& UnknownBuildSpaceStructure = LayerToUnknownBuildSpaceStructure.FindOrAdd(LayerIdx);

				check(BlockStructureHelpers::CountSize(UnknownLayerSpaceStructure) == BlockStructureHelpers::CountSize(UnknownBuildSpaceStructure));

				const uint64& LayerScannedSize = LayerToScannedSize.FindOrAdd(LayerIdx);
				const bool bLayerComplete = LayerToTotalDataSize.Contains(LayerIdx) && (LayerScannedSize >= LayerToTotalDataSize[LayerIdx]);

				FBlockStructure ChunkedLayerSpaceStructure;
				FBlockStructure ChunkedBuildSpaceStructure;
				const FBlockEntry* UnknownLayerBlock = UnknownLayerSpaceStructure.GetHead();
				const bool bIsFinalSingleBlock = bLayerComplete && UnknownLayerBlock != nullptr && UnknownLayerBlock == UnknownLayerSpaceStructure.GetTail();
				uint64 UnknownBlockByteCount = 0;

				if (UnknownLayerBlock != nullptr)
				{
					UE_LOGF(LogPatchGeneration, VeryVerbose, "Unknown layer[%d] data at %llu bytes", LayerIdx, BlockStructureHelpers::CountSize(UnknownLayerSpaceStructure));
				}

				TBlockData<uint8>& LayerSpaceBlockData = LayerToLayerSpaceBlockData.FindOrAdd(LayerIdx);
				FBlockStructure BlockDataToRemove;
				while (!bShouldAbort && UnknownLayerBlock != nullptr)
				{
					uint64 UnknownBlockOffset = UnknownLayerBlock->GetOffset();
					uint64 UnknownBlockSize = UnknownLayerBlock->GetSize();
					bool bFinalLayerChunk = false;
					while (!bShouldAbort && (UnknownBlockSize >= LargestWindowSize || (bIsFinalSingleBlock && !bFinalLayerChunk)))
					{
						// Copy out the chunk data.
						FBlockStructure NewChunkLayerSpace(UnknownBlockOffset, FMath::Min<uint64>(Configuration.OutputChunkWindowSize, UnknownBlockSize));
						checkf((BlockStructureHelpers::CountSize(NewChunkLayerSpace) == Configuration.OutputChunkWindowSize) || bIsFinalSingleBlock, TEXT("Expected correct chunk size! (%llu == %u) || %d"), BlockStructureHelpers::CountSize(NewChunkLayerSpace), Configuration.OutputChunkWindowSize, bIsFinalSingleBlock);
						TArray<uint8> NewChunkDataArray;
						LayerSpaceBlockData.CopyTo(NewChunkDataArray, NewChunkLayerSpace);
						check(bIsFinalSingleBlock || NewChunkDataArray.Num() == Configuration.OutputChunkWindowSize);
						check(!bIsFinalSingleBlock || NewChunkDataArray.Num() == FMath::Min<uint64>(Configuration.OutputChunkWindowSize, UnknownBlockSize));
						// Ensure padding if necessary.
						NewChunkDataArray.SetNumZeroed(Configuration.OutputChunkWindowSize);

						// Create data for new chunk.
						const FGuid NewChunkGuid = FGuid::NewGuid();
						const uint64 NewChunkHash = FRollingHash::GetHashForDataSet(NewChunkDataArray.GetData(), Configuration.OutputChunkWindowSize);
						const FSHAHash NewChunkSha = PatchGenerationHelpers::GetShaForDataSet(NewChunkDataArray.GetData(), Configuration.OutputChunkWindowSize);

						// Save it out.
						if (!ChunkWriter->AddChunkData(MoveTemp(NewChunkDataArray), NewChunkGuid, NewChunkHash, NewChunkSha))
						{
							return false;
						}
						ChunkShaHashes.Add(NewChunkGuid, NewChunkSha);
						ChunkInventory.FindOrAdd(NewChunkHash).Add(NewChunkGuid);
						BlockDataToRemove.Add(NewChunkLayerSpace);

						UE_LOGF(LogPatchGeneration, Verbose, "Created layer[%d] chunk @ %llu for %d out of %llu", LayerIdx, UnknownBlockOffset, Configuration.OutputChunkWindowSize, UnknownBlockSize);

						// Add to manifest builder.
						FBlockStructure BuildSpaceChunkStructure;
						const uint64 ChunkBuildSize = UnknownBuildSpaceStructure.SelectSerialBytes(UnknownBlockByteCount, Configuration.OutputChunkWindowSize, BuildSpaceChunkStructure);
						bFinalLayerChunk = bIsFinalSingleBlock && (UnknownBlockSize == ChunkBuildSize);

						// Chunk build space should either be window size, or size minus any padding if the final piece.
						check(bIsFinalSingleBlock || ChunkBuildSize == Configuration.OutputChunkWindowSize);
						check(!bIsFinalSingleBlock || ChunkBuildSize == FMath::Min<uint64>(Configuration.OutputChunkWindowSize, UnknownBlockSize));

						// This new chunk must not overlap any previous chunks.
						check(CreatedBuildSpaceMatches.Intersect(BuildSpaceChunkStructure).GetHead() == nullptr);
						check(AcceptedBuildSpaceMatches.Intersect(BuildSpaceChunkStructure).GetHead() == nullptr);

						CreatedBuildSpaceMatches.Add(BuildSpaceChunkStructure);
						NewCreatedChunks.Add(NewChunkGuid);

						LayerCreatingChunksTest.FindOrAdd(LayerIdx).Add(BuildSpaceChunkStructure);
						ManifestBuilder->AddChunkMatch(NewChunkGuid, BuildSpaceChunkStructure);

						// Track data selected.
						ChunkedLayerSpaceStructure.Add(UnknownBlockOffset, ChunkBuildSize);
						ChunkedBuildSpaceStructure.Add(BuildSpaceChunkStructure);

						check(BlockStructureHelpers::CountSize(ChunkedLayerSpaceStructure) == BlockStructureHelpers::CountSize(ChunkedBuildSpaceStructure));

						UnknownBlockOffset += ChunkBuildSize;
						UnknownBlockSize -= ChunkBuildSize;
						UnknownBlockByteCount += ChunkBuildSize;
						check(!bFinalLayerChunk || UnknownBlockSize == 0);
					}
					UnknownBlockByteCount += UnknownBlockSize;
					UnknownLayerBlock = UnknownLayerBlock->GetNext();
					check(!bFinalLayerChunk || UnknownLayerBlock == nullptr);
				}
				UnknownLayerSpaceStructure.Remove(ChunkedLayerSpaceStructure);
				UnknownBuildSpaceStructure.Remove(ChunkedBuildSpaceStructure);
				LayerSpaceBlockData.RemoveData(BlockDataToRemove);
			}

			if (bShouldAbort)
			{
				UE_LOGF(LogPatchGeneration, Error, "Aborting chunk generation process.");
				return false;
			}

			// Create new scanners from unknown data.
			while (!bShouldAbort && !PatchGenerationHelpers::ScannerArrayFull(Scanners))
			{
				bool bScannerCreated = false;
				for (int32 LayerIdx = 0; LayerIdx <= MaxLayer; ++LayerIdx)
				{
					// Check that we have enough slack space in the data array to be queuing up more scanners on the next layer.
					const int32 NextLayer = LayerIdx + 1;
					const uint64 NextLayerSpaceBlockCount = LayerToLayerSpaceBlockData.FindOrAdd(NextLayer).GetDataCount();
					const int32 OneGigabyte = 1073741824;
					const bool bQueuedDataFull = NextLayerSpaceBlockCount > OneGigabyte;
					if (bQueuedDataFull)
					{
						UE_LOGF(LogPatchGeneration, Verbose, "Not making new scanners on next layer %d due to current backlog %llu bytes", NextLayer, NextLayerSpaceBlockCount);
						break;
					}

					FBlockStructure& UnknownLayerSpaceStructure = LayerToUnknownLayerSpaceStructure.FindOrAdd(LayerIdx);
					FBlockStructure& UnknownBuildSpaceStructure = LayerToUnknownBuildSpaceStructure.FindOrAdd(LayerIdx);
					TBlockData<uint8>& LayerSpaceBlockData = LayerToLayerSpaceBlockData.FindOrAdd(LayerIdx);
					const uint64& LayerScannedSize = LayerToScannedSize.FindOrAdd(LayerIdx);
					const bool bLayerComplete = LayerToTotalDataSize.Contains(LayerIdx) && (LayerScannedSize >= LayerToTotalDataSize[LayerIdx]);

					FBlockStructure NewScannerBuildSpaceStructure;
					FBlockStructure NewScannerLayerSpaceStructure;
					const uint64 SelectedBuildSpaceSize = UnknownBuildSpaceStructure.SelectSerialBytes(0, ScannerDataSize, NewScannerBuildSpaceStructure);
					const uint64 UnknownDataSize = BlockStructureHelpers::CountSize(UnknownBuildSpaceStructure);

					// Make sure there are enough bytes available for a scanner, plus a chunk, so that we know no more chunks will get
					// made from this sequential unknown data.
					const uint64 RequiredScannerBytes = ScannerDataSize + LargestWindowSize;
					const bool bHasEnoughData = bLayerComplete || UnknownDataSize > RequiredScannerBytes;

					if (bHasEnoughData && (SelectedBuildSpaceSize == ScannerDataSize || (bLayerComplete && SelectedBuildSpaceSize > 0)))
					{
						check(bHasEnoughData || bLayerComplete);

						const uint64 SelectedLayerSpaceSize = UnknownLayerSpaceStructure.SelectSerialBytes(0, ScannerDataSize, NewScannerLayerSpaceStructure);
						check(SelectedBuildSpaceSize == SelectedLayerSpaceSize);
						bScannerCreated = true;
						LayerToScannerCount.FindOrAdd(NextLayer)++;
						MaxLayer = FMath::Max<int64>(MaxLayer, NextLayer);
						FStatsCollector::Set(StatLayers, MaxLayer);

						uint64& NextLayerScannerOffset = LayerToCreatedScannerOffset.FindOrAdd(NextLayer);
						TArray<uint8> ScannerData;
						LayerSpaceBlockData.CopyTo(ScannerData, NewScannerLayerSpaceStructure);
						check(ScannerData.Num() == SelectedLayerSpaceSize);

						const bool bIsFinalScanner = bLayerComplete && UnknownDataSize <= SelectedBuildSpaceSize;
						const uint64 PadSize = bIsFinalScanner ? ScannerOverlapSize : 0;
						ScannerData.AddZeroed(PadSize);

						// Test overlaps.
						FBlockStructure OverlapStructure = LayerCreatingScannersTest.FindOrAdd(NextLayer).Intersect(NewScannerBuildSpaceStructure);
						uint64 OverlapBytes = BlockStructureHelpers::CountSize(OverlapStructure);
						FBlockStructure OverlapLayerSpaceStructure = LayerCreatingScannersLayerSpaceTest.FindOrAdd(NextLayer).Intersect(NewScannerLayerSpaceStructure);
						uint64 OverlapLayerSpaceBytes = BlockStructureHelpers::CountSize(OverlapLayerSpaceStructure);
						check(OverlapLayerSpaceBytes == ScannerOverlapSize || NextLayerScannerOffset == 0);
						check(OverlapBytes == ScannerOverlapSize || NextLayerScannerOffset == 0);
						LayerCreatingScannersTest.FindOrAdd(NextLayer).Add(NewScannerBuildSpaceStructure);
						LayerCreatingScannersLayerSpaceTest.FindOrAdd(NextLayer).Add(NewScannerLayerSpaceStructure);

						// Check only one final scanner.
						if (bIsFinalScanner)
						{
							check(LayerCreatingFinalTest.Contains(NextLayer) == false);
							LayerCreatingFinalTest.Add(NextLayer, true);
						}

						UE_LOGF(LogPatchGeneration, Verbose, "Creating scanner on layer %d at %llu. IsFinal:%d. Mapping:%ls, BuildMapping:%ls", NextLayer, NextLayerScannerOffset, bIsFinalScanner, *NewScannerLayerSpaceStructure.ToString(), *NewScannerBuildSpaceStructure.ToString());
						Scanners.Emplace(new FScannerDetails(NextLayer, NextLayerScannerOffset, bIsFinalScanner, PadSize, MoveTemp(ScannerData), NewScannerBuildSpaceStructure, WindowSizes, CloudEnumeration.Get(), StatsCollector.Get()));
						NextLayerScannerOffset += ScannerDataSize - ScannerOverlapSize;

						// Remove blocks from structures.
						NewScannerBuildSpaceStructure.Empty();
						NewScannerLayerSpaceStructure.Empty();
						const uint64 SerialBytesToSelect = bIsFinalScanner ? ScannerDataSize : ScannerDataSize - ScannerOverlapSize;
						const uint64 SizeBuildRemoving = UnknownBuildSpaceStructure.SelectSerialBytes(0, SerialBytesToSelect, NewScannerBuildSpaceStructure);
						const uint64 SizeLayerRemoving = UnknownLayerSpaceStructure.SelectSerialBytes(0, SerialBytesToSelect, NewScannerLayerSpaceStructure);
						UnknownBuildSpaceStructure.Remove(NewScannerBuildSpaceStructure);
						UnknownLayerSpaceStructure.Remove(NewScannerLayerSpaceStructure);
						LayerSpaceBlockData.RemoveData(NewScannerLayerSpaceStructure);
						check(SizeBuildRemoving == SizeLayerRemoving);
						check(SizeBuildRemoving == SerialBytesToSelect || SizeBuildRemoving == UnknownDataSize);
						check(!bIsFinalScanner || BlockStructureHelpers::CountSize(UnknownBuildSpaceStructure) == 0);
						check(!bIsFinalScanner || BlockStructureHelpers::CountSize(UnknownLayerSpaceStructure) == 0);
					}
					else
					{
						UE_LOGF(LogPatchGeneration, Verbose, "Not making Layer[%d] unknown data scanners.. RequiredScannerBytes:%llu UnknownDataSize:%llu", LayerIdx, RequiredScannerBytes, UnknownDataSize);
					}
				}
				// Stop when we cannot make scanners anymore.
				if (!bScannerCreated)
				{
					break;
				}
			}

			if (bShouldAbort)
			{
				UE_LOGF(LogPatchGeneration, Error, "Aborting chunk generation process.");
				return false;
			}

			// Stream some build data.
			if (!PatchGenerationHelpers::ScannerArrayFull(Scanners))
			{
				// Check that we have enough slack space in the data array to be queuing up more scanners on layer 0.
				const uint64 BottomLayerSpaceBlockCount = LayerToLayerSpaceBlockData.FindOrAdd(0).GetDataCount();
				const int32 OneGigabyte = 1073741824;
				const bool bQueuedDataFull = BottomLayerSpaceBlockCount > OneGigabyte;
				if (bQueuedDataFull)
				{
					UE_LOGF(LogPatchGeneration, Verbose, "Not making new scanners on layer 0 due to current backlog %llu bytes.", BottomLayerSpaceBlockCount);
				}
				else
				{
					// Create a scanner from new build data?
					if (!BuildStream->IsEndOfData())
					{
						// Keep the overlap data from previous scanner.
						int32 PreviousSize = DataBuffer.Num();
						if (PreviousSize > 0)
						{
							check(PreviousSize > ScannerOverlapSize);
							uint8* CopyTo = DataBuffer.GetData();
							uint8* CopyFrom = CopyTo + (PreviousSize - ScannerOverlapSize);
							FMemory::Memcpy(CopyTo, CopyFrom, ScannerOverlapSize);
							DataBuffer.SetNum(ScannerOverlapSize, EAllowShrinking::Yes);
							DataBufferFirstIdx += (PreviousSize - ScannerOverlapSize);
						}

						// Grab some data from the build stream.
						PreviousSize = DataBuffer.Num();
						DataBuffer.SetNumUninitialized(ScannerDataSize);
						const bool bWaitForData = true;
						ReadLen = BuildStream->DequeueData(DataBuffer.GetData() + PreviousSize, ScannerDataSize - PreviousSize, bWaitForData);
						DataBuffer.SetNum(PreviousSize + ReadLen, EAllowShrinking::Yes);

						// Only make a scanner if we are getting new data.
						if (ReadLen > 0)
						{
							// Pad scanner data if end of build
							uint64 PadSize = BuildStream->IsEndOfData() ? ScannerOverlapSize : 0;
							DataBuffer.AddZeroed(PadSize);

							// Create data scanner.
							const bool bIsFinalScanner = BuildStream->IsEndOfData();
							FBlockStructure Structure;
							Structure.Add(DataBufferFirstIdx, DataBuffer.Num() - PadSize);

							// Test overlaps.
							FBlockStructure OverlapStructure = LayerCreatingScannersTest.FindOrAdd(0).Intersect(Structure);
							FBlockStructure OverlapLayerSpaceStructure = LayerCreatingScannersLayerSpaceTest.FindOrAdd(0).Intersect(Structure);
							uint64 OverlapBytes = BlockStructureHelpers::CountSize(OverlapStructure);
							uint64 OverlapLayerSpaceBytes = BlockStructureHelpers::CountSize(OverlapLayerSpaceStructure);
							check(OverlapBytes == ScannerOverlapSize || DataBufferFirstIdx == 0);
							check(OverlapLayerSpaceBytes == ScannerOverlapSize || DataBufferFirstIdx == 0);
							LayerCreatingScannersTest.FindOrAdd(0).Add(Structure);
							LayerCreatingScannersLayerSpaceTest.FindOrAdd(0).Add(Structure);

							// Check only one final scanner.
							if (bIsFinalScanner)
							{
								check(LayerCreatingFinalTest.Contains(0) == false);
								LayerCreatingFinalTest.Add(0, true);
							}

							UE_LOGF(LogPatchGeneration, Verbose, "Creating scanner on layer 0 at %llu. IsFinal:%d. Mapping:%ls", DataBufferFirstIdx, BuildStream->IsEndOfData(), *Structure.ToString());
							Scanners.Emplace(new FScannerDetails(0, DataBufferFirstIdx, BuildStream->IsEndOfData(), PadSize, DataBuffer, MoveTemp(Structure), WindowSizes, CloudEnumeration.Get(), StatsCollector.Get()));
							LayerToScannerCount.FindOrAdd(0)++;
						}
					}
				}
			}

			// Did we run out of unknown data?
			bHasUnknownData = false;
			for (const TPair<int32, FBlockStructure>& UnknownBuildSpaceStructurePair : LayerToUnknownBuildSpaceStructure)
			{
				if (UnknownBuildSpaceStructurePair.Value.GetHead() != nullptr)
				{
					bHasUnknownData = true;
					break;
				}
			}

			// Update some stats.
			int64 UnknownDataAlloc = 0;
			int64 UnknownDataNum = 0;
			for (const TPair<int32, TBlockData<uint8>>& LayerToLayerSpaceBlockDataPair : LayerToLayerSpaceBlockData)
			{
				UnknownDataNum += LayerToLayerSpaceBlockDataPair.Value.GetDataCount();
				UnknownDataAlloc += LayerToLayerSpaceBlockDataPair.Value.GetAllocatedSize();
			}
			FStatsCollector::Set(StatUnknownDataAlloc, UnknownDataAlloc);
			FStatsCollector::Set(StatUnknownDataNum, UnknownDataNum);
			FStatsCollector::Set(StatNumScanners, Scanners.Num());

			FStatsCollector::Set(StatTotalTime, FStatsCollector::GetCycles() - StartTime);
			StatsCollector->LogStats(StatsLoggerTimeSeconds);
		}

		if (BuildStream->HasAborted())
		{
			UE_LOGF(LogPatchGeneration, Error, "Directory Build Streamer aborted");
			return false;
		}

		// Complete chunk writer.
		FParallelChunkWriterSummaries ChunkWriterSummaries = ChunkWriter->OnProcessComplete();

		// Remove pump from ChunkWriter, making sure a final pump is called.
		ChunkWriterEventsHandler.RemoveTicker();

		// Produce final stats log.
		const uint64 EndTime = FStatsCollector::GetCycles();
		FStatsCollector::Set(StatTotalTime, EndTime - StartTime);
		StatsCollector->LogStats();
		
		if (ChunkWriter->HasAborted())
		{
			return false;
		}

		// Collect chunk info for the manifest builder.
		TMap<FGuid, int64> ChunkFileSizes = CloudEnumeration->GetChunkFileSizes();
		ChunkFileSizes.Append(ChunkWriterSummaries.ChunkOutputFileSizes);
		TMap<FGuid, int64> ChunkCompressedSizes = CloudEnumeration->GetChunkCompressedSizes();
		ChunkCompressedSizes.Append(ChunkWriterSummaries.ChunkOutputCompressedSizes);
		TMap<FGuid, uint32> ChunkWindowSizes = CloudEnumeration->GetChunkWindowSizes();
		ChunkShaHashes.Append(CloudEnumeration->GetChunkShaHashes());
		TMap<FGuid, FGuid> ChunkSecretIds = CloudEnumeration->GetChunkSecretIds();
		for (const TPair<FGuid, FGuid>& EncryptedChunk : ChunkWriterSummaries.ChunkOutputSecretIds)
		{
			ChunkSecretIds.FindOrAdd(EncryptedChunk.Key) = EncryptedChunk.Value;
		}
		TMap<FGuid, FAESAuthTag> ChunkAESAuthTags = CloudEnumeration->GetChunkAESAuthTags();
		for (const TPair<FGuid, FAESAuthTag>& EncryptedChunk : ChunkWriterSummaries.ChunkOutputAuthTags)
		{
			ChunkAESAuthTags.FindOrAdd(EncryptedChunk.Key) = EncryptedChunk.Value;
		}
		if (!bEncryptNewData)
		{
			check(ChunkWriterSummaries.ChunkOutputSecretIds.Num() == 0);
			check(ChunkWriterSummaries.ChunkOutputAuthTags.Num() == 0);
		}
		for (const TPair<uint64, TSet<FGuid>>& ChunkInventoryPair : CloudEnumeration->GetChunkInventory())
		{
			TSet<FGuid>& ChunkSet = ChunkInventory.FindOrAdd(ChunkInventoryPair.Key);
			ChunkSet = ChunkSet.Union(ChunkInventoryPair.Value);
		}

		TMap<FGuid, FChunkInfo> ChunkInfoMap;
		for (const TPair<uint64, TSet<FGuid>>& ChunkInventoryPair : ChunkInventory)
		{
			for (const FGuid& ChunkGuid : ChunkInventoryPair.Value)
			{
				if (ChunkShaHashes.Contains(ChunkGuid) && ChunkFileSizes.Contains(ChunkGuid))
				{
					FChunkInfo& ChunkInfo = ChunkInfoMap.FindOrAdd(ChunkGuid);
					ChunkInfo.Guid = ChunkGuid;
					ChunkInfo.Hash = ChunkInventoryPair.Key;
					FMemory::Memcpy(ChunkInfo.ShaHash.Hash, ChunkShaHashes[ChunkGuid].Hash, FSHA1::DigestSize);
					ChunkInfo.FileSize = ChunkFileSizes[ChunkGuid];
					ChunkInfo.GroupNumber = FCrc::MemCrc32(&ChunkGuid, sizeof(FGuid)) % 100;
					ChunkInfo.DataSizeCompressed = ChunkCompressedSizes.FindRef(ChunkGuid);
					ChunkInfo.DataSizeUncompressed = ChunkWindowSizes.Contains(ChunkGuid) ? ChunkWindowSizes[ChunkGuid] : Configuration.OutputChunkWindowSize;
					ChunkInfo.EncryptionSecretId = ChunkSecretIds.FindRef(ChunkGuid);
					ChunkInfo.AESAuthTag = ChunkAESAuthTags.FindRef(ChunkGuid);
				}
			}
		}
		
		if (bShouldAbort)
		{
			UE_LOGF(LogPatchGeneration, Error, "Aborting manifest finalization.");
			return false;
		}

		// Finalize the manifest data.
		TArray<FChunkInfo> ChunkInfoList;
		ChunkInfoMap.GenerateValueArray(ChunkInfoList);

		TArray<FFileSpan> FileSpans = BuildStream->GetAllFiles();

		bool bSkipWindowsFilenameLengthCheckSpecified = FParse::Param(FCommandLine::Get(), TEXT("SkipWindowsFilenameLengthCheck"));
		if (!bSkipWindowsFilenameLengthCheckSpecified)
		{
			int32 WindowsFileLengthWarning = 220;
			GConfig->GetInt(TEXT("BuildPatchServices"), TEXT("WindowsFileLengthWarning"), WindowsFileLengthWarning, GEngineIni);

			TArray<FString> TooLongFilenames;
			for (const FFileSpan& FileSpan : FileSpans)
			{
				if (FileSpan.Filename.Len() >= WindowsFileLengthWarning)
				{
					TooLongFilenames.Add(FileSpan.Filename);
				}
			}

			if (TooLongFilenames.Num() > 0)
			{
				FString WarningMsgText = FString::Format(TEXT("The following filenames are over {0} in length, which may cause issues for some users on Windows who use the default install location:"), { WindowsFileLengthWarning });
				WarningMsgText += "\n         (This warning can be suppressed with the -SkipWindowsFilenameLengthCheck command line arg)";
				for (const FString& Filename : TooLongFilenames)
				{
					WarningMsgText += "\n         " + Filename;
				}
				UE_LOGF(LogPatchGeneration, Warning, "%ls", *WarningMsgText);
			}
		}

		if (ManifestBuilder->FinalizeData(FileSpans, MoveTemp(ChunkInfoList)) == false)
		{
			UE_LOGF(LogPatchGeneration, Error, "Finalizing manifest failed.");
			return false;
		}
		uint64 NewChunkBytes = 0;
		for (const FGuid& NewChunk : NewCreatedChunks)
		{
			NewChunkBytes += ChunkWriterSummaries.ChunkOutputFileSizes[NewChunk];
		}
		UE_LOGF(LogPatchGeneration, Log, "Created %d chunks (%llu build bytes) (%llu compressed bytes)", NewCreatedChunks.Num(), BlockStructureHelpers::CountSize(CreatedBuildSpaceMatches), NewChunkBytes);
		UE_LOGF(LogPatchGeneration, Log, "Completed in %ls.", *FPlatformTime::PrettyTime(FStatsCollector::CyclesToSeconds(*StatTotalTime)));

		AsyncHelpers::ExecuteOnGameThread<void>([this, &ManifestBuilder]()
		{
			OnManifestGenerated().Broadcast(ManifestBuilder->GetManifest());
		}).Wait();

		// Save manifest out to the cloud directory.
		FMD5Hash MD5Hash;
		FSHAHash SHAHash;
		const FString OutputFilename = Configuration.CloudDirectory / Configuration.OutputFilename;
		if (ManifestBuilder->SaveToFile(OutputFilename, &SHAHash, &MD5Hash) == false)
		{
			UE_LOGF(LogPatchGeneration, Error, "Saving manifest failed.");
			return false;
		}
		UE_LOGF(LogPatchGeneration, Log, "Saved manifest to %ls.", *OutputFilename);

		AsyncHelpers::ExecuteOnGameThread<void>([this, OutputFilename, MD5Hash, SHAHash]()
		{
			OnManifestFileWritten().Broadcast(OutputFilename, MD5Hash, SHAHash);
		}).Wait();

		TMap<FGuid, TArray<uint8>> EncryptionSecret;
		EncryptionSecret.Add(Configuration.EncryptionSecretId, Configuration.EncryptionSecretKey);
		FBuildPatchAppManifest& Manifest = ManifestBuilder->GetManifest();
		Manifest.DecryptData(EncryptionSecret);
		AsyncHelpers::ExecuteOnGameThread<void>([this, &Manifest]()
		{
			OnManifestGeneratedDecrypted().Broadcast(Manifest);
		}).Wait();

		return true;
	}

	void FDirectoryChunker::Abort()
	{
		bShouldAbort = true;
	}

	IDirectoryChunker* FDirectoryChunkerFactory::Create(FDirectoryChunkerConfiguration Configuration)
	{
		return new FDirectoryChunker(MoveTemp(Configuration));
	}
}
