// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BuildPatchHash.cpp: Implements the static struct FRollingHashConst
=============================================================================*/

#include "BuildPatchHash.h"

// We'll use the commonly used in CRC64, ECMA polynomial defined in ECMA 182.
static const uint64 HashPoly64 = 0xC96C5795D7870F42;

uint64 FRollingHashConst::HashTable[ 256 ] = { 0 };

void FRollingHashConst::Init()
{
	for( uint32 TableIdx = 0; TableIdx < 256; ++TableIdx )
	{
		uint64 val = TableIdx;
		for( uint32 ShiftCount = 0; ShiftCount < 8; ++ShiftCount )
		{
			if( ( val & 1 ) == 1 )
			{
				val >>= 1;
				val ^= HashPoly64;
			}
			else
			{
				val >>= 1;
			}
		}
		HashTable[ TableIdx ] = val;
	}
}

void FRollingHash::Clear()
{
	HashState = 0;
	NumBytesConsumed = 0;
	WindowData.Empty();
}

const uint64 FRollingHash::GetWindowHash() const
{
	// We must have consumed enough bytes to function correctly
	check(NumBytesConsumed == WindowSize);
	return HashState;
}

const TRingBuffer< uint8 >& FRollingHash::GetWindowData() const
{
	return WindowData;
}

const uint32 FRollingHash::GetWindowSize() const
{
	return WindowSize;
}

const uint32 FRollingHash::GetNumDataNeeded() const
{
	return WindowSize - NumBytesConsumed;
}

void FRollingHash::ConsumeBytes(const uint8* NewBytes, const uint32& NumBytes)
{
	for (uint32 i = 0; i < NumBytes; ++i)
	{
		ConsumeByte(NewBytes[i]);
	}
}

void FRollingHash::RollForward(const uint8& NewByte)
{
	// We must have consumed enough bytes to function correctly
	check(NumBytesConsumed == WindowSize);
	uint8 OldByte = 0;
	WindowData.Dequeue(OldByte);
	WindowData.Enqueue(NewByte);
	// Update our HashState
	uint64 OldTerm = FRollingHashConst::HashTable[OldByte];
	ROTLEFT_64B(OldTerm, WindowSize);
	ROTLEFT_64B(HashState, 1);
	HashState ^= OldTerm;
	HashState ^= FRollingHashConst::HashTable[NewByte];
}

uint64 FRollingHash::GetHashForDataSet(const uint8* DataSet, uint32 WindowSize)
{
	uint64 HashState = 0;
	for (uint32 i = 0; i < WindowSize; ++i)
	{
		ROTLEFT_64B(HashState, 1);
		HashState ^= FRollingHashConst::HashTable[DataSet[i]];
	}
	return HashState;
}

FRollingHash::FRollingHash(uint32 InWindowSize)
	: WindowSize(InWindowSize)
	, WindowData(WindowSize)
{
}

FRollingHash::FRollingHash()
{
}

void FRollingHash::ConsumeByte(const uint8& NewByte)
{
	// We must be setup by consuming the correct amount of bytes
	check(NumBytesConsumed < WindowSize);
	++NumBytesConsumed;

	// Add the byte to our buffer
	WindowData.Enqueue(NewByte);
	// Add to our HashState
	ROTLEFT_64B(HashState, 1);
	HashState ^= FRollingHashConst::HashTable[NewByte];
}
