// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Core/RingBuffer.h"

/*=============================================================================
	BuildPatchHash.h: Declares and implements template classes for use with the 
	Build and Patching hash functionality.
=============================================================================*/

/**
 * A macro for barrel rolling a 64 bit value n times to the left.
 * @param Value		The value to shift
 * @param Shifts	The number of times to shift
 */
#define ROTLEFT_64B( Value, Shifts ) Value = ( ( ( Value ) << ( ( Shifts ) % 64 ) ) | ( ( Value ) >> ( ( 64 - ( ( Shifts ) % 64 ) ) % 64 ) ) )

/**
 * A static struct containing 64bit polynomial constant and hash table lookup for use with FRollingHash.
 */
struct FRollingHashConst
{
	// The lookup hash table
	static uint64 HashTable[ 256 ];
	
	/**
	 * Builds the hash table for use when hashing. Must be called before using FRollingHash.
	 */
	static void Init();
};

/**
 * A class that performs a rolling hash
 */
class FRollingHash
{
	// A typedef for our data ring buffer
	typedef TRingBuffer< uint8 > HashRingBuffer;

public:
	/**
	 * Constructor
	 */
	FRollingHash(uint32 WindowSize);

	/**
	 * Pass this function the initial data set to start the Rolling Hash with.
	 * @param NewByte		The byte to add to the hash state
	 */
	void ConsumeByte( const uint8& NewByte );

	/**
	 * Helper to consume from a byte array
	 * @param NewBytes		The byte array
	 * @param NumBytes		The number of bytes to consume
	 */
	void ConsumeBytes( const uint8* NewBytes, const uint32& NumBytes );

	/**
	 * Rolls the window by one byte forwards.
	 * @param NewByte		The byte that will be added to the front of the window
	 */
	void RollForward( const uint8& NewByte );

	/**
	 * Clears all data ready for a new entire data set
	 */
	void Clear();

	/**
	 * Get the HashState for the current window
	 * @return		The hash state
	 */
	const uint64 GetWindowHash() const;

	/**
	 * Get the Ring Buffer for the current window
	 * @return		The ring buffer
	 */
	const HashRingBuffer& GetWindowData() const;

	/**
	 * Get how many DataValueType values we still need to consume until our window is full 
	 * @return		The number of DataValueType we still need
	 */
	const uint32 GetNumDataNeeded() const;

	/**
	 * @return the size of our window
	 */
	const uint32 GetWindowSize() const;

	/**
	 * Static function to simply return the hash for a given data range.
	 * @param DataSet     The buffer to the data.
	 * @param WindowSize  The buffer of length.
	 * @return The hash state for the provided data
	 */
	static uint64 GetHashForDataSet( const uint8* DataSet, uint32 WindowSize );

private:
	FRollingHash();
	// The data size that we roll over.
	const uint32 WindowSize = 0;
	// The current hash value
	uint64 HashState = 0;
	// The number of bytes we have consumed so far, used in hash function and to check validity of calls
	uint32 NumBytesConsumed = 0;
	// Store the data to make access and rolling easier
	HashRingBuffer WindowData = 0;
};

/**
 * A static function to perform sanity checks on the rolling hash class.
 */
static bool CheckRollingHashAlgorithm()
{
	bool bCheckOk = true;
	// Sanity Check the RollingHash code!!
	FString IndivWords[6];
	IndivWords[0] = "123456";	IndivWords[1] = "7890-=";	IndivWords[2] = "qwerty";	IndivWords[3] = "uiop[]";	IndivWords[4] = "asdfgh";	IndivWords[5] = "jkl;'#";
	FString DataToRollOver = FString::Printf( TEXT( "%s%s%s%s%s%s" ), *IndivWords[0], *IndivWords[1], *IndivWords[2], *IndivWords[3], *IndivWords[4], *IndivWords[5] );
	uint64 IndivHashes[6];
	for (uint32 idx = 0; idx < 6; ++idx )
	{
		uint8 Converted[6];
		for (uint32 iChar = 0; iChar < 6; ++iChar )
			Converted[iChar] = IndivWords[idx][iChar];
		IndivHashes[idx] = FRollingHash::GetHashForDataSet( Converted, 6 );
	}

	FRollingHash RollingHash(6);
	uint32 StrIdx = 0;
	for (uint32 k=0; k<6; ++k)
		RollingHash.ConsumeByte( DataToRollOver[ StrIdx++ ] );
	bCheckOk = bCheckOk && IndivHashes[0] == RollingHash.GetWindowHash();
	for (uint32 k=0; k<6; ++k)
		RollingHash.RollForward( DataToRollOver[ StrIdx++ ] );
	bCheckOk = bCheckOk && IndivHashes[1] == RollingHash.GetWindowHash();
	for (uint32 k=0; k<6; ++k)
		RollingHash.RollForward( DataToRollOver[ StrIdx++ ] );
	bCheckOk = bCheckOk && IndivHashes[2] == RollingHash.GetWindowHash();
	for (uint32 k=0; k<6; ++k)
		RollingHash.RollForward( DataToRollOver[ StrIdx++ ] );
	bCheckOk = bCheckOk && IndivHashes[3] == RollingHash.GetWindowHash();
	for (uint32 k=0; k<6; ++k)
		RollingHash.RollForward( DataToRollOver[ StrIdx++ ] );
	bCheckOk = bCheckOk && IndivHashes[4] == RollingHash.GetWindowHash();
	for (uint32 k=0; k<6; ++k)
		RollingHash.RollForward( DataToRollOver[ StrIdx++ ] );
	bCheckOk = bCheckOk && IndivHashes[5] == RollingHash.GetWindowHash();

	return bCheckOk;
}
