// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchInstall.h"

#include "BuildPatchFileConstructor.h"

uint64 BuildPatchServices::CalculateRequiredDiskSpace(const IBuildManifestPtr& CurrentManifest, const IBuildManifestRef& BuildManifest, const BuildPatchServices::EInstallMode & InstallMode, const TSet<FString>& InstallTags)
{
	return FileConstructorHelpers::CalculateRequiredDiskSpace(StaticCastSharedPtr<FBuildPatchAppManifest>(CurrentManifest), StaticCastSharedRef<FBuildPatchAppManifest>(BuildManifest), InstallMode, InstallTags);
}
