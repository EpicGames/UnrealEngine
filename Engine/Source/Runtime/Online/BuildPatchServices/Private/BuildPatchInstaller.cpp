// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BuildPatchInstaller.cpp: Implements the FBuildPatchInstaller class which
	controls the process of installing a build described by a build manifest.
=============================================================================*/

#include "BuildPatchInstaller.h"

#include "Algo/AnyOf.h"
#include "Algo/Transform.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/RunnableThread.h"
#include "HttpModule.h"
#include "Math/UnitConversion.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

#if PLATFORM_MAC || PLATFORM_LINUX
#include <unistd.h>
#endif

#include "BuildPatchFileConstructor.h"
#include "BuildPatchServicesModule.h"
#include "BuildPatchServicesPrivate.h"
#include "BuildPatchSettings.h"
#include "BuildPatchUtil.h"
#include "Common/ChunkDataSizeProvider.h"
#include "Common/Crypto.h"
#include "Common/FileSystem.h"
#include "Common/HttpManager.h"
#include "Generation/ChunkDatabaseWriter.h"
#include "Generation/PackageChunkData.h"
#include "IBuildManifestSet.h"
#include "Installer/ChunkDbChunkSource.h"
#include "Installer/ChunkEvictionPolicy.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/CloudChunkSource.h"
#include "Installer/CustomUninstallAction.h"
#include "Installer/DownloadConnectionCount.h"
#include "Installer/DownloadService.h"
#include "Installer/FileAttribution.h"
#include "Installer/InstallChunkSource.h"
#include "Installer/InstallerAnalytics.h"
#include "Installer/InstallerError.h"
#include "Installer/InstallerSharedContext.h"
#include "Installer/MachineConfig.h"
#include "Installer/MemoryChunkStore.h"
#include "Installer/MessagePump.h"
#include "Installer/OptimisedDelta.h"
#include "Installer/Prerequisites.h"
#include "Installer/Statistics/ChunkDbChunkSourceStatistics.h"
#include "Installer/Statistics/CloudChunkSourceStatistics.h"
#include "Installer/Statistics/DownloadServiceStatistics.h"
#include "Installer/Statistics/Experiments.h"
#include "Installer/Statistics/FileConstructorStatistics.h"
#include "Installer/Statistics/FileOperationTracker.h"
#include "Installer/Statistics/InstallChunkSourceStatistics.h"
#include "Installer/Statistics/MemoryChunkStoreStatistics.h"
#include "Installer/Statistics/VerifierStatistics.h"
#include "Installer/Statistics/InstallActionStatistics.h"
#include "Installer/UriProvider.h"
#include "Installer/Verifier.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBPSInstallerConfig, Log, All);
DEFINE_LOG_CATEGORY(LogBPSInstallerConfig);

namespace ConfigHelpers
{
	using namespace BuildPatchServices;

	int32 LoadNumFileMoveRetries()
	{
		int32 MoveRetries = 5;
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("NumFileMoveRetries"), MoveRetries, GEngineIni);
		return FMath::Clamp<int32>(MoveRetries, 1, 50);
	}

	int32 LoadNumInstallerRetries()
	{
		int32 InstallerRetries = 5;
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("NumInstallerRetries"), InstallerRetries, GEngineIni);
		return FMath::Clamp<int32>(InstallerRetries, 1, 50);
	}

	float LoadDownloadSpeedAverageTime()
	{
		float AverageTime = 10.0;
		GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("DownloadSpeedAverageTime"), AverageTime, GEngineIni);
		return FMath::Clamp<float>(AverageTime, 1.0f, 30.0f);
	}

	float LoadDownloadSpeedAverageTime_Throttled()
	{
		float AverageTime = 10.0;
		GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("DownloadSpeedAverageTime_Throttled"), AverageTime, GEngineIni);
		return FMath::Clamp<float>(AverageTime, 1.0f, 30.0f);
	}

	uint64 LoadPreloadMaxChunkDbSize()
	{
		int32 PreloadMaxChunkDbSizeMB = 3000;
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("PreloadMaxChunkDbSizeMB"), PreloadMaxChunkDbSizeMB, GEngineIni);
		return uint64(FMath::Clamp<int32>(PreloadMaxChunkDbSizeMB, 1, TNumericLimits<int32>::Max())) * 1000 * 1000;
	}

	float DownloadSpeedAverageTime(bool bIsThrottlingDownload)
	{
		static float AverageTime = LoadDownloadSpeedAverageTime();
		static float AverageTimeThrottled = LoadDownloadSpeedAverageTime_Throttled();
		return bIsThrottlingDownload ? AverageTimeThrottled : AverageTime;
	}

	int32 NumFileMoveRetries()
	{
		static int32 MoveRetries = LoadNumFileMoveRetries();
		return MoveRetries;
	}

	int32 NumInstallerRetries()
	{
		static int32 InstallerRetries = LoadNumInstallerRetries();
		return InstallerRetries;
	}

	FOptimisedDeltaConfiguration BuildOptimisedDeltaConfig(const FBuildInstallerConfiguration& Config, const FBuildPatchInstallerAction& InstallerAction, const int32_t RetriesNumber)
	{
		check(InstallerAction.IsUpdate());
		// The optimised delta can deal with getting dupe manifests so lets just allow that for easy config.
		FOptimisedDeltaConfiguration OptimisedDeltaConfiguration(InstallerAction.GetSharedInstallOrCurrentManifest());
		OptimisedDeltaConfiguration.SourceManifest = InstallerAction.TryGetSharedCurrentManifest();
		OptimisedDeltaConfiguration.DeltaPolicy = Config.DeltaPolicy;
		OptimisedDeltaConfiguration.InstallMode = Config.InstallMode;
		OptimisedDeltaConfiguration.RetriesNumber = RetriesNumber;
		OptimisedDeltaConfiguration.DeltaFilenameTrailer = Config.DeltaFilenameTrailer;
		return OptimisedDeltaConfiguration;
	}

	TArray<FString> EnumeratePreloadChunkDbs(const TCHAR* SearchPath)
	{
		TArray<FString> ChunkDbResults;
		IFileManager::Get().FindFiles(ChunkDbResults, SearchPath, TEXT("chunkdb"));
		return ChunkDbResults;
	}
}

namespace InstallerHelpers
{
	using namespace BuildPatchServices;

	void LogBuildStatInfo(const FBuildInstallStats& BuildStats, const FGuid& InstallerId)
	{
		using namespace BuildPatchServices;
		static FCriticalSection DoNotInterleaveLogs;
		FScopeLock ScopeLock(&DoNotInterleaveLogs);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: InstallerId: %ls", *InstallerId.ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumFilesInBuild: %u", BuildStats.NumFilesInBuild);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumFilesOutdated: %u", BuildStats.NumFilesOutdated);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumFilesToRemove: %u", BuildStats.NumFilesToRemove);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumChunksRequired: %u", BuildStats.NumChunksRequired);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ChunksQueuedForDownload: %u", BuildStats.ChunksQueuedForDownload);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ChunksLocallyAvailable: %u", BuildStats.ChunksLocallyAvailable);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ChunksInChunkDbs: %u", BuildStats.ChunksInChunkDbs);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumChunksDownloaded: %u", BuildStats.NumChunksDownloaded);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumChunksRecycled: %u", BuildStats.NumChunksRecycled);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumChunksReadFromChunkDbs: %u", BuildStats.NumChunksReadFromChunkDbs);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumFailedDownloads: %u", BuildStats.NumFailedDownloads);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumBadDownloads: %u", BuildStats.NumBadDownloads);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumAbortedDownloads: %u", BuildStats.NumAbortedDownloads);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumRecycleFailures: %u", BuildStats.NumRecycleFailures);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumDriveStoreLostChunks: %u", BuildStats.NumDriveStoreLostChunks);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumDriveStoreChunkLoads: %u", BuildStats.NumDriveStoreChunkLoads);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumDriveStoreLoadFailures: %u", BuildStats.NumDriveStoreLoadFailures);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumChunkDbChunksFailed: %u", BuildStats.NumChunkDbChunksFailed);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: TotalDownloadedData: %llu", BuildStats.TotalDownloadedData);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ActiveRequestCountPeak : %u", BuildStats.ActiveRequestCountPeak);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: AverageDownloadSpeed: %ls bytes (%ls, %ls) /sec", *FText::AsNumber(BuildStats.AverageDownloadSpeed).ToString(), *FText::AsMemory(BuildStats.AverageDownloadSpeed, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(BuildStats.AverageDownloadSpeed, EMemoryUnitStandard::IEC).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: PeakDownloadSpeed: %ls bytes (%ls, %ls) /sec", *FText::AsNumber(BuildStats.PeakDownloadSpeed).ToString(), *FText::AsMemory(BuildStats.PeakDownloadSpeed, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(BuildStats.PeakDownloadSpeed, EMemoryUnitStandard::IEC).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: TotalReadData: %ls", *FText::AsNumber(BuildStats.TotalReadData).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: AverageDiskReadSpeed: %ls bytes (%ls, %ls) /sec", *FText::AsNumber(BuildStats.AverageDiskReadSpeed).ToString(), *FText::AsMemory(BuildStats.AverageDiskReadSpeed, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(BuildStats.AverageDiskReadSpeed, EMemoryUnitStandard::IEC).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: PeakDiskReadSpeed: %ls bytes (%ls, %ls) /sec", *FText::AsNumber(BuildStats.PeakDiskReadSpeed).ToString(), *FText::AsMemory(BuildStats.PeakDiskReadSpeed, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(BuildStats.PeakDiskReadSpeed, EMemoryUnitStandard::IEC).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: TotalWrittenData: %ls", *FText::AsNumber(BuildStats.TotalWrittenData).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: AverageDiskWriteSpeed: %ls bytes (%ls, %ls) /sec", *FText::AsNumber(BuildStats.AverageDiskWriteSpeed).ToString(), *FText::AsMemory(BuildStats.AverageDiskWriteSpeed, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(BuildStats.AverageDiskWriteSpeed, EMemoryUnitStandard::IEC).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: PeakDiskWriteSpeed: %ls bytes (%ls, %ls) /sec", *FText::AsNumber(BuildStats.PeakDiskWriteSpeed).ToString(), *FText::AsMemory(BuildStats.PeakDiskWriteSpeed, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(BuildStats.PeakDiskWriteSpeed, EMemoryUnitStandard::IEC).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumFilesConstructed: %u", BuildStats.NumFilesConstructed);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: InitializeTime: %ls", *FPlatformTime::PrettyTime(BuildStats.InitializeTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: TheoreticalDownloadTime: %ls", *FPlatformTime::PrettyTime(BuildStats.TheoreticalDownloadTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ConstructTime: %ls", *FPlatformTime::PrettyTime(BuildStats.ConstructTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: TotalDecryptTime: %ls", *FPlatformTime::PrettyTime(BuildStats.TotalDecryptTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: UninstallActionTime: %ls", *FPlatformTime::PrettyTime(BuildStats.UninstallActionTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: MoveFromStageTime: %ls", *FPlatformTime::PrettyTime(BuildStats.MoveFromStageTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: FileAttributesTime: %ls", *FPlatformTime::PrettyTime(BuildStats.FileAttributesTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: VerifyTime: %ls", *FPlatformTime::PrettyTime(BuildStats.VerifyTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: CleanUpTime: %ls", *FPlatformTime::PrettyTime(BuildStats.CleanUpTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: PrereqTime: %ls", *FPlatformTime::PrettyTime(BuildStats.PrereqTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ProcessPausedTime: %ls", *FPlatformTime::PrettyTime(BuildStats.ProcessPausedTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ProcessActiveTime: %ls", *FPlatformTime::PrettyTime(BuildStats.ProcessActiveTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ProcessExecuteTime: %ls", *FPlatformTime::PrettyTime(BuildStats.ProcessExecuteTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ProcessSuccess: %ls", BuildStats.ProcessSuccess ? TEXT("TRUE") : TEXT("FALSE"));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ErrorCode: %ls", *BuildStats.ErrorCode);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: FailureReasonText: %ls", *BuildStats.FailureReasonText.BuildSourceString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: FailureType: %ls", LexToString(BuildStats.FailureType));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: NumInstallRetries: %u", BuildStats.NumInstallRetries);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: MaxDiskSpaceNeededWhenDeletingChunkDbsIfRequested: %ls", *FText::AsNumber(BuildStats.MaxDiskSpaceNeededWhenDeletingChunkDbsIfRequested).ToString());
		check(BuildStats.NumInstallRetries == BuildStats.RetryFailureTypes.Num() && BuildStats.NumInstallRetries == BuildStats.RetryErrorCodes.Num());
		for (uint32 RetryIdx = 0; RetryIdx < BuildStats.NumInstallRetries; ++RetryIdx)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Build Stat: RetryFailureType %u: %ls", RetryIdx, LexToString(BuildStats.RetryFailureTypes[RetryIdx]));
			UE_LOGF(LogBuildPatchServices, Log, "Build Stat: RetryErrorCodes %u: %ls", RetryIdx, *BuildStats.RetryErrorCodes[RetryIdx]);
		}
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: FinalProgressValue: %f", BuildStats.FinalProgress);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: OverallRequestSuccessRate: %f", BuildStats.OverallRequestSuccessRate);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ExcellentDownloadHealthTime: %ls", *FPlatformTime::PrettyTime(BuildStats.ExcellentDownloadHealthTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: GoodDownloadHealthTime: %ls", *FPlatformTime::PrettyTime(BuildStats.GoodDownloadHealthTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: OkDownloadHealthTime: %ls", *FPlatformTime::PrettyTime(BuildStats.OkDownloadHealthTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: PoorDownloadHealthTime: %ls", *FPlatformTime::PrettyTime(BuildStats.PoorDownloadHealthTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: DisconnectedDownloadHealthTime: %ls", *FPlatformTime::PrettyTime(BuildStats.DisconnectedDownloadHealthTime));
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: DriveStorePeakBytes: %u", BuildStats.DriveStorePeakBytes);
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: MemoryStoreSizePeakBytes: %ls", *FText::AsNumber(BuildStats.MemoryStoreSizePeakBytes).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: MemoryStoreSizeLimitBytes: %ls", *FText::AsNumber(BuildStats.MemoryStoreSizeLimitBytes).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ProcessRequiredDiskSpace: %ls", *FText::AsNumber(BuildStats.ProcessRequiredDiskSpace).ToString());
		UE_LOGF(LogBuildPatchServices, Log, "Build Stat: ProcessAvailableDiskSpace: %ls", *FText::AsNumber(BuildStats.ProcessAvailableDiskSpace).ToString());
	}

	const TCHAR* GetActionTypeLog(const FInstallerAction& InstallerAction)
	{
		if (InstallerAction.IsInstall())
		{
			return TEXT("Install");
		}
		else if (InstallerAction.IsUpdate())
		{
			return TEXT("Update");
		}
		else if (InstallerAction.IsRepair())
		{
			return TEXT("Repair");
		}
		else if (InstallerAction.IsVerifyOnly())
		{
			return TEXT("VerifyOnly");
		}
		else if (InstallerAction.IsUninstall())
		{
			return TEXT("Uninstall");
		}
		return TEXT("Invalid");
	}

	FString GetManifestLog(const FBuildPatchAppManifestPtr& Manifest)
	{
		if (Manifest.IsValid())
		{
			return FString::Printf(TEXT("%s %s (%s)"), *Manifest->GetAppName(), *Manifest->GetVersionString(), *Manifest->GetBuildId());
		}
		else
		{
			return TEXT("NULL");
		}
	}

	void LogBuildConfiguration(const BuildPatchServices::FBuildInstallerConfiguration& InstallerConfiguration, const FGuid& InstallerId)
	{
		if (!UE_LOG_ACTIVE(LogBPSInstallerConfig, Log)) // Just skip all this iteration if we aren't going to log anything
			return;

		using namespace BuildPatchServices;
		static FCriticalSection DoNotInterleaveLogs;
		FScopeLock ScopeLock(&DoNotInterleaveLogs);
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: InstallerId: %ls", *InstallerId.ToString());
		for (const FInstallerAction& InstallerAction : InstallerConfiguration.InstallerActions)
		{
			FBuildPatchAppManifestPtr CurrentManifest = StaticCastSharedPtr<FBuildPatchAppManifest>(InstallerAction.TryGetCurrentManifest());
			FBuildPatchAppManifestPtr InstallManifest = StaticCastSharedPtr<FBuildPatchAppManifest>(InstallerAction.TryGetInstallManifest());
			UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: ActionType: %ls", GetActionTypeLog(InstallerAction));
			UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: CurrentManifest: %ls", *GetManifestLog(CurrentManifest));
			UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: InstallManifest: %ls", *GetManifestLog(InstallManifest));
			TSet<FGuid> NecessarySecrets = InstallManifest.IsValid() ? InstallManifest->GetNecessaryEncryptionSecretIds() : TSet<FGuid>();
			for (const FGuid& NecessarySecret : NecessarySecrets)
			{
				UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: InstallManifest: NecessarySecret: %ls", *LexToString(NecessarySecret));
			}
			for (const FString& Tag : InstallerAction.GetInstallTags())
			{
				UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: InstallTags: %ls", *Tag);
			}
			TSet<FString> ValidTags;
			InstallerAction.GetInstallOrCurrentManifest()->GetFileTagList(ValidTags);
			for (const FString& Tag : ValidTags)
			{
				UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: ValidTags: %ls", *Tag);
			}
			if (!InstallerAction.GetInstallSubdirectory().IsEmpty())
			{
				UE_LOGF(LogBuildPatchServices, Log, "Build Config: InstallSubdirectory: %ls", *InstallerAction.GetInstallSubdirectory());
			}
			if (!InstallerAction.GetCloudSubdirectory().IsEmpty())
			{
				UE_LOGF(LogBuildPatchServices, Log, "Build Config: CloudSubdirectory: %ls", *InstallerAction.GetCloudSubdirectory());
			}
		}

		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: InstallDirectory: %ls", *InstallerConfiguration.InstallDirectory);
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: StagingDirectory: %ls", *InstallerConfiguration.StagingDirectory);
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: BackupDirectory: %ls", *InstallerConfiguration.BackupDirectory);

		for (const FString& DatabaseFile : InstallerConfiguration.ChunkDatabaseFiles)
		{
			UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: ChunkDatabaseFile: %ls", *DatabaseFile);
		}

		for (const FString& CloudDirectory : InstallerConfiguration.CloudDirectories)
		{
			UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: CloudDirectories: %ls", *CloudDirectory);
		}

		for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : InstallerConfiguration.EncryptionSecrets)
		{
			UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: AvailableSecretId: %ls", *EncryptionSecret.Key.ToString());
		}

		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: InstallMode: %ls", LexToString(InstallerConfiguration.InstallMode));
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: VerifyMode: %ls", LexToString(InstallerConfiguration.VerifyMode));
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: DeltaPolicy: %ls", LexToString(InstallerConfiguration.DeltaPolicy));
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: bRunRequiredPrereqs: %ls", *LexToString(InstallerConfiguration.bRunRequiredPrereqs));
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: bAllowConcurrentExecution: %ls", *LexToString(InstallerConfiguration.bAllowConcurrentExecution));
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: DownloadRateLimitBps: %llu", InstallerConfiguration.DownloadRateLimitBps);
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: bRejectSymlinks: %ls", *LexToString(InstallerConfiguration.bRejectSymlinks));
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: bConstructFilesInMemory: %ls", *LexToString(InstallerConfiguration.bConstructFilesInMemory));
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: bInstallToMemory: %ls", *LexToString(InstallerConfiguration.bInstallToMemory));
		UE_LOGF(LogBPSInstallerConfig, Log, "Build Config: bSkipInitialDiskSizeCheck: %ls", *LexToString(InstallerConfiguration.bSkipInitialDiskSizeCheck));
	}

	TSet<FGuid> GetMultipleReferencedChunks(IBuildManifestSet* ManifestSet)
	{
		using namespace BuildPatchServices;
		TSet<FGuid> MultipleReferencedChunks;
		TSet<FGuid> AllReferencedChunks;
		TSet<FString> ExpectedFiles;
		ManifestSet->GetExpectedFiles(ExpectedFiles);
		for (const FString& File : ExpectedFiles)
		{
			const FFileManifest* NewFileManifest = ManifestSet->GetNewFileManifest(File);
			if (NewFileManifest != nullptr)
			{
				for (const FChunkPart& ChunkPart : NewFileManifest->ChunkParts)
				{
					if (AllReferencedChunks.Contains(ChunkPart.Guid))
					{
						MultipleReferencedChunks.Add(ChunkPart.Guid);
					}
					else
					{
						AllReferencedChunks.Add(ChunkPart.Guid);
					}
				}
			}
		}
		return MultipleReferencedChunks;
	}

	const TCHAR* GetVerifyErrorCode(const BuildPatchServices::EVerifyResult& VerifyResult)
	{
		using namespace BuildPatchServices;
		switch (VerifyResult)
		{
		case EVerifyResult::FileMissing: return VerifyErrorCodes::FileMissing;
		case EVerifyResult::OpenFileFailed: return VerifyErrorCodes::OpenFileFailed;
		case EVerifyResult::HashCheckFailed: return VerifyErrorCodes::HashCheckFailed;
		case EVerifyResult::FileSizeFailed: return VerifyErrorCodes::FileSizeFailed;
		}

		return VerifyErrorCodes::UnknownFail;
	}

	void LogAdditionalVerifyErrors(BuildPatchServices::EVerifyError Error, int32 Count)
	{
		using namespace BuildPatchServices;
		EVerifyResult VerifyResult;
		if (TryConvertToVerifyResult(Error, VerifyResult))
		{
			FString Prefix = InstallErrorPrefixes::ErrorTypeStrings[static_cast<int32>(EBuildPatchInstallError::BuildVerifyFail)];
			FString Suffix = InstallerHelpers::GetVerifyErrorCode(VerifyResult);
			UE_LOGF(LogBuildPatchServices, Log, "Build verification error encountered: %ls: %d", *(Prefix + Suffix), Count);
		}
	}

	FOptimisedDeltaDependencies BuildOptimisedDeltaDependencies(const TUniquePtr<IDownloadService>& DownloadService, TSharedRef<IUriProvider, ESPMode::ThreadSafe> UriProvider)
	{
		FOptimisedDeltaDependencies OptimisedDeltaDependencies(MoveTemp(UriProvider));
		OptimisedDeltaDependencies.DownloadService = DownloadService.Get();
		return OptimisedDeltaDependencies;
	}
}

namespace BuildPatchServices
{
	struct FScopedControllables
	{
	public:
		FScopedControllables(FCriticalSection* InSyncObject, TArray<IControllable*>& InRegistrationArray, bool& bInIsPaused, bool& bInShouldAbort)
			: SyncObject(InSyncObject)
			, RegistrationArray(InRegistrationArray)
			, bIsPaused(bInIsPaused)
			, bShouldAbort(bInShouldAbort)
		{
		}

		~FScopedControllables()
		{
			FScopeLock ScopeLock(SyncObject);
			for (IControllable* Controllable : RegisteredArray)
			{
				RegistrationArray.Remove(Controllable);
			}
		}

		void Register(IControllable* Controllable)
		{
			FScopeLock ScopeLock(SyncObject);
			RegistrationArray.Add(Controllable);
			RegisteredArray.Add(Controllable);
			if (bShouldAbort)
			{
				Controllable->Abort();
			}
			else
			{
				Controllable->SetPaused(bIsPaused);
			}
		}

	private:
		FCriticalSection* SyncObject;
		TArray<IControllable*>& RegistrationArray;
		TArray<IControllable*> RegisteredArray;
		bool& bIsPaused;
		bool& bShouldAbort;
	};

	struct FBuildPatchDownloadRecord
	{
		double StartTime;
		double EndTime;
		int64 DownloadSize;

		FBuildPatchDownloadRecord()
			: StartTime(0)
			, EndTime(0)
			, DownloadSize(0)
		{}

		friend bool operator<(const FBuildPatchDownloadRecord& Lhs, const FBuildPatchDownloadRecord& Rhs)
		{
			return Lhs.StartTime < Rhs.StartTime;
		}
	};

	static uint64 DetermineInstallMaxDiskSizeIfDeletingChunkDbs(const TArray<FString>& ChunkDbFiles, IBuildManifestSet* ManifestSet, IFileSystem* FileSystem, const FString& InstallDirectory, const TArray<FString>& CorruptFiles, EInstallMode InstallMode);

	static FString FormatNumber(uint64 Value) { return FText::AsNumber(Value).ToString(); }

	namespace ConstructionHelpers
	{
		FBuildInstallerConfiguration&& ProcessOverrides(FBuildInstallerConfiguration&& Configuration)
		{
			// Clean up directory members.
			FPaths::NormalizeDirectoryName(Configuration.InstallDirectory);
			FPaths::CollapseRelativeDirectories(Configuration.InstallDirectory);
			FPaths::NormalizeDirectoryName(Configuration.StagingDirectory);
			FPaths::CollapseRelativeDirectories(Configuration.StagingDirectory);
			FPaths::NormalizeDirectoryName(Configuration.BackupDirectory);
			FPaths::CollapseRelativeDirectories(Configuration.BackupDirectory);

			// Force stage with RAW filenames, if legacy staging locations exist.
			IFileManager& FileManager = IFileManager::Get();
			const FString LegacyDataStagingDir(Configuration.StagingDirectory / TEXT("PatchData"));
			const FString LegacyInstallStagingDir(Configuration.StagingDirectory / TEXT("Install"));
			const FString LegacyMetaStagingDir(Configuration.StagingDirectory / TEXT("Meta"));
			Configuration.bStageWithRawFilenames = Configuration.bStageWithRawFilenames
				|| FileManager.DirectoryExists(*LegacyDataStagingDir)
				|| FileManager.DirectoryExists(*LegacyInstallStagingDir)
				|| FileManager.DirectoryExists(*LegacyMetaStagingDir);

			// Create a default shared context.
			if (!Configuration.SharedContext)
			{
				Configuration.SharedContext = FBuildInstallerSharedContextFactory::Create(TEXT("BuildPatchInstaller"));
				const bool bUseChunkDBs = !Configuration.ChunkDatabaseFiles.IsEmpty();
				const bool bHasCurrentManifest = Algo::AnyOf(Configuration.InstallerActions, [](const FBuildPatchInstallerAction& Elem) { return Elem.TryGetCurrentManifest() != nullptr; });
				const uint32 NumExpectedThreads = Configuration.SharedContext->NumThreadsPerInstaller(bUseChunkDBs, bHasCurrentManifest);
				Configuration.SharedContext->PreallocateThreads(NumExpectedThreads);
			}

			return MoveTemp(Configuration);
		}

		FString GetDataStagingDir(const FBuildInstallerConfiguration& Configuration)
		{
			FString DataStagingDir;
			if (Configuration.bStageWithRawFilenames)
			{
				DataStagingDir = Configuration.StagingDirectory / TEXT("PatchData");
			}
			else
			{
				DataStagingDir = Configuration.StagingDirectory / TEXT("c"); // c for chunk
			}
			FPaths::NormalizeDirectoryName(DataStagingDir);
			FPaths::CollapseRelativeDirectories(DataStagingDir);
			return DataStagingDir;
		}

		FString GetInstallStagingDir(const FBuildInstallerConfiguration& Configuration)
		{
			FString InstallStagingDir;
			if (Configuration.bStageWithRawFilenames)
			{
				InstallStagingDir = Configuration.StagingDirectory / TEXT("Install");
			}
			else
			{
				InstallStagingDir = Configuration.StagingDirectory / TEXT("f"); // f for files
			}
			FPaths::NormalizeDirectoryName(InstallStagingDir);
			FPaths::CollapseRelativeDirectories(InstallStagingDir);
			return InstallStagingDir;
		}

		FString GetMetaStagingDir(const FBuildInstallerConfiguration& Configuration)
		{
			FString MetaStagingDir;
			if (Configuration.bStageWithRawFilenames)
			{
				MetaStagingDir = Configuration.StagingDirectory / TEXT("Meta");
			}
			else
			{
				MetaStagingDir = Configuration.StagingDirectory / TEXT("m"); // m for meta
			}
			FPaths::NormalizeDirectoryName(MetaStagingDir);
			FPaths::CollapseRelativeDirectories(MetaStagingDir);
			return MetaStagingDir;
		}

		TArray<FBuildPatchInstallerAction> UpCastInstallerActions(const TArray<FInstallerAction>& InInstallerActions)
		{
			TArray<FBuildPatchInstallerAction> InstallerActions;
			for (const FInstallerAction& InstallerAction : InInstallerActions)
			{
				InstallerActions.Emplace(InstallerAction);
			}
			return InstallerActions;
		}

		TMultiMap<FString, FBuildPatchAppManifestRef>&& AppendInstallationInfo(TMultiMap<FString, FBuildPatchAppManifestRef>&& InstallationInfo, TArray<FBuildPatchInstallerAction>& InstallerActions, const FBuildInstallerConfiguration& Configuration)
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				// Make sure existing manifests are added to installation info.
				FBuildPatchAppManifestPtr CurrentManifest = InstallerAction.TryGetSharedCurrentManifest();
				if (CurrentManifest.IsValid())
				{
					InstallationInfo.Add(Configuration.InstallDirectory / InstallerAction.GetInstallSubdirectory(), CurrentManifest.ToSharedRef());
				}
			}
			return MoveTemp(InstallationInfo);
		}

		TArray<FBuildPatchAppManifestPtr> ToManifestArray(const TArray<FBuildPatchInstallerAction>& InstallerActions)
		{
			TArray<FBuildPatchAppManifestPtr> ManifestAraray;
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				ManifestAraray.AddUnique(InstallerAction.TryGetSharedCurrentManifest());
				ManifestAraray.AddUnique(InstallerAction.TryGetSharedInstallManifest());
			}
			return ManifestAraray;
		}
	}

	/* FBuildPatchInstaller implementation
	*****************************************************************************/
	FBuildPatchInstaller::FBuildPatchInstaller(FBuildInstallerConfiguration InConfiguration, TMultiMap<FString, FBuildPatchAppManifestRef> InInstallationInfo, const FString& InLocalMachineConfigFile, TSharedPtr<IAnalyticsProvider> InAnalytics, FBuildPatchInstallerDelegate InStartDelegate, FBuildPatchInstallerDelegate InCompleteDelegate)
		: Configuration(ConstructionHelpers::ProcessOverrides(MoveTemp(InConfiguration)))
		, SessionId(FGuid::NewGuid())
		, StartDelegate(InStartDelegate)
		, CompleteDelegate(InCompleteDelegate)
		, InstallerActions(ConstructionHelpers::UpCastInstallerActions(Configuration.InstallerActions))
		, DataStagingDir(ConstructionHelpers::GetDataStagingDir(Configuration))
		, InstallStagingDir(ConstructionHelpers::GetInstallStagingDir(Configuration))
		, MetaStagingDir(ConstructionHelpers::GetMetaStagingDir(Configuration))
		, PreviousMoveMarker(Configuration.InstallDirectory / TEXT("$movedMarker"))
		, HddSpaceReservationFile(DataStagingDir / TEXT("reserve"))
		, InstallationInfo(ConstructionHelpers::AppendInstallationInfo(MoveTemp(InInstallationInfo), InstallerActions, Configuration))
		, ThreadLock()
		, Thread(nullptr)
		, bSuccess(false)
		, bIsRunning(false)
		, bIsInited(false)
		, bFirstInstallIteration(true)
		, PreviousTotalDownloadRequired(0)
		, bAreResultsEncrypted(false)
		, BuildStats()
		, BuildProgress()
		, bIsPaused(false)
		, bShouldAbort(false)
		, FilesInstalled()
		, TaggedFiles()
		, FilesToConstruct()
		, LocalMachineConfigFile(InLocalMachineConfigFile)
		, HttpManager(FHttpManagerFactory::Create())
		, FileSystem(FFileSystemFactory::Create())
		, Crypto(FCryptoFactory::Create())
		, Platform(FPlatformFactory::Create())
		, InstallerError(FInstallerErrorFactory::Create())
		, Analytics(MoveTemp(InAnalytics))
		, InstallerAnalytics(FInstallerAnalyticsFactory::Create(Analytics.Get()))
		, InstallActionStatistics(FInstallActionStatisticsFactory::Create(InstallerActions))
		, FileOperationTracker(Configuration.bTrackFileOperations ? FFileOperationTrackerFactory::Create(FTSTicker::GetCoreTicker()) : FFileOperationTrackerFactory::CreateNull())
		, DownloadSpeedRecorder(FSpeedRecorderFactory::Create())
		, DiskReadSpeedRecorder(FSpeedRecorderFactory::Create())
		, DiskWriteSpeedRecorder(FSpeedRecorderFactory::Create())
		, ChunkDataSizeProvider(FChunkDataSizeProviderFactory::Create(ConstructionHelpers::ToManifestArray(InstallerActions)))
		, DownloadServiceStatistics(FDownloadServiceStatisticsFactory::Create(DownloadSpeedRecorder, ChunkDataSizeProvider, InstallerAnalytics))
		, ChunkDbChunkSourceStatistics(FChunkDbChunkSourceStatisticsFactory::Create(DiskReadSpeedRecorder.Get(), FileOperationTracker.Get()))
		, InstallChunkSourceStatistics(FInstallChunkSourceStatisticsFactory::Create(DiskReadSpeedRecorder.Get(), InstallerAnalytics.Get(), InstallActionStatistics.Get(), FileOperationTracker.Get()))
		, CloudChunkSourceStatistics(FCloudChunkSourceStatisticsFactory::Create(InstallerAnalytics.Get(), &BuildProgress, FileOperationTracker.Get()))
		, FileConstructorStatistics(FFileConstructorStatisticsFactory::Create(DiskReadSpeedRecorder.Get(), DiskWriteSpeedRecorder.Get(), &BuildProgress, FileOperationTracker.Get()))
		, VerifierStatistics(FVerifierStatisticsFactory::Create(DiskReadSpeedRecorder.Get(), &BuildProgress, FileOperationTracker.Get()))
		, DownloadService(FDownloadServiceFactory::Create(HttpManager, FileSystem, DownloadServiceStatistics, InstallerAnalytics))
		, MessagePump(FMessagePumpFactory::Create())
		, Controllables()
		, DownloadRateLimitBps(Configuration.DownloadRateLimitBps)
	{
		InstallerError->RegisterForErrors([this]() { CancelInstall(); });
		Controllables.Add(&BuildProgress);
	}

	FBuildPatchInstaller::~FBuildPatchInstaller()
	{
		PreExit();
	}

	void FBuildPatchInstaller::PreExit()
	{
		// Set shutdown error so any running threads will exit if no error has already been set.
		if (bIsRunning)
		{
			InstallerError->SetError(EBuildPatchInstallError::ApplicationClosing, ApplicationClosedErrorCodes::ApplicationClosed);
		}

		CleanupThread();

		if (InstallerAnalytics.IsValid())
		{
			InstallerAnalytics->Flush();
		}
	}

	bool FBuildPatchInstaller::Tick()
	{
		bool bStillTicking = true;
		PumpMessages();
		if (IsComplete())
		{
			ExecuteCompleteDelegate();
			CleanupThread();
			bStillTicking = false;
		}
		return bStillTicking;
	}

	const IFileOperationTracker* FBuildPatchInstaller::GetFileOperationTracker() const
	{
		return FileOperationTracker.Get();
	}

	const ISpeedRecorder* FBuildPatchInstaller::GetDownloadSpeedRecorder() const
	{
		check(DownloadSpeedRecorder.IsValid());
		return DownloadSpeedRecorder.Get();
	}

	const ISpeedRecorder* FBuildPatchInstaller::GetDiskReadSpeedRecorder() const
	{
		check(DiskReadSpeedRecorder.IsValid());
		return DiskReadSpeedRecorder.Get();
	}

	const ISpeedRecorder* FBuildPatchInstaller::GetDiskWriteSpeedRecorder() const
	{
		check(DiskWriteSpeedRecorder.IsValid());
		return DiskWriteSpeedRecorder.Get();
	}

	const IDownloadServiceStatistics* FBuildPatchInstaller::GetDownloadServiceStatistics() const
	{
		check(DownloadServiceStatistics.IsValid());
		return DownloadServiceStatistics.Get();
	}

	const IInstallChunkSourceStatistics* FBuildPatchInstaller::GetInstallChunkSourceStatistics() const
	{
		return InstallChunkSourceStatistics.Get();
	}

	const ICloudChunkSourceStatistics* FBuildPatchInstaller::GetCloudChunkSourceStatistics() const
	{
		return CloudChunkSourceStatistics.Get();
	}

	const IFileConstructorStatistics* FBuildPatchInstaller::GetFileConstructorStatistics() const
	{
		return FileConstructorStatistics.Get();
	}

	const IChunkDbChunkSourceStatistics* FBuildPatchInstaller::GetChunkDbChunkSourceStatistics() const
	{
		return ChunkDbChunkSourceStatistics.Get();
	}

	const IVerifierStatistics* FBuildPatchInstaller::GetVerifierStatistics() const
	{
		return VerifierStatistics.Get();
	}

	const FBuildInstallerConfiguration& FBuildPatchInstaller::GetConfiguration() const
	{
		return Configuration;
	}

#if !UE_BUILD_SHIPPING
	void FBuildPatchInstaller::GetDebugText(TArray<FString>& Output)
	{
		if (!DownloadServiceStatistics.IsValid())
		{
			return;
		}

		TArray<FDownload> Downloads = DownloadServiceStatistics->GetCurrentDownloads();
		for (const FDownload& Download : Downloads)
		{
			Output.Add(FString::Printf(TEXT("BPI %s downloaded %.2f MBytes / %.2f MBytes"),
				*Download.Data,
				(double)Download.Received / (1024.0 * 1024.0),
				(double)Download.Size / (1024.0 * 1024.0)
			));
		}
	}
#endif

	void FBuildPatchInstaller::SetDownloadBytesPerSecondLimit(uint64 InDownloadRateLimitBps)
	{
		UE::TUniqueLock _(DownloadRateLimitLock);


		if (InDownloadRateLimitBps != DownloadRateLimitBps)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Build Config: InstallerId: %ls: SetDownloadRateLimit: %llu", *SessionId.ToString(), InDownloadRateLimitBps);
			DownloadRateLimitBps = InDownloadRateLimitBps;

			if (DownloadRateUpdatedFn)
			{
				DownloadRateUpdatedFn(DownloadRateLimitBps);
			}
		}


	}

	uint64 FBuildPatchInstaller::GetDownloadBytesPerSecondLimit() const
	{
		uint64 ReturnRate;
		{
			UE::TUniqueLock _(const_cast<UE::FMutex&>(DownloadRateLimitLock));
			ReturnRate = DownloadRateLimitBps;
		}

		return ReturnRate;
	}

	bool FBuildPatchInstaller::StartInstallation()
	{
		if (Thread == nullptr)
		{
			// Start thread!
			check(Configuration.SharedContext);
			Thread = Configuration.SharedContext->CreateThread();
			Thread->RunTask([this] { Run(); });

			StartDelegate.ExecuteIfBound(AsShared());
		}
		return Thread != nullptr;
	}

	template<typename T>
	bool WaitForResultOrAbort(T& DataToWait, bool& bShouldAbort, float PeriodToCheckSeconds = 0.05 /* 50 ms by default */)
	{
		while (!DataToWait.IsReady())
		{
			FPlatformProcess::Sleep(PeriodToCheckSeconds);
			if (bShouldAbort)
			{
				return false;
			}
		}
		return true;
	}

	bool FBuildPatchInstaller::Initialize()
	{
		bool bInstallerInitSuccess = true;
		InstallerHelpers::LogBuildConfiguration(Configuration, SessionId);

		// Check provided tags are all valid.
		for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
		{
			TSet<FString> ValidTags;
			InstallerAction.GetInstallOrCurrentManifest().GetFileTagList(ValidTags);
			ValidTags.Add(TEXT(""));
			TSet<FString> InvalidTags = InstallerAction.GetInstallTags().Difference(ValidTags);
			if (InvalidTags.Num() > 0)
			{
				UE_LOGF(LogBuildPatchServices, Error, "Installer configuration: Invalid InstallTags provided: %ls", *FString::Join(InvalidTags, TEXT(",")));
				InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::InvalidInstallTags, 0,
					NSLOCTEXT("BuildPatchInstallError", "InvalidInstallTags", "This installation could not continue due to a configuration issue. Please contact support."));
				bInstallerInitSuccess = false;
				break;
			}
		}

		// Check the actions to ensure if VerifyOnly is used, then it is exclusively used.
		bool bHasVerifyOnly = false;
		bool bHasOtherActionType = false;
		for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
		{
			if (InstallerAction.IsVerifyOnly())
			{
				bHasVerifyOnly = true;
			}
			else
			{
				bHasOtherActionType = true;
			}
		}
		if (bHasVerifyOnly && bHasOtherActionType)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Installer configuration: Verify only actions must not be mixed with others.");
			InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::VerifyOnlyActionMixedWithOtherActions, 0,
				NSLOCTEXT("BuildPatchInstallError", "VerifyOnlyActionMixed", "This installation could not continue due to a configuration issue. Please contact support."));
			bInstallerInitSuccess = false;
		}

		if (!bHasVerifyOnly && !Configuration.CloudDirectories.Num())
		{
			UE_LOGF(LogBuildPatchServices, Error, "Installer configuration: Cloud directories are missing.");
			InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::EmptyCloudDirectories, 0,
				NSLOCTEXT("BuildPatchInstallError", "EmptyCloudDirectories", "This installation could not continue due to a configuration issue. Please contact support."));
			bInstallerInitSuccess = false;

			UE_LOGF(LogBuildPatchServices, Error, "Installer configuration: Cloud Directories is empty.");
			InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::EmptyCloudDirectories);
			return false;
		}

		if (Configuration.bInstallToMemory && Configuration.bConstructFilesInMemory)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Installer configuration: Can't construct in memory and install to memory.");
			InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::InvalidMemoryConstructionConfig, 0,
				NSLOCTEXT("BuildPatchInstallError", "InvalidMemoryConstructionConfig", "This installation could not continue due to a configuration issue. Please contact support."));
			bInstallerInitSuccess = false;
		}

		// We can't be patching if we are installing to memory.
		if (Configuration.bInstallToMemory)
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (!InstallerAction.IsInstall())
				{
					UE_LOGF(LogBuildPatchServices, Error, "Installer configuration: Installation to memory only supports installs (e.g. no updates/repairs).");
					InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::InvalidInstallToMemory, 0,
						NSLOCTEXT("BuildPatchInstallError", "InvalidInstallToMemory", "This installation could not continue due to a configuration issue. Please contact support."));
					bInstallerInitSuccess = false;
					break;
				}
			}
		}

		// Check that we were provided with a bound delegate.
		if (!CompleteDelegate.IsBound())
		{
			UE_LOGF(LogBuildPatchServices, Error, "Installer configuration: Completion delegate not provided.");
			InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::MissingCompleteDelegate);
			bInstallerInitSuccess = false;
		}

		// Make sure we have install directory access.
		if (!Configuration.bInstallToMemory)
		{
			IFileManager::Get().MakeDirectory(*Configuration.InstallDirectory, true);
			if (!IFileManager::Get().DirectoryExists(*Configuration.InstallDirectory))
			{
				UE_LOGF(LogBuildPatchServices, Error, "Installer setup: Inability to create InstallDirectory %ls.", *Configuration.InstallDirectory);
				InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::MissingInstallDirectory, 0,
					FText::Format(NSLOCTEXT("BuildPatchInstallError", "MissingInstallDirectory", "The installation directory could not be created.\n{0}"), FText::FromString(Configuration.InstallDirectory)));
				bInstallerInitSuccess = false;
			}

			// Make sure we have staging directory access.
			IFileManager::Get().MakeDirectory(*Configuration.StagingDirectory, true);
			if (!IFileManager::Get().DirectoryExists(*Configuration.StagingDirectory))
			{
				UE_LOGF(LogBuildPatchServices, Error, "Installer setup: Inability to create StagingDirectory %ls.", *Configuration.StagingDirectory);
				InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::MissingStageDirectory, 0,
					FText::Format(NSLOCTEXT("BuildPatchInstallError", "MissingStageDirectory", "The following directory could not be created.\n{0}"), FText::FromString(Configuration.StagingDirectory)));
				bInstallerInitSuccess = false;
			}
		}

		// Make sure that we have a prereq if we've specified a prereq only install.
		if (Configuration.InstallMode == EInstallMode::PrereqOnly)
		{
			bool bMissingPrereqPath = true;
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (!InstallerAction.GetInstallOrCurrentManifest().GetPrereqPath().IsEmpty())
				{
					bMissingPrereqPath = false;
					break;
				}
			}
			if (bMissingPrereqPath)
			{
				UE_LOGF(LogBuildPatchServices, Error, "Installer setup: PrereqOnly install selected for manifest with no prereq.");
				InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::MissingPrereqForPrereqOnlyInstall, 0,
					NSLOCTEXT("BuildPatchInstallError", "MissingPrereqForPrereqOnlyInstall", "This installation could not continue due to a prerequisite configuration issue. Please contact support."));
				bInstallerInitSuccess = false;
			}
		}

		// Make sure that we have all necessary secrets available unless we are in preload mode.
		if (Configuration.InstallMode != EInstallMode::Preload)
		{
			TSet<FGuid> NecessarySecrets;
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				// Only test actions with install manifests.
				// An uninstall action can complete without any encryption keys, and we don't need the keys for the current install if they were not needed for a new update.
				const FBuildPatchAppManifest* InstallManifest = InstallerAction.TryGetInstallManifest();
				if (InstallManifest && InstallManifest->GetFeatureLevel() >= EFeatureLevel::ChunkEncryptionSupport)
				{
					NecessarySecrets.Append(InstallManifest->GetNecessaryEncryptionSecretIds());
				}
			}
			TSet<FGuid> AvailableSecrets;
			TSet<FGuid> InvalidSecrets;
			for (const TPair<FGuid, TArray<uint8>>& EncryptionSecret : Configuration.EncryptionSecrets)
			{
				AvailableSecrets.Add(EncryptionSecret.Key);
				if (!Crypto->IsValidKey_AES_256_GCM(EncryptionSecret.Value))
				{
					InvalidSecrets.Add(EncryptionSecret.Key);
				}
			}
			if (InvalidSecrets.Num() > 0)
			{
				TArray<FString> InvalidSecretsIds;
				Algo::Transform(InvalidSecrets, InvalidSecretsIds, [](const FGuid& Id) { return Id.ToString(); });
				UE_LOGF(LogBuildPatchServices, Error, "Installer setup: Installation cannot complete, invalid encryption secret(s) provided for id(s) %ls", *FString::Join(InvalidSecretsIds, TEXT(", ")));
				InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::InvalidEncryptionKeyForNonPreload, 0,
					NSLOCTEXT("BuildPatchInstallError", "InvalidEncryptionKeyForNonPreload", "This installation could not continue as the required decryption data is not valid. Please contact support."));
				bInstallerInitSuccess = false;
			}
			bool bMissingSecrets = NecessarySecrets.Difference(AvailableSecrets).Num() > 0;
			if (bMissingSecrets)
			{
				TArray<FString> AvailableSecretsIds;
				Algo::Transform(AvailableSecrets, AvailableSecretsIds, [](const FGuid& Id) { return Id.ToString(); });
				UE_LOGF(LogBuildPatchServices, Error, "Installer setup: Installation cannot complete without all necessary encryption secrets. Available %d of %d: %ls", AvailableSecretsIds.Num(), NecessarySecrets.Num(), *FString::Join(AvailableSecretsIds, TEXT(", ")));
				InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::MissingEncryptionKeyForNonPreload, 0,
					NSLOCTEXT("BuildPatchInstallError", "MissingEncryptionKeyForNonPreload", "This installation could not continue as the required decryption data is missing. Please contact support."));
				bInstallerInitSuccess = false;
			}
		}

		// do the delta optimization
		typedef TTuple<FBuildPatchInstallerAction&, TSharedRef<IOptimisedDelta>> FManifestInfoDeltaPair;
		TArray<FManifestInfoDeltaPair> RunningOptimisedDeltas;
		for (FBuildPatchInstallerAction& InstallerAction : InstallerActions)
		{
			if (InstallerAction.IsUpdate())
			{
				TArray<FString> CloudDirectories = Configuration.CloudDirectories;
				if (!InstallerAction.GetCloudSubdirectory().IsEmpty())
				{
					for (FString& CloudDirectory : CloudDirectories)
					{
						CloudDirectory /= InstallerAction.GetCloudSubdirectory();
					}
				}
				const int32_t NumberOfDirectories = CloudDirectories.Num();

				TSharedRef<IUriProvider, ESPMode::ThreadSafe> UriProvider = FUriProviderFactory::Create(MessagePump.Get(), MoveTemp(CloudDirectories));
				FOptimisedDeltaDependencies OptimisedDeltaDependencies = InstallerHelpers::BuildOptimisedDeltaDependencies(DownloadService, MoveTemp(UriProvider));
				TSharedRef<IOptimisedDelta> OptimisedDelta = FOptimisedDeltaFactory::Create(ConfigHelpers::BuildOptimisedDeltaConfig(Configuration, InstallerAction, NumberOfDirectories), MoveTemp(OptimisedDeltaDependencies));
				RunningOptimisedDeltas.Add(FManifestInfoDeltaPair{ InstallerAction, MoveTemp(OptimisedDelta) });
			}
		}
		for (FManifestInfoDeltaPair& RunningOptimisedDelta : RunningOptimisedDeltas)
		{
			FBuildPatchInstallerAction& InstallerAction = RunningOptimisedDelta.Get<0>();
			TSharedRef<IOptimisedDelta> OptimisedDelta = RunningOptimisedDelta.Get<1>();
			if (WaitForResultOrAbort(*OptimisedDelta, bShouldAbort))
			{
				const IOptimisedDelta::FResultValueOrError& OptimisedDeltaResult = OptimisedDelta->GetResult();
				PreviousTotalDownloadRequired += OptimisedDelta->GetMetaDownloadSize();
				// The OptimiseDelta class handles policy, so if we get a nullptr back, that is a hard error.
				if (!OptimisedDeltaResult.IsValid())
				{
					UE_LOGF(LogBuildPatchServices, Error, "Installer setup: Destination manifest could not be obtained.");
					InstallerError->SetError(EBuildPatchInstallError::DownloadError, *OptimisedDeltaResult.GetError());
					bInstallerInitSuccess = false;
					break;
				}
				else
				{
					InstallerAction.SetDeltaManifest(OptimisedDeltaResult.GetValue().ToSharedRef());
				}
				// Report delta manifest use and response code to analytics.
				InstallActionStatistics->OnUsedOptimizedDelta(InstallerAction, true, OptimisedDelta->GetDownloadResponseCode());
			}
			else
			{
				UE_LOGF(LogBuildPatchServices, Error, "Installer setup is aborted.");
				InstallerError->SetError(EBuildPatchInstallError::UserCanceled, UserCancelErrorCodes::UserRequested);
				bInstallerInitSuccess = false;
				break;
			}
		}

		// We can now build out any systems that need late construction but can survive between retries.
		ManifestSet.Reset(FBuildManifestSetFactory::Create(InstallerActions));
		Verifier.Reset(FVerifierFactory::Create(FileSystem.Get(), VerifierStatistics.Get(), Configuration.VerifyMode, Configuration.SharedContext, 
			ManifestSet.Get(), Configuration.InstallDirectory, Configuration.InstallMode == EInstallMode::StageFiles ? InstallStagingDir : FString(), 
			!Configuration.bStageWithRawFilenames, NewPerFileSubdirectories));

		// Add systems to controllables.
		{
			FScopeLock Lock(&ThreadLock);
			Controllables.Add(Verifier.Get());
		}

		// Queue update to chunk data size cache on main thread
		if (bInstallerInitSuccess)
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				ChunkDataSizeProvider->AddManifestData(InstallerAction.TryGetSharedInstallManifest());
			}
		}

		// Init build statistics that are known.
		{
			FScopeLock Lock(&ThreadLock);
			BuildStats.NumFilesInBuild = ManifestSet->GetNumExpectedFiles();
			BuildStats.ProcessSuccess = bInstallerInitSuccess;
			BuildStats.ErrorCode = InstallerError->GetErrorCode();
			BuildStats.FailureReasonText = InstallerError->GetErrorText();
			BuildStats.FailureType = InstallerError->GetErrorType();
		}

		// Check for any filepath violations.
		FString InstallDirectoryWithSlash = Configuration.InstallDirectory;
		FPaths::NormalizeDirectoryName(InstallDirectoryWithSlash);
		FPaths::CollapseRelativeDirectories(InstallDirectoryWithSlash);
		InstallDirectoryWithSlash /= TEXT("/");
		TSet<FString> ExpectedFiles;
		ManifestSet->GetExpectedFiles(ExpectedFiles);
		for (const FString& ExpectedFile : ExpectedFiles)
		{
			// We're not checking per file subdirs here, which could allow things to escape, but we also are specifically
			// adding per file subdirs so that it can since we're wanting to move files to a sibling directory of the install
			// location. 
			const FString InstallConstructionFile = FPaths::ConvertRelativePathToFull(Configuration.InstallDirectory, ExpectedFile);
			if (!InstallConstructionFile.StartsWith(InstallDirectoryWithSlash))
			{
				UE_LOGF(LogBuildPatchServices, Error, "Installer setup: Filepath in manifest escaped install directory. %ls -> %ls", *ExpectedFile, *InstallConstructionFile);
				InstallerError->SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::InvalidDataInManifest);
				bInstallerInitSuccess = false;
			}
		}

		bIsInited = true;
		return bInstallerInitSuccess;
	}

	uint32 FBuildPatchInstaller::Run()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildPatchInstaller::Run);

		// Make sure this function can never be parallelized
		static FCriticalSection SingletonFunctionLockCS;
		const bool bShouldLock = !Configuration.bAllowConcurrentExecution;
		if (bShouldLock)
		{
			SingletonFunctionLockCS.Lock();
		}
		bIsRunning = true;
		ProcessExecuteTimer.Start();
		ProcessActiveTimer.Start();

		// No longer queued
		BuildProgress.SetStateProgress(EBuildPatchState::Queued, 1.0f);

		// If we have resuming data, then we immediately go to Resuming
		const FString LegacyResumeDataFilename = InstallStagingDir / TEXT("$resumeData");
		const FString ResumeDataFilename = MetaStagingDir / TEXT("$resumeData");
		const bool bHasResumeData = FileSystem->FileExists(*ResumeDataFilename) || FileSystem->FileExists(*LegacyResumeDataFilename);
		if (bHasResumeData)
		{
			BuildProgress.SetStateProgress(EBuildPatchState::Initializing, 1.0f);
		}

		// Init prereqs progress value
		bool bHasPrereqPath = false;
		for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
		{
			if (!InstallerAction.GetInstallOrCurrentManifest().GetPrereqPath().IsEmpty())
			{
				bHasPrereqPath = true;
				break;
			}
		}
		const bool bInstallPrereqs = Configuration.bRunRequiredPrereqs && bHasPrereqPath;
		BuildProgress.SetStateProgress(EBuildPatchState::PrerequisitesInstall, bInstallPrereqs ? 0.0f : 1.0f);

		// Initialization
		InitializeTimer.Start();
		bool bProcessSuccess = Initialize();

		uint64 MaxDiskSpaceNeededWhenDeletingChunkDbsIfRequested = 0;
		if (Configuration.bCalculateDeleteChunkDbMaxDiskSpaceAndExit)
		{
			if (bProcessSuccess)
			{
				MaxDiskSpaceNeededWhenDeletingChunkDbsIfRequested = DetermineInstallMaxDiskSizeIfDeletingChunkDbs(Configuration.ChunkDatabaseFiles, ManifestSet.Get(), FileSystem.Get(), Configuration.InstallDirectory, {}, Configuration.InstallMode);
			}
		}
		else
		{
			// Run if successful init 
			if (bProcessSuccess)
			{
				// Keep track of files that failed verify
				TArray<FString> CorruptFiles;

				// Keep retrying the install while it is not canceled, or caused by download error
				bProcessSuccess = false;
				bool bInstallSuccess = false;
				bool bCanRetry = true;
				bool bWillRetry = false;
				int32 InstallRetries = ConfigHelpers::NumInstallerRetries();
				while (!bProcessSuccess && bCanRetry)
				{
					bLastInstallIteration = 1 == InstallRetries;
					// Inform file operation tracker of the selected manifest.
					FileOperationTracker->OnManifestSelection(ManifestSet.Get());

					if (Configuration.InstallMode == EInstallMode::Preload)
					{
						// Run the install
						bInstallSuccess = bProcessSuccess = RunPreload();
					}
					else
					{
						// Run the install
						bInstallSuccess = RunInstallation(CorruptFiles);

						// Runs the custom uninstall actions if specified
						bInstallSuccess = bInstallSuccess && RunCustomUninstallAction();

						// Backup local changes then move generated files
						bInstallSuccess = bInstallSuccess && RunBackupAndMove();

						// Setup file attributes
						bInstallSuccess = bInstallSuccess && RunFileAttributes();

						// Run Verification
						CorruptFiles.Empty();
						bProcessSuccess = bInstallSuccess && RunVerification(CorruptFiles);
					}

					// Set if we can retry
					--InstallRetries;
					bCanRetry = InstallRetries > 0 && !InstallerError->IsCancelled() && InstallerError->CanRetry() && !ManifestSet->IsExclusivelyVerifying();
					bWillRetry = !bProcessSuccess && bCanRetry;

					// Clean staging if INSTALL success, we still do cleanup if we failed at the verify stage.
					const bool bShouldClean = bInstallSuccess && !Configuration.bInstallToMemory && Configuration.InstallMode != EInstallMode::Preload;
					if (bShouldClean)
					{
						CleanUpTimer.Start();
						if (Configuration.InstallMode == EInstallMode::StageFiles || bAreResultsEncrypted)
						{
							UE_LOGF(LogBuildPatchServices, Log, "Deleting litter from staging area.");
							IFileManager::Get().DeleteDirectory(*DataStagingDir, false, true);
						}
						else
						{
							UE_LOGF(LogBuildPatchServices, Log, "Deleting staging area.");
							IFileManager::Get().DeleteDirectory(*Configuration.StagingDirectory, false, true);
							if (!bWillRetry)
							{
								CleanupEmptyDirectories(Configuration.InstallDirectory);
							}
						}
						CleanUpTimer.Stop();
					}
					BuildProgress.SetStateProgress(EBuildPatchState::CleanUp, 1.0f);

					// If successful or we will retry, remove the moved files marker
					if (!Configuration.bInstallToMemory)
					{
						if (bProcessSuccess || bCanRetry)
						{
							UE_LOGF(LogBuildPatchServices, Log, "Reset MM.");
							IFileManager::Get().Delete(*PreviousMoveMarker, false, true);
						}
					}

					// Setup end of attempt stats
					bFirstInstallIteration = false;
					float TempFinalProgress = BuildProgress.GetProgressNoMarquee();
					{
						FScopeLock Lock(&ThreadLock);
						const uint32 NumInstallRetries = ConfigHelpers::NumInstallerRetries() - (InstallRetries + 1);
						BuildStats.NumInstallRetries = NumInstallRetries;
						BuildStats.FinalProgress = TempFinalProgress;
						BuildStats.NumInstallRetries = NumInstallRetries;
						// If we failed, and will retry, record this failure type and reset the abort flag
						if (bWillRetry)
						{
							BuildStats.RetryFailureTypes.Add(InstallerError->GetErrorType());
							BuildStats.RetryErrorCodes.Add(InstallerError->GetErrorCode());
							bShouldAbort = false;
						}
					}

					// If we will retry the install, reset progress states.
					if (bWillRetry)
					{
						InitializeTimer.Start();
						Verifier->Reset();
						BuildProgress.Reset();
						BuildProgress.SetStateProgress(EBuildPatchState::Initializing, 0.0f);
						BuildProgress.SetStateProgress(EBuildPatchState::Resuming, 0.0f);
						BuildProgress.SetStateProgress(EBuildPatchState::Downloading, 0.0f);
						BuildProgress.SetStateProgress(EBuildPatchState::Installing, 0.0f);
						BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, 0.0f);
						BuildProgress.SetStateProgress(EBuildPatchState::SettingAttributes, 0.0f);
						BuildProgress.SetStateProgress(EBuildPatchState::BuildVerification, 0.0f);
						BuildProgress.SetStateProgress(EBuildPatchState::CleanUp, 0.0f);
					}
				}
			}

			if (bProcessSuccess)
			{
				// Run the prerequisites installer if this is our first install and the manifest has prerequisites info
				if (bInstallPrereqs)
				{
					PrereqTimer.Start();
					bProcessSuccess &= RunPrerequisites();
					PrereqTimer.Stop();
				}
			}
		} // end if doing normal installation

		// Make sure all timers are stopped
		InitializeTimer.Stop();
		ConstructTimer.Stop();
		UninstallActionTimer.Stop();
		MoveFromStageTimer.Stop();
		FileAttributesTimer.Stop();
		VerifyTimer.Stop();
		CleanUpTimer.Stop();
		PrereqTimer.Stop();
		ProcessPausedTimer.Stop();
		ProcessActiveTimer.Stop();
		ProcessExecuteTimer.Stop();

		// Set final stat values and log out results
		bSuccess = bProcessSuccess;
		{
			FScopeLock Lock(&ThreadLock);
			BuildStats.InitializeTime = InitializeTimer.GetSeconds();
			BuildStats.ConstructTime = ConstructTimer.GetSeconds();
			BuildStats.UninstallActionTime = UninstallActionTimer.GetSeconds();
			BuildStats.MoveFromStageTime = MoveFromStageTimer.GetSeconds();
			BuildStats.FileAttributesTime = FileAttributesTimer.GetSeconds();
			BuildStats.VerifyTime = VerifyTimer.GetSeconds();
			BuildStats.CleanUpTime = CleanUpTimer.GetSeconds();
			BuildStats.PrereqTime = PrereqTimer.GetSeconds();
			BuildStats.ProcessPausedTime = ProcessPausedTimer.GetSeconds();
			BuildStats.ProcessActiveTime = ProcessActiveTimer.GetSeconds();
			BuildStats.ProcessExecuteTime = ProcessExecuteTimer.GetSeconds();
			BuildStats.ProcessSuccess = bProcessSuccess;
			BuildStats.ErrorCode = InstallerError->GetErrorCode();
			BuildStats.FailureReasonText = InstallerError->GetErrorText();
			BuildStats.FailureType = InstallerError->GetErrorType();
			BuildStats.MaxDiskSpaceNeededWhenDeletingChunkDbsIfRequested = MaxDiskSpaceNeededWhenDeletingChunkDbsIfRequested;
			BuildStats.ActionsStats = InstallActionStatistics->GetAllStats();
		}

		// Mark that we are done
		bIsRunning = false;
		if (bShouldLock)
		{
			SingletonFunctionLockCS.Unlock();
		}
		return bSuccess ? 0 : 1;
	}

	bool FBuildPatchInstaller::CheckForExternallyInstalledFiles(const TSet<FString>& FilesToCheck)
	{
		// Check the marker file for a previous installation unfinished, if we find it, we'll continue to move files and then verify them.
		if (FileSystem->FileExists(*PreviousMoveMarker))
		{
			return true;
		}

		// This check is only valid for an installer that is performing no updates.
		if (ManifestSet->ContainsUpdate())
		{
			return false;
		}

		// Check if any of the provided files exist on disk. If they do, we'll be starting the installation with a verify to find out what work needs to be done.
		for (const FString& FileToCheck : FilesToCheck)
		{
			const FString DiskFileName = FBuildPatchUtils::ResolveInstallationFileName(Configuration.InstallDirectory, NewPerFileSubdirectories, FileToCheck);
			if (FileSystem->FileExists(*DiskFileName))
			{
				return true;
			}
		}
		return false;
	}

	static FChunkDbSourceConfig BuildChunkDbSourceConfig(const FBuildInstallerConfiguration& Configuration, const FString& DataStagingDir)
	{
		TArray<FString> ChunkDatabaseFiles = Configuration.ChunkDatabaseFiles;
		Algo::Transform(ConfigHelpers::EnumeratePreloadChunkDbs(*DataStagingDir), ChunkDatabaseFiles, [&DataStagingDir](const FString& ChunkDbFile) {return DataStagingDir / ChunkDbFile; });
		FChunkDbSourceConfig ChunkDbSourceConfig(MoveTemp(ChunkDatabaseFiles));
		ChunkDbSourceConfig.bDeleteChunkDBAfterUse = Configuration.bDeleteChunkDbFilesAfterUse;
		ChunkDbSourceConfig.bTruncateChunkDbs = Configuration.bTruncateChunkDbFilesAsUsed;
		return ChunkDbSourceConfig;
	}

	static FConstructorCloudChunkSourceConfig BuildConstructorCloudSourceConfig(const FBuildInstallerConfiguration& InConfiguration)
	{
		FConstructorCloudChunkSourceConfig CloudSourceConfig(InConfiguration.CloudDirectories);

		// Load max download retry count from engine config.
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkRetries"), CloudSourceConfig.MaxRetryCount, GEngineIni);
		CloudSourceConfig.MaxRetryCount = FMath::Clamp<int32>(CloudSourceConfig.MaxRetryCount, -1, 1000);

		// Load retry times from engine config.
		TArray<FString> ConfigStrings;
		GConfig->GetArray(TEXT("Portal.BuildPatch"), TEXT("RetryTimes"), ConfigStrings, GEngineIni);
		bool bReadArraySuccess = ConfigStrings.Num() > 0;
		TArray<float> RetryDelayTimes;
		RetryDelayTimes.AddZeroed(ConfigStrings.Num());
		for (int32 TimeIdx = 0; TimeIdx < ConfigStrings.Num() && bReadArraySuccess; ++TimeIdx)
		{
			float TimeValue = FPlatformString::Atof(*ConfigStrings[TimeIdx]);
			// Atof will return 0.0 if failed to parse, and we don't expect a time of 0.0 so presume error
			if (TimeValue > 0.0f)
			{
				RetryDelayTimes[TimeIdx] = FMath::Clamp<float>(TimeValue, 0.5f, 300.0f);
			}
			else
			{
				bReadArraySuccess = false;
			}
		}
		// If the retry array was parsed successfully, set on config.
		if (bReadArraySuccess)
		{
			CloudSourceConfig.RetryDelayTimes = MoveTemp(RetryDelayTimes);
		}

		// Load percentiles for download health groupings from engine config.
		// If the enum was changed since writing, the config here needs updating.
		check((int32)EBuildPatchDownloadHealth::NUM_Values == 5);
		TArray<float> HealthPercentages;
		HealthPercentages.AddZeroed((int32)EBuildPatchDownloadHealth::NUM_Values);
		if (GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("OKHealth"), HealthPercentages[(int32)EBuildPatchDownloadHealth::OK], GEngineIni)
			&& GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("GoodHealth"), HealthPercentages[(int32)EBuildPatchDownloadHealth::Good], GEngineIni)
			&& GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("ExcellentHealth"), HealthPercentages[(int32)EBuildPatchDownloadHealth::Excellent], GEngineIni))
		{
			CloudSourceConfig.HealthPercentages = MoveTemp(HealthPercentages);
		}

		// Load the delay for how long we get no data for until determining the health as disconnected.
		GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("DisconnectedDelay"), CloudSourceConfig.DisconnectedDelay, GEngineIni);
		CloudSourceConfig.DisconnectedDelay = FMath::Clamp<float>(CloudSourceConfig.DisconnectedDelay, 1.0f, 30.0f);

		CloudSourceConfig.DownloadRateLimitBps = InConfiguration.DownloadRateLimitBps;

		return CloudSourceConfig;
	}

	FCloudSourceConfig FBuildPatchInstaller::BuildCloudSourceConfig()
	{
		FCloudSourceConfig CloudSourceConfig(Configuration.CloudDirectories);
		CloudSourceConfig.SharedContext = Configuration.SharedContext.Get();
		CloudSourceConfig.DownloadRateLimitBps = Configuration.DownloadRateLimitBps;

		// Load max download retry count from engine config.
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkRetries"), CloudSourceConfig.MaxRetryCount, GEngineIni);
		CloudSourceConfig.MaxRetryCount = FMath::Clamp<int32>(CloudSourceConfig.MaxRetryCount, -1, 1000);

		// Load the number of retries before enabling detailed logging.
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("RetriesToEnableVerbose"), CloudSourceConfig.RetriesToEnableVerbose, GEngineIni);
		CloudSourceConfig.RetriesToEnableVerbose = FMath::Clamp<int32>(CloudSourceConfig.RetriesToEnableVerbose, 0, 1000);

		// Load prefetch config
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("CloudSourcePreFetchMinimum"), CloudSourceConfig.PreFetchMinimum, GEngineIni);
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("CloudSourcePreFetchMaximum"), CloudSourceConfig.PreFetchMaximum, GEngineIni);
		CloudSourceConfig.PreFetchMinimum = FMath::Clamp<int32>(CloudSourceConfig.PreFetchMinimum, 1, 1000);
		CloudSourceConfig.PreFetchMaximum = FMath::Clamp<int32>(CloudSourceConfig.PreFetchMaximum, CloudSourceConfig.PreFetchMinimum, 1000);

		// Load retry times from engine config.
		TArray<FString> ConfigStrings;
		GConfig->GetArray(TEXT("Portal.BuildPatch"), TEXT("RetryTimes"), ConfigStrings, GEngineIni);
		bool bReadArraySuccess = ConfigStrings.Num() > 0;
		TArray<float> RetryDelayTimes;
		RetryDelayTimes.AddZeroed(ConfigStrings.Num());
		for (int32 TimeIdx = 0; TimeIdx < ConfigStrings.Num() && bReadArraySuccess; ++TimeIdx)
		{
			float TimeValue = FPlatformString::Atof(*ConfigStrings[TimeIdx]);
			// Atof will return 0.0 if failed to parse, and we don't expect a time of 0.0 so presume error
			if (TimeValue > 0.0f)
			{
				RetryDelayTimes[TimeIdx] = FMath::Clamp<float>(TimeValue, 0.5f, 300.0f);
			}
			else
			{
				bReadArraySuccess = false;
			}
		}
		// If the retry array was parsed successfully, set on config.
		if (bReadArraySuccess)
		{
			CloudSourceConfig.RetryDelayTimes = MoveTemp(RetryDelayTimes);
		}

		// Load percentiles for download health groupings from engine config.
		// If the enum was changed since writing, the config here needs updating.
		check((int32)EBuildPatchDownloadHealth::NUM_Values == 5);
		TArray<float> HealthPercentages;
		HealthPercentages.AddZeroed((int32)EBuildPatchDownloadHealth::NUM_Values);
		if (GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("OKHealth"), HealthPercentages[(int32)EBuildPatchDownloadHealth::OK], GEngineIni)
			&& GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("GoodHealth"), HealthPercentages[(int32)EBuildPatchDownloadHealth::Good], GEngineIni)
			&& GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("ExcellentHealth"), HealthPercentages[(int32)EBuildPatchDownloadHealth::Excellent], GEngineIni))
		{
			CloudSourceConfig.HealthPercentages = MoveTemp(HealthPercentages);
		}

		// Load the delay for how long we get no data for until determining the health as disconnected.
		GConfig->GetFloat(TEXT("Portal.BuildPatch"), TEXT("DisconnectedDelay"), CloudSourceConfig.DisconnectedDelay, GEngineIni);
		CloudSourceConfig.DisconnectedDelay = FMath::Clamp<float>(CloudSourceConfig.DisconnectedDelay, 1.0f, 30.0f);

		// We tell the cloud source to only start downloads once it receives the first get call.
		CloudSourceConfig.bBeginDownloadsOnFirstGet = true;

		return CloudSourceConfig;
	}

	FDownloadConnectionCountConfig FBuildPatchInstaller::BuildConnectionCountConfig()
	{
		FDownloadConnectionCountConfig ConnectionCountConfiguration;

		// Load simultaneous downloads from engine config.
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloads"), (int32&)ConnectionCountConfiguration.FallbackCount, GEngineIni);
		ConnectionCountConfiguration.FallbackCount = FMath::Clamp<uint32>(ConnectionCountConfiguration.FallbackCount, 1, 100);

		// Check if download connection scaling is disabled via config or download throttling.
		GConfig->GetBool(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsDisableConnectionScaling"), ConnectionCountConfiguration.bDisableConnectionScaling, GEngineIni);
		ConnectionCountConfiguration.bDisableConnectionScaling = ConnectionCountConfiguration.bDisableConnectionScaling || DownloadRateLimitBps != 0;

		uint32 MaxLibCurlConnections = 0;
		uint32 MaxDownloadCount = ConnectionCountConfiguration.MaxLimit;
		if (!GConfig->GetInt(TEXT("HTTP"), TEXT("HttpMaxConnectionsPerServer"), (int32&)MaxLibCurlConnections, GEngineIni))
		{
			MaxDownloadCount = 16;
			UE_LOGF(LogBuildPatchServices, Warning, "HttpMaxConnectionsPerServer=0 is not set in the Engine.ini [HTTP] section. Simultaneous downloads will be limited to %d", MaxDownloadCount);
		}
		else
		{
			if (0 != MaxLibCurlConnections)
			{
				MaxDownloadCount = MaxLibCurlConnections;
				UE_LOGF(LogBuildPatchServices, Warning, "HttpMaxConnectionsPerServer is set to a non-zero value in the Engine.ini [HTTP] section. Simultaneous downloads will be limited to %d", MaxDownloadCount);
			}
		}

		uint32 MinLimit = ConnectionCountConfiguration.MinLimit;
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsLowerLimit"), (int32&)ConnectionCountConfiguration.MinLimit, GEngineIni);
		ConnectionCountConfiguration.MinLimit = FMath::Clamp<uint32>(ConnectionCountConfiguration.MinLimit,
			FMath::Min(MinLimit, MaxDownloadCount - 2),
			FMath::Min(32U, MaxDownloadCount - 2));

		// Overriding MinLimit for A/B Testing
		const FString UserId = FPlatformMisc::GetEpicAccountId();
		const FString Variant = Experiments::GetVariant(Experiments::DownloadSpeed(), UserId);

		if (Variant == TEXT("Connection_Floor_16"))
		{
			ConnectionCountConfiguration.MinLimit = 16;
		}

		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsUpperLimit"), (int32&)ConnectionCountConfiguration.MaxLimit, GEngineIni);
		ConnectionCountConfiguration.MaxLimit = FMath::Clamp<uint32>(ConnectionCountConfiguration.MaxLimit, ConnectionCountConfiguration.MinLimit + 2, MaxDownloadCount);

		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsSlowdownHysteresis"), (int32&)ConnectionCountConfiguration.NegativeHysteresis, GEngineIni);
		ConnectionCountConfiguration.NegativeHysteresis = FMath::Clamp<uint32>(ConnectionCountConfiguration.NegativeHysteresis, 1, 256);

		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsSpeedupHysteresis"), (int32&)ConnectionCountConfiguration.PositiveHysteresis, GEngineIni);
		ConnectionCountConfiguration.PositiveHysteresis = FMath::Clamp<uint32>(ConnectionCountConfiguration.PositiveHysteresis, 1, 256);

		GConfig->GetDouble(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsLowBandwidthFactor"), ConnectionCountConfiguration.LowBandwidthFactor, GEngineIni);
		ConnectionCountConfiguration.LowBandwidthFactor = FMath::Clamp<double>(ConnectionCountConfiguration.LowBandwidthFactor, 0.2L, 0.8L);

		GConfig->GetDouble(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsHighBandwidthFactor"), ConnectionCountConfiguration.HighBandwidthFactor, GEngineIni);
		ConnectionCountConfiguration.HighBandwidthFactor = FMath::Clamp<double>(ConnectionCountConfiguration.HighBandwidthFactor, ConnectionCountConfiguration.LowBandwidthFactor + 0.1L, 1.0L);

		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsAverageMinCount"), (int32&)ConnectionCountConfiguration.AverageSpeedMinCount, GEngineIni);
		ConnectionCountConfiguration.AverageSpeedMinCount = FMath::Clamp<uint32>(ConnectionCountConfiguration.AverageSpeedMinCount, 1, 32);

		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkDownloadsHealthHysteresis"), (int32&)ConnectionCountConfiguration.HealthHysteresis, GEngineIni);
		ConnectionCountConfiguration.HealthHysteresis = FMath::Clamp<uint32>(ConnectionCountConfiguration.HealthHysteresis, 1, 256);

		return ConnectionCountConfiguration;
	}

	FString FBuildPatchInstaller::GetSHA1Filename(const FFileManifest& FileManifest)
	{
		FString FileHashStringValue;
		FBuildPatchUtils::SHAToBase32(FileManifest.SHA1Hash, FileHashStringValue);

		return FileHashStringValue;
	}

	static void GenerateFilesToConstruct(TSet<FString>& FilesToConstruct, IBuildManifestSet* ManifestSet, const TArray<FString>& CorruptFiles, const TSet<FString>& TaggedFiles, const TSet<FString>& OutdatedFiles, bool bIsPrereqOnly)
	{
		// Get the list of files actually needing construction
		FilesToConstruct.Empty();
		if (CorruptFiles.Num())
		{
			FilesToConstruct.Append(CorruptFiles);
		}
		else if (bIsPrereqOnly)
		{
			TArray<FPreReqInfo> PreReqInfos;
			ManifestSet->GetPreReqInfo(PreReqInfos);
			for (FPreReqInfo PreReqInfo : PreReqInfos)
			{
				FilesToConstruct.Add(PreReqInfo.Path);
			}
		}
		else
		{
			FilesToConstruct = OutdatedFiles.Intersect(TaggedFiles);
		}
		// This is sorted coming in and needs to stay in that order to pass BPT test suite
		//FilesToConstruct.Sort(TLess<FString>());
	}

	//
	// Returns how much disk space is needed in order to complete the install or patch.
	// 
	// This includes the ChunkDB size, the staging size (with destructive install), and any increases to the
	// installation directory.
	//
	// It's possible this is 0 if, for example, the patch perfectly deletes segments from files such
	// that no new chunks are required, the first file is completely deleted (no staging size), and the
	// staging size of further files fits in that freed up space.
	//
	static uint64 DetermineInstallMaxDiskSizeIfDeletingChunkDbs(const TArray<FString>& ChunkDbFiles, IBuildManifestSet* ManifestSet, IFileSystem* FileSystem, const FString& InstallDirectory, const TArray<FString>& CorruptFiles, EInstallMode InstallMode)
	{
		TSet<FString> FilesToConstructSet;

		TSet<FString> TaggedFiles;
		ManifestSet->GetExpectedFiles(TaggedFiles);
		TSet<FString> OutdatedFiles;
		ManifestSet->GetOutdatedFiles(InstallDirectory, OutdatedFiles);

		GenerateFilesToConstruct(FilesToConstructSet, ManifestSet, CorruptFiles, TaggedFiles, OutdatedFiles, InstallMode == EInstallMode::PrereqOnly);

		// Generate the list of chunks we will consume, in order.
		TUniquePtr<IChunkReferenceTracker> ChunkReferenceTracker(FChunkReferenceTrackerFactory::Create(
			ManifestSet,
			FilesToConstructSet));
		TArray<FGuid> ReferenceChain;
		ChunkReferenceTracker->CopyOutOrderedUseList(ReferenceChain);

		TArray<FString> FilesToConstruct = FilesToConstructSet.Array();

		// NOTE - removed files are done at the END of installation, not the beginning (because of chunk sources? could be better!)
		// so it doesn't affect how much disk space we need (it's consumed the whole time).
	
		//
		// Calculating the disk size requried to install:
		//
		// We have a bunch of chunkdbs on disk that sum to the size of the end install
		// We know that we can delete many - but not all - of the input chunkdbs when a file is
		// completed.
		//
		// After each file completion we have:
		// 
		// Size of all remaining chunkdbs + size of installed files thus far.
		//
		// So we get the actual required size by iterating over the file list
		// and asking the chunk db system how many chunkdbs are left at the current
		// reference level.
		//
		// For patches, we adjust this by compensating the installed file size with
		// the size of the replaced files.
		int32 CurrentPosition = 0;

		TArray<int32> FileCompletionPositions;
		for (const FString& FileToConstruct : FilesToConstruct)
		{
			const FFileManifest* FileManifest = ManifestSet->GetNewFileManifest(FileToConstruct);
			if (!FileManifest)
			{
				// This is an error condition that will fail to install, but we don't want to crash here...
				continue;
			}

			// We will be advancing the chunk reference tracker by this many chunks.
			int32 AdvanceCount = FileManifest->ChunkParts.Num();

			CurrentPosition += AdvanceCount;

			FileCompletionPositions.Add(CurrentPosition);
		}

		TArray<uint64> ChunkDbSizesAtPosition;
		IConstructorChunkDbChunkSource::GetChunkDbSizesAtIndexes(ChunkDbFiles, FileSystem, ReferenceChain, FileCompletionPositions, ChunkDbSizesAtPosition);

		uint64 MaxDiskSize = FBuildPatchUtils::CalculateDiskSpaceRequirementsWithDeleteDuringInstall(
			FilesToConstruct, 0, 0, ManifestSet,
			ChunkDbSizesAtPosition, InstallDirectory, InstallMode);

		return MaxDiskSize;
	}

	bool FBuildPatchInstaller::RunInstallation(TArray<FString>& CorruptFiles)
	{
		// Reset all errors except for canceled errors to preserve the canceled state between install retries.
		InstallerError->ResetError(true /*bPreserveCancelled*/);

		// Installation is not ran for VerifyOnly actions. This mode is read-only.
		if (ManifestSet->IsExclusivelyVerifying())
		{
			UE_LOGF(LogBuildPatchServices, Log, "Skipping installation for VerifyOnly action");
			// Set weights for verify only
			BuildProgress.SetStateWeight(EBuildPatchState::Downloading, 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::Installing, 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::MovingToInstall, 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::SettingAttributes, 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::BuildVerification, 1.0f);
			// Mark all installation steps complete
			BuildProgress.SetStateProgress(EBuildPatchState::Initializing, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Resuming, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Downloading, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Installing, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::SettingAttributes, 1.0f);
			// Stop relevant timers
			InitializeTimer.Stop();
			return true;
		}
		UE_LOGF(LogBuildPatchServices, Log, "Starting Installation");

		// Get the list of required files, by the tags
		TaggedFiles.Empty();
		ManifestSet->GetExpectedFiles(TaggedFiles);
		OutdatedFiles.Empty();
		
		if (Configuration.bInstallToMemory)
		{
			// When installing to memory we know we aren't patching and we know we don't have an installation
			// directory, so we are always installing everything.
			OutdatedFiles = TaggedFiles;
		}
		else
		{
			// The only reason the install directory is passed to this is to do a "just in case" check where even if a file isn't
			// supposed to have changed, if the file size on disk DID change, we add it to the file list as outdated. IMO this should be done as a separate function,
			// but for the time being we don't really care about this verification so we remove the install directory to disable this when we have
			// per file subdirectories rather than try and pipe the per file subdirectory stuff all through the manifest set.
			ManifestSet->GetOutdatedFiles(NewPerFileSubdirectories.Num() ? FString() : Configuration.InstallDirectory, OutdatedFiles);
		}

		const bool bIsPrereqOnly = Configuration.InstallMode == EInstallMode::PrereqOnly;
		const bool bHasCorruptFiles = CorruptFiles.Num() > 0;

		GenerateFilesToConstruct(FilesToConstruct, ManifestSet.Get(), CorruptFiles, TaggedFiles, OutdatedFiles, bIsPrereqOnly);
		UE_LOGF(LogBuildPatchServices, Log, "Requiring %d files", FilesToConstruct.Num());

		// Check if we should skip out of this process due to existing installation,
		// that will mean we start with the verification stage
		if (!Configuration.bInstallToMemory &&
			!bHasCorruptFiles && 
			(bIsPrereqOnly || CheckForExternallyInstalledFiles(FilesToConstruct)))
		{
			UE_LOGF(LogBuildPatchServices, Log, "Detected previous staging completed, or existing files in target directory");
			// Add required files to the verifier as 'touched' since we do not know their state.
			Verifier->AddTouchedFiles(FilesToConstruct);
			// Set weights for verify only
			BuildProgress.SetStateWeight(EBuildPatchState::Downloading, 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::Installing, 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::MovingToInstall, 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::SettingAttributes, 0.2f);
			BuildProgress.SetStateWeight(EBuildPatchState::BuildVerification, 1.0f);
			// Mark all installation steps complete
			BuildProgress.SetStateProgress(EBuildPatchState::Initializing, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Resuming, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Downloading, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Installing, 1.0f);
			// Stop relevant timers
			InitializeTimer.Stop();
			return true;
		}

		if (!bHasCorruptFiles)
		{
			FScopeLock Lock(&ThreadLock);
			BuildStats.NumFilesOutdated = FilesToConstruct.Num();
		}

		// Make sure all the files won't exceed the maximum path length
		if (!Configuration.bInstallToMemory)
		{
			{
				// +1 means extra char for path delimiter
				int32 StagedConstructionFilePathLen = InstallStagingDir.Len() + 1 + BuildPatchConstants::SHAToBase32StringLength;

				if (StagedConstructionFilePathLen >= FPlatformMisc::GetMaxPathLength())
				{
					UE_LOGF(LogBuildPatchServices, Error, "Staged filenames (length %d) will exceed maximum path length when saved to %ls", BuildPatchConstants::SHAToBase32StringLength, *InstallStagingDir);
					InstallerError->SetError(EBuildPatchInstallError::PathLengthExceeded, PathLengthErrorCodes::StagingDirectory);
					return false;
				}
			}
			for (const FString& FileToConstruct : FilesToConstruct)
			{
				const FString InstallConstructionFile = FBuildPatchUtils::ResolveInstallationFileName(Configuration.InstallDirectory, NewPerFileSubdirectories, FileToConstruct);

				if (InstallConstructionFile.Len() >= FPlatformMisc::GetMaxPathLength())
				{
					UE_LOGF(LogBuildPatchServices, Error, "Could not create new file due to exceeding maximum path length %ls", *InstallConstructionFile);
					InstallerError->SetError(EBuildPatchInstallError::PathLengthExceeded, PathLengthErrorCodes::InstallDirectory);
					return false;
				}

				if (Configuration.bStageWithRawFilenames)
				{
					FString StagedConstructionFile = InstallStagingDir / FileToConstruct;

					if (StagedConstructionFile.Len() >= FPlatformMisc::GetMaxPathLength())
					{
						UE_LOGF(LogBuildPatchServices, Error, "Could not create new file due to exceeding maximum path length %ls", *StagedConstructionFile);
						InstallerError->SetError(EBuildPatchInstallError::PathLengthExceeded, PathLengthErrorCodes::StagingDirectory);
						return false;
					}
				}
			}
		}

		// Set initial states on IO state tracker.
		const bool bVerifyAllFiles = Configuration.VerifyMode == EVerifyMode::ShaVerifyAllFiles || Configuration.VerifyMode == EVerifyMode::FileSizeCheckAllFiles;
		const EFileOperationState UntouchedFileState = bVerifyAllFiles ? EFileOperationState::Installed : EFileOperationState::Complete;
		for (const FString& TaggedFile : TaggedFiles)
		{
			if (!FilesToConstruct.Contains(TaggedFile))
			{
				FileOperationTracker->OnFileStateUpdate(TaggedFile, UntouchedFileState);
			}
		}

		// Cache the last download requirement in case we are running a retry.
		PreviousTotalDownloadRequired += CloudChunkSourceStatistics->GetDownloadedBytes();
		// Reset so that we don't double count data.
		CloudChunkSourceStatistics->OnRequiredDataUpdated(0);
		CloudChunkSourceStatistics->OnReceivedDataUpdated(0);

		// De-dupe the file construction list if we are using SHA1 filenames for staging. In this case we don't build duplicate files, we copy them.
		TSet<FString> DeDupedFilesToConstruct;
		if (!Configuration.bStageWithRawFilenames)
		{
			TSet<FSHAHash> FileShaReferences;
			for (const FString& File : FilesToConstruct)
			{
				const FFileManifest* NewFileManifest = ManifestSet->GetNewFileManifest(File);
				if (NewFileManifest != nullptr && !FileShaReferences.Contains(NewFileManifest->SHA1Hash))
				{
					FileShaReferences.Add(NewFileManifest->SHA1Hash);
					DeDupedFilesToConstruct.Add(File);
				}
			}
		}

		// Before beginning work, make sure we have deleted the HDD reservation file if it existed.
		FileSystem->DeleteFile(*HddSpaceReservationFile);

		// Scoped systems composition and execution.
		{
			TUniquePtr<IChunkDataSerialization> ChunkDataSerialization(FChunkDataSerializationFactory::Create(
				FileSystem.Get(), Crypto.Get(), { EFeatureLevel::Latest, Configuration.EncryptionSecrets, FGuid() /* UE5 MERGE TODO : , CloudChunkSourceStatistics.Get()*/}));

			// Generate the list of chunks we will need and the order in which we will need them.
			TUniquePtr<IChunkReferenceTracker> ChunkReferenceTracker(FChunkReferenceTrackerFactory::Create(
				ManifestSet.Get(),
				Configuration.bStageWithRawFilenames ? FilesToConstruct : DeDupedFilesToConstruct));
			TSet<FGuid> ReferencedChunks = ChunkReferenceTracker->GetReferencedChunks();
			
			// Add a source for pulling from chunk "databases", basically tarballs of chunks.
			TArray<FGuid> ChunkAccessOrderedList;
			ChunkReferenceTracker->CopyOutOrderedUseList(ChunkAccessOrderedList);
			TUniquePtr<IConstructorChunkDbChunkSource> ChunkDbChunkSource(IConstructorChunkDbChunkSource::CreateChunkDbSource(
				BuildChunkDbSourceConfig(Configuration, DataStagingDir),
				FileSystem.Get(),
				ChunkAccessOrderedList,
				ChunkDataSerialization.Get(),
				ChunkDbChunkSourceStatistics.Get()));

			// Add a source for pulling from the existing directory (for patching). This uses the manifest for the
			// currently deployed files to create a list of available chunks to read from.
			TUniquePtr<IConstructorInstallChunkSource> InstallChunkSource;
			
			TSet<FGuid> EmptySet;
			if (!Configuration.bInstallToMemory)
			{
				InstallChunkSource.Reset(IConstructorInstallChunkSource::CreateInstallSource(
					FileSystem.Get(),
					InstallChunkSourceStatistics.Get(),
					InstallationInfo,
					ReferencedChunks,
					false,
					OldPerFileSubdirectories
				));
			}

			// Sub out the chunks we have available on disk - anything we don't we have to download.
			const TSet<FGuid>& ChunkDbChunksAvailable = ChunkDbChunkSource->GetAvailableChunks();
			const TSet<FGuid>& InstallChunksAvailable = InstallChunkSource.IsValid() ? InstallChunkSource->GetAvailableChunks() : EmptySet;
			
			TSet<FGuid> CloudChunks = ReferencedChunks.Difference(InstallChunksAvailable).Difference(ChunkDbChunksAvailable);
			TSet<FGuid> ChunkDbChunks = ReferencedChunks.Intersect(ChunkDbChunksAvailable);

			// Note ordering - if a chunk is available via install and chunkdb, we take it from chunkdb.
			TSet<FGuid> InstallChunks = ReferencedChunks.Intersect(InstallChunksAvailable).Difference(ChunkDbChunksAvailable);

			if (InstallChunks.Num() == 0)
			{
				InstallChunkSource.Reset();
			}

			// Gather statistics on expected and available chunks for each action.
			InstallActionStatistics->CalculateChunkStatistics(InstallChunks);

			// Set up our chunk location tracking.
			int32 DownloadChunkCount = CloudChunks.Num();
			TMap<FGuid, EConstructorChunkLocation> ChunkLocations;	
			uint64 InstallBytes = 0;
			uint64 ChunkDbBytes = 0;
			uint64 CloudBytes = 0;
			{
				for (const FGuid& InstallChunk : InstallChunks)
				{
					ChunkLocations.Add(InstallChunk, EConstructorChunkLocation::Install);
					InstallBytes += ManifestSet->GetDownloadSize(InstallChunk);
				}
				for (const FGuid& ChunkDbChunk : ChunkDbChunks)
				{
					ChunkLocations.Add(ChunkDbChunk, EConstructorChunkLocation::ChunkDb);
					ChunkDbBytes += ManifestSet->GetDownloadSize(ChunkDbChunk);
				}
				for (const FGuid& CloudChunk : CloudChunks)
				{
					ChunkLocations.Add(CloudChunk, EConstructorChunkLocation::Cloud);
					CloudBytes += ManifestSet->GetDownloadSize(CloudChunk);
				}
			}

			FileOperationTracker->OnDataStateUpdate(ChunkDbChunks, EFileOperationState::PendingLocalChunkDbData);
			FileOperationTracker->OnDataStateUpdate(InstallChunks, EFileOperationState::PendingLocalInstallData);
			FileOperationTracker->OnDataStateUpdate(CloudChunks, EFileOperationState::PendingRemoteCloudData);

			// Add the source for downloading chunks we don't have on disk.
			TUniquePtr<IDownloadConnectionCount> DownloadConnectionCount(FDownloadConnectionCountFactory::Create(BuildConnectionCountConfig(), DownloadServiceStatistics.Get()));

			TUniquePtr<IConstructorCloudChunkSource> CloudChunkSource(
				IConstructorCloudChunkSource::CreateCloudSource(
					BuildConstructorCloudSourceConfig(Configuration),
					DownloadService.Get(),
					ChunkDataSerialization.Get(),
					DownloadConnectionCount.Get(),
					MessagePump.Get(),
					CloudChunkSourceStatistics.Get(),
					ManifestSet.Get())
			);

			// Set up the class that actually requests chunks and writes them to disk.
			FFileConstructorConfig FCC;
			FCC.bInstallToMemory = Configuration.bInstallToMemory;
			FCC.bConstructInMemory = Configuration.bConstructFilesInMemory;
			FCC.bSkipInitialDiskSizeCheck = Configuration.bSkipInitialDiskSizeCheck;
			FCC.ConstructList = (!Configuration.bStageWithRawFilenames ? DeDupedFilesToConstruct : FilesToConstruct).Array();
			FCC.InstallDirectory = Configuration.InstallDirectory;
			FPaths::NormalizeDirectoryName(FCC.InstallDirectory);
			FCC.InstallMode = Configuration.InstallMode;
			FCC.PerFileSubdirectories = &NewPerFileSubdirectories;
			FCC.ManifestSet = ManifestSet.Get();
			FCC.MetaDirectory = MetaStagingDir;
			FCC.SharedContext = Configuration.SharedContext.Get();
			FCC.StagingDirectory = InstallStagingDir;
			FCC.bDeleteChunkDBFilesAfterUse = Configuration.bDeleteChunkDbFilesAfterUse;
			FCC.BackingStoreDirectory = DataStagingDir;
			FCC.SpawnAdditionalIOThreads = Configuration.ConstructorSpawnAdditionalIOThreads;
			FCC.IOBufferSizeMB = Configuration.ConstructorIOBufferSizeMB;
			FCC.IOBatchSizeMB = Configuration.ConstructorIOBatchSizeMB;
			FCC.StallWhenFileSystemThrottled = Configuration.ConstructorStallWhenFileSystemThrottled;
			FCC.DisableResumeBelowMB = Configuration.ConstructorDisableResumeBelowMB;
			FCC.bUseSHA1StageFilenames = !Configuration.bStageWithRawFilenames;
			FCC.bRejectSymlinks = Configuration.bRejectSymlinks;
			
			TUniquePtr<FBuildPatchFileConstructor> FileConstructor(
				new FBuildPatchFileConstructor(
					FCC, 
					FileSystem.Get(), 
					ChunkDbChunkSource.Get(), 
					CloudChunkSource.Get(),
					InstallChunkSource.Get(),
					ChunkReferenceTracker.Get(), 
					InstallerError.Get(),
					InstallerAnalytics.Get(),
					MessagePump.Get(),
					FileConstructorStatistics.Get(),
					MoveTemp(ChunkLocations)
				)
			);

			FDelegateHandle OnBeforeDeleteFileHandle;
			if (InstallChunkSource)
			{
				OnBeforeDeleteFileHandle = FileConstructor->OnBeforeDeleteFile().AddLambda([this, &InstallChunkSource](const FString& FilePath)
				{
					FString BuildRelativeFilename = FilePath;
					BuildRelativeFilename.RemoveFromStart(Configuration.InstallDirectory);
					BuildRelativeFilename.RemoveFromStart(TEXT("/"));
					OldFilesRemovedBySystem.Add(BuildRelativeFilename);
					InstallChunkSource->OnBeforeDeleteFile(FilePath);
				});		
			}
			FDelegateHandle OnFileIsStagedHandle = FileConstructor->OnFileIsStaged().AddLambda([this](const FString& FilePath)
				{
					MessagePump->SendMessage(FInstallationFileAction{ FInstallationFileAction::EType::Staged, FilePath });
				});
			// Register controllables.
			FScopedControllables ScopedControllables(&ThreadLock, Controllables, bIsPaused, bShouldAbort);
			ScopedControllables.Register(FileConstructor.Get());

			// Set chunk counter stats.
			if (!bHasCorruptFiles)
			{
				FScopeLock Lock(&ThreadLock);
				BuildStats.NumChunksRequired = ReferencedChunks.Num();
				BuildStats.ChunksQueuedForDownload = CloudChunks.Num();
				BuildStats.ChunksLocallyAvailable = InstallChunks.Num();
				BuildStats.ChunksInChunkDbs = ChunkDbChunks.Num();
				UE_LOGF(LogBPSInstallerConfig, Display, "Chunk Locations: Cloud %d (%ls bytes) Install %d (%ls bytes) ChunkDb %d (%ls bytes)", 
					BuildStats.ChunksQueuedForDownload, *FText::AsNumber(CloudBytes).ToString(), 
					BuildStats.ChunksLocallyAvailable, *FText::AsNumber(InstallBytes).ToString(),
					BuildStats.ChunksInChunkDbs, *FText::AsNumber(ChunkDbBytes).ToString());
			}

			// Setup some weightings for the progress tracking
			const bool bExclusivelyRepairing = ManifestSet->IsExclusivelyRepairing() && FilesToConstruct.Num() == 0;
			const float NumRequiredChunksFloat = ReferencedChunks.Num();
			const bool bHasFileAttributes = ManifestSet->HasFileAttributes();
			const float AttributesWeight = bHasFileAttributes ? bExclusivelyRepairing ? 1.0f / 50.0f : 1.0f / 20.0f : 0.0f;
			const float VerifyWeight = Configuration.VerifyMode == EVerifyMode::ShaVerifyAllFiles || Configuration.VerifyMode == EVerifyMode::ShaVerifyTouchedFiles ? 1.1f / 9.0f : 0.3f / 9.0f;
			BuildProgress.SetStateWeight(EBuildPatchState::Downloading, 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::MovingToInstall, 0.05f);
			BuildProgress.SetStateWeight(EBuildPatchState::Installing, FilesToConstruct.Num() > 0 ? 1.0f : 0.0f);
			BuildProgress.SetStateWeight(EBuildPatchState::SettingAttributes, AttributesWeight);
			BuildProgress.SetStateWeight(EBuildPatchState::BuildVerification, VerifyWeight);

			// If this is a repair operation, start off with install and download complete
			if (bExclusivelyRepairing)
			{
				UE_LOGF(LogBuildPatchServices, Log, "Performing a repair operation");
				BuildProgress.SetStateProgress(EBuildPatchState::Downloading, 1.0f);
				BuildProgress.SetStateProgress(EBuildPatchState::Installing, 1.0f);
				BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, 1.0f);
			}

			// Initializing is now complete if we are constructing files
			if (FilesToConstruct.Num() > 0)
			{
				BuildProgress.SetStateProgress(EBuildPatchState::Initializing, 1.0f);
				InitializeTimer.Stop();
			}

			// Win64 Static Analysis warn about lambda lifetime when assigned directly to member variable.
			// e.g: warning V1047: Lifetime of the lambda is greater than lifetime of the local variable 'CloudChunkSource' captured by reference.
			// Instead we set a local variable, and use that to set the TFunction. The member variable is properly reset a few lines below.
			auto DLRateLambda = [&DownloadConnectionCount, &CloudChunkSource](uint64 NewDownloadRateLimitBps)
				{
					CloudChunkSource->SetDownloadBytesPerSecondLimit(NewDownloadRateLimitBps);
					DownloadConnectionCount->SetConnectionScaling(NewDownloadRateLimitBps == 0);
				};

			{
				UE::TUniqueLock _(DownloadRateLimitLock);
				DownloadRateUpdatedFn = DLRateLambda;
			}

			// Wait for the file constructor to complete
			ConstructTimer.Start();
			FileConstructor->Run();
			ConstructTimer.Stop();
			FileConstructor->OnFileIsStaged().Remove(OnFileIsStagedHandle);

			{
				UE::TUniqueLock _(DownloadRateLimitLock);
				DownloadRateUpdatedFn.Reset();
			}

			if (OnBeforeDeleteFileHandle.IsValid())
			{
				FileConstructor->OnBeforeDeleteFile().Remove(OnBeforeDeleteFileHandle);
			}
			const bool bReportAnalytic = InstallerError->HasError() == false;
			if (bReportAnalytic && bLastInstallIteration && InstallerAnalytics.IsValid() && FileConstructor->HasCorruptFiles())
			{
				InstallerAnalytics->RecordConstructionError({}, INDEX_NONE, TEXT("Serialised Verify Fail"));
			}

			UE_LOGF(LogBuildPatchServices, Log, "File construction complete");

			if (Configuration.bInstallToMemory)
			{
				FileConstructor->GrabFilesInstalledToMemory(FilesInstalledToMemory);
			}

			// Set any final stats before system destruction.
			{
				FScopeLock Lock(&ThreadLock);
				BuildStats.ProcessRequiredDiskSpace = FileConstructor->GetRequiredDiskSpace();
				BuildStats.ProcessAvailableDiskSpace = FileConstructor->GetAvailableDiskSpace();

				FBuildPatchFileConstructor::FBackingStoreStats BackingStoreStats = FileConstructor->GetBackingStoreStats();

				BuildStats.DriveStorePeakBytes = BackingStoreStats.DiskPeakUsageBytes;
				BuildStats.NumDriveStoreLoadFailures = BackingStoreStats.DiskLoadFailureCount;
				BuildStats.NumDriveStoreLostChunks = BackingStoreStats.DiskLostChunkCount;
				BuildStats.NumDriveStoreChunkLoads = BackingStoreStats.DiskChunkLoadCount;

				BuildStats.MemoryStoreSizePeakBytes = BackingStoreStats.MemoryPeakUsageBytes;
				BuildStats.MemoryStoreSizeLimitBytes = BackingStoreStats.MemoryLimitBytes;
			}

			// Let the verifier know which files we built.
			Verifier->AddTouchedFiles(FilesToConstruct);
		}

		// Process some final stats.
		SetFinalInstallStats();

		UE_LOGF(LogBuildPatchServices, Log, "Staged install complete");
		const bool bReturnInstallationSuccess = !InstallerError->HasError();

		// Ensure all progress complete
		if (bReturnInstallationSuccess)
		{
			BuildProgress.SetStateProgress(EBuildPatchState::Initializing, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Resuming, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Downloading, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Installing, 1.0f);
			InitializeTimer.Stop();
		}

		return bReturnInstallationSuccess;
	}

	bool FBuildPatchInstaller::RunPreload()
	{
		// Reset all errors except for canceled errors to preserve the canceled state between install retries.
		InstallerError->ResetError(true /*bPreserveCancelled*/);

		// Reset build progress
		UE_LOGF(LogBuildPatchServices, Log, "Starting Preload");

		// Set weights for preload only
		BuildProgress.SetStateWeight(EBuildPatchState::Resuming, 0.0f);
		BuildProgress.SetStateWeight(EBuildPatchState::Downloading, 1.0f);
		BuildProgress.SetStateWeight(EBuildPatchState::Installing, 1.0f);
		BuildProgress.SetStateWeight(EBuildPatchState::MovingToInstall, 0.0f);
		BuildProgress.SetStateWeight(EBuildPatchState::SettingAttributes, 0.0f);
		BuildProgress.SetStateWeight(EBuildPatchState::BuildVerification, 0.0f);
		BuildProgress.SetStateWeight(EBuildPatchState::RunningCustomUninstallAction, 0.0f);

		// Mark all irrelevant steps complete
		BuildProgress.SetStateProgress(EBuildPatchState::Resuming, 1.0f);
		BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, 1.0f);
		BuildProgress.SetStateProgress(EBuildPatchState::SettingAttributes, 1.0f);
		BuildProgress.SetStateProgress(EBuildPatchState::BuildVerification, 1.0f);
		BuildProgress.SetStateProgress(EBuildPatchState::RunningCustomUninstallAction, 1.0f);

		// We need an array of sorted, duplicate chunks, in order to produce the chunkDB files properly.
		// So we use the TArray type function instead of TSet
		TArray<FGuid> ReferencedChunks;
		ManifestSet->GetReferencedChunks(ReferencedChunks);
		Algo::Reverse(ReferencedChunks);
		
		// Default chunk store size to tie in with the default prefetch maxes for source configs.
		int32 ChunkStoreMemorySize = 500;
		// Load overridden size from config.
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkStoreMemorySize"), ChunkStoreMemorySize, GEngineIni);
		// Clamp to sensible limits.
		ChunkStoreMemorySize = FMath::Clamp<int32>(ChunkStoreMemorySize, 64, 2048);
		// Cache the last download requirement in case we are running a retry.
		PreviousTotalDownloadRequired += CloudChunkSourceStatistics->GetDownloadedBytes();
		// Reset so that we don't double count data.
		CloudChunkSourceStatistics->OnRequiredDataUpdated(0);
		CloudChunkSourceStatistics->OnReceivedDataUpdated(0);
		bAreResultsEncrypted = true;

		// Scoped systems composition and execution.
		{
			TUniquePtr<IChunkDataSerialization> ChunkDataSerialization(FChunkDataSerializationFactory::Create(
				FileSystem.Get(), Crypto.Get(), { EFeatureLevel::Latest, Configuration.EncryptionSecrets, FGuid() /* UE5 MERGE TODO : , CloudChunkSourceStatistics.Get()*/ }));

			const TSet<FGuid> ReferencedChunksSet{ ReferencedChunks };

			// We don't actually need a chunkdb source, but we'll use it to tell us what we already downloaded.
			const TSet<FGuid> AlreadyFetchedChunks = [&]()
				{
					TUniquePtr<IConstructorChunkDbChunkSource> ChunkDbChunkSource(IConstructorChunkDbChunkSource::CreateChunkDbSource(
						BuildChunkDbSourceConfig(Configuration, DataStagingDir),
						FileSystem.Get(),
						ReferencedChunks,
						ChunkDataSerialization.Get(),
						ChunkDbChunkSourceStatistics.Get()));
					return ChunkDbChunkSource->GetAvailableChunks();
				}();
			const TSet<FGuid> InitialDownloadChunks = ReferencedChunksSet.Difference(AlreadyFetchedChunks);

			// Set chunk counter stats.
			{
				FScopeLock Lock(&ThreadLock);
				BuildStats.NumChunksRequired = ReferencedChunks.Num();
				BuildStats.ChunksQueuedForDownload = InitialDownloadChunks.Num();
				BuildStats.ChunksLocallyAvailable = AlreadyFetchedChunks.Num();
				BuildStats.ChunksInChunkDbs = AlreadyFetchedChunks.Num();
			}

			// Initialisation complete.
			BuildProgress.SetStateProgress(EBuildPatchState::Initializing, 1.0f);
			FileConstructorStatistics->OnResumeCompleted();
			InitializeTimer.Stop();

			// If we don't have any InitialDownloadChunks, then we already finished building all of the chunkdb files necessary
			TAtomic<bool> bChunkDbSuccess(true);
			if (InitialDownloadChunks.Num() > 0)
			{
				TArray<FGuid> OrderedRequiredChunks;
				TArray<FChunkDatabaseFile> ChunkDbFiles;
				bChunkDbSuccess = false;
				// Figure out the details for each individual chunkdb we want to save out.
				const uint64 ChunkDbHeaderSize = ChunkDbHelpers::GetChunkDbHeaderSize();
				const uint64 PerEntryHeaderSize = ChunkDbHelpers::GetPerChunkEntryHeaderSize();
				const uint64 ChunkDbMaxSize = ConfigHelpers::LoadPreloadMaxChunkDbSize();
				uint64 RequiredDiskSpace = 0;
				uint64 AvailableFileSize = 0;
				int32 ChunkDbFileIdx = 0;
				OrderedRequiredChunks.Reserve(ReferencedChunks.Num());
				for (const FGuid& DataId : ReferencedChunks)
				{
					if (!AlreadyFetchedChunks.Contains(DataId))
					{
						OrderedRequiredChunks.Add(DataId);
						const uint64 DataSize = ManifestSet->GetDownloadSize(DataId) + PerEntryHeaderSize;
						RequiredDiskSpace += DataSize;
						if (AvailableFileSize < DataSize)
						{
							ChunkDbFileIdx = ChunkDbFiles.AddDefaulted();
							AvailableFileSize = ChunkDbMaxSize - ChunkDbHeaderSize;
							RequiredDiskSpace += ChunkDbHeaderSize;
						}

						ChunkDbFiles[ChunkDbFileIdx].DataList.Add(DataId);
						AvailableFileSize -= DataSize;
					}
				}

				// Count how many chunkdb files already exist, from a previous preload progress.
				TArray<FString> PreExistingChunkDbs = ConfigHelpers::EnumeratePreloadChunkDbs(*DataStagingDir);

				// Produce nicely named chunkDBs.
				// Technically, there are mathematical solutions to this, however there can be floating point errors in log that cause edge cases there, so we'll just use the obvious simple method.
				const int32 NumDigitsForParts = FString::Printf(TEXT("%d"), ChunkDbFiles.Num() + PreExistingChunkDbs.Num()).Len();
				for (ChunkDbFileIdx = 0; ChunkDbFileIdx < ChunkDbFiles.Num(); ++ChunkDbFileIdx)
				{
					ChunkDbFiles[ChunkDbFileIdx].DatabaseFilename = DataStagingDir / FString::Printf(TEXT("clouddata.part%0*d.chunkdb"), NumDigitsForParts, ChunkDbFileIdx + 1 + PreExistingChunkDbs.Num());
				}

				// Setup file stats
				{
					FScopeLock Lock(&ThreadLock);
					BuildStats.NumFilesInBuild = PreExistingChunkDbs.Num() + ChunkDbFiles.Num();
					BuildStats.NumFilesOutdated = ChunkDbFiles.Num();
				}

				// Check we have enough disk space to proceed
				const TOptional<FileConstructorHelpers::FDiskSpaceInfo> DiskSpaceInfo = FileConstructorHelpers::GetRemainingDiskSpace(Configuration.InstallDirectory, RequiredDiskSpace);
				if (!DiskSpaceInfo.IsSet() || !DiskSpaceInfo->bHaveSufficientFreeSpace)
				{
					UE_LOGF(LogBuildPatchServices, Display, "Preload: Initial Disk Sizes: Required: %ls Available: %ls", *FormatNumber(RequiredDiskSpace), DiskSpaceInfo.IsSet() ? *FormatNumber(DiskSpaceInfo->AvailableDiskSpace) : TEXT("<failed>"));
					InstallerError->SetError(
						EBuildPatchInstallError::OutOfDiskSpace,
						PreExistingChunkDbs.Num() > 0 ? DiskSpaceErrorCodes::DuringInstallation : DiskSpaceErrorCodes::InitialSpaceCheck,
						0,
						BuildPatchServices::GetDiskSpaceMessage(Configuration.InstallDirectory, RequiredDiskSpace, DiskSpaceInfo.IsSet() ? DiskSpaceInfo->AvailableDiskSpace : 0));
					bChunkDbSuccess = false;
				}
				else
				{
					ConstructTimer.Start();
					// Database writer dependencies.
					TUniquePtr<IDownloadConnectionCount> DownloadConnectionCount(FDownloadConnectionCountFactory::Create(BuildConnectionCountConfig(), DownloadServiceStatistics.Get()));
					TUniquePtr<IChunkReferenceTracker> ChunkReferenceTracker(FChunkReferenceTrackerFactory::Create(OrderedRequiredChunks));
					TUniquePtr<IChunkEvictionPolicy> MemoryEvictionPolicy(FChunkEvictionPolicyFactory::Create(
						ChunkReferenceTracker.Get()));
					TUniquePtr<IMemoryChunkStoreStatistics> MemoryChunkStoreStatistics(FMemoryChunkStoreStatisticsFactory::Create(FileOperationTracker.Get()));
					TUniquePtr<IMemoryChunkStore> CloudChunkStore(FMemoryChunkStoreFactory::Create(
						ChunkStoreMemorySize,
						MemoryEvictionPolicy.Get(),
						nullptr,
						MemoryChunkStoreStatistics.Get(),
						ChunkReferenceTracker.Get()));
						FCloudSourceConfig CloudSourceConfig = BuildCloudSourceConfig();
					TUniquePtr<ICloudChunkSource> CloudChunkSource(FCloudChunkSourceFactory::Create(
							MoveTemp(CloudSourceConfig),
							Platform.Get(),
							CloudChunkStore.Get(),
							DownloadService.Get(),
							ChunkReferenceTracker.Get(),
							ChunkDataSerialization.Get(),
							MessagePump.Get(),
							InstallerError.Get(),
							DownloadConnectionCount.Get(),
							CloudChunkSourceStatistics.Get(),
							ManifestSet.Get(),
							InitialDownloadChunks));

					// Start database writer that saves all the chunks to the chunkdbs.
					FEvent* CompletionTrigger = FPlatformProcess::GetSynchEventFromPool(true);
					FChunkDbWriterConfig ChunkDbWriterConfig(ChunkDbFiles);
					ChunkDbWriterConfig.bUseTempFile = false;
					ChunkDbWriterConfig.HeaderUpdateFrequency = 5.0f;
					ChunkDbWriterConfig.bReserialise = false;
					ChunkDbWriterConfig.bDeleteFilesOnFailure = false;
					ChunkDbWriterConfig.bCallbackOnMainThread = false;
					TUniquePtr<IChunkDatabaseWriter> ChunkDatabaseWriter(FChunkDatabaseWriterFactory::Create(
						ChunkDbWriterConfig,
						CloudChunkSource.Get(),
						FileSystem.Get(),
						InstallerError.Get(),
						ChunkReferenceTracker.Get(),
						ChunkDataSerialization.Get(),
						FileConstructorStatistics.Get(),
						[this](float Progress) { BuildProgress.SetStateProgress(EBuildPatchState::Installing, Progress); },
						[&bChunkDbSuccess, CompletionTrigger](bool bInSuccess) { bChunkDbSuccess = bInSuccess; CompletionTrigger->Trigger(); }));

					// Register controllables.
					FScopedControllables ScopedControllables(&ThreadLock, Controllables, bIsPaused, bShouldAbort);
					ScopedControllables.Register(CloudChunkSource.Get());
					ScopedControllables.Register(ChunkDatabaseWriter.Get());

					// Win64 Static Analysis warn about lambda lifetime when assigned directly to member variable.
					// e.g: warning V1047: Lifetime of the lambda is greater than lifetime of the local variable 'CloudChunkSource' captured by reference.
					// Instead we set a local variable, and use that to set the TFunction. The member variable is properly reset a few lines below.
					auto DLRateLambda = [&DownloadConnectionCount, &CloudChunkSource](uint64 NewDownloadRateLimitBps)
						{
							CloudChunkSource->SetDownloadBytesPerSecondLimit(NewDownloadRateLimitBps);
							DownloadConnectionCount->SetConnectionScaling(NewDownloadRateLimitBps == 0);
						};

					// Register download rate function.
					{
						UE::TUniqueLock _(DownloadRateLimitLock);
						DownloadRateUpdatedFn = DLRateLambda;
					}

					// Wait for preload complete
					CompletionTrigger->Wait();
					FPlatformProcess::ReturnSynchEventToPool(CompletionTrigger);
					ConstructTimer.Stop();

					{
						UE::TUniqueLock _(DownloadRateLimitLock);
						DownloadRateUpdatedFn.Reset();
					}
				}

				// Setup file stats
				{
					FScopeLock Lock(&ThreadLock);
					BuildStats.NumFilesConstructed = FileConstructorStatistics->GetFilesConstructed();
					BuildStats.ProcessRequiredDiskSpace = RequiredDiskSpace;
					BuildStats.ProcessAvailableDiskSpace = DiskSpaceInfo.IsSet() ? DiskSpaceInfo->AvailableDiskSpace : 0;
				}

				// Process some final stats.
				SetFinalInstallStats();
			}

			if (!bChunkDbSuccess)
			{
				// This call will be ignored, if the database writer correctly set an error code. Otherwise we will set unknown so we can track that an error ocurred and see we need more info.
				InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::UnknownFail);
			}
			else
			{
				// Calculate if we want to try to reserve extra HDD space for the unlock process.
				TArray<FString> AllChunkDbs = ConfigHelpers::EnumeratePreloadChunkDbs(*DataStagingDir);
				uint64 ChunkDbSize = 0;
				for (const FString& ChunkDbFile : AllChunkDbs)
				{
					int64 FileSize = 0;
					FileSystem->GetFileSize(*ChunkDbFile, FileSize);
					ChunkDbSize += FileSize;
				}
				TSet<FString> ExpectedFiles;
				ManifestSet->GetExpectedFiles(ExpectedFiles);
				const uint64 InstallSize = ManifestSet->GetTotalNewFileSize(ExpectedFiles);
				if (InstallSize > ChunkDbSize)
				{
					const uint64 ReserveSize = InstallSize - ChunkDbSize;
					FileSystem->SetFileSize(*HddSpaceReservationFile, ReserveSize);
				}
			}
		}

		UE_LOGF(LogBuildPatchServices, Log, "Preload complete");
		const bool bReturnPreloadSuccess = !InstallerError->HasError();

		// Ensure all progress complete
		if (bReturnPreloadSuccess)
		{
			BuildProgress.SetStateProgress(EBuildPatchState::Initializing, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Resuming, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Downloading, 1.0f);
			BuildProgress.SetStateProgress(EBuildPatchState::Installing, 1.0f);
			InitializeTimer.Stop();
		}

		return bReturnPreloadSuccess;
	}

	bool FBuildPatchInstaller::RunPrerequisites()
	{
		TUniquePtr<IMachineConfig> MachineConfig(FMachineConfigFactory::Create(LocalMachineConfigFile, true));
		TUniquePtr<IPrerequisites> Prerequisites(FPrerequisitesFactory::Create(
			MachineConfig.Get(),
			InstallerAnalytics.Get(),
			InstallerError.Get(),
			FileSystem.Get(),
			Platform.Get()));

		return Prerequisites->RunPrereqs(ManifestSet.Get(), Configuration, InstallStagingDir, BuildProgress);
	}

	void FBuildPatchInstaller::SetFinalInstallStats()
	{
		FScopeLock Lock(&ThreadLock);
		BuildStats.NumChunksDownloaded = DownloadServiceStatistics->GetNumSuccessfulChunkDownloads();
		BuildStats.NumFailedDownloads = DownloadServiceStatistics->GetNumFailedChunkDownloads();
		BuildStats.NumBadDownloads = CloudChunkSourceStatistics->GetNumCorruptChunkDownloads();
		BuildStats.NumAbortedDownloads = CloudChunkSourceStatistics->GetNumAbortedChunkDownloads();
		BuildStats.OverallRequestSuccessRate = CloudChunkSourceStatistics->GetDownloadSuccessRate();
		BuildStats.NumChunksRecycled = InstallChunkSourceStatistics->GetNumSuccessfulChunkRecycles();
		BuildStats.NumChunksReadFromChunkDbs = ChunkDbChunkSourceStatistics->GetNumSuccessfulLoads();
		BuildStats.NumRecycleFailures = InstallChunkSourceStatistics->GetNumFailedChunkRecycles();
		BuildStats.NumChunkDbChunksFailed = ChunkDbChunkSourceStatistics->GetNumFailedLoads();
		TArray<float> HealthTimers = CloudChunkSourceStatistics->GetDownloadHealthTimers();
		BuildStats.ExcellentDownloadHealthTime = HealthTimers[(int32)EBuildPatchDownloadHealth::Excellent];
		BuildStats.GoodDownloadHealthTime = HealthTimers[(int32)EBuildPatchDownloadHealth::Good];
		BuildStats.OkDownloadHealthTime = HealthTimers[(int32)EBuildPatchDownloadHealth::OK];
		BuildStats.PoorDownloadHealthTime = HealthTimers[(int32)EBuildPatchDownloadHealth::Poor];
		BuildStats.DisconnectedDownloadHealthTime = HealthTimers[(int32)EBuildPatchDownloadHealth::Disconnected];
		BuildStats.ActiveRequestCountPeak = CloudChunkSourceStatistics->GetPeakRequestCount();
		BuildStats.ActiveRequestCountAvg = CloudChunkSourceStatistics->GetAvgRequestCount();
		const FString UserId = FPlatformMisc::GetEpicAccountId();
		const FString Variant = Experiments::GetVariant(Experiments::DownloadSpeed(), UserId);
		BuildStats.ExperimentName = FString(Experiments::DownloadSpeed().ExperimentName);
		BuildStats.ExerimentVariantName = *Variant;
		BuildStats.TotalDecryptTime = CloudChunkSourceStatistics->GetTotalDecryptTime();
		BuildStats.CloudDirectoryStats = CloudChunkSourceStatistics->GetCloudDirectoryStats();
	}

	class FParallelCleanUpDirectoryEnumerator : public IPlatformFile::FDirectoryVisitor
	{
	public:
		FParallelCleanUpDirectoryEnumerator(const FString& InRootDirectory) : RootDirectory(InRootDirectory) {}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			if (bIsDirectory)
			{
				FString Directory = FilenameOrDirectory;
				DirectoryFutures.Enqueue(Async(EAsyncExecution::ThreadPool, [this, Directory = MoveTemp(Directory)]()
					{
						struct FEmptyDirVisitor : public IPlatformFile::FDirectoryVisitor
						{
							// Once visited means either a folder or a file exists inside of the dir
							virtual bool Visit(const TCHAR*, bool) override { return false; }
						};
						static FEmptyDirVisitor EmptyDirVisitor;
						if (IFileManager::Get().IterateDirectory(*Directory, EmptyDirVisitor))
						{
							return Directory;
						}
						IFileManager::Get().IterateDirectory(*Directory, *this);
						return FString();
					}));
			}
			return true;
		}

		void GetCleanUpFolders(TSet<FString>& Results)
		{
			TFuture<FString> Future;
			while (DirectoryFutures.Dequeue(Future))
			{
				FString Dir = Future.Get();
				if (!Dir.IsEmpty())
				{
					Results.Add(MoveTemp(Dir));
				}
			}
		}

	private:

		static bool IsSlashOrBackslash(TCHAR C)
		{
			return C == TEXT('/') || C == TEXT('\\');
		}

	private:
		const FString& RootDirectory;
		TQueue<TFuture<FString>, EQueueMode::Mpsc> DirectoryFutures;
	};

	void CleanupEmptyDirectoriesImpl(const FString& RootDirectory, TSet<FString> HandledFolders = {})
	{
		TSet<FString> CleanUpFolders;
		FParallelCleanUpDirectoryEnumerator DirectoryEnumerator(RootDirectory);
		IFileManager::Get().IterateDirectory(*RootDirectory, DirectoryEnumerator);
		DirectoryEnumerator.GetCleanUpFolders(CleanUpFolders);
		// to avoid recursion if we cannot delete a folder
		CleanUpFolders = CleanUpFolders.Difference(HandledFolders);

		for (const FString& FolderToDelete : CleanUpFolders)
		{
#if PLATFORM_MAC
			// On Mac we need to delete the .DS_Store file, but FindFiles() skips .DS_Store files.
			IFileManager::Get().Delete(*(FolderToDelete / TEXT(".DS_Store")), false, true);
#endif

			bool bDeleteSuccess = IFileManager::Get().DeleteDirectory(*FolderToDelete, false, true);
			const uint32 LastError = FPlatformMisc::GetLastError();
			UE_LOGF(LogBuildPatchServices, Log, "Deleted Empty Folder (%u,%u) %ls", bDeleteSuccess ? 1 : 0, LastError, *FolderToDelete);
			HandledFolders.Add(FolderToDelete);
		}

		if (CleanUpFolders.Num() > 0)
		{
			CleanupEmptyDirectoriesImpl(RootDirectory, MoveTemp(HandledFolders));
		}
	}

	void FBuildPatchInstaller::CleanupEmptyDirectories(const FString& RootDirectory)
	{
		CleanupEmptyDirectoriesImpl(RootDirectory);
	}

	void FBuildPatchInstaller::FilterToExistingFiles(const FString& RootDirectory, const TMap<FString, FString>& PerFileSubdirectories, TSet<FString>& Files, bool bResolveInstallationFileName)
	{
		for (auto FilesIt = Files.CreateIterator(); FilesIt; ++FilesIt)
		{
			const FString DiskFileName = bResolveInstallationFileName ? 
				FBuildPatchUtils::ResolveInstallationFileName(Configuration.InstallDirectory, PerFileSubdirectories, *FilesIt)
			: (RootDirectory / *FilesIt);
			if (!FileSystem->FileExists(*DiskFileName))
			{
				FilesIt.RemoveCurrent();
			}
		}
	}

	bool FBuildPatchInstaller::RemoveFileWithRetries(const FString& FullFilename, uint32& ErrorCode)
	{
		int32 DeleteRetries = ConfigHelpers::NumFileMoveRetries();
		bool bDeleteSuccess = false;
		ErrorCode = 0;
		while (DeleteRetries >= 0 && !bDeleteSuccess)
		{
			bDeleteSuccess = FileSystem->DeleteFile(*FullFilename);
			ErrorCode = FPlatformMisc::GetLastError();
			if (!bDeleteSuccess && (--DeleteRetries) >= 0)
			{
				UE_LOGF(LogBuildPatchServices, Warning, "Failed to delete file %ls (%d), retying after 0.5 sec..", *FullFilename, ErrorCode);
				FPlatformProcess::Sleep(0.5f);
			}
		}
		if (!bDeleteSuccess)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Failed to delete file %ls (%d), installer will exit.", *FullFilename, ErrorCode);
		}
		return bDeleteSuccess;
	}

	bool FBuildPatchInstaller::RelocateFileWithRetries(const FString& ToFullFilename, const FString& FromFullFilename, uint32& RenameErrorCode, uint32& CopyErrorCode)
	{
		int32 RelocateRetries = ConfigHelpers::NumFileMoveRetries();
		bool bRelocateSuccess = false;
		RenameErrorCode = 0;
		CopyErrorCode = 0;
		while (RelocateRetries >= 0 && !bRelocateSuccess)
		{
			const bool bIsDirectory = FileSystem->DirectoryExists(*ToFullFilename);
			if (bIsDirectory)
			{
				const bool bDeleteDirSuccess = FileSystem->DeleteDirectoryRecursively(*ToFullFilename);
				if (!bDeleteDirSuccess)
				{
					const uint32 ErrorCode = FPlatformMisc::GetLastError();
					UE_LOGF(LogBuildPatchServices, Warning, "Failed to delete directory %ls (%d), trying move anyway..", *ToFullFilename, ErrorCode);
				}
			}
			bRelocateSuccess = FileSystem->MoveFile(*ToFullFilename, *FromFullFilename);
			RenameErrorCode = FPlatformMisc::GetLastError();
			if (!bRelocateSuccess)
			{
				UE_LOGF(LogBuildPatchServices, Warning, "Failed to move file %ls (%d), trying copy..", *FromFullFilename, RenameErrorCode);
				bRelocateSuccess = FileSystem->CopyFile(*ToFullFilename, *FromFullFilename);
				CopyErrorCode = FPlatformMisc::GetLastError();
				if (bRelocateSuccess)
				{
					FileSystem->DeleteFile(*FromFullFilename);
				}
				else if ((--RelocateRetries) >= 0)
				{
					UE_LOGF(LogBuildPatchServices, Warning, "Failed to copy too (%d), retying after 0.5 sec..", CopyErrorCode);
					FPlatformProcess::Sleep(0.5f);
				}
			}
		}
		if (!bRelocateSuccess)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Failed to relocated file %ls (%d-%d), installer will exit.", *FromFullFilename, RenameErrorCode, CopyErrorCode);
		}
		return bRelocateSuccess;
	}

	bool FBuildPatchInstaller::CopyFileWithRetries(const FString& ToFullFilename, const FString& FromFullFilename, uint32& CopyErrorCode)
	{
		int32 RelocateRetries = ConfigHelpers::NumFileMoveRetries();
		bool bRelocateSuccess = false;
		CopyErrorCode = 0;
		while (RelocateRetries >= 0 && !bRelocateSuccess)
		{
			bRelocateSuccess = FileSystem->CopyFile(*ToFullFilename, *FromFullFilename);
			CopyErrorCode = FPlatformMisc::GetLastError();
			if (!bRelocateSuccess && (--RelocateRetries) >= 0)
			{
				UE_LOGF(LogBuildPatchServices, Warning, "Failed to copy file %ls (%d), retying after 0.5 sec..", *FromFullFilename, CopyErrorCode);
				FPlatformProcess::Sleep(0.5f);
			}
		}
		if (!bRelocateSuccess)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Failed to copy file %ls (%d), installer will exit.", *FromFullFilename, CopyErrorCode);
		}
		return bRelocateSuccess;
	}

	bool FBuildPatchInstaller::RunCustomUninstallAction()
	{
		BuildProgress.SetStateProgress(EBuildPatchState::RunningCustomUninstallAction, 0.0f);

		const bool bSkipThisStep = Configuration.InstallMode == EInstallMode::PrereqOnly
			|| InstallerError->HasError()
			|| InstallerError->IsCancelled();

		if (bSkipThisStep)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Skipping custom uninstall action");
			BuildProgress.SetStateProgress(EBuildPatchState::RunningCustomUninstallAction, 1.0f);
			return true;
		}

		UninstallActionTimer.Start();
		TUniquePtr<ICustomUninstallAction> CustomUninstallAction(FCustomUninstallActionFactory::Create(InstallerError.Get(), FileSystem.Get(), Platform.Get()));
		bool bRunSuccessfull = CustomUninstallAction->RunAction(ManifestSet.Get(), Configuration, InstallStagingDir, BuildProgress);
		UninstallActionTimer.Stop();

		return bRunSuccessfull;
	}

	bool FBuildPatchInstaller::RunBackupAndMove()
	{
		BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, 0.0f);
		// We skip this step if performing stage only
		bool bRunBackupAndMoveSuccess = true;
		if (Configuration.InstallMode == EInstallMode::StageFiles
			|| (bFirstInstallIteration && Configuration.InstallMode == EInstallMode::PrereqOnly)
			|| ManifestSet->IsExclusivelyVerifying()
			|| bAreResultsEncrypted
			|| Configuration.bInstallToMemory)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Skipping backup and stage relocation");
			BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, 1.0f);
		}
		else
		{
			MoveFromStageTimer.Start();
			UE_LOGF(LogBuildPatchServices, Log, "Running backup and stage relocation");
			// If there's no error, move all complete files
			bRunBackupAndMoveSuccess = InstallerError->HasError() == false;
			if (bRunBackupAndMoveSuccess)
			{
				// Get list of all expected files
				TSet<FString> FilesToRelocate;
				ManifestSet->GetExpectedFiles(FilesToRelocate);

				// Filter symlinks from files to relocate
				TSet<FString> SymlinksToCreate;
				for (auto FilesIt = FilesToRelocate.CreateIterator(); FilesIt; ++FilesIt)
				{
					const FFileManifest* FileManifest = ManifestSet->GetNewFileManifest(**FilesIt);
					if (FileManifest != nullptr && !FileManifest->SymlinkTarget.IsEmpty())
					{
						SymlinksToCreate.Add(*FilesIt);
						FilesIt.RemoveCurrent();
					}
				}

				TMap<FSHAHash, uint32> FileHashToCountMap;
				if (!Configuration.bStageWithRawFilenames)
				{
					// Filter to existing files with SHA1 hash value as filename
					for (auto FilesIt = FilesToRelocate.CreateIterator(); FilesIt; ++FilesIt)
					{
						if (const FFileManifest* FileManifest = ManifestSet->GetNewFileManifest(**FilesIt))
						{
							FString FileToRelocateHashStringValue = GetSHA1Filename(*FileManifest);

							if (!FileSystem->FileExists(*(InstallStagingDir / *FileToRelocateHashStringValue)))
							{
								FilesIt.RemoveCurrent();
								continue;
							}

							// We need to map each hash to its file count for proper relocation it
							FileHashToCountMap.FindOrAdd(FileManifest->SHA1Hash)++;
						}
					}
				}
				else
				{
					FilterToExistingFiles(InstallStagingDir, NewPerFileSubdirectories, FilesToRelocate, false);
				}
				FilesToRelocate.Sort(TLess<FString>());

				// First handle files that should be removed for patching
				TSet<FString> FilesToRemove;
				ManifestSet->GetRemovableFiles(FilesToRemove);
				// Note, this is using the OLD subdirectory mapping since we are concerned with files that may not exist in the new manifest
				FilterToExistingFiles(Configuration.InstallDirectory, OldPerFileSubdirectories, FilesToRemove, true);
				FilesToRemove.Sort(TLess<FString>());

				// Counters for progress tracking.
				float NumOperationsFloat = FilesToRelocate.Num() + FilesToRemove.Num() + SymlinksToCreate.Num();
				float PerformedOperationsFloat = 0;

				// Add to build stats
				ThreadLock.Lock();
				BuildStats.NumFilesToRemove = FilesToRemove.Num();
				ThreadLock.Unlock();

				if (NumOperationsFloat > 0.0f)
				{
					UE_LOGF(LogBuildPatchServices, Log, "Create MM");
					TUniquePtr<FArchive> MoveMarkerFile = FileSystem->CreateFileWriter(*PreviousMoveMarker, EWriteFlags::EvenIfReadOnly);
				}

				// Perform all of the removals
				for (auto FileToRemove = FilesToRemove.CreateConstIterator(); FileToRemove && !InstallerError->HasError(); ++FileToRemove)
				{
					BackupFileIfNecessary(*FileToRemove, OldPerFileSubdirectories);
					const FString FullFilename = FBuildPatchUtils::ResolveInstallationFileName(Configuration.InstallDirectory, OldPerFileSubdirectories, *FileToRemove);
					uint32 ErrorCode = 0;
					bool bDeleteSuccess = RemoveFileWithRetries(FullFilename, ErrorCode);
					if (bDeleteSuccess)
					{
						MessagePump->SendMessage(FInstallationFileAction{ FInstallationFileAction::EType::Removed, *FileToRemove });
					}
					else
					{
						InstallerError->SetError(EBuildPatchInstallError::MoveFileToInstall, MoveErrorCodes::DeleteOldFileFailed, ErrorCode);
						bRunBackupAndMoveSuccess = false;
						break;
					}
					BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, PerformedOperationsFloat / NumOperationsFloat);
					PerformedOperationsFloat += 1.0f;
				}

				// Perform all of the relocations
				for (auto FileToRelocate = FilesToRelocate.CreateConstIterator(); FileToRelocate && !InstallerError->HasError(); ++FileToRelocate)
				{
					const FString DestFilename = FBuildPatchUtils::ResolveInstallationFileName(Configuration.InstallDirectory, NewPerFileSubdirectories, *FileToRelocate);
					const bool bOldDirectoryExists = FileSystem->DirectoryExists(*DestFilename);
					const bool bOldFileExists = FileSystem->FileExists(*DestFilename);
					if (bOldDirectoryExists)
					{
						CleanupEmptyDirectories(DestFilename);
					}
					if (bOldFileExists)
					{
						BackupFileIfNecessary(*FileToRelocate, NewPerFileSubdirectories);
						uint32 ErrorCode = 0;
						bool bDeleteSuccess = RemoveFileWithRetries(DestFilename, ErrorCode);
						if (!bDeleteSuccess)
						{
							InstallerError->SetError(EBuildPatchInstallError::MoveFileToInstall, MoveErrorCodes::DeletePrevFileFailed, ErrorCode);
							bRunBackupAndMoveSuccess = false;
							break;
						}
					}
					uint32 RenameErrorCode = 0;
					uint32 CopyErrorCode = 0;

					bool bRelocateSuccess = false;

					if (!Configuration.bStageWithRawFilenames)
					{
						if (const FFileManifest* FileManifest = ManifestSet->GetNewFileManifest(*FileToRelocate))
						{
							FString FileHashStringValue = GetSHA1Filename(*FileManifest);

							const FString SrcFilename = InstallStagingDir / *FileHashStringValue;

							// several files can have same SHA1 hash
							// so we need to copy this files Count times to dest dir
							// and at the end move it to dest dir
							uint32* Count = FileHashToCountMap.Find(FileManifest->SHA1Hash);
							if (*Count == 1)
							{
								bRelocateSuccess = RelocateFileWithRetries(DestFilename, SrcFilename, RenameErrorCode, CopyErrorCode);
							}
							else
							{
								bRelocateSuccess = CopyFileWithRetries(DestFilename, SrcFilename, CopyErrorCode);
								*Count = *Count - 1;
							}
						}
						else
						{
							UE_LOGF(LogBuildPatchServices, Error, "Unable to find a manifest for a file %ls", **FileToRelocate);
						}
					}
					else
					{
						const FString SrcFilename = InstallStagingDir / *FileToRelocate;
						bRelocateSuccess = RelocateFileWithRetries(DestFilename, SrcFilename, RenameErrorCode, CopyErrorCode);
					}

					if (bRelocateSuccess)
					{
						FilesInstalled.Add(*FileToRelocate);
						FileOperationTracker->OnFileStateUpdate(*FileToRelocate, EFileOperationState::Installed);
						FInstallationFileAction::EType Action = OldFilesRemovedBySystem.Contains(*FileToRelocate) || bOldFileExists ? FInstallationFileAction::EType::Updated : FInstallationFileAction::EType::Added;
						MessagePump->SendMessage(FInstallationFileAction{ Action, *FileToRelocate });
					}
					else
					{
						FString ErrorString = FString::Printf(TEXT("%s-%u-%u"), MoveErrorCodes::StageToInstall, RenameErrorCode, CopyErrorCode);
						InstallerError->SetError(EBuildPatchInstallError::MoveFileToInstall, (RenameErrorCode != 0 || CopyErrorCode != 0) ? *ErrorString : MoveErrorCodes::StageToInstall);
						bRunBackupAndMoveSuccess = false;
						break;
					}
					BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, PerformedOperationsFloat / NumOperationsFloat);
					PerformedOperationsFloat += 1.0f;
				}

				// Create all symlinks
				for (auto SymlinkToCreate = SymlinksToCreate.CreateConstIterator(); SymlinkToCreate && !InstallerError->HasError(); ++SymlinkToCreate)
				{
					const FString SymlinkFilename = Configuration.InstallDirectory / *SymlinkToCreate;

					const bool bOldFileExists = FileSystem->FileExists(*SymlinkFilename);
					if (bOldFileExists)
					{
						uint32 ErrorCode = 0;
						bool bDeleteSuccess = RemoveFileWithRetries(SymlinkFilename, ErrorCode);
						if (!bDeleteSuccess)
						{
							InstallerError->SetError(EBuildPatchInstallError::MoveFileToInstall, MoveErrorCodes::DeletePrevFileFailed, ErrorCode);
							bRunBackupAndMoveSuccess = false;
							break;
						}
					}

					bool bCreateSuccess = false;
					uint32 ErrorCode = 0;

					const FFileManifest* FileManifest = ManifestSet->GetNewFileManifest(*SymlinkToCreate);
					FileSystem->MakeDirectory(*FPaths::GetPath(SymlinkFilename));

#if PLATFORM_MAC || PLATFORM_LINUX
					unlink(TCHAR_TO_UTF8(*SymlinkFilename));
					bCreateSuccess = symlink(TCHAR_TO_UTF8(*FileManifest->SymlinkTarget), TCHAR_TO_UTF8(*SymlinkFilename)) == 0;
					ErrorCode = errno;
#endif

					if (!bCreateSuccess)
					{
						// Report if we could not create the symlink.
						InstallerError->SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::SymlinkCreateFail, ErrorCode);
						bRunBackupAndMoveSuccess = false;
						break;
					}

					FilesInstalled.Add(*SymlinkToCreate);
					FileOperationTracker->OnFileStateUpdate(*SymlinkToCreate, EFileOperationState::Installed);
					FInstallationFileAction::EType Action = OldFilesRemovedBySystem.Contains(*SymlinkToCreate) || bOldFileExists ? FInstallationFileAction::EType::Updated : FInstallationFileAction::EType::Added;
					MessagePump->SendMessage(FInstallationFileAction{ Action, *SymlinkToCreate });

					BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, PerformedOperationsFloat / NumOperationsFloat);
					PerformedOperationsFloat += 1.0f;
				}

				bRunBackupAndMoveSuccess = bRunBackupAndMoveSuccess && (InstallerError->HasError() == false);
				if (bRunBackupAndMoveSuccess)
				{
					BuildProgress.SetStateProgress(EBuildPatchState::MovingToInstall, 1.0f);
				}
			}
			UE_LOGF(LogBuildPatchServices, Log, "Relocation complete %d", bRunBackupAndMoveSuccess ? 1 : 0);
			MoveFromStageTimer.Stop();
		}
		return bRunBackupAndMoveSuccess;
	}

	bool FBuildPatchInstaller::RunFileAttributes()
	{
		if (ManifestSet->IsExclusivelyVerifying() || bAreResultsEncrypted || Configuration.bInstallToMemory)
		{
			return true;
		}

		// Only provide stage directory if stage-only mode
		FString EmptyString;
		const FString& OptionalStageDirectory = Configuration.InstallMode == EInstallMode::StageFiles ? InstallStagingDir : EmptyString;

		// Construct the attributes class
		FileAttributesTimer.Start();
		TUniquePtr<IFileAttribution> Attributes(FFileAttributionFactory::Create(FileSystem.Get(), ManifestSet.Get(), FilesToConstruct, Configuration.InstallDirectory, OptionalStageDirectory, &BuildProgress, NewPerFileSubdirectories));
		FScopedControllables ScopedControllables(&ThreadLock, Controllables, bIsPaused, bShouldAbort);
		ScopedControllables.Register(Attributes.Get());
		Attributes->ApplyAttributes();
		FileAttributesTimer.Stop();

		// We don't fail on this step currently
		return true;
	}

	bool FBuildPatchInstaller::RunVerification(TArray< FString >& CorruptFiles)
	{
		if (Configuration.bInstallToMemory)
		{
			// We hash all the files during construction and can only install (not repair) so this
			// doesn't do anything other than rehash files in memory.
			return true;
		}

		// If we are storing encrypted data, we can't verify. This will be done by a later 'unlocking' stage.
		if (bAreResultsEncrypted)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Verify stage skipping due to preloading encrypted data.");
			BuildProgress.SetStateProgress(EBuildPatchState::BuildVerification, 1.0f);
			return true;
		}

		// Make sure this function can never be parallelized
		static FCriticalSection SingletonFunctionLockCS;
		const bool bShouldLock = !Configuration.bAllowConcurrentExecution;
		if (bShouldLock)
		{
			SingletonFunctionLockCS.Lock();
		}

		VerifyTimer.Start();
		BuildProgress.SetStateProgress(EBuildPatchState::BuildVerification, 0.0f);

		// Verify the installation
		UE_LOGF(LogBuildPatchServices, Log, "Verifying install");
		CorruptFiles.Empty();

		// Verify the build
		EVerifyResult VerifyResult = Verifier->Verify(CorruptFiles);
		bool bVerifySuccessful = VerifyResult == EVerifyResult::Success;
		if (!bVerifySuccessful)
		{
			if (VerifyResult == EVerifyResult::Aborted)
			{
				UE_LOGF(LogBuildPatchServices, Error, "User aborted verification");
				InstallerError->SetError(EBuildPatchInstallError::UserCanceled, UserCancelErrorCodes::UserRequested);
			}
			else
			{
				UE_LOGF(LogBuildPatchServices, Error, "Build verification failed on %u file(s)", CorruptFiles.Num());
				InstallerError->SetError(EBuildPatchInstallError::BuildVerifyFail, InstallerHelpers::GetVerifyErrorCode(VerifyResult));
			}
		}
		TMap<EVerifyError, int32> VerifyErrorCounts = VerifierStatistics->GetVerifyErrorCounts();
		for (const TPair<EVerifyError, int32>& VerifyErrorCount : VerifyErrorCounts)
		{
			const int32 CachedCount = CachedVerifyErrorCounts.FindRef(VerifyErrorCount.Key);
			if (CachedCount < VerifyErrorCount.Value)
			{
				InstallerHelpers::LogAdditionalVerifyErrors(VerifyErrorCount.Key, VerifyErrorCount.Value - CachedCount);
			}
		}
		CachedVerifyErrorCounts = MoveTemp(VerifyErrorCounts);

		BuildProgress.SetStateProgress(EBuildPatchState::BuildVerification, 1.0f);

		// Delete/Backup any incorrect files if failure was not cancellation
		if (!InstallerError->IsCancelled())
		{
			for (const FString& CorruptFile : CorruptFiles)
			{
				// Getting staged file name, it can be different for cases when SHA1 are used instead
				FString StagedFilenameToUse;
				if (!Configuration.bStageWithRawFilenames)
				{
					const FFileManifest* FileManifest = ManifestSet->GetNewFileManifest(CorruptFile);
					if (FileManifest == nullptr)
					{
						UE_LOGF(LogBuildPatchServices, Error, "Can't get manifest for file %ls", *CorruptFile);
						continue;
					}

					StagedFilenameToUse = GetSHA1Filename(*FileManifest);
				}
				else
				{
					StagedFilenameToUse = CorruptFile;
				}

				// Deleting files in a staged directory
				FString StagingPathToUse = InstallStagingDir / StagedFilenameToUse;
				if (!FileSystem->DeleteFile(*StagingPathToUse))
				{
					uint32 ErrorCode = FPlatformMisc::GetLastError();
					UE_LOGF(LogBuildPatchServices, Error, "Failed to delete corrupted file %ls with error code %d", *StagingPathToUse, ErrorCode);
					InstallerError->SetError(EBuildPatchInstallError::BuildVerifyFail, VerifyErrorCodes::DeleteCorruptedFileFailed, ErrorCode);
					bVerifySuccessful = false;
				}

				// Deleting files in the destination install directory when non-stagefiles mode is used
				if (Configuration.InstallMode != EInstallMode::StageFiles)
				{
					FString InstalledFile = FBuildPatchUtils::ResolveInstallationFileName(Configuration.InstallDirectory, NewPerFileSubdirectories, CorruptFile);
					if (FileSystem->FileExists(*InstalledFile))
					{
						BackupFileIfNecessary(CorruptFile, NewPerFileSubdirectories, true);
						FileSystem->DeleteFile(*InstalledFile);
						OldFilesRemovedBySystem.Add(CorruptFile);
					}
				}
			}
		}

		UE_LOGF(LogBuildPatchServices, Log, "Verify stage complete %d", bVerifySuccessful ? 1 : 0);

		VerifyTimer.Stop();
		if (bShouldLock)
		{
			SingletonFunctionLockCS.Unlock();
		}
		return bVerifySuccessful;
	}

	bool FBuildPatchInstaller::BackupFileIfNecessary(const FString& Filename, const TMap<FString, FString>& PerFileSubdirectories, bool bDiscoveredByVerification /*= false */)
	{
		const FString InstalledFilename = FBuildPatchUtils::ResolveInstallationFileName(Configuration.InstallDirectory, PerFileSubdirectories, Filename);
		const FString BackupFilename = Configuration.BackupDirectory / Filename;
		const bool bBackupOriginals = !Configuration.BackupDirectory.IsEmpty();
		// Skip if not doing backups
		if (!bBackupOriginals)
		{
			return true;
		}
		// Skip if no file to backup
		const bool bInstalledFileExists = FPlatformFileManager::Get().GetPlatformFile().FileExists(*InstalledFilename);
		if (!bInstalledFileExists)
		{
			return true;
		}
		// Skip if already backed up
		const bool bAlreadyBackedUp = FPlatformFileManager::Get().GetPlatformFile().FileExists(*BackupFilename);
		if (bAlreadyBackedUp)
		{
			return true;
		}
		// Skip if the target file was already copied to the installation
		const bool bAlreadyInstalled = FilesInstalled.Contains(Filename);
		if (bAlreadyInstalled)
		{
			return true;
		}
		// If discovered by verification, but the patching system did not touch the file, we know it must be backed up.
		// If patching system touched the file it would already have been backed up
		if (bDiscoveredByVerification && !OutdatedFiles.Contains(Filename))
		{
			return IFileManager::Get().Move(*BackupFilename, *InstalledFilename, true, true, true);
		}
		bool bUserEditedFile = bDiscoveredByVerification;
		const bool bCheckFileChanges = !bDiscoveredByVerification;
		if (bCheckFileChanges)
		{
			const FFileManifest* OldFileManifest = ManifestSet->GetCurrentFileManifest(Filename);
			const FFileManifest* NewFileManifest = ManifestSet->GetNewFileManifest(Filename);
			const int64 InstalledFilesize = IFileManager::Get().FileSize(*InstalledFilename);
			const int64 OriginalFileSize = OldFileManifest ? OldFileManifest->FileSize : INDEX_NONE;
			const int64 NewFileSize = NewFileManifest ? NewFileManifest->FileSize : INDEX_NONE;
			const FSHAHash HashZero;
			const FSHAHash& HashOld = OldFileManifest ? OldFileManifest->SHA1Hash : HashZero;
			const FSHAHash& HashNew = NewFileManifest ? NewFileManifest->SHA1Hash : HashZero;
			const bool bFileSizeDiffers = OriginalFileSize != InstalledFilesize && NewFileSize != InstalledFilesize;
			bUserEditedFile = bFileSizeDiffers || FBuildPatchUtils::VerifyFile(FileSystem.Get(), InstalledFilename, HashOld, HashNew) == 0;
		}
		// Finally, use the above logic to determine if we must do the backup
		const bool bNeedBackup = bUserEditedFile;
		bool bBackupSuccess = true;
		if (bNeedBackup)
		{
			UE_LOGF(LogBuildPatchServices, Log, "Backing up %ls", *Filename);
			bBackupSuccess = IFileManager::Get().Move(*BackupFilename, *InstalledFilename, true, true, true);
		}
		return bBackupSuccess;
	}

	void FBuildPatchInstaller::CleanupThread()
	{
		Configuration.SharedContext->ReleaseThread(Thread);
		Thread = nullptr;
	}

	double FBuildPatchInstaller::GetDownloadSpeed() const
	{
		const bool bThrottlingEnabled = DownloadRateLimitBps != 0;
		return DownloadSpeedRecorder->GetAverageSpeed(ConfigHelpers::DownloadSpeedAverageTime(bThrottlingEnabled), bThrottlingEnabled);
	}

	int64 FBuildPatchInstaller::GetTotalDownloadRequired() const
	{
		return CloudChunkSourceStatistics->GetRequiredDownloadSize() + PreviousTotalDownloadRequired;
	}

	int64 FBuildPatchInstaller::GetTotalDownloaded() const
	{
		return DownloadServiceStatistics->GetBytesDownloaded();
	}

	bool FBuildPatchInstaller::IsComplete() const
	{
		return !bIsRunning && bIsInited;
	}

	bool FBuildPatchInstaller::IsCanceled() const
	{
		return InstallerError->IsCancelled();
	}

	bool FBuildPatchInstaller::IsPaused() const
	{
		FScopeLock Lock(&ThreadLock);
		return bIsPaused;
	}

	bool FBuildPatchInstaller::IsResumable() const
	{
		FScopeLock Lock(&ThreadLock);
		if (BuildStats.FailureType == EBuildPatchInstallError::PathLengthExceeded)
		{
			return false;
		}
		return !BuildStats.ProcessSuccess;
	}

	bool FBuildPatchInstaller::IsUpdate() const
	{
		for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
		{
			if (InstallerAction.IsUpdate())
			{
				return true;
			}
		}
		return false;
	}

	bool FBuildPatchInstaller::IsRepair() const
	{
		for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
		{
			if (InstallerAction.IsRepair())
			{
				return true;
			}
		}
		return false;
	}

	bool FBuildPatchInstaller::IsVerifyOnly() const
	{
		for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
		{
			if (InstallerAction.IsVerifyOnly())
			{
				return true;
			}
		}
		return false;
	}

	bool FBuildPatchInstaller::IsUninstall() const
	{
		for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
		{
			if (!InstallerAction.IsUninstall())
			{
				return false;
			}
		}
		return true;
	}

	bool FBuildPatchInstaller::CompletedSuccessfully() const
	{
		return IsComplete() && bSuccess;
	}

	bool FBuildPatchInstaller::HasError() const
	{
		FScopeLock Lock(&ThreadLock);
		if (BuildStats.FailureType == EBuildPatchInstallError::UserCanceled)
		{
			return false;
		}
		return !BuildStats.ProcessSuccess;
	}

	EBuildPatchInstallError FBuildPatchInstaller::GetErrorType() const
	{
		FScopeLock Lock(&ThreadLock);
		return BuildStats.FailureType;
	}

	FString FBuildPatchInstaller::GetErrorCode() const
	{
		FScopeLock Lock(&ThreadLock);
		return BuildStats.ErrorCode;
	}

	TMap<FString, TArray64<uint8>>& FBuildPatchInstaller::GetFilesInstalledToMemory()
	{
		return FilesInstalledToMemory;
	}

	void FBuildPatchInstaller::SetPerFileSubdirectories(TMap<FString, FString>&& InOldPerFileSubdirectories, TMap<FString, FString>&& InNewPerFileSubdirectories)
	{
		NewPerFileSubdirectories = MoveTemp(InNewPerFileSubdirectories);
		OldPerFileSubdirectories = MoveTemp(InOldPerFileSubdirectories);
	}

	//@todo this is deprecated and shouldn't be used anymore [6/4/2014 justin.sargent]
	FText FBuildPatchInstaller::GetPercentageText() const
	{
		static const FText PleaseWait = NSLOCTEXT("BuildPatchInstaller", "BuildPatchInstaller_GenericProgress", "Please Wait");

		FScopeLock Lock(&ThreadLock);

		float Progress = GetUpdateProgress() * 100.0f;
		if (Progress <= 0.0f)
		{
			return PleaseWait;
		}

		FNumberFormattingOptions PercentFormattingOptions;
		PercentFormattingOptions.MaximumFractionalDigits = 0;
		PercentFormattingOptions.MinimumFractionalDigits = 0;

		return FText::AsPercent(GetUpdateProgress(), &PercentFormattingOptions);
	}

	//@todo this is deprecated and shouldn't be used anymore [6/4/2014 justin.sargent]
	FText FBuildPatchInstaller::GetDownloadSpeedText() const
	{
		static const FText DownloadSpeedFormat = NSLOCTEXT("BuildPatchInstaller", "BuildPatchInstaller_DownloadSpeedFormat", "{Current} / {Total} ({Speed}/sec)");

		FScopeLock Lock(&ThreadLock);
		FText SpeedDisplayedText;
		double DownloadSpeed = GetDownloadSpeed();
		double InitialDownloadSize = GetTotalDownloadRequired();
		double TotalDownloaded = GetTotalDownloaded();
		if (DownloadSpeed >= 0)
		{
			FNumberFormattingOptions FormattingOptions;
			FormattingOptions.MaximumFractionalDigits = 1;
			FormattingOptions.MinimumFractionalDigits = 1;

			FFormatNamedArguments Args;
			Args.Add(TEXT("Speed"), FText::AsMemory(DownloadSpeed, &FormattingOptions));
			Args.Add(TEXT("Total"), FText::AsMemory(InitialDownloadSize, &FormattingOptions));
			Args.Add(TEXT("Current"), FText::AsMemory(TotalDownloaded, &FormattingOptions));

			return FText::Format(DownloadSpeedFormat, Args);
		}

		return FText();
	}

	EBuildPatchState FBuildPatchInstaller::GetState() const
	{
		return BuildProgress.GetState();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FText FBuildPatchInstaller::GetStatusText() const
	{
		return BuildPatchServices::StateToText(GetState());
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

		float FBuildPatchInstaller::GetUpdateProgress() const
	{
		return BuildProgress.GetProgress();
	}

	FBuildInstallStats FBuildPatchInstaller::GetBuildStatistics() const
	{
		FScopeLock Lock(&ThreadLock);
		FBuildInstallStats CurrentStats(BuildStats);

		if (!IsComplete())
		{
			CurrentStats.AverageDownloadSpeed = DownloadSpeedRecorder->GetAverageSpeed(TNumericLimits<float>::Max());
			CurrentStats.PeakDownloadSpeed = DownloadSpeedRecorder->GetPeakSpeed();
			CurrentStats.NumChunksDownloaded = DownloadServiceStatistics->GetNumSuccessfulChunkDownloads();
			CurrentStats.NumFailedDownloads = DownloadServiceStatistics->GetNumFailedChunkDownloads();
		}

		return CurrentStats;
	}

	EBuildPatchDownloadHealth FBuildPatchInstaller::GetDownloadHealth() const
	{
		return CloudChunkSourceStatistics->GetDownloadHealth();
	}

	FText FBuildPatchInstaller::GetErrorText() const
	{
		return InstallerError->GetErrorText();
	}

	void FBuildPatchInstaller::CancelInstall()
	{
		InstallerError->SetError(EBuildPatchInstallError::UserCanceled, UserCancelErrorCodes::UserRequested);

		// Make sure we are not paused
		if (IsPaused())
		{
			TogglePauseInstall();
		}

		// Abort all controllable classes
		ThreadLock.Lock();
		bShouldAbort = true;
		for (IControllable* Controllable : Controllables)
		{
			Controllable->Abort();
		}
		ThreadLock.Unlock();
	}

	bool FBuildPatchInstaller::TogglePauseInstall()
	{
		FScopeLock Lock(&ThreadLock);
		// If there is an error, we don't allow pausing.
		const bool bShouldBePaused = !bIsPaused && !InstallerError->HasError();
		if (bIsPaused)
		{
			// Stop pause timer.
			ProcessPausedTimer.Stop();
		}
		else if (bShouldBePaused)
		{
			// Start pause timer.
			ProcessPausedTimer.Start();
		}
		bIsPaused = bShouldBePaused;
		// Set pause state on all controllable classes
		for (IControllable* Controllable : Controllables)
		{
			Controllable->SetPaused(bShouldBePaused);
		}
		// Set pause state on pausable process timers.
		ConstructTimer.SetPause(bIsPaused);
		UninstallActionTimer.SetPause(bIsPaused);
		MoveFromStageTimer.SetPause(bIsPaused);
		FileAttributesTimer.SetPause(bIsPaused);
		VerifyTimer.SetPause(bIsPaused);
		CleanUpTimer.SetPause(bIsPaused);
		ProcessActiveTimer.SetPause(bIsPaused);
		return bShouldBePaused;
	}

	void FBuildPatchInstaller::RegisterMessageHandler(FMessageHandler* MessageHandler)
	{
		check(IsInGameThread());
		check(MessageHandler != nullptr);
		MessagePump->RegisterMessageHandler(MessageHandler);
	}

	void FBuildPatchInstaller::UnregisterMessageHandler(FMessageHandler* MessageHandler)
	{
		check(IsInGameThread());
		MessagePump->UnregisterMessageHandler(MessageHandler);
	}

	void FBuildPatchInstaller::ExecuteCompleteDelegate()
	{
		// Should be executed in main thread, and already be complete.
		check(IsInGameThread());
		check(IsComplete());
		// Finish applying build statistics.
		{
			FScopeLock Lock(&ThreadLock);
			BuildStats.FinalDownloadSpeed = GetDownloadSpeed();
			BuildStats.AverageDownloadSpeed = DownloadSpeedRecorder->GetAverageSpeed(TNumericLimits<float>::Max());
			BuildStats.PeakDownloadSpeed = DownloadSpeedRecorder->GetPeakSpeed();
			BuildStats.AverageDiskReadSpeed = DiskReadSpeedRecorder->GetAverageSpeed(TNumericLimits<float>::Max());
			BuildStats.PeakDiskReadSpeed = DiskReadSpeedRecorder->GetPeakSpeed();
			BuildStats.AverageDiskWriteSpeed = DiskWriteSpeedRecorder->GetAverageSpeed(TNumericLimits<float>::Max());
			BuildStats.PeakDiskWriteSpeed = DiskWriteSpeedRecorder->GetPeakSpeed();
			BuildStats.TotalDownloadedData = DownloadServiceStatistics->GetBytesDownloaded();
			BuildStats.TotalReadData = InstallChunkSourceStatistics->GetBytesRead();
			BuildStats.TotalReadData += VerifierStatistics->GetBytesVerified();
			BuildStats.TotalWrittenData = FileConstructorStatistics->GetBytesConstructed();
			BuildStats.NumFilesConstructed = FileConstructorStatistics->GetFilesConstructed();
			BuildStats.TheoreticalDownloadTime = BuildStats.AverageDownloadSpeed > 0 ? BuildStats.TotalDownloadedData / BuildStats.AverageDownloadSpeed : 0;
			InstallerHelpers::LogBuildStatInfo(BuildStats, SessionId);
		}
		// Call the complete delegate.
		CompleteDelegate.ExecuteIfBound(AsShared());
	}

	void FBuildPatchInstaller::PumpMessages()
	{
		check(IsInGameThread());
		MessagePump->PumpMessages();
	}
}
