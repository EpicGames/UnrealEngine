// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BuildPatchInstaller.h: Declares the FBuildPatchInstaller class which
	controls the process of installing a build described by a build manifest.
=============================================================================*/

#pragma once

#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"
#include "Templates/SharedPointer.h"
#include "CoreMinimal.h"

#include "Interfaces/IBuildInstaller.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "BuildPatchSettings.h"

#include "Common/HttpManager.h"
#include "Core/AsyncHelpers.h"
#include "Core/Platform.h"
#include "Core/ProcessTimer.h"
#include "BuildPatchManifest.h"
#include "BuildPatchProgress.h"
#include "IBuildManifestSet.h"
#include "Installer/InstallChunkSource.h"

#include <atomic>

PRAGMA_DISABLE_DEPRECATION_WARNINGS

namespace BuildPatchServices
{
	struct FChunkDbSourceConfig;
	struct FCloudSourceConfig;
	struct FDownloadConnectionCountConfig;
	struct FInstallSourceConfig;
	class IInstallerError;
	class IFileOperationTracker;
	class IMemoryChunkStoreStatistics;
	class IDiskChunkStoreStatistics;
	class ISpeedRecorder;
	class IChunkDataSizeProvider;
	class IDownloadServiceStatistics;
	class IChunkDbChunkSourceStatistics;
	class ICloudChunkSourceStatistics;
	class IInstallChunkSourceStatistics;
	class IFileConstructorStatistics;
	class IInstallActionStatistics;
	class IVerifierStatistics;
	class IInstallerAnalytics;
	class IDownloadService;
	class IOptimisedDelta;
	class IMessagePump;
	class IVerifier;
	class IBuildManifestSet;

	/**
	 * FBuildPatchInstaller
	 * This class controls a thread that wraps the code to install/patch an app from manifests.
	 */
	class FBuildPatchInstaller
		: public IBuildInstaller
		, public FRunnable
		, public TSharedFromThis<FBuildPatchInstaller, ESPMode::ThreadSafe>
	{
	private:
		// The installer configuration.
		const FBuildInstallerConfiguration Configuration;

		// Simple unique installer guid to help logging functions have context.
		const FGuid SessionId;

		// The delegates that we will be calling when started.
		const FBuildPatchInstallerDelegate StartDelegate;

		// The delegates that we will be calling on complete.
		const FBuildPatchInstallerDelegate CompleteDelegate;

		// The Configuration.InstallerActions array converted into private class type.
		// Cannot be const due to Delta Optimisation modifying the manifests.
		TArray<FBuildPatchInstallerAction> InstallerActions;

		// The directory created in staging, to store local patch data.
		const FString DataStagingDir;

		// The directory created in staging to construct install files to.
		const FString InstallStagingDir;

		// The directory created in staging to store any required meta, such as resume info.
		const FString MetaStagingDir;

		// The filename used to mark a previous install that did not complete but moved staged files into the install directory.
		const FString PreviousMoveMarker;

		// The filename used to reserve diskspace when preloading.
		const FString HddSpaceReservationFile;

		// Map of registered installations.
		const TMultiMap<FString, FBuildPatchAppManifestRef> InstallationInfo;

		// A critical section to protect variables.
		mutable FCriticalSection ThreadLock;

		// Hold a pointer to my thread for easier deleting.
		IBuildInstallerThread* Thread;

		// A flag storing whether the process was a success.
		std::atomic<bool> bSuccess;

		// A flag marking that we a running.
		std::atomic<bool> bIsRunning;

		// A flag marking that we initialized correctly.
		std::atomic<bool> bIsInited;

		// A flag that stores whether we are on the first install iteration.
		std::atomic<bool> bFirstInstallIteration;

		// A flag that indicates whether we are on the last install iteration.
		std::atomic<bool> bLastInstallIteration = false;

		// Tracks required download data between install retries.
		std::atomic<int64> PreviousTotalDownloadRequired;

		// Tracks if any results of the file construction are encrypted. If they are, we'll be staging only, and no point verifying.
		std::atomic<bool> bAreResultsEncrypted;

		// Keep track of build stats.
		FBuildInstallStats BuildStats;

		// Keep track of install progress.
		FBuildPatchProgress BuildProgress;

		// Whether we are currently paused.
		bool bIsPaused;

		// Whether we are needing to abort.
		bool bShouldAbort;

		// Holds a list of files that have been placed into the install directory.
		TArray<FString> FilesInstalled;

		// Holds the files which are all required.
		TSet<FString> TaggedFiles;

		// Holds the files which are outdated.
		TSet<FString> OutdatedFiles;

		// The files to be constructed in the current install attempt.
		TSet<FString> FilesToConstruct;

		// The files that were removed during installation due to destructive patch behavior.
		TSet<FString> OldFilesRemovedBySystem;

		// With bInstallToMemory, files end up here after construction.
		TMap<FString, TArray64<uint8>> FilesInstalledToMemory;

		// Keyed off the file manifest filename, value is appended to the installation directory.
		// Where to find the files in the installation when patching.
		TMap<FString, FString> OldPerFileSubdirectories;
		// Where to put the files in the new installation.
		TMap<FString, FString> NewPerFileSubdirectories;

		// The file which contains per machine configuration information.
		FString LocalMachineConfigFile;

		// HTTP manager used to make requests for download data.
		TSharedPtr<IHttpManager> HttpManager;

		// File systems for classes requiring disk access.
		TSharedPtr<IFileSystem> FileSystem;

		// File systems for classes requiring crypto.
		TUniquePtr<ICrypto> Crypto;

		// Platform abstraction.
		TUniquePtr<IPlatform> Platform;

		// Installer error tracking system.
		TUniquePtr<IInstallerError> InstallerError;

		// The analytics provider interface.
		TSharedPtr<IAnalyticsProvider> Analytics;

		// Installer analytics handler.
		TSharedPtr<IInstallerAnalytics> InstallerAnalytics;

		// Installer statistics tracking.
		TUniquePtr<IInstallActionStatistics> InstallActionStatistics;
		TUniquePtr<IFileOperationTracker> FileOperationTracker;
		TSharedPtr<ISpeedRecorder> DownloadSpeedRecorder;
		TSharedPtr<ISpeedRecorder> DiskReadSpeedRecorder;
		TSharedPtr<ISpeedRecorder> DiskWriteSpeedRecorder;
		TSharedPtr<IChunkDataSizeProvider> ChunkDataSizeProvider;
		TSharedPtr<IDownloadServiceStatistics> DownloadServiceStatistics;
		TUniquePtr<IChunkDbChunkSourceStatistics> ChunkDbChunkSourceStatistics;
		TUniquePtr<IInstallChunkSourceStatistics> InstallChunkSourceStatistics;
		TUniquePtr<ICloudChunkSourceStatistics> CloudChunkSourceStatistics;
		TUniquePtr<IFileConstructorStatistics> FileConstructorStatistics;
		TUniquePtr<IVerifierStatistics> VerifierStatistics;

		// Download service.
		TUniquePtr<IDownloadService> DownloadService;

		// The message pump controller.
		TUniquePtr<IMessagePump> MessagePump;

		// The interface for manifest data aggregation.
		TUniquePtr<IBuildManifestSet> ManifestSet;

		// The interface for verification of files built.
		TUniquePtr<IVerifier> Verifier;

		// List of controllable classes that have been constructed.
		TArray<IControllable*> Controllables;

		// Download rate limit in Bytes per second, 0 is unlimited.
		// Since we can get a change on any thread and we may need to pipe it to
		// the file constructor, rather than poll the rate we register a function
		// that gets called. The function must be set within the mutex and will be called
		// holding the mutex.
		UE::FMutex DownloadRateLimitLock;
		uint64 DownloadRateLimitBps;
		TUniqueFunction<void(uint64 NewRate)> DownloadRateUpdatedFn;

		// List of message handlers that have been registered.
		TArray<FMessageHandler*> MessageHandlers;

		// Stage timers for build stats.
		FProcessTimer InitializeTimer;
		FProcessTimer ConstructTimer;
		FProcessTimer UninstallActionTimer;
		FProcessTimer MoveFromStageTimer;
		FProcessTimer FileAttributesTimer;
		FProcessTimer VerifyTimer;
		FProcessTimer CleanUpTimer;
		FProcessTimer PrereqTimer;
		FProcessTimer ProcessPausedTimer;
		FProcessTimer ProcessActiveTimer;
		FProcessTimer ProcessExecuteTimer;

		// Caches verification errors encountered each run.
		TMap<BuildPatchServices::EVerifyError, int32> CachedVerifyErrorCounts;

	public:
		/**
		 * Constructor takes configuration and dependencies.
		 * @param Configuration             The installer configuration structure.
		 * @param InstallationInfo          Map of locally installed apps for use as chunk sources.
		 * @param LocalMachineConfigFile    Filename for the local machine's config. This is used for per-machine configuration rather than shipped or user config.
		 * @param Analytics                 Optionally valid ptr to an analytics provider for sending events.
		 * @param StartDelegate             Delegate for when the process has started.
		 * @param CompleteDelegate          Delegate for when the process has completed.
		 */
		FBuildPatchInstaller(FBuildInstallerConfiguration Configuration, TMultiMap<FString, FBuildPatchAppManifestRef> InstallationInfo, const FString& LocalMachineConfigFile, TSharedPtr<IAnalyticsProvider> Analytics, FBuildPatchInstallerDelegate StartDelegate, FBuildPatchInstallerDelegate CompleteDelegate);

		/**
		 * Default Destructor, will delete the allocated Thread.
		 */
		~FBuildPatchInstaller();

		// FRunnable interface begin.
		uint32 Run() override;
		// FRunnable interface end.

		// IBuildInstaller interface begin.
		virtual bool StartInstallation() override;
		virtual bool IsComplete() const override;
		virtual bool IsCanceled() const override;
		virtual bool IsPaused() const override;
		virtual bool IsResumable() const override;
		virtual bool IsUpdate() const override;
		virtual bool IsRepair() const override;
		virtual bool IsVerifyOnly() const override;
		virtual bool IsUninstall() const override;
		virtual bool CompletedSuccessfully() const override;
		virtual bool HasError() const override;
		virtual EBuildPatchInstallError GetErrorType() const override;
		virtual FString GetErrorCode() const override;
		//@todo this is deprecated and shouldn't be used anymore [6/4/2014 justin.sargent]
		virtual FText GetPercentageText() const override;
		//@todo this is deprecated and shouldn't be used anymore [6/4/2014 justin.sargent]
		virtual FText GetDownloadSpeedText() const override;
		virtual double GetDownloadSpeed() const override;
		virtual int64 GetTotalDownloadRequired() const override;
		virtual int64 GetTotalDownloaded() const override;
		virtual EBuildPatchState GetState() const override;
		virtual FText GetStatusText() const override;
		virtual float GetUpdateProgress() const override;
		virtual FBuildInstallStats GetBuildStatistics() const override;
		virtual EBuildPatchDownloadHealth GetDownloadHealth() const override;
		virtual FText GetErrorText() const override;
		virtual void CancelInstall() override;
		virtual bool TogglePauseInstall() override;
		virtual void RegisterMessageHandler(FMessageHandler* MessageHandler) override;
		virtual void UnregisterMessageHandler(FMessageHandler* MessageHandler) override;
		virtual const FBuildInstallerConfiguration& GetConfiguration() const override;
		virtual void SetDownloadBytesPerSecondLimit(uint64 InDownloadRateLimitBps) override;
		virtual uint64 GetDownloadBytesPerSecondLimit() const override;
#if !UE_BUILD_SHIPPING
		virtual void GetDebugText(TArray<FString>& Output) override;
#endif
		virtual TMap<FString, TArray64<uint8>>& GetFilesInstalledToMemory() override;
		virtual void SetPerFileSubdirectories(TMap<FString, FString>&& InOldPerFileSubdirectories, TMap<FString, FString>&& InNewPerFileSubdirectories) override;
		// IBuildInstaller interface end.

		/**
		 * Tick function called from the module to give us game thread time.
		 */
		bool Tick();

		/**
		 * Called by the module during shutdown.
		 */
		void PreExit();

		/**
		 * @return the file operation tracker for granular data progress states.
		 */
		const IFileOperationTracker* GetFileOperationTracker() const;

		/**
		 * @return the speed recorder for download speed.
		 */
		const ISpeedRecorder* GetDownloadSpeedRecorder() const;

		/**
		 * @return the speed recorder for disk read speed.
		 */
		const ISpeedRecorder* GetDiskReadSpeedRecorder() const;

		/**
		 * @return the speed recorder for disk write speed.
		 */
		const ISpeedRecorder* GetDiskWriteSpeedRecorder() const;

		/**
		 * @return the download service statistics interface.
		 */
		const IDownloadServiceStatistics* GetDownloadServiceStatistics() const;

		/**
		 * @return the install chunk source statistics interface.
		 */
		const IInstallChunkSourceStatistics* GetInstallChunkSourceStatistics() const;

		/**
		 * @return the cloud chunk source statistics interface.
		 */
		const ICloudChunkSourceStatistics* GetCloudChunkSourceStatistics() const;

		/**
		 * @return the file constructor statistics interface.
		 */
		const IFileConstructorStatistics* GetFileConstructorStatistics() const;

		const IChunkDbChunkSourceStatistics* GetChunkDbChunkSourceStatistics() const;

		/**
		 * @return the verifier statistics interface.
		 */
		const IVerifierStatistics* GetVerifierStatistics() const;


	private:

		/**
		 * Initialize the installer.
		 * @return Whether initialization was successful.
		 */
		bool Initialize();

		/**
		 * Executes the on complete delegate. This should only be called when completed, and is separated out
		 * to allow control to make this call on the main thread.
		 */
		void ExecuteCompleteDelegate();

		/**
		 * Pumps all queued messages to registered handlers.
		 */
		void PumpMessages();

		/**
		 * Checks the installation directory for any already existing files of the correct size, with may account for manual
		 * installation. Should be used for new installation detecting existing files.
		 * NB: Not useful for patches, where we'd expect existing files anyway.
		 * @param FilesToCheck      The list of files that we would expect to not exist yet.
		 * @return    Returns true if there were potentially already installed files.
		 */
		bool CheckForExternallyInstalledFiles(const TSet<FString>& FilesToCheck);

		/**
		 * Runs the installation process.
		 * @param   CorruptFiles    A list of files that were corrupt, to only install those.
		 * @return  Returns true if there were no errors blocking installation.
		 */
		bool RunInstallation(TArray<FString>& CorruptFiles);

		/**
		 * Runs the preload download process.
		 * @return  Returns true if there were no errors blocking the preload.
		 */
		bool RunPreload();

		/**
		 * Runs the prerequisite installation process.
		 */
		bool RunPrerequisites();

		/**
		 * Sets installation stats from various systems.
		 */
		void SetFinalInstallStats();

		/**
		 * Runs the custom uninstall action if specified.
		 * @return    Returns true if there were no errors.
		 */
		bool RunCustomUninstallAction();

		/**
		 * Runs the backup process for locally changed files, and then moves new files into installation directory.
		 * @return    Returns true if there were no errors.
		 */
		bool RunBackupAndMove();

		/**
		 * Runs the process to setup all file attributes required.
		 * @return    Returns true if there were no errors.
		 */
		bool RunFileAttributes();

		/**
		 * Runs the verification process.
		 * @param CorruptFiles  OUT     Receives the list of files that failed verification.
		 * @return    Returns true if there were no corrupt files.
		 */
		bool RunVerification(TArray<FString>& CorruptFiles);

		/**
		 * Checks a particular file in the install directory to see if it needs backing up, and does so if necessary.
		 * @param Filename                      The filename to check, which should match a filename in a manifest.
		 * @param PerFileSubdirectories         Per-file subdirectory mapping to use when resolving paths
		 * @param bDiscoveredByVerification     Optional, whether the file was detected changed already by verification stage. Default: false.
		 * @return    Returns true if there were no errors.
		 */
		bool BackupFileIfNecessary(const FString& Filename, const TMap<FString, FString>& PerFileSubdirectories, bool bDiscoveredByVerification = false);

		/**
		 * Delete empty directories from an installation.
		 * @param RootDirectory     Root Directory for search.
		 */
		void CleanupEmptyDirectories(const FString& RootDirectory);

		/**
		 * Remove from the set any files that do not exist.
		 * @param RootDirectory             Root Directory for search.
		 * @param PerFileSubdirectories     Per-file subdirectory mapping to use when resolving paths
		 * @param Files                     Set of files to filter.
		 * @param bResolveInstallationFileName If true, instead of appending the RootDirectory / FilesName, call ResolveInstallationFileName on the file.
		 */
		void FilterToExistingFiles(const FString& RootDirectory, const TMap<FString, FString>& PerFileSubdirectories, TSet<FString>& Files, bool bResolveInstallationFileName);

		/**
		 * Attempt to remove a file, using configured retries.
		 * @param FullFilename      The full filename to remove.
		 * @param ErrorCode     OUT The error code if failed.
		 * @return true if the file was removed.
		 */
		bool RemoveFileWithRetries(const FString& FullFilename, uint32& ErrorCode);

		/**
		 * Attempt to relocate a file by move or copy, using configured retries
		 * @param ToFullFilename          The full filename of the destination.
		 * @param FromFullFilename        The full filename of the source.
		 * @param RenameErrorCode     OUT The move/rename error code if failed.
		 * @param CopyErrorCode       OUT The copy error code if failed.
		 * @return true if the file was relocated.
		 */
		bool RelocateFileWithRetries(const FString& ToFullFilename, const FString& FromFullFilename, uint32& RenameErrorCode, uint32& CopyErrorCode);

		/**
		 * Attempt to copy a file using configured retries
		 * @param ToFullFilename          The full filename of the destination.
		 * @param FromFullFilename        The full filename of the source.
		 * @param CopyErrorCode       OUT The copy error code if failed.
		 * @return true if the file was copied.
		 */
		bool CopyFileWithRetries(const FString& ToFullFilename, const FString& FromFullFilename, uint32& CopyErrorCode);

		/**
		 * Builds the cloud source configuration struct.
		 */
		FCloudSourceConfig BuildCloudSourceConfig();

		/**
		 * Builds the cloud source configuration struct.
		 */
		FDownloadConnectionCountConfig BuildConnectionCountConfig();

		/**
		 * Get the filename as SHA string value.
		 * @param FileManifest    The FFileManifest for the file.
		 * @return filename as SHA string value.
		 */
		FString GetSHA1Filename(const FFileManifest& FileManifest);

		/**
		 * Destroys the internal worker thread
		 */
		void CleanupThread();
	};
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

typedef TSharedPtr< BuildPatchServices::FBuildPatchInstaller, ESPMode::ThreadSafe > FBuildPatchInstallerPtr;
typedef TSharedRef< BuildPatchServices::FBuildPatchInstaller, ESPMode::ThreadSafe > FBuildPatchInstallerRef;
typedef TWeakPtr< BuildPatchServices::FBuildPatchInstaller, ESPMode::ThreadSafe > FBuildPatchInstallerWeakPtr;
