// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchManifest.h"
#include "HAL/FileManager.h"
#include "Misc/Base64.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Algo/Sort.h"
#include "Algo/AllOf.h"
#include "Algo/Accumulate.h"
#include "Algo/Transform.h"
#include "Algo/RemoveIf.h"
#include "Common/FileSystem.h"
#include "Core/BlockStructure.h"
#include "Core/AsyncHelpers.h"
#include "Data/ChunkData.h"
#include "Data/ManifestData.h"
#include "BuildPatchUtil.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBuildPatchManifest, Log, All);
DEFINE_LOG_CATEGORY(LogBuildPatchManifest);

using namespace BuildPatchServices;

#define LOCTEXT_NAMESPACE "BuildPatchManifest"

/**
 * Helper functions that convert generic types to and from string blobs for use with JSON parsing.
 * It's kind of horrible but guarantees no loss of data as the JSON reader/writer only supports float functionality 
 * which would result in data loss with high int32 values, and we'll be using uint64.
 */
template< typename DataType >
bool FromStringBlob( const FString& StringBlob, DataType& ValueOut )
{
	void* AsBuffer = &ValueOut;
	return FString::ToBlob( StringBlob, static_cast< uint8* >( AsBuffer ), sizeof( DataType ) );
}
template< typename DataType >
FString ToStringBlob( const DataType& DataVal )
{
	const void* AsBuffer = &DataVal;
	return FString::FromBlob( static_cast<const uint8*>( AsBuffer ), sizeof( DataType ) );
}
template< typename DataType >
bool FromHexString(const FString& HexString, DataType& ValueOut)
{
	void* AsBuffer = &ValueOut;
	if (HexString.Len() == (sizeof(DataType)* 2))
	{
		HexToBytes(HexString, static_cast<uint8*>(AsBuffer));
		return true;
	}
	return false;
}
template< typename DataType >
FString ToHexString(const DataType& DataVal)
{
	const void* AsBuffer = &DataVal;
	return BytesToHex(static_cast<const uint8*>(AsBuffer), sizeof(DataType));
}

/**
 * Helper functions to decide whether the passed in data is a JSON string we expect to deserialize a manifest from
 */
bool BufferIsJsonManifest(const TArray<uint8>& DataInput)
{
	// The best we can do is look for the mandatory first character open curly brace,
	// it will be within the first 4 characters (may have BOM)
	for (int32 idx = 0; idx < 4 && idx < DataInput.Num(); ++idx)
	{
		if (DataInput[idx] == TEXT('{'))
		{
			return true;
		}
	}
	return false;
}

FString GetFilename(const FFileManifest& FileManifest, const FString& RootDirectory, bool bIsStaged)
{
	static constexpr const TCHAR* EncryptedExt = TEXT(".e");
	static constexpr const TCHAR* CompletedExt = TEXT(".c");
	if (bIsStaged)
	{
		FString FileSHA1HashStringValue;
		FBuildPatchUtils::SHAToBase32(FileManifest.SHA1Hash, FileSHA1HashStringValue);
		return IFileManager::Get().FileExists(*(RootDirectory / FileSHA1HashStringValue + CompletedExt)) ? FileSHA1HashStringValue + CompletedExt : FileSHA1HashStringValue + EncryptedExt;
	}
	else
	{
		return FileManifest.Filename;
	}
}

FString GetFilename(const FString& Filename, const FString& /*RootDirectory*/, bool bIsStaged)
{
	// Staged files not supported if using filenames not file manifests
	check(bIsStaged == false);
	return Filename;
}

/* FBuildPatchCustomField implementation
*****************************************************************************/
FBuildPatchCustomField::FBuildPatchCustomField(const FString& Value)
	: CustomValue(Value)
{
}

FString FBuildPatchCustomField::AsString() const
{
	return CustomValue;
}

double FBuildPatchCustomField::AsDouble() const
{
	// The Json parser currently only supports float so we have to decode string blob instead
	double Rtn;
	if( FromStringBlob( CustomValue, Rtn ) )
	{
		return Rtn;
	}
	return 0;
}

int64 FBuildPatchCustomField::AsInteger() const
{
	// The Json parser currently only supports float so we have to decode string blob instead
	int64 Rtn;
	if( FromStringBlob( CustomValue, Rtn ) )
	{
		return Rtn;
	}
	return 0;
}

/* FCipherHeader implementation
*****************************************************************************/
struct FCipherHeader
{
public:
	EFeatureLevel Version = EFeatureLevel::Latest;
	uint32 HeaderSize = 0;
	EManifestStorageFlags DataStoredAs = EManifestStorageFlags::RawData;
	uint32 DataSizeUncompressed = 0;
	uint32 DataSizeCompressed = 0;
	TArray<uint8> InitializationVector;
};

FArchive& operator<<(FArchive& Ar, FCipherHeader& Header)
{
	if (Ar.IsError())
	{
		return Ar;
	}

	const int64 StartPos = Ar.Tell();
	Ar << Header.HeaderSize;

	int32 Version = (int32)Header.Version;
	Ar << Version;
	Header.Version = (EFeatureLevel)Version;

	Ar << Header.DataStoredAs;
	Ar << Header.DataSizeUncompressed;
	Ar << Header.DataSizeCompressed;
	Ar << Header.InitializationVector;

	if (Ar.IsSaving())
	{
		const int64 EndPos = Ar.Tell();
		Ar.Seek(StartPos);
		Ar << (Header.HeaderSize = EndPos - StartPos);
		Ar.Seek(EndPos);
	}

	Ar.Seek(StartPos + Header.HeaderSize);
	return Ar;
}

/* FBuildPatchAppManifest implementation
*****************************************************************************/

FBuildPatchAppManifest::FBuildPatchAppManifest()
	: TotalBuildSize(INDEX_NONE)
	, TotalDownloadSize(INDEX_NONE)
	, bNeedsResaving(false)
{
}

FBuildPatchAppManifest::FBuildPatchAppManifest(const uint32& InAppID, const FString& InAppName)
	: FBuildPatchAppManifest()
{
	ManifestMeta.AppID = InAppID;
	ManifestMeta.AppName = InAppName;
}

FBuildPatchAppManifest::FBuildPatchAppManifest(const FBuildPatchAppManifest& Other)
	: ManifestMeta(Other.ManifestMeta)
	, ChunkDataList(Other.ChunkDataList)
	, FileManifestList(Other.FileManifestList)
	, CustomFields(Other.CustomFields)
	, EncryptedData(Other.EncryptedData)
	, EncryptionSecretId(Other.EncryptionSecretId)
	, bIsManifestEncrypted(Other.bIsManifestEncrypted)
	, TotalBuildSize(Other.TotalBuildSize)
	, TotalDownloadSize(Other.TotalDownloadSize)
	, bNeedsResaving(Other.bNeedsResaving)
{
	FMemory::Memcpy(EncryptionAuthTag.AuthTag, Other.EncryptionAuthTag.AuthTag, AES256_GCM_AuthTagSizeInBytes);
	InitLookups();
}

FBuildPatchAppManifest::~FBuildPatchAppManifest()
{
	DestroyData();
}

bool FBuildPatchAppManifest::SaveToFile(const FString& Filename, BuildPatchServices::EFeatureLevel SaveFormat, FSHAHash* OutSHA1Hash, FMD5Hash* OutMD5Hash)
{
	bool bSuccess = SaveFormat >= GetFeatureLevel();
	if (bSuccess)
	{
		TArray<uint8> ManifestData;
		FMemoryWriter MemoryWriter(ManifestData);
		// Serialize into the desired format.
		if (SaveFormat >= BuildPatchServices::EFeatureLevel::StoredAsBinaryData)
		{
			bSuccess = FManifestData::Serialize(MemoryWriter, *this, SaveFormat);
		}
		else
		{
			FString JSONOutput;
			SerializeToJSON(JSONOutput);
			FTCHARToUTF8 JsonUTF8(*JSONOutput);
			MemoryWriter.Serialize((UTF8CHAR*)JsonUTF8.Get(), JsonUTF8.Length() * sizeof(UTF8CHAR));
		}
		// Save the file if successful.
		if (bSuccess)
		{
			bSuccess = FFileHelper::SaveArrayToFile(ManifestData, *Filename);
		}
		// Set the output hash values if desired.
		if (bSuccess)
		{
			if (OutMD5Hash != nullptr)
			{
				FMD5 Md5Gen;
				Md5Gen.Update(ManifestData.GetData(), ManifestData.Num());
				OutMD5Hash->Set(Md5Gen);
			}
			if (OutSHA1Hash != nullptr)
			{
				FSHA1::HashBuffer(ManifestData.GetData(), ManifestData.Num(), OutSHA1Hash->Hash);
			}
		}
	}
	return bSuccess;
}

bool FBuildPatchAppManifest::LoadFromFile(const FString& Filename)
{
	TArray<uint8> FileData;
	if (FFileHelper::LoadFileToArray(FileData, *Filename, FILEREAD_Silent))
	{
		return DeserializeFromData(FileData);
	}
	return false;
}

bool FBuildPatchAppManifest::DeserializeFromData(const TArray<uint8>& DataInput)
{
	if (DataInput.Num())
	{
		if (BufferIsJsonManifest(DataInput))
		{
			FString JsonManifest;
			FFileHelper::BufferToString(JsonManifest, DataInput.GetData(), DataInput.Num());
			return DeserializeFromJSON(JsonManifest);
		}
		else
		{
			FMemoryReader MemoryReader(DataInput);
			return FManifestData::Serialize(MemoryReader, *this);
		}
	}
	return false;
}

void FBuildPatchAppManifest::DestroyData()
{
	// Clear Manifest data
	ManifestMeta = FManifestMeta();
	ChunkDataList = FChunkDataList();
	FileManifestList = FFileManifestList();
	CustomFields = FCustomFields();
	EncryptionSecretId = FGuid();
	FileNameLookup.Empty();
	FileManifestLookup.Empty();
	TaggedFilesLookup.Empty();
	ChunkInfoLookup.Empty();
	TotalBuildSize = INDEX_NONE;
	TotalDownloadSize = INDEX_NONE;
	bNeedsResaving = false;
}

void FBuildPatchAppManifest::InitLookups()
{
	// Create file lookups.
	const int32 NumFiles = FileManifestList.FileList.Num();
	FileNameLookup.Empty(ManifestMeta.bIsFileData ? NumFiles : 0);
	FileManifestLookup.Empty(NumFiles);
	TaggedFilesLookup.Empty();
	for (const FFileManifest& FileManifest : FileManifestList.FileList)
	{
		FileManifestLookup.Add(FileManifest.Filename, &FileManifest);
		if (ManifestMeta.bIsFileData)
		{
			FileNameLookup.Add(FileManifest.ChunkParts[0].Guid, &FileManifest.Filename);
		}
		if (FileManifest.InstallTags.Num() == 0)
		{
			TaggedFilesLookup.FindOrAdd(TEXT("")).Add(&FileManifest);
		}
		else
		{
			for (const FString& FileTag : FileManifest.InstallTags)
			{
				TaggedFilesLookup.FindOrAdd(FileTag).Add(&FileManifest);
			}
		}
	}

	// Create chunk lookup.
	const int32 NumChunks = ChunkDataList.ChunkList.Num();
	ChunkInfoLookup.Empty(NumChunks);
	for (const FChunkInfo& ChunkInfo : ChunkDataList.ChunkList)
	{
		ChunkInfoLookup.Add(ChunkInfo.Guid, &ChunkInfo);
	}

	// Calculate build sizes.
	TotalBuildSize = 0;
	TotalDownloadSize = 0;
	for (const FFileManifest& FileManifest : FileManifestList.FileList)
	{
		TotalBuildSize += FileManifest.FileSize;
	}
	for (const FChunkInfo& Chunk : ChunkDataList.ChunkList)
	{
		TotalDownloadSize += Chunk.FileSize;
	}
}

void FBuildPatchAppManifest::SerializeToJSON(FString& JSONOutput)
{
	using namespace BuildPatchServices;
#if UE_BUILD_DEBUG // We'll use this to switch between human readable JSON
	TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy< TCHAR > > > Writer = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy< TCHAR > >::Create(&JSONOutput);
#else
	TSharedRef< TJsonWriter< TCHAR, TCondensedJsonPrintPolicy< TCHAR > > > Writer = TJsonWriterFactory< TCHAR, TCondensedJsonPrintPolicy< TCHAR > >::Create(&JSONOutput);
#endif //ALLOW_DEBUG_FILES

	Writer->WriteObjectStart();
	{
		// Write general data
		Writer->WriteValue(TEXT("ManifestFileVersion"), ToStringBlob(static_cast<int32>(ManifestMeta.FeatureLevel)));
		Writer->WriteValue(TEXT("bIsFileData"), ManifestMeta.bIsFileData);
		Writer->WriteValue(TEXT("AppID"), ToStringBlob(ManifestMeta.AppID));
		Writer->WriteValue(TEXT("AppNameString"), ManifestMeta.AppName);
		Writer->WriteValue(TEXT("BuildVersionString"), ManifestMeta.BuildVersion);
		Writer->WriteValue(TEXT("LaunchExeString"), ManifestMeta.LaunchExe);
		Writer->WriteValue(TEXT("LaunchCommand"), ManifestMeta.LaunchCommand);
		Writer->WriteArrayStart(TEXT("PrereqIds"));
		for (const FString& PrereqId : ManifestMeta.PrereqIds)
		{
			Writer->WriteValue(PrereqId);
		}
		Writer->WriteArrayEnd();
		Writer->WriteValue(TEXT("PrereqName"), ManifestMeta.PrereqName);
		Writer->WriteValue(TEXT("PrereqPath"), ManifestMeta.PrereqPath);
		Writer->WriteValue(TEXT("PrereqArgs"), ManifestMeta.PrereqArgs);
		// Write file manifest data
		Writer->WriteArrayStart(TEXT("FileManifestList"));
		for (const FFileManifest& FileManifest : FileManifestList.FileList)
		{
			Writer->WriteObjectStart();
			{
				Writer->WriteValue(TEXT("Filename"), FileManifest.Filename);
				Writer->WriteValue(TEXT("FileHash"), FString::FromBlob(FileManifest.SHA1Hash.Hash, FSHA1::DigestSize));
				if (EnumHasAllFlags(FileManifest.FileMetaFlags, EFileMetaFlags::UnixExecutable))
				{
					Writer->WriteValue(TEXT("bIsUnixExecutable"), true);
				}
				if (EnumHasAllFlags(FileManifest.FileMetaFlags, EFileMetaFlags::ReadOnly))
				{
					Writer->WriteValue(TEXT("bIsReadOnly"), true);
				}
				if (EnumHasAllFlags(FileManifest.FileMetaFlags, EFileMetaFlags::Compressed))
				{
					Writer->WriteValue(TEXT("bIsCompressed"), true);
				}
				const bool bIsSymlink = !FileManifest.SymlinkTarget.IsEmpty();
				if (bIsSymlink)
				{
					Writer->WriteValue(TEXT("SymlinkTarget"), FileManifest.SymlinkTarget);
				}
				else
				{
					Writer->WriteArrayStart(TEXT("FileChunkParts"));
					{
						for (const FChunkPart& ChunkPart : FileManifest.ChunkParts)
						{
							Writer->WriteObjectStart();
							{
								Writer->WriteValue(TEXT("Guid"), ChunkPart.Guid.ToString());
								Writer->WriteValue(TEXT("Offset"), ToStringBlob(ChunkPart.Offset));
								Writer->WriteValue(TEXT("Size"), ToStringBlob(ChunkPart.Size));
							}
							Writer->WriteObjectEnd();
						}
					}
					Writer->WriteArrayEnd();
				}
				if (FileManifest.InstallTags.Num() > 0)
				{
					Writer->WriteArrayStart(TEXT("InstallTags"));
					{
						for (const FString& InstallTag : FileManifest.InstallTags)
						{
							Writer->WriteValue(InstallTag);
						}
					}
					Writer->WriteArrayEnd();
				}
			}
			Writer->WriteObjectEnd();
		}
		Writer->WriteArrayEnd();
		// Write chunk hash list
		Writer->WriteObjectStart(TEXT("ChunkHashList"));
		for (const FChunkInfo& ChunkInfo : ChunkDataList.ChunkList)
		{
			const FGuid& ChunkGuid = ChunkInfo.Guid;
			const uint64& ChunkHash = ChunkInfo.Hash;
			Writer->WriteValue(ChunkGuid.ToString(), ToStringBlob(ChunkHash));
		}
		Writer->WriteObjectEnd();
		// Write chunk sha list
		Writer->WriteObjectStart(TEXT("ChunkShaList"));
		for (const FChunkInfo& ChunkInfo : ChunkDataList.ChunkList)
		{
			const FGuid& ChunkGuid = ChunkInfo.Guid;
			const FSHAHash& ChunkSha = ChunkInfo.ShaHash;
			Writer->WriteValue(ChunkGuid.ToString(), ToHexString(ChunkSha));
		}
		Writer->WriteObjectEnd();
		// Write data group list
		Writer->WriteObjectStart(TEXT("DataGroupList"));
		for (const FChunkInfo& ChunkInfo : ChunkDataList.ChunkList)
		{
			const FGuid& DataGuid = ChunkInfo.Guid;
			const uint8& DataGroup = ChunkInfo.GroupNumber;
			Writer->WriteValue(DataGuid.ToString(), ToStringBlob(DataGroup));
		}
		Writer->WriteObjectEnd();
		// Write chunk size list
		Writer->WriteObjectStart(TEXT("ChunkFilesizeList"));
		for (const FChunkInfo& ChunkInfo : ChunkDataList.ChunkList)
		{
			const FGuid& ChunkGuid = ChunkInfo.Guid;
			const int64& ChunkSize = ChunkInfo.FileSize;
			Writer->WriteValue(ChunkGuid.ToString(), ToStringBlob(ChunkSize));
		}
		Writer->WriteObjectEnd();
		// Write custom fields
		Writer->WriteObjectStart(TEXT("CustomFields"));
		for (const TPair<FString, FString>& CustomField : CustomFields.Fields)
		{
			Writer->WriteValue(CustomField.Key, CustomField.Value);
		}
		Writer->WriteObjectEnd();
	}
	Writer->WriteObjectEnd();

	Writer->Close();
}

// @TODO LSwift: Perhaps replace FromBlob and ToBlob usage with hexadecimal notation instead
bool FBuildPatchAppManifest::DeserializeFromJSON( const FString& JSONInput )
{
	bool bSuccess = true;
	TSharedPtr<FJsonObject> JSONManifestObject;
	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(JSONInput);

	// Clear current data
	DestroyData();

	// Attempt to deserialize JSON
	if (!FJsonSerializer::Deserialize(Reader, JSONManifestObject) || !JSONManifestObject.IsValid())
	{
		return false;
	}

	// Store a list of all data GUID for later use
	TSet<FGuid> AllDataGuids;

	// Get the values map
	const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonValueMap = JSONManifestObject->Values;

	// Feature Level did not always exist
	int32 FeatureLevelInt = 0;
	TSharedPtr<FJsonValue> JsonFeatureLevel = JsonValueMap.FindRef(TEXT("ManifestFileVersion"));
	if (JsonFeatureLevel.IsValid() && FromStringBlob(JsonFeatureLevel->AsString(), FeatureLevelInt))
	{
		ManifestMeta.FeatureLevel = static_cast<EFeatureLevel>(FeatureLevelInt);
	}
	else
	{
		// Then we presume version just before we started outputting the version
		ManifestMeta.FeatureLevel = EFeatureLevel::CustomFields;
	}

	// If we loaded the default version number, we know it was saved is a CL range that was bugged,
	// when the correct version should have been StoresChunkFileSizes.
	if (ManifestMeta.FeatureLevel == EFeatureLevel::BrokenJsonVersion)
	{
		ManifestMeta.FeatureLevel = EFeatureLevel::StoresChunkFileSizes;
	}

	// Get the app and version strings
	TSharedPtr<FJsonValue> JsonAppID = JsonValueMap.FindRef(TEXT("AppID"));
	TSharedPtr<FJsonValue> JsonAppNameString = JsonValueMap.FindRef(TEXT("AppNameString"));
	TSharedPtr<FJsonValue> JsonBuildVersionString = JsonValueMap.FindRef(TEXT("BuildVersionString"));
	TSharedPtr<FJsonValue> JsonLaunchExe = JsonValueMap.FindRef(TEXT("LaunchExeString"));
	TSharedPtr<FJsonValue> JsonLaunchCommand = JsonValueMap.FindRef(TEXT("LaunchCommand"));
	TSharedPtr<FJsonValue> JsonPrereqName = JsonValueMap.FindRef(TEXT("PrereqName"));
	TSharedPtr<FJsonValue> JsonPrereqPath = JsonValueMap.FindRef(TEXT("PrereqPath"));
	TSharedPtr<FJsonValue> JsonPrereqArgs = JsonValueMap.FindRef(TEXT("PrereqArgs"));
	bSuccess = bSuccess && JsonAppID.IsValid();
	if( bSuccess )
	{
		bSuccess = bSuccess && FromStringBlob( JsonAppID->AsString(), ManifestMeta.AppID );
	}
	bSuccess = bSuccess && JsonAppNameString.IsValid();
	if( bSuccess )
	{
		ManifestMeta.AppName = JsonAppNameString->AsString();
	}
	bSuccess = bSuccess && JsonBuildVersionString.IsValid();
	if( bSuccess )
	{
		ManifestMeta.BuildVersion = JsonBuildVersionString->AsString();
	}
	bSuccess = bSuccess && JsonLaunchExe.IsValid();
	if( bSuccess )
	{
		ManifestMeta.LaunchExe = JsonLaunchExe->AsString();
	}
	bSuccess = bSuccess && JsonLaunchCommand.IsValid();
	if( bSuccess )
	{
		ManifestMeta.LaunchCommand = JsonLaunchCommand->AsString();
	}

	// Get the prerequisites installer info.  These are optional entries.
	ManifestMeta.PrereqName = JsonPrereqName.IsValid() ? JsonPrereqName->AsString() : FString();
	ManifestMeta.PrereqPath = JsonPrereqPath.IsValid() ? JsonPrereqPath->AsString() : FString();
	ManifestMeta.PrereqArgs = JsonPrereqArgs.IsValid() ? JsonPrereqArgs->AsString() : FString();

	// Get the FileManifestList
	TSharedPtr<FJsonValue> JsonFileManifestList = JsonValueMap.FindRef(TEXT("FileManifestList"));
	bSuccess = bSuccess && JsonFileManifestList.IsValid();
	if( bSuccess )
	{
		TArray<TSharedPtr<FJsonValue>> JsonFileManifestArray = JsonFileManifestList->AsArray();
		for (auto JsonFileManifestIt = JsonFileManifestArray.CreateConstIterator(); JsonFileManifestIt && bSuccess; ++JsonFileManifestIt)
		{
			TSharedPtr<FJsonObject> JsonFileManifest = (*JsonFileManifestIt)->AsObject();

			const int32 FileIndex = FileManifestList.FileList.Add(FFileManifest());
			FFileManifest& FileManifest = FileManifestList.FileList[FileIndex];
			FileManifest.Filename = JsonFileManifest->GetStringField(TEXT("Filename"));
			bSuccess = bSuccess && FString::ToBlob(JsonFileManifest->GetStringField(TEXT("FileHash")), FileManifest.SHA1Hash.Hash, FSHA1::DigestSize);
			TArray<TSharedPtr<FJsonValue>> JsonChunkPartArray = JsonFileManifest->GetArrayField(TEXT("FileChunkParts"));
			for (auto JsonChunkPartIt = JsonChunkPartArray.CreateConstIterator(); JsonChunkPartIt && bSuccess; ++JsonChunkPartIt)
			{
				const int32 ChunkIndex = FileManifest.ChunkParts.Add(FChunkPart());
				FChunkPart& FileChunkPart = FileManifest.ChunkParts[ChunkIndex];
				TSharedPtr<FJsonObject> JsonChunkPart = (*JsonChunkPartIt)->AsObject();
				bSuccess = bSuccess && FGuid::Parse(JsonChunkPart->GetStringField(TEXT("Guid")), FileChunkPart.Guid);
				bSuccess = bSuccess && FromStringBlob(JsonChunkPart->GetStringField(TEXT("Offset")), FileChunkPart.Offset);
				bSuccess = bSuccess && FromStringBlob(JsonChunkPart->GetStringField(TEXT("Size")), FileChunkPart.Size);
				AllDataGuids.Add(FileChunkPart.Guid);
			}
			if (JsonFileManifest->HasTypedField<EJson::Array>(TEXT("InstallTags")))
			{
				TArray<TSharedPtr<FJsonValue>> JsonInstallTagsArray = JsonFileManifest->GetArrayField(TEXT("InstallTags"));
				for (auto JsonInstallTagIt = JsonInstallTagsArray.CreateConstIterator(); JsonInstallTagIt && bSuccess; ++JsonInstallTagIt)
				{
					FileManifest.InstallTags.Add((*JsonInstallTagIt)->AsString());
				}
			}
			if (JsonFileManifest->HasField(TEXT("bIsUnixExecutable")) && JsonFileManifest->GetBoolField(TEXT("bIsUnixExecutable")))
			{
				FileManifest.FileMetaFlags |= EFileMetaFlags::UnixExecutable;
			}
			if (JsonFileManifest->HasField(TEXT("bIsReadOnly")) && JsonFileManifest->GetBoolField(TEXT("bIsReadOnly")))
			{
				FileManifest.FileMetaFlags |= EFileMetaFlags::ReadOnly;
			}
			if (JsonFileManifest->HasField(TEXT("bIsCompressed")) && JsonFileManifest->GetBoolField(TEXT("bIsCompressed")))
			{
				FileManifest.FileMetaFlags |= EFileMetaFlags::Compressed;
			}
			FileManifest.SymlinkTarget = JsonFileManifest->HasField(TEXT("SymlinkTarget")) ? JsonFileManifest->GetStringField(TEXT("SymlinkTarget")) : TEXT("");
		}
	}

	for (FFileManifest& FileManifest : FileManifestList.FileList)
	{
		FileManifestLookup.Add(FileManifest.Filename, &FileManifest);
	}

	// For each chunk setup its info
	for (const FGuid& DataGuid : AllDataGuids)
	{
		int32 ChunkIndex = ChunkDataList.ChunkList.Add(FChunkInfo());
		ChunkDataList.ChunkList[ChunkIndex].Guid = DataGuid;
	}

	// Create a lookup table for chunks to speed up parsing
	TMap<FGuid,FChunkInfo*> MutableChunkInfoLookup;
	for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList)
	{
		MutableChunkInfoLookup.Add(ChunkInfo.Guid, &ChunkInfo);
	}

	// Get the ChunkHashList
	bool bHasChunkHashList = false;
	TSharedPtr<FJsonValue> JsonChunkHashList = JsonValueMap.FindRef(TEXT("ChunkHashList"));
	bSuccess = bSuccess && JsonChunkHashList.IsValid();
	if (bSuccess)
	{
		TSharedPtr<FJsonObject> JsonChunkHashListObj = JsonChunkHashList->AsObject();
		for (auto ChunkHashIt = JsonChunkHashListObj->Values.CreateConstIterator(); ChunkHashIt && bSuccess; ++ChunkHashIt)
		{
			FGuid ChunkGuid;
			uint64 ChunkHash = 0;
			bSuccess = bSuccess && FGuid::Parse(ChunkHashIt.Key(), ChunkGuid);
			bSuccess = bSuccess && FromStringBlob(ChunkHashIt.Value()->AsString(), ChunkHash);
			if (bSuccess && MutableChunkInfoLookup.Contains(ChunkGuid))
			{
				FChunkInfo* ChunkInfoData = MutableChunkInfoLookup[ChunkGuid];
				ChunkInfoData->Hash = ChunkHash;
				bHasChunkHashList = true;
			}
		}
	}

	// Get the ChunkShaList (optional)
	TSharedPtr<FJsonValue> JsonChunkShaList = JsonValueMap.FindRef(TEXT("ChunkShaList"));
	if (JsonChunkShaList.IsValid())
	{
		TSharedPtr<FJsonObject> JsonChunkHashListObj = JsonChunkShaList->AsObject();
		for (auto ChunkHashIt = JsonChunkHashListObj->Values.CreateConstIterator(); ChunkHashIt && bSuccess; ++ChunkHashIt)
		{
			FGuid ChunkGuid;
			FSHAHash ChunkSha;
			bSuccess = bSuccess && FGuid::Parse(ChunkHashIt.Key(), ChunkGuid);
			bSuccess = bSuccess && FromHexString(ChunkHashIt.Value()->AsString(), ChunkSha);
			if (bSuccess && MutableChunkInfoLookup.Contains(ChunkGuid))
			{
				FChunkInfo* ChunkInfoData = MutableChunkInfoLookup[ChunkGuid];
				ChunkInfoData->ShaHash = ChunkSha;
			}
		}
	}

	// Get the PrereqIds (optional)
	TSharedPtr<FJsonValue> JsonPrereqIds = JsonValueMap.FindRef(TEXT("PrereqIds"));
	if (bSuccess && JsonPrereqIds.IsValid())
	{
		TArray<TSharedPtr<FJsonValue>> JsonPrereqIdsArray = JsonPrereqIds->AsArray();
		for (TSharedPtr<FJsonValue> JsonPrereqId : JsonPrereqIdsArray)
		{
			ManifestMeta.PrereqIds.Add(JsonPrereqId->AsString());
		}
	}
	else
	{
		// We fall back to using the hash of the prereq exe if we have no prereq ids specified
		FString PrereqFilename = ManifestMeta.PrereqPath;
		PrereqFilename.ReplaceInline(TEXT("\\"), TEXT("/"));
		const FFileManifest* const * FoundFileManifest = FileManifestLookup.Find(PrereqFilename);
		if (FoundFileManifest)
		{
			FSHAHash PrereqHash;
			FMemory::Memcpy(PrereqHash.Hash, (*FoundFileManifest)->SHA1Hash.Hash, FSHA1::DigestSize);
			ManifestMeta.PrereqIds.Add(PrereqHash.ToString());
		}
	}

	// Get the DataGroupList
	TSharedPtr<FJsonValue> JsonDataGroupList = JsonValueMap.FindRef(TEXT("DataGroupList"));
	if (JsonDataGroupList.IsValid())
	{
		TSharedPtr<FJsonObject> JsonDataGroupListObj = JsonDataGroupList->AsObject();
		for (auto DataGroupIt = JsonDataGroupListObj->Values.CreateConstIterator(); DataGroupIt && bSuccess; ++DataGroupIt)
		{
			FGuid DataGuid;
			uint8 DataGroup = INDEX_NONE;
			// If the list exists, we must be able to parse it ok otherwise error
			bSuccess = bSuccess && FGuid::Parse(DataGroupIt.Key(), DataGuid);
			bSuccess = bSuccess && FromStringBlob(DataGroupIt.Value()->AsString(), DataGroup);
			if (bSuccess && MutableChunkInfoLookup.Contains(DataGuid))
			{
				FChunkInfo* ChunkInfoData = MutableChunkInfoLookup[DataGuid];
				ChunkInfoData->GroupNumber = DataGroup;
			}
		}
	}
	else if (bSuccess)
	{
		// If the list did not exist in the manifest then the grouping is the deprecated crc functionality, as long
		// as there are no previous parsing errors we can build the group list from the Guids.
		for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList)
		{
			ChunkInfo.GroupNumber = FCrc::MemCrc_DEPRECATED(&ChunkInfo.Guid, sizeof(FGuid)) % 100;
		}
	}

	// Get the ChunkFilesizeList
	bool bHasChunkFilesizeList = false;
	TSharedPtr< FJsonValue > JsonChunkFilesizeList = JsonValueMap.FindRef(TEXT("ChunkFilesizeList"));
	if (JsonChunkFilesizeList.IsValid())
	{
		TSharedPtr< FJsonObject > JsonChunkFilesizeListObj = JsonChunkFilesizeList->AsObject();
		for (auto ChunkFilesizeIt = JsonChunkFilesizeListObj->Values.CreateConstIterator(); ChunkFilesizeIt; ++ChunkFilesizeIt)
		{
			FGuid ChunkGuid;
			int64 ChunkSize = 0;
			if (FGuid::Parse(ChunkFilesizeIt.Key(), ChunkGuid))
			{
				FromStringBlob(ChunkFilesizeIt.Value()->AsString(), ChunkSize);
				if (MutableChunkInfoLookup.Contains(ChunkGuid))
				{
					FChunkInfo* ChunkInfoData = MutableChunkInfoLookup[ChunkGuid];
					ChunkInfoData->FileSize = ChunkSize;
					bHasChunkFilesizeList = true;
				}
			}
		}
	}
	if (bHasChunkFilesizeList == false)
	{
		// Missing chunk list, version before we saved them compressed. Assume original fixed chunk size of 1 MiB.
		for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList)
		{
			ChunkInfo.FileSize = 1048576;
		}
	}

	// Get the bIsFileData value. The variable will exist in versions of StoresIfChunkOrFileData or later, otherwise the previous method is to check
	// if ChunkHashList is empty.
	TSharedPtr<FJsonValue> JsonIsFileData = JsonValueMap.FindRef(TEXT("bIsFileData"));
	if (JsonIsFileData.IsValid() && JsonIsFileData->Type == EJson::Boolean)
	{
		ManifestMeta.bIsFileData = JsonIsFileData->AsBool();
	}
	else
	{
		ManifestMeta.bIsFileData = !bHasChunkHashList;
	}

	// Get the custom fields. This is optional, and should not fail if it does not exist
	TSharedPtr< FJsonValue > JsonCustomFields = JsonValueMap.FindRef( TEXT( "CustomFields" ) );
	if( JsonCustomFields.IsValid() )
	{
		TSharedPtr< FJsonObject > JsonCustomFieldsObj = JsonCustomFields->AsObject();
		for( auto CustomFieldIt = JsonCustomFieldsObj->Values.CreateConstIterator(); CustomFieldIt && bSuccess; ++CustomFieldIt )
		{
			CustomFields.Fields.Add(FString(CustomFieldIt.Key()), CustomFieldIt.Value()->AsString());
		}
	}

	// If this is file data, fill out the guid to filename lookup, and chunk file size and SHA.
	if (ManifestMeta.bIsFileData)
	{
		for (FFileManifest& FileManifest : FileManifestList.FileList)
		{
			if (FileManifest.ChunkParts.Num() == 1)
			{
				const FGuid& Guid = FileManifest.ChunkParts[0].Guid;
				FileNameLookup.Add(Guid, &FileManifest.Filename);
				if (MutableChunkInfoLookup.Contains(Guid))
				{
					FChunkInfo* ChunkInfoData = MutableChunkInfoLookup[Guid];
					ChunkInfoData->FileSize = FileManifest.FileSize;
					ChunkInfoData->ShaHash = FileManifest.SHA1Hash;
				}
			}
			else
			{
				bSuccess = false;
			}
		}
	}

	// Setup build id from backwards compat route
	ManifestMeta.BuildId = FBuildPatchUtils::GetBackwardsCompatibleBuildId(ManifestMeta);

	// Copy FeatureLevel to each complex serialisable member
	ChunkDataList.FeatureLevel = ManifestMeta.FeatureLevel;
	FileManifestList.FeatureLevel = ManifestMeta.FeatureLevel;
	CustomFields.FeatureLevel = ManifestMeta.FeatureLevel;

	// Call OnPostLoad for the file manifest list
	FileManifestList.OnPostLoad();

	// Mark as should be re-saved, client that stores manifests should start using binary
	bNeedsResaving = true;

	// Setup internal lookups
	InitLookups();

	// Make sure we don't have any half loaded data
	if( !bSuccess )
	{
		DestroyData();
	}

	return bSuccess;
}

void FBuildPatchAppManifest::GetChunksRequiredForFiles(const TSet<FString>& Filenames, TSet<FGuid>& RequiredChunks) const
{
	for (const FString& Filename : Filenames)
	{
		const FFileManifest* FileManifest = GetFileManifest(Filename);
		if (FileManifest != nullptr)
		{
			for (const FChunkPart& ChunkPart : FileManifest->ChunkParts)
			{
				RequiredChunks.Add(ChunkPart.Guid);
			}
		}
	}
}

int64 FBuildPatchAppManifest::GetDownloadSize() const
{
	return TotalDownloadSize;
}

int64 FBuildPatchAppManifest::GetDownloadSize(const TSet<FString>& Tags) const
{
	return GetDownloadSize(GetTaggedFileManifests(Tags));
}

int64 FBuildPatchAppManifest::GetDownloadSize(const TSet<const FFileManifest*>& TaggedFiles) const
{
	// For each tagged file and for each new chunk we find we add the download size for it.
	TSet<FGuid> RequiredChunks;
	int64 TotalSize = 0;
	for (const FFileManifest* File : TaggedFiles)
	{
		for (const FChunkPart& ChunkPart : File->ChunkParts)
		{
			bool bAlreadyInSet;
			RequiredChunks.Add(ChunkPart.Guid, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				const FChunkInfo* const* ChunkInfo = ChunkInfoLookup.Find(ChunkPart.Guid);
				if (ChunkInfo != nullptr)
				{
					TotalSize += (*ChunkInfo)->FileSize;
				}
			}
		}
	}
	return TotalSize;
}

int64 FBuildPatchAppManifest::GetDeltaDownloadSize(const TSet<FString>& Tags, const IBuildManifestRef& PreviousVersion) const
{
	return GetDeltaDownloadSize(Tags, PreviousVersion, Tags);
}

int64 FBuildPatchAppManifest::GetDeltaDownloadSize(const TSet<FString>& InTags, const IBuildManifestRef& InPreviousVersion, const TSet<FString>& InPreviousTags) const
{
	const TSet<FString>* Tags = &InTags;
	FBuildPatchAppManifestRef PreviousVersion = StaticCastSharedRef< FBuildPatchAppManifest >(InPreviousVersion);
	const TSet<FString>* PreviousTags = &InPreviousTags;
	TSet<FString> TagsManifest, PreviousTagsManifest;
	if (Tags->Num() == 0)
	{
		GetFileTagList(TagsManifest);
		Tags = &TagsManifest;
	}
	if (PreviousTags->Num() == 0)
	{
		PreviousVersion->GetFileTagList(PreviousTagsManifest);
		PreviousTags = &PreviousTagsManifest;
	}

	// Enumerate what is available.
	TSet<FString> FilesInstalled;
	TSet<FGuid> ChunksRequired;
	TSet<FGuid> ChunksInstalled;
	PreviousVersion->GetTaggedFileList(*PreviousTags, FilesInstalled);
	PreviousVersion->GetChunksRequiredForFiles(FilesInstalled, ChunksRequired);
	PreviousVersion->EnumerateProducibleChunks(*PreviousTags, ChunksRequired, ChunksInstalled);

	// Enumerate what is needed for the update.
	TSet<FString> AllTaggedFiles;
	GetTaggedFileList(*Tags, AllTaggedFiles);

	return GetDeltaDownloadSize(AllTaggedFiles, InPreviousVersion, FilesInstalled, ChunksInstalled);
}

int64 FBuildPatchAppManifest::GetDeltaDownloadSize(const TSet<const FFileManifest*>& InTaggedManifests, const IBuildManifestRef& InPreviousVersion, const TSet<FString>& InFilesInstalled, const TSet<FGuid>& InChunksInstalled) const
{
	// Enumerate what is needed for the update.
	TSet<FString> AllTaggedFiles;
	Algo::Transform(InTaggedManifests, AllTaggedFiles, [](const FFileManifest* File) { return File->Filename; });

	return GetDeltaDownloadSize(AllTaggedFiles, InPreviousVersion, InFilesInstalled, InChunksInstalled);
}

int64 FBuildPatchAppManifest::GetDeltaDownloadSize(const TSet<FString>& InAllTaggedFiles, const IBuildManifestRef& InPreviousVersion, const TSet<FString>& InFilesInstalled, const TSet<FGuid>& InChunksInstalled) const
{
	FBuildPatchAppManifestRef PreviousVersion = StaticCastSharedRef< FBuildPatchAppManifest >(InPreviousVersion);

	// Enumerate what has changed.
	FString DummyString;
	TSet<FString> OutdatedFiles;
	GetOutdatedFiles(&PreviousVersion.Get(), DummyString, InAllTaggedFiles, OutdatedFiles);

	const TSet<FString> NewFilesNeeded = InAllTaggedFiles.Difference(InFilesInstalled);
	const TSet<FString> UpdatedFilesNeeded = OutdatedFiles.Intersect(InAllTaggedFiles);
	const TSet<FString> FilesNeeded = NewFilesNeeded.Union(UpdatedFilesNeeded);

	TSet<FGuid> ChunksNeeded;
	GetChunksRequiredForFiles(FilesNeeded, ChunksNeeded);
	ChunksNeeded = ChunksNeeded.Difference(InChunksInstalled);

	// Return download size of required chunks.
	return GetDataSize(ChunksNeeded);
}

int64 FBuildPatchAppManifest::GetBuildSize() const
{
	return TotalBuildSize;
}

int64 FBuildPatchAppManifest::GetBuildSize(const TSet<FString>& Tags) const
{
	return GetBuildSize(GetTaggedFileManifests(Tags));
}

int64 FBuildPatchAppManifest::GetBuildSize(const TSet<const FFileManifest*>& TaggedFiles) const
{
	// For each tag file and for each new file we find we add the size for it.
	return Algo::Accumulate<int64>(TaggedFiles, 0, [](int64 Result, const FFileManifest* File) { return Result + File->FileSize; });
}

TArray<FString> FBuildPatchAppManifest::GetBuildFileList() const
{
	TArray<FString> Filenames;
	GetFileList(Filenames);
	return Filenames;
}

TArray<FStringView> FBuildPatchAppManifest::GetBuildFileListView() const
{
	TArray<FStringView> Filenames;
	GetFileList(Filenames);
	return Filenames;
}

TArray<FString> FBuildPatchAppManifest::GetBuildFileList(const TSet<FString>& Tags) const
{
	TArray<FString> Filenames;
	GetTaggedFileList(Tags, Filenames);
	return Filenames;
}

TArray<FStringView> FBuildPatchAppManifest::GetBuildFileListView(const TSet<FString>& Tags) const
{
	TArray<FStringView> Filenames;
	GetTaggedFileList(Tags, Filenames);
	return Filenames;
}

int64 FBuildPatchAppManifest::GetFileSize(const TArray<FString>& Filenames) const
{
	return Algo::Accumulate<int64>(Filenames, 0, [this](int64 Size, const FString& Filename){ return Size + GetFileSize(Filename); });
}

int64 FBuildPatchAppManifest::GetFileSize(const TSet<FString>& Filenames) const
{
	return Algo::Accumulate<int64>(Filenames, 0, [this](int64 Size, const FString& Filename){ return Size + GetFileSize(Filename); });
}

int64 FBuildPatchAppManifest::GetFileSize(FStringView Filename) const
{
	const FFileManifest *const *const FileManifest = FileManifestLookup.Find(Filename);
	if (FileManifest != nullptr)
	{
		return (*FileManifest)->FileSize;
	}
	return 0;
}

EFileMetaFlags FBuildPatchAppManifest::GetFileMetaFlags(const FString& Filename) const
{
	const FFileManifest *const *const FileManifest = FileManifestLookup.Find(Filename);
	if (FileManifest != nullptr)
	{
		return (*FileManifest)->FileMetaFlags;
	}
	return EFileMetaFlags::None;
}

const FMD5Hash* FBuildPatchAppManifest::GetFileMD5Hash(const FString& Filename) const
{
	if (GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType)
	{
		const FFileManifest* const* const FileManifest = FileManifestLookup.Find(Filename);
		if (FileManifest != nullptr)
		{
			return &(*FileManifest)->MD5Hash;
		}
	}
	return nullptr;
}

const FSHAHash* FBuildPatchAppManifest::GetFileSHA1Hash(const FString& Filename) const
{
	const FFileManifest* const* const FileManifest = FileManifestLookup.Find(Filename);
	if (FileManifest != nullptr)
	{
		return &(*FileManifest)->SHA1Hash;
	}
	return nullptr;
}

const FSHA256Signature* FBuildPatchAppManifest::GetFileSHA256Hash(const FString& Filename) const
{
	if (GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes)
	{
		const FFileManifest* const* const FileManifest = FileManifestLookup.Find(Filename);
		if (FileManifest != nullptr)
		{
			return &(*FileManifest)->SHA256Hash;
		}
	}
	return nullptr;
}

const FString* FBuildPatchAppManifest::GetFileMIMEType(const FString& Filename) const
{
	if (GetFeatureLevel() >= BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType)
	{
		const FFileManifest* const* const FileManifest = FileManifestLookup.Find(Filename);
		if (FileManifest != nullptr)
		{
			return &(*FileManifest)->MIMEType;
		}
	}
	return nullptr;
}

int64 FBuildPatchAppManifest::GetDataSize(const FGuid& DataGuid) const
{
	if (ChunkInfoLookup.Contains(DataGuid))
	{
		// Chunk file sizes are stored in the info
		return ChunkInfoLookup[DataGuid]->FileSize;
	}
	else if (ManifestMeta.bIsFileData)
	{
		// For file data, the file must exist in the list
		check(FileNameLookup.Contains(DataGuid));
		return GetFileSize(*FileNameLookup[DataGuid]);
	}
	else
	{
		// Default chunk size to be the original fixed data size of 1 MiB. Inaccurate, but represents original behavior.
		return 1024 * 1024;
	}
}

int64 FBuildPatchAppManifest::GetDataSize(const TArray<FGuid>& DataGuids) const
{
	return Algo::Accumulate<int64>(DataGuids, 0, [this](int64 Size, const FGuid& DataGuid){ return Size + GetDataSize(DataGuid); });
}

int64 FBuildPatchAppManifest::GetDataSize(const TSet<FGuid>& DataGuids) const
{
	return Algo::Accumulate<int64>(DataGuids, 0, [this](int64 Size, const FGuid& DataGuid){ return Size + GetDataSize(DataGuid); });
}

uint32 FBuildPatchAppManifest::GetNumFiles() const
{
	return FileManifestList.FileList.Num();
}

void FBuildPatchAppManifest::GetFileList(TArray<FString>& Filenames) const
{
	Algo::Transform(FileManifestList.FileList, Filenames, &FFileManifest::Filename);
}

void FBuildPatchAppManifest::GetFileList(TArray<FStringView>& Filenames) const
{
	Algo::Transform(FileManifestList.FileList, Filenames, &FFileManifest::Filename);
}

void FBuildPatchAppManifest::GetFileList(TSet<FString>& Filenames) const
{
	Algo::Transform(FileManifestList.FileList, Filenames, &FFileManifest::Filename);
}

TSet<FString> FBuildPatchAppManifest::GetFileTagList() const
{
	TSet<FString> TagSet;
	GetFileTagList(TagSet);
	return TagSet;
}

void FBuildPatchAppManifest::GetFileTagList(TSet<FString>& Tags) const
{
	Algo::Transform(TaggedFilesLookup, Tags, &TPair<FString, TArray<const BuildPatchServices::FFileManifest*>>::Key);
}

void FBuildPatchAppManifest::GetTaggedFileList(const TSet<FString>& Tags, TArray<FString>& TaggedFiles) const
{
	Algo::Transform(GetTaggedFileManifests(Tags), TaggedFiles, [&](const FFileManifest* FileManifest) { return FileManifest->Filename; });
}

void FBuildPatchAppManifest::GetTaggedFileList(const TSet<FString>& Tags, TArray<FStringView>& TaggedFiles) const
{
	for (const FString& Tag : Tags)
	{
		const TArray<const FFileManifest*>* const Files = TaggedFilesLookup.Find(Tag);
		if (Files != nullptr)
		{
			for (const FFileManifest* File : *Files)
			{
				TaggedFiles.Add(File->Filename);
			}
		}
	}
}

void FBuildPatchAppManifest::GetTaggedFileList(const TSet<FString>& Tags, TSet<FString>& TaggedFiles) const
{
	Algo::Transform(GetTaggedFileManifests(Tags), TaggedFiles, [&](const FFileManifest* FileManifest) { return FileManifest->Filename; });
}

TSet<const FFileManifest*> FBuildPatchAppManifest::GetTaggedFileManifests(const TSet<FString>& Tags) const
{
	TSet<const FFileManifest*> TaggedFiles;
	for (const FString& Tag : Tags)
	{
		if (const TArray<const FFileManifest*>* const Files = TaggedFilesLookup.Find(Tag))
		{
			TaggedFiles.Append(*Files);
		}
	}
	return TaggedFiles;
}

void FBuildPatchAppManifest::GetDataList(TArray<FGuid>& DataGuids) const
{
	Algo::Transform(ChunkDataList.ChunkList, DataGuids, &FChunkInfo::Guid);
}

void FBuildPatchAppManifest::GetDataList(TSet<FGuid>& DataGuids) const
{
	Algo::Transform(ChunkDataList.ChunkList, DataGuids, &FChunkInfo::Guid);
}

const FFileManifest* FBuildPatchAppManifest::GetFileManifest(const FString& Filename) const
{
	const FFileManifest* const * FileManifest = FileManifestLookup.Find(Filename);
	return (FileManifest != nullptr) ? (*FileManifest) : nullptr;
}

TArray<FFileManifest>::TConstIterator FBuildPatchAppManifest::GetFileManifestIterator() const
{
	return FileManifestList.FileList.CreateConstIterator();
}

bool FBuildPatchAppManifest::IsFileDataManifest() const
{
	return ManifestMeta.bIsFileData;
}

bool FBuildPatchAppManifest::GetChunkHash(const FGuid& ChunkGuid, uint64& OutHash) const
{
	const FChunkInfo* const * ChunkInfo = ChunkInfoLookup.Find(ChunkGuid);
	if (ChunkInfo != nullptr)
	{
		OutHash = (*ChunkInfo)->Hash;
		return true;
	}
	return false;
}

bool FBuildPatchAppManifest::GetChunkShaHash(const FGuid& ChunkGuid, FSHAHash& OutHash) const
{
	static const uint8 Zero[FSHA1::DigestSize] = {0};
	const FChunkInfo* const * ChunkInfo = ChunkInfoLookup.Find(ChunkGuid);
	if (ChunkInfo != nullptr)
	{
		OutHash = (*ChunkInfo)->ShaHash;
		return FMemory::Memcmp(OutHash.Hash, Zero, FSHA1::DigestSize) != 0;
	}
	return false;
}

bool FBuildPatchAppManifest::GetChunkEncryptionSecretId(const FGuid& ChunkGuid, FGuid& OutEncryptionSecretId) const
{
	const FChunkInfo* const* ChunkInfo = ChunkInfoLookup.Find(ChunkGuid);
	if (ChunkInfo != nullptr)
	{
		OutEncryptionSecretId = (*ChunkInfo)->EncryptionSecretId;
		return OutEncryptionSecretId.IsValid();
	}
	return false;
}

const BuildPatchServices::FChunkInfo* FBuildPatchAppManifest::GetChunkInfo(const FGuid& ChunkGuid) const
{
	const BuildPatchServices::FChunkInfo* const * const ChunkInfoPtrPtr = ChunkInfoLookup.Find(ChunkGuid);
	const BuildPatchServices::FChunkInfo* const ChunkInfoPtr = ChunkInfoPtrPtr == nullptr ? nullptr : *ChunkInfoPtrPtr;
	return ChunkInfoPtr;
}

bool FBuildPatchAppManifest::GetFileHash(const FGuid& FileGuid, FSHAHash& OutHash) const
{
	const FString* const * FoundFilename = FileNameLookup.Find(FileGuid);
	if (FoundFilename != nullptr)
	{
		return GetFileHash(**FoundFilename, OutHash);
	}
	return false;
}

bool FBuildPatchAppManifest::GetFileHash(const FString& Filename, FSHAHash& OutHash) const
{
	const FFileManifest* const * FoundFileManifest = FileManifestLookup.Find(Filename);
	if (FoundFileManifest != nullptr)
	{
		FMemory::Memcpy(OutHash.Hash, (*FoundFileManifest)->SHA1Hash.Hash, FSHA1::DigestSize);
		return true;
	}
	return false;
}

bool FBuildPatchAppManifest::GetFilePartHash(const FGuid& FilePartGuid, uint64& OutHash) const
{
	const FChunkInfo* const * FilePartInfo = ChunkInfoLookup.Find(FilePartGuid);
	if (FilePartInfo != nullptr)
	{
		OutHash = (*FilePartInfo)->Hash;
		return true;
	}
	return false;
}

BuildPatchServices::EFeatureLevel FBuildPatchAppManifest::GetFeatureLevel() const
{
	return ManifestMeta.FeatureLevel;
}

uint32 FBuildPatchAppManifest::GetAppID() const
{
	return ManifestMeta.AppID;
}

const FString& FBuildPatchAppManifest::GetAppName() const
{
	return ManifestMeta.AppName;
}

const FString& FBuildPatchAppManifest::GetVersionString() const
{
	return ManifestMeta.BuildVersion;
}

const FString& FBuildPatchAppManifest::GetUniqueBuildId() const
{
	return ManifestMeta.BuildId;
}

const FGuid& FBuildPatchAppManifest::GetEncryptionSecretId(bool* bOutIsManifestEncrypted) const
{
	if (bOutIsManifestEncrypted)
	{
		*bOutIsManifestEncrypted = bIsManifestEncrypted;
	}
	return EncryptionSecretId;
}

bool FBuildPatchAppManifest::IsManifestEncrypted() const
{
	return bIsManifestEncrypted;
}

bool FBuildPatchAppManifest::IsOriginallyFullyEncrypted() const
{
	return EncryptionSecretId.IsValid() 
		&& Algo::AllOf(ChunkDataList.ChunkList, [](const FChunkInfo& Info) { return Info.EncryptionSecretId.IsValid(); });
}

TSet<FGuid> FBuildPatchAppManifest::GetNecessaryEncryptionSecretIds() const
{
	TSet<FGuid> EncryptionSecretIds;
	EncryptionSecretIds.Add(EncryptionSecretId);
	Algo::Transform(ChunkDataList.ChunkList, EncryptionSecretIds, &FChunkInfo::EncryptionSecretId);
	EncryptionSecretIds.Remove(FGuid());
	return EncryptionSecretIds;
}

const FString& FBuildPatchAppManifest::GetLaunchExe() const
{
	return ManifestMeta.LaunchExe;
}

const FString& FBuildPatchAppManifest::GetLaunchCommand() const
{
	return ManifestMeta.LaunchCommand;
}

const TSet<FString>& FBuildPatchAppManifest::GetPrereqIds() const
{
	return ManifestMeta.PrereqIds;
}

const FString& FBuildPatchAppManifest::GetPrereqName() const
{
	return ManifestMeta.PrereqName;
}

const FString& FBuildPatchAppManifest::GetPrereqPath() const
{
	return ManifestMeta.PrereqPath;
}

const FString& FBuildPatchAppManifest::GetPrereqArgs() const
{
	return ManifestMeta.PrereqArgs;
}

const FString& FBuildPatchAppManifest::GetUninstallActionPath() const
{
	return ManifestMeta.UninstallActionPath;
}

const FString& FBuildPatchAppManifest::GetUninstallActionArgs() const
{
	return ManifestMeta.UninstallActionArgs;
}

const FString& FBuildPatchAppManifest::GetBuildId() const
{
	return ManifestMeta.BuildId;
}

IBuildManifestRef FBuildPatchAppManifest::Duplicate() const
{
	return MakeShareable(new FBuildPatchAppManifest(*this));
}

void FBuildPatchAppManifest::CopyCustomFields(const IBuildManifestRef& InOther, bool bClobber)
{
	// Cast manifest parameters
	FBuildPatchAppManifestRef Other = StaticCastSharedRef< FBuildPatchAppManifest >(InOther);

	for (const TPair<FString, FString>& CustomField : Other->CustomFields.Fields)
	{
		if (bClobber || !CustomFields.Fields.Contains(CustomField.Key))
		{
			CustomFields.Fields.Add(CustomField.Key, CustomField.Value);
		}
	}
}

TArray<FString> FBuildPatchAppManifest::GetCustomFieldNames() const
{
	TArray<FString> CustomFieldNames;
	CustomFields.Fields.GenerateKeyArray(CustomFieldNames);
	return CustomFieldNames;
}

const IManifestFieldPtr FBuildPatchAppManifest::GetCustomField(const FString& FieldName) const
{
	IManifestFieldPtr Return;
	if (CustomFields.Fields.Contains(FieldName))
	{
		Return = MakeShareable(new FBuildPatchCustomField(CustomFields.Fields[FieldName]));
	}
	return Return;
}

const IManifestFieldPtr FBuildPatchAppManifest::SetCustomField(const FString& FieldName, const FString& Value)
{
	CustomFields.Fields.Add(FieldName, Value);
	return GetCustomField(FieldName);
}

const IManifestFieldPtr FBuildPatchAppManifest::SetCustomField(const FString& FieldName, const double& Value)
{
	return SetCustomField(FieldName, ToStringBlob(Value));
}

const IManifestFieldPtr FBuildPatchAppManifest::SetCustomField(const FString& FieldName, const int64& Value)
{
	return SetCustomField(FieldName, ToStringBlob(Value));
}

void FBuildPatchAppManifest::RemoveCustomField(const FString& FieldName)
{
	CustomFields.Fields.Remove(FieldName);
}


int32 FBuildPatchAppManifest::EnumerateProducibleChunks(const TSet<FString>& TagSet, const TSet<FGuid>& ChunksRequired, TSet<FGuid>& ChunksAvailable) const
{
	TSet<FString> AvailableFiles;
	GetTaggedFileList(TagSet, AvailableFiles);
	return EnumerateProducibleChunks_Internal([&](const FString& Filename) { return AvailableFiles.Contains(Filename); }, ChunksRequired, ChunksAvailable);
}

int32 FBuildPatchAppManifest::EnumerateProducibleChunks(const TSet<const FFileManifest*>& TaggedFiles, const TSet<FGuid>& ChunksRequired, TSet<FGuid>& ChunksAvailable) const
{
	TSet<FString> AvailableFiles;
	Algo::Transform(TaggedFiles, AvailableFiles, [](const FFileManifest* File) { return File->Filename; });
	return EnumerateProducibleChunks_Internal([&](const FString& Filename) { return AvailableFiles.Contains(Filename); }, ChunksRequired, ChunksAvailable);
}

int32 FBuildPatchAppManifest::EnumerateProducibleChunks_Internal(const TFunction<bool(const FString&)>& FileAccessChecker, const TSet<FGuid>& ChunksRequired, TSet<FGuid>& ChunksAvailable) const
{
	int32 Count = 0;
	TMap<FGuid, FBlockStructure> ChunkBlockStructures;
	ChunkBlockStructures.Reserve(ChunksRequired.Num());
	// Re-assemble the information about chunk available blocks from *.e or *.c files. 
	for (const FFileManifest& FileManifest : FileManifestList.FileList)
	{
		const bool bHasFile = FileAccessChecker(FileManifest.Filename);
		if (bHasFile)
		{
			for (const FChunkPart& ChunkPart : FileManifest.ChunkParts)
			{
				if (ChunksRequired.Contains(ChunkPart.Guid))
				{
					FBlockStructure& FoundChunkParts = ChunkBlockStructures.FindOrAdd(ChunkPart.Guid);
					FoundChunkParts.Add(ChunkPart.Offset, ChunkPart.Size, ESearchDir::FromEnd);
				}
			}
		}
	}
	for (const TPair<FGuid, FBlockStructure>& ChunkBlockStructurePair : ChunkBlockStructures)
	{
		const FGuid& ChunkBlockId = ChunkBlockStructurePair.Key;
		const FBlockStructure& ChunkBlocks = ChunkBlockStructurePair.Value;
		const FChunkInfo* ChunkInfoPtr = ChunkInfoLookup[ChunkBlockId];
		// A chunk is treated as fully uploaded if there are no gaps between the Head and Tail, 
		// and the total block size is equal to the total expected chunk size.
		if (ChunkBlocks.GetHead() == ChunkBlocks.GetTail() && ChunkBlocks.GetHead() && ChunkBlocks.GetHead()->GetSize() == ChunkInfoPtr->DataSizeUncompressed)
		{
			ChunksAvailable.Add(ChunkBlockId);
			++Count;
		}
	}
	return Count;
}


bool FBuildPatchAppManifest::HasFileAttributes() const
{
	for (const FFileManifest& FileManifest : FileManifestList.FileList)
	{
		if (FileManifest.FileMetaFlags != EFileMetaFlags::None)
		{
			return true;
		}
	}
	return false;
}

bool FBuildPatchAppManifest::SerialiseSensitiveFields(FArchive& Ar)
{
	// If the metadata is changed, we need to consider those changes here and work by version.
	// If the new featurelevel does not add fields that require encryption, just bump the assert here.
	static_assert((int32)EFeatureLevel::Latest == 24, "Please check the below code is still of for new feature level, and update this assert.");

	// This is the list of metadata fields that get encrypted.
	Ar << ManifestMeta.LaunchExe;
	Ar << ManifestMeta.LaunchCommand;
	Ar << ManifestMeta.PrereqIds;
	Ar << ManifestMeta.PrereqName;
	Ar << ManifestMeta.PrereqPath;
	Ar << ManifestMeta.PrereqArgs;
	Ar << ManifestMeta.UninstallActionPath;
	Ar << ManifestMeta.UninstallActionArgs;

	// Next, serialise all of the potentially sensitive FileManifestList fields.
	// Unfortunately, we cannot be const correct here due to FArchive API.
	for (FFileManifest& FileManifest : FileManifestList.FileList)
	{
		Ar << FileManifest.Filename;
		Ar << FileManifest.SymlinkTarget;
	}

	return !Ar.IsError();
}

void FBuildPatchAppManifest::ObfuscateSensitiveFields()
{
	// If the metadata is changed, we need to consider those changes here and work by version.
	// If the new featurelevel does not add fields that require encryption, just bump the assert here.
	static_assert((int32)EFeatureLevel::Latest == 24, "Please check the below code is still of for new feature level, and update this assert.");

	ManifestMeta.LaunchExe = ManifestDataHelpers::ObfuscateString(ManifestMeta.LaunchExe);
	ManifestMeta.LaunchCommand = ManifestDataHelpers::ObfuscateString(ManifestMeta.LaunchCommand);
	ManifestMeta.PrereqIds = ManifestDataHelpers::ObfuscateStrings(ManifestMeta.PrereqIds);
	ManifestMeta.PrereqName = ManifestDataHelpers::ObfuscateString(ManifestMeta.PrereqName);
	ManifestMeta.PrereqPath = ManifestDataHelpers::ObfuscateString(ManifestMeta.PrereqPath);
	ManifestMeta.PrereqArgs = ManifestDataHelpers::ObfuscateString(ManifestMeta.PrereqArgs);
	ManifestMeta.UninstallActionPath = ManifestDataHelpers::ObfuscateString(ManifestMeta.UninstallActionPath);
	ManifestMeta.UninstallActionArgs = ManifestDataHelpers::ObfuscateString(ManifestMeta.UninstallActionArgs);
	for (FFileManifest& FileManifest : FileManifestList.FileList)
	{
		FileManifest.Filename = ManifestDataHelpers::ObfuscateString(FileManifest.Filename);
		FileManifest.SymlinkTarget = ManifestDataHelpers::ObfuscateString(FileManifest.SymlinkTarget);
	}
}

bool FBuildPatchAppManifest::EncryptData(const FGuid& InEncryptionSecretId, const TArray<uint8>& InEncryptionSecretKey)
{
	using namespace BuildPatchServices;
	checkf(!bIsManifestEncrypted, TEXT("Cannot encrypt already encrypted manifest file."));
	checkf(GetFeatureLevel() >= EFeatureLevel::ManifestEncryptionSupport, TEXT("Cannot encrypt a manifest unless the support is new enough"));

	TArray<uint8> DataToEncrypt;
	FMemoryWriter RawAr(DataToEncrypt);
	if (SerialiseSensitiveFields(RawAr))
	{
		// Now compress,because usually the sensitive manifest string fields would compress well, but if we encrypt without compression, we risk increasing the manifest file size dramatically.
		FCipherHeader CipherHeader;
		CipherHeader.DataSizeUncompressed = DataToEncrypt.Num();
		CipherHeader.DataStoredAs = ManifestDataHelpers::CompressManifestMemory(DataToEncrypt);
		CipherHeader.DataSizeCompressed = DataToEncrypt.Num();

		// Encrypt the compressed data.
		TUniquePtr<ICrypto> Crypto{ FCryptoFactory::Create() };
		TArray<uint8> AuthTag;
		CipherHeader.InitializationVector.SetNumUninitialized(AES256_GCM_InitializationVectorSizeInBytes);
		Crypto->CreateRandomBytes(CipherHeader.InitializationVector);
		bool bEncryptResult = false;
		TArray<uint8> Ciphertext = Crypto->Encrypt_AES_256_GCM(DataToEncrypt, InEncryptionSecretKey, CipherHeader.InitializationVector, AuthTag, bEncryptResult);
		EnumAddFlags(CipherHeader.DataStoredAs, EManifestStorageFlags::Encrypted);

		// Create the actual encryption data block, we don't save the SecretId or AuthTag here since that will get saved in the manifest header.
		EncryptedData.Data.Reset();
		FMemoryWriter FinalAr(EncryptedData.Data);
		FinalAr << CipherHeader;
		FinalAr << Ciphertext;

		if (bEncryptResult && !FinalAr.IsError())
		{
			ObfuscateSensitiveFields();
			bIsManifestEncrypted = true;
			FMemory::Memcpy(EncryptionAuthTag.AuthTag, AuthTag.GetData(), BuildPatchServices::AES256_GCM_AuthTagSizeInBytes);
			EncryptionSecretId = InEncryptionSecretId;
		}
		else
		{
			UE_LOGF(LogBuildPatchManifest, Error, "FBuildPatchAppManifest::EncryptData: encryption cipher failed. %ls", *InEncryptionSecretId.ToString());
			bIsManifestEncrypted = false;
			EncryptedData = FEncryptedData();
			EncryptionAuthTag.Reset();
			EncryptionSecretId.Invalidate();
		}

		// Restore internal lookups either way
		InitLookups();
	}
	else
	{
		UE_LOGF(LogBuildPatchManifest, Error, "FBuildPatchAppManifest::EncryptData: SerialiseSensitiveFields failed. %ls", *InEncryptionSecretId.ToString());
	}
	return bIsManifestEncrypted;
}

bool FBuildPatchAppManifest::DecryptData(const TMap<FGuid, TArray<uint8>>& AvailableEncryptionSecrets)
{
	if (EncryptionSecretId.IsValid() && bIsManifestEncrypted)
	{
		const TArray<uint8>* EncryptionSecretKey = AvailableEncryptionSecrets.Find(EncryptionSecretId);
		if (EncryptionSecretKey != nullptr)
		{
			FCipherHeader CipherHeader;
			TArray<uint8> Ciphertext;
			// Read the required values from the encryption block.
			FMemoryReader EncryptedReader(EncryptedData.Data);
			EncryptedReader << CipherHeader;
			EncryptedReader << Ciphertext;

			// Attempt to decrypt the data
			TUniquePtr<ICrypto> Crypto{ FCryptoFactory::Create() };
			bool bDecryptResult = false;
			TArray<uint8> DecryptedData = Crypto->Decrypt_AES_256_GCM(Ciphertext, *EncryptionSecretKey, CipherHeader.InitializationVector, EncryptionAuthTag.AuthTag, bDecryptResult);
			if (bDecryptResult)
			{
				// Uncompress if necessary
				if (EnumHasAllFlags(CipherHeader.DataStoredAs, EManifestStorageFlags::Compressed))
				{
					ManifestDataHelpers::UncompressManifestMemory(DecryptedData, CipherHeader.DataSizeUncompressed);
				}

				FMemoryReader RawAr(DecryptedData);
				if (SerialiseSensitiveFields(RawAr))
				{
					// Remove the encryption data, leaving EncryptionSecretId intact for info.
					bIsManifestEncrypted = false;
					EncryptedData = FEncryptedData();
					EncryptionAuthTag.Reset();

					// Restore internal lookups.
					InitLookups();

					return true;
				}
				else
				{
					UE_LOGF(LogBuildPatchManifest, Error, "FBuildPatchAppManifest::DecryptData: SerialiseSensitiveFields failed. %ls", *EncryptionSecretId.ToString());
				}
			}
			else
			{
				UE_LOGF(LogBuildPatchManifest, Error, "FBuildPatchAppManifest::DecryptData: decryption cipher failed. %ls", *EncryptionSecretId.ToString());
			}
		}
		else
		{
			UE_LOGF(LogBuildPatchManifest, Error, "FBuildPatchAppManifest::DecryptData: encryption key missing. %ls", *EncryptionSecretId.ToString());
		}
	}
	return false;
}

void FBuildPatchAppManifest::GetRemovableFiles(const IBuildManifestRef& InOldManifest, TArray<FString>& RemovableFiles) const
{
	// Cast manifest parameters
	const FBuildPatchAppManifestRef OldManifest = StaticCastSharedRef<FBuildPatchAppManifest>(InOldManifest);
	GetRemovableFiles(OldManifest.Get(), RemovableFiles);
}

void FBuildPatchAppManifest::GetRemovableFiles(const FBuildPatchAppManifest& OldManifest, TArray<FString>& RemovableFiles) const
{
	// Simply put, any files that exist in the OldManifest file list, but do not in this manifest's file list, are assumed
	// to be files no longer required by the build
	for (const FFileManifest& OldFile : OldManifest.FileManifestList.FileList)
	{
		if (!FileManifestLookup.Contains(OldFile.Filename))
		{
			RemovableFiles.Add(OldFile.Filename);
		}
	}
}

void FBuildPatchAppManifest::GetRemovableFiles(const TCHAR* InInstallPath, TArray<FString>& RemovableFiles) const
{
	// For the logic below our InstallPath must be collapsed, normalised and end with a /
	// Normalising a directory removes trailing slashes, and we need the slash back on before calling CollapseRelativeDirectories.
	FString InstallPath = InInstallPath;
	FPaths::NormalizeDirectoryName(InstallPath);
	InstallPath += TEXT("/");
	FPaths::CollapseRelativeDirectories(InstallPath);

	TArray<FString> AllFiles;
	IFileManager::Get().FindFilesRecursive(AllFiles, *InstallPath, TEXT("*"), true, false);

#if PLATFORM_MAC
	// On Mac paths in manifest start with app bundle name
	if (InstallPath.EndsWith(".app/"))
	{
		InstallPath = FPaths::GetPath(InstallPath.LeftChop(1)) + "/";
	}
#endif

	for (int32 FileIndex = 0; FileIndex < AllFiles.Num(); ++FileIndex)
	{
		const FString& Filename = AllFiles[FileIndex].RightChop(InstallPath.Len());
		if (!FileManifestLookup.Contains(Filename))
		{
			RemovableFiles.Add(AllFiles[FileIndex]);
		}
	}
}

bool FBuildPatchAppManifest::NeedsResaving() const
{
	// The bool is marked during file load if we load an old version that should be upgraded
	return bNeedsResaving;
}

void FBuildPatchAppManifest::GetOutdatedFiles(const IBuildManifestRef& InOldManifest, TSet<FString>& OutdatedFiles) const
{
	GetOutdatedFiles(FBuildPatchAppManifestPtr(StaticCastSharedRef<FBuildPatchAppManifest>(InOldManifest)), TEXT(""), OutdatedFiles);
}

void FBuildPatchAppManifest::GetOutdatedFiles(const FBuildPatchAppManifestPtr& OldManifest, const FString& InstallDirectory, TSet<FString>& OutDatedFiles) const
{
	GetOutdatedFiles(OldManifest.Get(), InstallDirectory, OutDatedFiles);
}

void FBuildPatchAppManifest::GetOutdatedFiles(const FBuildPatchAppManifest* OldManifest, const FString& InstallDirectory, TSet<FString>& OutDatedFiles) const
{
	TSet<FString> FilesToCheck;
	GetFileList(FilesToCheck);
	GetOutdatedFiles(OldManifest, InstallDirectory, FilesToCheck, OutDatedFiles);
}

void FBuildPatchAppManifest::GetOutdatedFiles(const FBuildPatchAppManifest* OldManifest, const FString& InstallDirectory, const TSet<FString>& FilesToCheck, TSet<FString>& OutDatedFiles) const
{
	const bool bCheckExistingFile = InstallDirectory.IsEmpty() == false;
	if (nullptr == OldManifest)
	{
		// All files are outdated if no OldManifest
		TSet<FString> AllFiles;
		GetFileList(AllFiles);
		OutDatedFiles.Append(AllFiles.Intersect(FilesToCheck));
	}
	else
	{
		// Enumerate files in the this file list, that do not exist, or have different hashes in the OldManifest
		// to be files no longer required by the build
		TMap<FString, int64> OutdatedFileSizes = bCheckExistingFile ? ParallelGetFileSizes(FilesToCheck, InstallDirectory) : TMap<FString, int64>{};
		for (const FString& FileToCheck : FilesToCheck)
		{
			const FFileManifest* NewFile = GetFileManifest(FileToCheck);
			if (NewFile != nullptr)
			{
				// Check changed
				if (IsFileOutdated(*OldManifest, NewFile->Filename))
				{
					OutDatedFiles.Add(NewFile->Filename);
				}
				// Double check an unchanged file is not missing (size will be -1) or is incorrect size
				else if (bCheckExistingFile)
				{
					const int64* PrecalculatedSize = OutdatedFileSizes.Find(NewFile->Filename);
					const int64 ExistingFileSize = PrecalculatedSize ? *PrecalculatedSize : IFileManager::Get().FileSize(*(InstallDirectory / NewFile->Filename));
					if ((ExistingFileSize < 0) || (ExistingFileSize != NewFile->FileSize))
					{
						OutDatedFiles.Add(NewFile->Filename);
					}
				}
			}
		}
	}
}

void FBuildPatchAppManifest::GetMissedFiles(const FString& InstallDirectory, const TSet<FString>& FilesToCheck, TSet<FString>& OutMissedFiles) const
{
	ensure(!InstallDirectory.IsEmpty());
	if (InstallDirectory.IsEmpty())
	{
		return;
	}
	TSet<FString> Filenames;
	GetFileList(Filenames);
	// Enumerate files in the this file list, that do not exist on disk or missed in manifest
	const TMap<FString, int64> OutdatedFileSizes = ParallelGetFileSizes(FilesToCheck.Intersect(Filenames), InstallDirectory);
	
	for (const TPair<FString, int64>& Item : OutdatedFileSizes)
	{
		if (Item.Value < 0LL) // if file doesn't exist
		{
			OutMissedFiles.Add(Item.Key); // add to list of missing files.
		}
	}
}

bool FBuildPatchAppManifest::IsFileOutdated(const FBuildPatchAppManifestRef& OldManifest, const FString& Filename) const
{
	return IsFileOutdated(OldManifest.Get(), Filename);
}

bool FBuildPatchAppManifest::IsFileOutdated(const FBuildPatchAppManifest& OldManifest, const FString& Filename) const
{
	// If both app manifests are the same, return false as only repair would touch the file.
	if (&OldManifest == this)
	{
		return false;
	}
	// Get file manifests
	const FFileManifest* OldFile = OldManifest.GetFileManifest(Filename);
	const FFileManifest* NewFile = GetFileManifest(Filename);
	// Out of date if not in either manifest
	if (!OldFile || !NewFile)
	{
		return true;
	}
	// Different hash means different file
	if (OldFile->SHA1Hash != NewFile->SHA1Hash)
	{
		return true;
	}
	return false;
}

uint32 FBuildPatchAppManifest::GetNumberOfChunkReferences(const FGuid& ChunkGuid) const
{
	uint32 RefCount = 0;
	// For each file in the manifest, check if it has references to this chunk
	for (const FFileManifest& FileManifest : FileManifestList.FileList)
	{
		for (const FChunkPart& ChunkPart : FileManifest.ChunkParts)
		{
			if (ChunkPart.Guid == ChunkGuid)
			{
				++RefCount;
			}
		}
	}
	return RefCount;
}

#undef LOCTEXT_NAMESPACE
