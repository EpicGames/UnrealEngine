// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Interfaces/IBuildManifest.h"
#include "Data/ChunkData.h"
#include "Data/ManifestData.h"

class FBuildPatchAppManifest;
class FBuildPatchCustomField;

typedef TSharedPtr< class FBuildPatchCustomField, ESPMode::ThreadSafe > FBuildPatchCustomFieldPtr;
typedef TSharedRef< class FBuildPatchCustomField, ESPMode::ThreadSafe > FBuildPatchCustomFieldRef;
typedef TSharedPtr< class FBuildPatchAppManifest, ESPMode::ThreadSafe > FBuildPatchAppManifestPtr;
typedef TSharedRef< class FBuildPatchAppManifest, ESPMode::ThreadSafe > FBuildPatchAppManifestRef;

/**
 * Declare the FBuildPatchCustomField object class, which is the implementation of the object we return to
 * clients of the module
 */
class FBuildPatchCustomField
	: public IManifestField
{
public:
	/**
	 * Constructor taking the custom value
	 */
	FBuildPatchCustomField(const FString& Value);

	// START IBuildManifest Interface
	virtual FString AsString() const override;
	virtual double AsDouble() const override;
	virtual int64 AsInteger() const override;
	// END IBuildManifest Interface

private:
	/**
	 * Hide the default constructor
	 */
	FBuildPatchCustomField(){}

private:
	// Holds the underlying value
	FString CustomValue;
};

// Required to allow private access to manifest builder for now..
namespace BuildPatchServices
{
	class FBuildPatchInstaller;
	class FManifestBuilder;
	class FManifestData;
	class FDirectoryChunker;
	class FChunkDeltaOptimiser;
	class FBuildPatchManifestSet;
}

/**
 * Declare the FBuildPatchAppManifest object class. This holds the UObject data, and the implemented build manifest functionality
 */
class FBuildPatchAppManifest
	: public IBuildManifest
{
	// Allow access to build processor classes
	friend class FBuildDataGenerator;
	friend class FBuildDataFileProcessor;
	friend class BuildPatchServices::FBuildPatchInstaller;
	friend class BuildPatchServices::FManifestBuilder;
	friend class FBuildMergeManifests;
	friend class FBuildDiffManifests;
	friend class FManifestUObject;
	friend class BuildPatchServices::FManifestData;
	friend class BuildPatchServices::FDirectoryChunker;
	friend class BuildPatchServices::FChunkDeltaOptimiser;
	friend class BuildPatchServices::FBuildPatchManifestSet;
public:

	/**
	 * Default constructor
	 */
	FBuildPatchAppManifest();

	/**
	 * Basic details constructor
	 */
	FBuildPatchAppManifest(const uint32& InAppID, const FString& AppName);

	/**
	 * Copy constructor
	 */
	FBuildPatchAppManifest(const FBuildPatchAppManifest& Other);

	/**
	 * Default destructor
	 */
	~FBuildPatchAppManifest();

	// START IBuildManifest Interface
	virtual BuildPatchServices::EFeatureLevel GetFeatureLevel() const override;
	virtual uint32 GetAppID() const override;
	virtual const FString& GetAppName() const override;
	virtual const FString& GetVersionString() const override;
	virtual const FString& GetUniqueBuildId() const override;
	virtual const FGuid& GetEncryptionSecretId(bool* bOutIsManifestEncrypted = nullptr) const override;
	virtual bool IsManifestEncrypted() const override;
	virtual bool IsOriginallyFullyEncrypted() const override;
	virtual TSet<FGuid> GetNecessaryEncryptionSecretIds() const override;
	virtual const FString& GetLaunchExe() const override;
	virtual const FString& GetLaunchCommand() const override;
	virtual const TSet<FString>& GetPrereqIds() const override;
	virtual const FString& GetPrereqName() const override;
	virtual const FString& GetPrereqPath() const override;
	virtual const FString& GetPrereqArgs() const override;
	virtual const FString& GetUninstallActionPath() const override;
	virtual const FString& GetUninstallActionArgs() const override;
	virtual int64 GetDownloadSize() const override;
	virtual int64 GetDownloadSize(const TSet<FString>& Tags) const override;
	virtual int64 GetDownloadSize(const TSet<const BuildPatchServices::FFileManifest*>& TaggedFiles) const;
	virtual int64 GetDeltaDownloadSize(const TSet<FString>& Tags, const IBuildManifestRef& PreviousVersion) const override;
	virtual int64 GetDeltaDownloadSize(const TSet<FString>& Tags, const IBuildManifestRef& PreviousVersion, const TSet<FString>& PreviousTags) const override;
	virtual int64 GetDeltaDownloadSize(const TSet<const BuildPatchServices::FFileManifest*>& InTaggedManifests, const IBuildManifestRef& InPreviousVersion, const TSet<FString>& InFilesInstalled, const TSet<FGuid>& InChunksInstalled) const;
	virtual int64 GetDeltaDownloadSize(const TSet<FString>& AllTaggedFiles, const IBuildManifestRef& InPreviousVersion, const TSet<FString>& InFilesInstalled, const TSet<FGuid>& InChunksInstalled) const;
	virtual int64 GetBuildSize() const override;
	virtual int64 GetBuildSize(const TSet<FString>& Tags) const override;
	virtual int64 GetBuildSize(const TSet<const BuildPatchServices::FFileManifest*>& TaggedFiles) const;
	virtual TArray<FString> GetBuildFileList() const override;
	virtual TArray<FStringView> GetBuildFileListView() const override;
	virtual TArray<FString> GetBuildFileList(const TSet<FString>& Tags) const override;
	virtual TArray<FStringView> GetBuildFileListView(const TSet<FString>& Tags) const override;
	virtual int64 GetFileSize(FStringView Filename) const override;
	virtual int64 GetFileSize(const TArray<FString>& Filenames) const override;
	virtual int64 GetFileSize(const TSet  <FString>& Filenames) const override;
	virtual EFileMetaFlags GetFileMetaFlags(const FString& Filename) const override;
	virtual const FMD5Hash* GetFileMD5Hash(const FString& Filename) const override;
	virtual const FSHAHash* GetFileSHA1Hash(const FString& Filename) const override;
	virtual const FSHA256Signature* GetFileSHA256Hash(const FString& Filename) const override;
	virtual const FString* GetFileMIMEType(const FString& Filename) const override;
	virtual TSet<FString> GetFileTagList() const override;
	virtual void GetFileTagList(TSet<FString>& Tags) const override;
	virtual void GetOutdatedFiles(const IBuildManifestRef& OldManifest, TSet<FString>& OutdatedFiles) const override;
	virtual void GetRemovableFiles(const IBuildManifestRef& OldManifest, TArray<FString>& RemovableFiles) const override;
	virtual void GetRemovableFiles(const TCHAR* InstallPath, TArray<FString>& RemovableFiles) const override;
	virtual void GetChunksRequiredForFiles(const TSet<FString>& Filenames, TSet<FGuid>& RequiredChunks) const override;
	virtual int64 GetDataSize(const FGuid& DataGuid) const override;
	virtual bool GetChunkShaHash(const FGuid& ChunkGuid, FSHAHash& OutHash) const override;
	virtual bool GetChunkEncryptionSecretId(const FGuid& ChunkGuid, FGuid& OutEncryptionSecretId) const override;
	virtual bool NeedsResaving() const override;
	virtual void CopyCustomFields(const IBuildManifestRef& Other, bool bClobber) override;
	virtual TArray<FString> GetCustomFieldNames() const override;
	virtual const IManifestFieldPtr GetCustomField(const FString& FieldName) const override;
	virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const FString& Value) override;
	virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const double& Value) override;
	virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const int64& Value) override;
	virtual void RemoveCustomField(const FString& FieldName) override;
	virtual IBuildManifestRef Duplicate() const override;
	// END IBuildManifest Interface

	/**
	 * @return The unique build id of this manifest.
	 */
	virtual const FString& GetBuildId() const;

	/**
	 * Sets up the internal map from a file
	 * @param Filename		The file to load JSON from
	 * @return		True if successful.
	 */
	virtual bool LoadFromFile(const FString& Filename);

	/**
	 * Sets up the object from the passed in data
	 * @param DataInput		The data to deserialize from
	 * @return		True if successful.
	 */
	virtual bool DeserializeFromData(const TArray<uint8>& DataInput);

	/**
	 * Sets up the object from the passed in JSON string
	 * @param JSONInput		The JSON string to deserialize from
	 * @return		True if successful.
	 */
	virtual bool DeserializeFromJSON(const FString& JSONInput);

	/**
	 * Saves out the manifest information.
	 * @param Filename      IN   The file to save to.
	 * @param SaveFormat    IN   The feature level that the intended client has support for, which the manifest will need saving as.
	 *                           A manifest file cannot be downgraded, the function will fail if the provided value is less than GetFeatureLevel().
	 * @param OutSHA1Hash   OUT  An output variable which will be populated with the SHA1 hash of the written manifest.
	 * @param OutMD5Hash    OUT  An output variable which will be populated with the MD5 hash of the written manifest.
	 * @return True if successful.
	 */
	virtual bool SaveToFile(const FString& Filename, BuildPatchServices::EFeatureLevel SaveFormat, FSHAHash* OutSHA1Hash = nullptr, FMD5Hash* OutMD5Hash = nullptr);

	/**
	 * Creates the object in JSON format
	 * @param JSONOutput		A string to receive the JSON representation
	 */
	virtual void SerializeToJSON(FString& JSONOutput);

	/**
	 * Get the number of times a chunks is referenced in this manifest
	 * @param ChunkGuid		The chunk GUID
	 * @return	The number of references to this chunk
	 */
	virtual uint32 GetNumberOfChunkReferences(const FGuid& ChunkGuid) const;

	/**
	 * Returns the total size of all data files in it's list.
	 * @param DataGuids		The GUID array for the data
	 * @return		Total file size, or 0 if none of this data is in the manifest.
	 */
	virtual int64 GetDataSize(const TArray<FGuid>& DataGuids) const;
	virtual int64 GetDataSize(const TSet  <FGuid>& DataGuids) const;

	/**
	 * Returns the number of files in this build.
	 * @return		The number of files.
	 */
	virtual uint32 GetNumFiles() const;

	/**
	 * Get the list of files described by this manifest
	 * @param Filenames		OUT		Receives the list of files.
	 */
	virtual void GetFileList(TArray<FString>& Filenames) const;
	virtual void GetFileList(TArray<FStringView>& Filenames) const;
	virtual void GetFileList(TSet  <FString>& Filenames) const;

	/**
	 * Get the list of files that are tagged with the provided tags
	 * @param Tags					The tags for the required file groups.
	 * @param TaggedFiles	OUT		Receives the tagged files.
	 */
	virtual void GetTaggedFileList(const TSet<FString>& Tags, TArray<FString>& TaggedFiles) const;
	virtual void GetTaggedFileList(const TSet<FString>& Tags, TArray<FStringView>& TaggedFiles) const;
	virtual void GetTaggedFileList(const TSet<FString>& Tags, TSet<FString>& TaggedFiles) const;

	/**
	 * Get the set of manifests that are tagged with the provided tags
	 * @param Tags					The tags for the required file groups.
	 * @return The set of tagged file manifests.
	 */
	virtual TSet<const BuildPatchServices::FFileManifest*> GetTaggedFileManifests(const TSet<FString>& Tags) const;

	/**
	 * Get the list of Guids for all chunks referenced by this manifest
	 * @param DataGuids		OUT		Receives the array of Guids.
	 */
	virtual void GetDataList(TArray<FGuid>& DataGuids) const;
	virtual void GetDataList(TSet  <FGuid>& DataGuids) const;

	/**
	 * Returns the manifest for a particular file in the app, nullptr if non-existing
	 * @param Filename	The filename.
	 * @return	The file manifest, or invalid ptr
	 */
	virtual const BuildPatchServices::FFileManifest* GetFileManifest(const FString& Filename) const;

	/**
	 *  @return a const iterator for the containing file manifests.
	 *
	 */
	virtual TArray<BuildPatchServices::FFileManifest>::TConstIterator GetFileManifestIterator() const;

	/**
	 * Gets whether this manifest is made up of file data instead of chunk data
	 * @return	True if the build is made from file data. False if the build is constructed from chunk data.
	 */
	virtual bool IsFileDataManifest() const;

	/**
	 * Gets the chunk hash for a given chunk
	 * @param ChunkGuid		IN		The guid of the chunk to get hash for
	 * @param OutHash		OUT		Receives the hash value if found
	 * @return	true if we had the hash for this chunk
	 */
	virtual bool GetChunkHash(const FGuid& ChunkGuid, uint64& OutHash) const;

	/**
	 * Gets the FChunkInfo for a given chunk.
	 * @param ChunkGuid     The guid of the chunk to get hash for.
	 * @return ptr to the FChunkInfo or nullptr if not found.
	 */
	virtual const BuildPatchServices::FChunkInfo* GetChunkInfo(const FGuid& ChunkGuid) const;

	/**
	 * Gets the file hash for given file data
	 * @param FileGuid		IN		The guid of the file data to get hash for
	 * @param OutHash		OUT		Receives the hash value if found
	 * @return	true if we had the hash for this file
	 */
	virtual bool GetFileHash(const FGuid& FileGuid, FSHAHash& OutHash) const; // DEPRECATE ME

	/**
	 * Gets the file hash for a given file
	 * @param Filename		IN		The filename in the build
	 * @param OutHash		OUT		Receives the hash value if found
	 * @return	true if we had the hash for this file
	 */
	virtual bool GetFileHash(const FString& Filename, FSHAHash& OutHash) const;

	/**
	 * Gets the file hash for given file data. Valid for non-chunked manifest
	 * @param FileGuid		IN		The guid of the file data to get hash for
	 * @param OutHash		OUT		Receives the hash value if found
	 * @return	true if we had the hash for this file
	 */
	virtual bool GetFilePartHash(const FGuid& FilePartGuid, uint64& OutHash) const;


	/**
	 * Populates an array of chunks that should be producible from a local build, given the tagset that is expected to be installed.
	 * @param TagSet           IN   The tagset identifying files expected to be installed.
	 * @param ChunksRequired   IN   A list of chunks that are needed.
	 * @param ChunksAvailable  OUT  A list to receive the chunks from ChunksRequired that could be constructed locally.
	 * @return the number of chunks added to the ChunksAvailable set.
	 */
	virtual int32 EnumerateProducibleChunks(const TSet<FString>& TagSet, const TSet<FGuid>& ChunksRequired, TSet<FGuid>& ChunksAvailable) const;
	virtual int32 EnumerateProducibleChunks(const TSet<const BuildPatchServices::FFileManifest*>& TaggedFiles, const TSet<FGuid>& ChunksRequired, TSet<FGuid>& ChunksAvailable) const;

	/**
	 * Gets a list of files that have changed or are new in the this manifest, compared to those in the old manifest, or are missing from disk.
	 * @param OldManifest		IN		The Build Manifest that is currently installed. Shared Ptr - Can be invalid.
	 * @param InstallDirectory	IN		The Build installation directory, so that it can be checked for missing files.
	 * @param OutDatedFiles		OUT		The files that changed hash, are new, are wrong size, or missing on disk.
	 */
	virtual void GetOutdatedFiles(const FBuildPatchAppManifestPtr& OldManifest, const FString& InstallDirectory, TSet<FString>& OutDatedFiles) const;
	virtual void GetOutdatedFiles(const FBuildPatchAppManifest*    OldManifest, const FString& InstallDirectory, TSet<FString>& OutDatedFiles) const;
	virtual void GetOutdatedFiles(const FBuildPatchAppManifest*    OldManifest, const FString& InstallDirectory, const TSet<FString>& FilesToCheck, TSet<FString>& OutDatedFiles) const;

	/**
	 * Gets a list of files mentioned in the manifest that are missing from disk.
	 * @param InstallDirectory [in]  The build installation directory to check for missing files.
	 * @param FilesToCheck     [in]  The list of files to check.
	 * @param OutMissedFiles   [out] The files that are missing from disk.
	 */
	virtual void GetMissedFiles(const FString& InstallDirectory, const TSet<FString>& FilesToCheck, TSet<FString>& OutMissedFiles) const;

	/**
	 * Check a single file to see if it will be effected by patching from a previous version.
	 * @param OldManifest		The Build Manifest that is currently installed. Shared Ref - Implicitly valid.
	 * @param Filename			The Build installation directory, so that it can be checked for missing files.
	 */
	virtual bool IsFileOutdated(const FBuildPatchAppManifestRef& OldManifest, const FString& Filename) const;
	virtual bool IsFileOutdated(const FBuildPatchAppManifest&    OldManifest, const FString& Filename) const;

	/**
	 * Gets a list of files that were installed with the Old Manifest, but no longer required by this Manifest.
	 * @param OldManifest		IN		The Build Manifest that is currently installed.
	 * @param RemovableFiles	OUT		A list to receive the files that may be removed.
	 */
	virtual void GetRemovableFiles(const FBuildPatchAppManifest& OldManifest, TArray<FString>& RemovableFiles) const;


	/** @return True if any files in this manifest have file attributes to be set */
	virtual bool HasFileAttributes() const;

	bool EncryptData(const FGuid& InEncryptionSecretId, const TArray<uint8>& InEncryptionSecretKey);

	bool DecryptData(const TMap<FGuid, TArray<uint8>>& AvailableEncryptionSecrets);

private:
	/**
	 * Destroys any memory we have allocated and clears out ready for generation of a new manifest
	 */
	void DestroyData();

	bool SerialiseSensitiveFields(FArchive& Ar);
	void ObfuscateSensitiveFields();

protected:
	/**
	 * Setups the lookup maps that optimize data access, should be called when Data changes
	 */
	void InitLookups();
private:

	/**
	 * Helper for the public EnumerateProducibleChunks functions.
	 */
	int32 EnumerateProducibleChunks_Internal(const TFunction<bool(const FString&)>& FileAccessChecker, const TSet<FGuid>& ChunksRequired, TSet<FGuid>& ChunksAvailable) const;

protected:
	/** Holds the actual manifest data. Some other variables point to the memory held by these objects */

	BuildPatchServices::FManifestMeta ManifestMeta;
	BuildPatchServices::FChunkDataList ChunkDataList;
	BuildPatchServices::FFileManifestList FileManifestList;
	BuildPatchServices::FCustomFields CustomFields;
	BuildPatchServices::FEncryptedData EncryptedData;

private:
	// The encryption secret ID and hash used for this manifest, if encrypted.
	// These are serialised with the manifest header, not the main data.
	FGuid EncryptionSecretId;
	BuildPatchServices::FAESAuthTag EncryptionAuthTag;

	// Whether the manifest is currently encrypted.
	bool bIsManifestEncrypted = false;

	/** Holds the handle to our PreExit delegate */
	FDelegateHandle OnPreExitHandle;

	/** Some lookups to optimize data access */
	TMap<FGuid, const FString*> FileNameLookup;
	TMap<FStringView, const BuildPatchServices::FFileManifest*> FileManifestLookup;
	TMap<FString, TArray<const BuildPatchServices::FFileManifest*>> TaggedFilesLookup;
	TMap<FGuid, const BuildPatchServices::FChunkInfo*> ChunkInfoLookup;

	/** Holds the total build size in bytes */
	int64 TotalBuildSize;
	int64 TotalDownloadSize;

	/** Flag marked true if we loaded from disk as an old manifest version that should be updated */
	bool bNeedsResaving;
};
