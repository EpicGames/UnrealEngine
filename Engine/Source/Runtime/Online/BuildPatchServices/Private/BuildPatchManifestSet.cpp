// Copyright Epic Games, Inc. All Rights Reserved.

#include "IBuildManifestSet.h"
#include "Algo/Accumulate.h"
#include "Algo/AnyOf.h"
#include "Algo/Transform.h"

#include "BuildPatchSettings.h"

#include "Common/FileSystem.h"
#include "Data/ManifestData.h"
#include "Installer/OptimisedDelta.h"
#include "BuildPatchUtil.h"
#include "BuildPatchFeatureLevel.h"

namespace BuildPatchServices
{
	struct FManifestIterator
	{
	public:
		FManifestIterator() : InstallerAction(nullptr), Current(nullptr) { }
		FManifestIterator(const FBuildPatchInstallerAction& InInstallerAction) : InstallerAction(&InInstallerAction) { Current = InstallerAction->TryGetCurrentManifest() ? InstallerAction->TryGetCurrentManifest() : InstallerAction->TryGetInstallManifest(); }
		FManifestIterator operator++() { Current = (InstallerAction->TryGetInstallManifest() != Current) ? InstallerAction->TryGetInstallManifest() : nullptr; return *this; }
		bool operator!=(const FManifestIterator & Other) const { return Current != Other.Current; }
		const FBuildPatchAppManifest& operator*() const { return *Current; }
		FBuildPatchInstallerAction const * const InstallerAction;
		FBuildPatchAppManifest const * Current;
	};
	FManifestIterator begin(const FBuildPatchInstallerAction& InstallerAction) { return FManifestIterator(InstallerAction); }
	FManifestIterator end(const FBuildPatchInstallerAction& InstallerAction) { return FManifestIterator(); }

	class FBuildPatchManifestSet : public IBuildManifestSet
	{
		// Lookup for data references.
		typedef TTuple<FChunkInfo const* const, FBuildPatchAppManifest const * const, FBuildPatchInstallerAction const * const> FDataReference;
		TMap<FGuid, FDataReference> DataLookup;
		// Lookup for file references.
		typedef TTuple<FFileManifest const* const, FBuildPatchAppManifest const * const, FBuildPatchInstallerAction const * const> FFileReference;
		TMap<FString, FFileReference> CurrentFileLookup;
		TMap<FString, FFileReference> NewFileLookup;
	public:
		FBuildPatchManifestSet(TArray<FBuildPatchInstallerAction>&& InInstallerActions)
			: InstallerActions(MoveTemp(InInstallerActions))
		{
			// We need to perform our own lookups to avoid looping in some scenarios, and also allow other systems to not have to understand subdirectories.
			// All publically used API includes subdirectory in filenames.
			// Reverse iterate provided actions, making sure there is only one unique reference to each file in the entire action set.
			// Though an undesirable request, the expected behaviour for multiple actions producing the same file would be clobbering by later actions.
			for (int32 Idx = InstallerActions.Num() - 1; Idx >= 0; --Idx)
			{
				FBuildPatchInstallerAction& InstallerAction = InstallerActions[Idx];
				// We can add every DataLookup from all manifests, since any file could reference any data piece.
				// Duplicates are not an issue because any single ChunkGuid, will resolve to a reachable Chunk file.
				for (const FBuildPatchAppManifest& Manifest : InstallerAction)
				{
					for (const TPair<FGuid, const BuildPatchServices::FChunkInfo*>& ChunkInfoPair : Manifest.ChunkInfoLookup)
					{
						DataLookup.Add(ChunkInfoPair.Key, FDataReference{ ChunkInfoPair.Value, &Manifest, &InstallerAction });
					}
				}

				// Get all the CurrentFiles, if there are any.
				const FBuildPatchAppManifest* CurrentManifest = InstallerAction.TryGetCurrentManifest();
				if (CurrentManifest != nullptr)
				{
					for (const FFileManifest* CurrFileManifest : CurrentManifest->GetTaggedFileManifests(InstallerAction.GetInstallTags()))
					{
						FString FullFilename = InstallerAction.GetInstallSubdirectory() / CurrFileManifest->Filename;
						// Do not allow clobbering.
						if (!CurrentFileLookup.Contains(FullFilename))
						{
							CurrentFileLookup.Add(MoveTemp(FullFilename), FFileReference{CurrFileManifest, CurrentManifest, &InstallerAction});
						}
					}
				}

				// Get all the NewFiles, if there are any.
				const FBuildPatchAppManifest* InstallManifest = InstallerAction.TryGetInstallManifest();
				if (InstallManifest != nullptr)
				{
					// Track files that were actually placed into the lookups, so we can override the action's tagged file list.
					// This will help other functionality guarentee it cannot return the incorrect resolved file manifest, sice each file will only be referenced
					// by one action after this logic.
					// We are also going to iterate files in order of the manifest, rather than just the generic public functions to get tagged files, as this will allow us to preserve the desired manifest
					// installation order much easier, since we can treat the tagged file list as sorted because it is construct once.
					// REMEMBER: TaggedFiles refer to the manifest's filenames, so not including subdirectory. If two actions share a filename, but with different subdirectories, they both get installed.
					//           This is important because the tagged file list is used to iterate file manifest in the build manifest.
					TSet<FString> ResultingTaggedFiles;
					for (auto It = InstallManifest->GetFileManifestIterator(); It; ++It)
					{
						const bool bFileIsTagged = It->InstallTags.Num() == 0 || Algo::AnyOf(It->InstallTags, [&](const FString& Tag) { return InstallerAction.GetInstallTags().Contains(Tag); });
						if (bFileIsTagged)
						{
							FString FullFilename = InstallerAction.GetInstallSubdirectory() / It->Filename;
							// Do not allow clobbering.
							if (!NewFileLookup.Contains(FullFilename))
							{
								NewFileLookup.Add(MoveTemp(FullFilename), FFileReference{ &(*It), InstallManifest, &InstallerAction });
								ResultingTaggedFiles.Add(It->Filename);
							}
						}
					}
					InstallerAction.SetTaggedFiles(MoveTemp(ResultingTaggedFiles));
				}
			}
		}

		virtual FChunkInfo const* GetChunkInfo(const FGuid& DataGuid) const override
		{
			FDataReference const * const LookupResult = DataLookup.Find(DataGuid);
			if (LookupResult)
			{
				return LookupResult->Get<0>();
			}
			return 0;
		}

		virtual void GetInstallResumeIds(TSet<FString>& ResumeIds, bool bIncludeLegacy) const
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (!InstallerAction.IsUninstall())
				{
					if (bIncludeLegacy)
					{
						ResumeIds.Add(InstallerAction.GetInstallManifest().GetAppName() + InstallerAction.GetInstallManifest().GetVersionString());
					}
					ResumeIds.Add(InstallerAction.GetInstallManifest().GetBuildId());
				}
			}
		}

		virtual void GetInstallResumeIdsForFile(const FString& BuildFile, TSet<FString>& ResumeIds, bool bIncludeLegacy) const
		{
			const FFileReference* LookupResult = NewFileLookup.Find(BuildFile);
			if (LookupResult)
			{
				if (bIncludeLegacy)
				{
					ResumeIds.Add(LookupResult->Get<2>()->GetInstallManifest().GetAppName() + LookupResult->Get<2>()->GetInstallManifest().GetVersionString());
				}
				ResumeIds.Add(LookupResult->Get<2>()->GetInstallManifest().GetBuildId());
			}
		}

		virtual void GetReferencedChunks(TSet<FGuid>& DataGuids) const override
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (!InstallerAction.IsUninstall())
				{
					InstallerAction.GetInstallManifest().GetChunksRequiredForFiles(InstallerAction.GetTaggedFiles(), DataGuids);
				}
			}
		}

		virtual void GetReferencedChunks(TArray<FGuid>& DataGuids) const override
		{
			// For an array instead of set, we would expect to provide ordered duplicates.
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				// InstallerAction.GetTaggedFiles() has already been de-duped across actions in this class's constructor.
				for (const FString& TaggedFile : InstallerAction.GetTaggedFiles())
				{
					FFileManifest const * const TaggedFileManifest = GetNewFileManifest(InstallerAction.GetInstallSubdirectory() / TaggedFile);
					if (TaggedFileManifest)
					{
						for (const FChunkPart& ChunkPart : TaggedFileManifest->ChunkParts)
						{
							DataGuids.Add(ChunkPart.Guid);
						}
					}
				}
			}
		}

		FDataReference const* const DataLookupFind(const FGuid& DataGuid) const
		{
			// This used to be a thread_local static lookup cache however it fails with multiple installations as well
			// as if we have multiple installs in a row that happen to use the same guid. I'm not confident using a thread lock
			// is worth saving what should be a N(1) lookup.
			return DataLookup.Find(DataGuid);
		}

		virtual uint64 GetDownloadSize(const FGuid& DataGuid) const override
		{
			FDataReference const* const LookupResult = DataLookupFind(DataGuid);
			if (LookupResult)
			{
				return LookupResult->Get<0>()->FileSize;
			}
			return 0;
		}

		virtual uint64 GetDownloadSize(const TSet<FGuid>& DataGuids) const override
		{
			return Algo::Accumulate<uint64>(DataGuids, 0, [this](uint64 Size, const FGuid& DataGuid) { return Size + GetDownloadSize(DataGuid); });
		}

		virtual bool GetChunkShaHash(const FGuid& DataGuid, FSHAHash& OutHash) const override
		{
			static const uint8 Zero[FSHA1::DigestSize] = { 0 };
			FDataReference const* const LookupResult = DataLookupFind(DataGuid);
			if (LookupResult)
			{
				OutHash = LookupResult->Get<0>()->ShaHash;
				return FMemory::Memcmp(OutHash.Hash, Zero, FSHA1::DigestSize) != 0;
			}
			return false;
		}

		virtual FGuid GetDataSecretId(const FGuid& DataGuid) const override
		{
			FDataReference const* const LookupResult = DataLookupFind(DataGuid);
			if (LookupResult)
			{
				return LookupResult->Get<0>()->EncryptionSecretId;
			}
			return FGuid();
		}

		virtual FString GetChunkCloudSubdirectory(const FGuid& DataGuid) const override
		{
			FDataReference const* const LookupResult = DataLookupFind(DataGuid);
			if (LookupResult)
			{
				return LookupResult->Get<2>()->GetCloudSubdirectory();
			}
			return FString();
		}

		virtual FString GetChunkFilename(const FGuid& DataGuid) const override
		{
			FDataReference const* const LookupResult = DataLookupFind(DataGuid);
			if (LookupResult)
			{
				return FBuildPatchUtils::GetDataFilename(*LookupResult->Get<1>(), DataGuid);
			}
			return FString();
		}

		virtual FString GetChunkFilenameWithCloudSubdirectory(const FGuid& DataGuid) const override
		{
			FDataReference const* const LookupResult = DataLookupFind(DataGuid);
			if (LookupResult)
			{
				return LookupResult->Get<2>()->GetCloudSubdirectory() / FBuildPatchUtils::GetDataFilename(*LookupResult->Get<1>(), DataGuid);
			}
			return FString();
		}

		virtual FString GetChunkUniqueBuildId(const FGuid& DataGuid) const override
		{
			FDataReference const* const LookupResult = DataLookupFind(DataGuid);
			if (LookupResult)
			{
				return LookupResult->Get<2>()->GetUniqueBuildId();
			}
			return FString();
		}

		virtual bool GetChunkFromDeltaStatus(const FGuid& DataGuid) const override
		{
			FDataReference const* const LookupResult = DataLookupFind(DataGuid);
			if (LookupResult)
			{
				return LookupResult->Get<0>()->bChunkFromDelta;
			}
			return false;
		}

		virtual int32 GetNumExpectedFiles() const
		{
			return NewFileLookup.Num();
		}

		virtual void GetExpectedFiles(TSet<FString>& Filenames) const
		{
			Filenames.Reserve(Filenames.Num() + NewFileLookup.Num());
			for (const TPair<FString, FFileReference>& NewFilePair : NewFileLookup)
			{
				Filenames.Add(NewFilePair.Key);
			}
		}

		virtual void GetOutdatedFiles(const FString& InstallDirectory, TSet<FString>& OutdatedFiles) const override
		{
			// For outdated files, we must consider the difference by comparing the file manifests
			// according to CurrentFileLookup and NewFileLookup, since these have already been properly flattened
			// using the correct action resolution logic.

			// For this reason, we cannot use the GetOutdatedFiles function on the actual build manifest,
			// since it is not subdirectory aware, or multi-action aware.

			const bool bCheckFileSizeOnDisk = InstallDirectory.IsEmpty() == false;
			TSet<FString> FileSizesToCheck;
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				// InstallerAction.GetTaggedFiles() has already been de-duped across actions in this class's constructor.
				for (const FString& ManifestFilename : InstallerAction.GetTaggedFiles())
				{
					FString FullFilename = InstallerAction.GetInstallSubdirectory() / ManifestFilename;
					const FFileManifest* CurrentFileManifest = GetCurrentFileManifest(FullFilename);
					const FFileManifest* NewFileManifest = GetNewFileManifest(FullFilename);

					// Automatically considered outdated, if we have a new manifest without current manfiest.
					if (CurrentFileManifest == nullptr && NewFileManifest != nullptr)
					{
						OutdatedFiles.Add(MoveTemp(FullFilename));
					}
					// If we have both, check if the hash changed.
					else if (CurrentFileManifest != nullptr && NewFileManifest != nullptr
					     && (CurrentFileManifest->SHA1Hash != NewFileManifest->SHA1Hash))
					{
						OutdatedFiles.Add(MoveTemp(FullFilename));
					}
					// If we are checking disk sizes, add this 'not outdated' file for the check.
					else if (bCheckFileSizeOnDisk)
					{
						FileSizesToCheck.Add(MoveTemp(FullFilename));
					}
				}
			}
			// Run the disk size check for any files that need it.
			if (bCheckFileSizeOnDisk && FileSizesToCheck.Num() > 0)
			{
				TMap<FString, int64> FileSizes = ParallelGetFileSizes(FileSizesToCheck, InstallDirectory);
				for (FString& FullFilename : FileSizesToCheck)
				{
					const FFileManifest* NewFileManifest = GetNewFileManifest(FullFilename);
					if (NewFileManifest != nullptr)
					{
						const int64 ActualFileSize = FileSizes.FindRef(FullFilename, INDEX_NONE);
						if (NewFileManifest->FileSize != ActualFileSize)
						{
							OutdatedFiles.Add(MoveTemp(FullFilename));
						}
					}
				}
			}
		}

		virtual void GetMissedFiles(const FString& InstallDirectory, TSet<FString>& OutMissedFiles) const override
		{
			TSet<FString> TempFileSet;
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				// It expects to use method for Update.
				if (!InstallerAction.IsUninstall() && !InstallerAction.IsInstall())
				{
					TempFileSet.Reset();
					const FBuildPatchAppManifest* CurrentManifest = InstallerAction.TryGetCurrentManifest();
					if (CurrentManifest)
					{
						// There is no reliable way to determine the list of tags with which the binary was previously installed.
						// At the same time, InstallerAction.GetTaggedFiles() cannot be used because it contains files from the new manifest.
						TSet<FString> AllFilenames;
						CurrentManifest->GetFileList(AllFilenames);
						CurrentManifest->GetMissedFiles(InstallDirectory / InstallerAction.GetInstallSubdirectory(), AllFilenames, TempFileSet);
					}
					else
					{
						// Treat all files as missing if the current manifest is unavailable.
						TempFileSet.Append(InstallerAction.GetTaggedFiles());
					}
					
					Algo::Transform(TempFileSet, OutMissedFiles, [&](const FString& Filename) { return InstallerAction.GetInstallSubdirectory() / Filename; });
				}
			}
		}

		virtual void GetRemovableFiles(TSet<FString>& FilesToRemove) const override
		{
			// We enumerate all trackable files, adding anything not in the NewFileLookup map.
			// This way we gracefully handle tagged/untagged files, and also uninstalling actions.
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				for (const FBuildPatchAppManifest& Manifest : InstallerAction)
				{
					for (const FString& TrackedFile : Manifest.GetBuildFileList())
					{
						const FString FullFilename = InstallerAction.GetInstallSubdirectory() / TrackedFile;
						if (!NewFileLookup.Contains(FullFilename))
						{
							FilesToRemove.Add(FullFilename);
						}
					}
				}
			}
		}

		virtual const BuildPatchServices::FFileManifest* GetCurrentFileManifest(const FString& BuildFile) const override
		{
			FFileReference const* const LookupResult = CurrentFileLookup.Find(BuildFile);
			if (LookupResult)
			{
				return LookupResult->Get<0>();
			}
			return nullptr;
		}

		virtual const BuildPatchServices::FFileManifest* GetNewFileManifest(const FString& BuildFile) const
		{
			FFileReference const* const LookupResult = NewFileLookup.Find(BuildFile);
			if (LookupResult)
			{
				return LookupResult->Get<0>();
			}
			return nullptr;
		}

		template<typename ContainerType>
		uint64 GetTotalNewFileSizeHelper(const ContainerType& Filenames) const
		{
			uint64 TotalFileSize = 0;
			for (const FString& Filename : Filenames)
			{
				const BuildPatchServices::FFileManifest* FileManifest = GetNewFileManifest(Filename);
				if (nullptr != FileManifest)
				{
					TotalFileSize += FileManifest->FileSize;
				}
			}
			return TotalFileSize;
		}
		virtual uint64 GetTotalNewFileSize(const TArray<FString>& Filenames) const { return GetTotalNewFileSizeHelper(Filenames); }
		virtual uint64 GetTotalNewFileSize(const TSet<FString>& Filenames) const { return GetTotalNewFileSizeHelper(Filenames); }

		virtual void GetPreReqInfo(TArray<FPreReqInfo>& PreReqInfos) const override
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (!InstallerAction.IsUninstall())
				{
					const FBuildPatchAppManifest& Manifest = InstallerAction.GetInstallManifest();
					if (!Manifest.GetPrereqPath().IsEmpty())
					{
						FPreReqInfo& PreInfo = PreReqInfos.AddDefaulted_GetRef();
						PreInfo.IdSet = Manifest.GetPrereqIds();
						PreInfo.AppName = Manifest.GetAppName();
						PreInfo.Args = Manifest.GetPrereqArgs();
						PreInfo.Name = Manifest.GetPrereqName();
						PreInfo.Path = InstallerAction.GetInstallSubdirectory() / Manifest.GetPrereqPath();
						PreInfo.VersionString = Manifest.GetVersionString();
						PreInfo.bIsRepair = InstallerAction.IsRepair();
					}
				}
			}
		}

		virtual void GetFilesTaggedForRepair(TSet<FString>& Filenames) const override
		{
			for (const TPair<FString, FFileReference>& NewFileReference : NewFileLookup)
			{
				if (NewFileReference.Value.Get<2>()->IsRepair())
				{
					Filenames.Add(NewFileReference.Key);
				}
			}
		}

		virtual bool IsFileRepairAction(const FString& Filename) const override
		{
			FFileReference const* const LookupResult = NewFileLookup.Find(Filename);
			if (LookupResult)
			{
				return LookupResult->Get<2>()->IsRepair();
			}
			return false;
		}

		virtual bool ContainsUpdate() const override
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (InstallerAction.IsUpdate() && InstallerAction.GetCurrentManifest().GetBuildId() != InstallerAction.GetInstallManifest().GetBuildId())
				{
					return true;
				}
			}
			return false;
		}

		virtual bool IsExclusivelyRepairing() const override
		{
			bool bHasRepair = false;
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (InstallerAction.IsRepair())
				{
					bHasRepair = true;
				}
				else
				{
					return false;
				}
			}
			return bHasRepair;
		}

		virtual bool HasFileAttributes() const override
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (!InstallerAction.IsUninstall())
				{
					if (InstallerAction.GetInstallManifest().HasFileAttributes())
					{
						return true;
					}
				}
			}
			return false;
		}

		virtual bool IsExclusivelyVerifying() const override
		{
			bool bHasVerifyOnly = false;
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (InstallerAction.IsVerifyOnly())
				{
					bHasVerifyOnly = true;
				}
				else
				{
					return false;
				}
			}
			return bHasVerifyOnly;
		}

		virtual void GetUninstallActionInfo(TArray<FUninstallActionInfo>& UninstallActionsInfo) const override
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				if (InstallerAction.IsUninstall())
				{
					const FBuildPatchAppManifest& Manifest = InstallerAction.GetInstallOrCurrentManifest();
					const FString& UninstallActionPath = Manifest.GetUninstallActionPath();
					if (!UninstallActionPath.IsEmpty())
					{
						FUninstallActionInfo& UninstallActionInfo = UninstallActionsInfo.AddDefaulted_GetRef();
						UninstallActionInfo.Path = UninstallActionPath;
						UninstallActionInfo.Args = Manifest.GetUninstallActionArgs();
					}
				}
			}
		}

	private:
		TArray<FBuildPatchInstallerAction> InstallerActions;
	};

	IBuildManifestSet* FBuildManifestSetFactory::Create(TArray<FBuildPatchInstallerAction> InstallerActions)
	{
		return new FBuildPatchManifestSet(MoveTemp(InstallerActions));
	}
}