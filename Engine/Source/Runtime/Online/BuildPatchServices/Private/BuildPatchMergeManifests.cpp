// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchMergeManifests.h"
#include "HAL/ThreadSafeBool.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Guid.h"
#include "Algo/Sort.h"

#include "Common/FileSystem.h"
#include "Data/ManifestData.h"
#include "BuildPatchManifest.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMergeManifests, Log, All);
DEFINE_LOG_CATEGORY(LogMergeManifests);

namespace MergeHelpers
{
	FBuildPatchAppManifestPtr LoadManifestFile(const FString& ManifestFilePath, FCriticalSection* UObjectAllocationLock)
	{
		check(UObjectAllocationLock != nullptr);
		UObjectAllocationLock->Lock();
		FBuildPatchAppManifestPtr Manifest = MakeShareable(new FBuildPatchAppManifest());
		UObjectAllocationLock->Unlock();
		if (Manifest->LoadFromFile(ManifestFilePath))
		{
			return Manifest;
		}
		return FBuildPatchAppManifestPtr();
	}

	bool CopyFileDataFromManifestToArray(const TSet<FString>& Filenames, const FBuildPatchAppManifestPtr& Source, TArray<BuildPatchServices::FFileManifest>& DestArray)
	{
		bool bSuccess = true;
		for (const FString& Filename : Filenames)
		{
			check(Source.IsValid());
			const BuildPatchServices::FFileManifest* FileManifest = Source->GetFileManifest(Filename);
			if (FileManifest == nullptr)
			{
				UE_LOGF(LogMergeManifests, Error, "Could not find file in %ls %ls: %ls", *Source->GetAppName(), *Source->GetVersionString(), *Filename);
				bSuccess = false;
			}
			else
			{
				DestArray.Add(*FileManifest);
			}
		}
		return bSuccess;
	}

	bool ReinitialiseChunkInfoList(const TArray<BuildPatchServices::FFileManifest>& FileManifestList, const FBuildPatchAppManifest& ManifestA, const FBuildPatchAppManifest& ManifestB, TArray<BuildPatchServices::FChunkInfo>& ChunkList)
	{
		using namespace BuildPatchServices;
		ChunkList.Reset();
		TSet<FGuid> ReferencedChunks;
		for (const FFileManifest& FileManifest : FileManifestList)
		{
			for (const FChunkPart& FileChunkPart : FileManifest.ChunkParts)
			{
				bool bAlreadyInSet = false;
				ReferencedChunks.Add(FileChunkPart.Guid, &bAlreadyInSet);
				if (bAlreadyInSet == false)
				{
					// Find the chunk info
					const FChunkInfo* ChunkInfo = ManifestB.GetChunkInfo(FileChunkPart.Guid);
					if (ChunkInfo == nullptr)
					{
						ChunkInfo = ManifestA.GetChunkInfo(FileChunkPart.Guid);
					}
					if (ChunkInfo == nullptr)
					{
						UE_LOGF(LogMergeManifests, Error, "Failed to copy chunk meta for %ls used by %ls. Possible damaged manifest file as input.", *FileChunkPart.Guid.ToString(), *FileManifest.Filename);
						return false;
					}
					else
					{
						FChunkInfo ChunkTemp = *ChunkInfo;
						ChunkTemp.bChunkFromDelta = FileChunkPart.bChunkFromDelta;
						ChunkList.Add(MoveTemp(ChunkTemp));
					}
				}
			}
		}
		return true;
	}
}

bool FBuildMergeManifests::MergeManifests(const FString& ManifestFilePathA, const FString& ManifestFilePathB, const FString& ManifestFilePathC, const FString& NewVersionString, const FString& SelectionDetailFilePath, const TMap<FGuid, TArray<uint8>>& AvailableEncryptionSecrets)
{
	using namespace BuildPatchServices;

	FCriticalSection UObjectAllocationLock;

	TFunction<FBuildPatchAppManifestPtr()> TaskManifestA = [&UObjectAllocationLock, &ManifestFilePathA]()
	{
		return MergeHelpers::LoadManifestFile(ManifestFilePathA, &UObjectAllocationLock);
	};
	TFunction<FBuildPatchAppManifestPtr()> TaskManifestB = [&UObjectAllocationLock, &ManifestFilePathB]()
	{
		return MergeHelpers::LoadManifestFile(ManifestFilePathB, &UObjectAllocationLock);
	};
	typedef TPair<TSet<FString>, TSet<FString>> FStringSetPair;
	FThreadSafeBool bSelectionDetailSuccess = false;
	TFunction<FStringSetPair()> TaskSelectionInfo = [&SelectionDetailFilePath, &bSelectionDetailSuccess]()
	{
		bSelectionDetailSuccess = true;
		FStringSetPair StringSetPair;
		if (SelectionDetailFilePath.IsEmpty() == false)
		{
			FString SelectionDetailFileData;
			bSelectionDetailSuccess = FFileHelper::LoadFileToString(SelectionDetailFileData, *SelectionDetailFilePath);
			if (bSelectionDetailSuccess)
			{
				TArray<FString> SelectionDetailLines;
				SelectionDetailFileData.ParseIntoArrayLines(SelectionDetailLines);
				for (int32 LineIdx = 0; LineIdx < SelectionDetailLines.Num(); ++LineIdx)
				{
					FString Filename, Source;
					SelectionDetailLines[LineIdx].Split(TEXT("\t"), &Filename, &Source, ESearchCase::CaseSensitive);
					Filename = Filename.TrimStartAndEnd().TrimQuotes();
					FPaths::NormalizeDirectoryName(Filename);
					Source = Source.TrimStartAndEnd().TrimQuotes();
					if (Source == TEXT("A"))
					{
						StringSetPair.Key.Add(Filename);
					}
					else if (Source == TEXT("B"))
					{
						StringSetPair.Value.Add(Filename);
					}
					else
					{
						UE_LOGF(LogMergeManifests, Error, "Could not parse line %d from %ls", LineIdx + 1, *SelectionDetailFilePath);
						bSelectionDetailSuccess = false;
					}
				}
			}
			else
			{
				UE_LOGF(LogMergeManifests, Error, "Could not load selection detail file %ls", *SelectionDetailFilePath);
			}
		}
		return MoveTemp(StringSetPair);
	};

	TFuture<FBuildPatchAppManifestPtr> FutureManifestA = Async(EAsyncExecution::ThreadPool, MoveTemp(TaskManifestA));
	TFuture<FBuildPatchAppManifestPtr> FutureManifestB = Async(EAsyncExecution::ThreadPool, MoveTemp(TaskManifestB));
	TFuture<FStringSetPair> FutureSelectionDetail = Async(EAsyncExecution::ThreadPool, MoveTemp(TaskSelectionInfo));

	FBuildPatchAppManifestPtr ManifestA = FutureManifestA.Get();
	FBuildPatchAppManifestPtr ManifestB = FutureManifestB.Get();
	FStringSetPair SelectionDetail = FutureSelectionDetail.Get();

	// Flush any logs collected by tasks
	GLog->FlushThreadedLogs();

	// We must have loaded our manifests
	if (ManifestA.IsValid() == false)
	{
		UE_LOGF(LogMergeManifests, Error, "Could not load manifest %ls", *ManifestFilePathA);
		return false;
	}
	if (ManifestB.IsValid() == false)
	{
		UE_LOGF(LogMergeManifests, Error, "Could not load manifest %ls", *ManifestFilePathB);
		return false;
	}

	// We cannot use merge manifests unless we have all required secret keys for both, otherwise we cannot match file names, and encrypt the produced manifest file.
	TSet<FGuid> AvailableSecrets;
	AvailableEncryptionSecrets.GetKeys(AvailableSecrets);
	TSet<FGuid> NecessaryEncryptionSecrets;
	NecessaryEncryptionSecrets.Append(ManifestA->GetNecessaryEncryptionSecretIds());
	NecessaryEncryptionSecrets.Append(ManifestB->GetNecessaryEncryptionSecretIds());
	TSet<FGuid> MissingSecrets = NecessaryEncryptionSecrets.Difference(AvailableSecrets);
	if (MissingSecrets.Num() > 0)
	{
		UE_LOGF(LogMergeManifests, Error, "Cannot continue without all necessary secret keys.");
		for (const FGuid& SecretId : MissingSecrets)
		{
			UE_LOGF(LogMergeManifests, Error, "    Missing secret with ID: %ls", *SecretId.ToString());
		}
		return false;
	}

	// Check if the selection detail had an error
	if (bSelectionDetailSuccess == false)
	{
		return false;
	}

	// If we have no selection detail, then we take the union of all files, preferring the version from B
	if (SelectionDetail.Key.Num() == 0 && SelectionDetail.Value.Num() == 0)
	{
		TSet<FString> ManifestFilesA(ManifestA->GetBuildFileList());
		SelectionDetail.Value.Append(ManifestB->GetBuildFileList());
		SelectionDetail.Key = ManifestFilesA.Difference(SelectionDetail.Value);
	}
	else
	{
		// If we accepted a selection detail, make sure any dupes come from ManifestB
		SelectionDetail.Key = SelectionDetail.Key.Difference(SelectionDetail.Value);
	}

	IBuildManifestPtr Result = MergeManifests(ManifestA.ToSharedRef(), ManifestB.ToSharedRef(), NewVersionString, SelectionDetail.Key, SelectionDetail.Value);

	// Save the new manifest out if we didn't register a failure
	if (Result.IsValid())
	{
		FBuildPatchAppManifestRef ResultRef = StaticCastSharedRef<FBuildPatchAppManifest>(Result.ToSharedRef());
		// If ManifestB was encrypted, encrypt ManifestC if we had the key given.
		bool bEncryptionSuccess = true;
		const FGuid EncryptionSecretId = ManifestB->GetEncryptionSecretId();
		if (EncryptionSecretId.IsValid())
		{
			const TArray<uint8>* EncryptionSecretKey = AvailableEncryptionSecrets.Find(EncryptionSecretId);
			if (EncryptionSecretKey != nullptr)
			{
				UE_LOGF(LogMergeManifests, Log, "Encrypting output manifest with secret %ls.", *EncryptionSecretId.ToString());
				bEncryptionSuccess = ResultRef->EncryptData(EncryptionSecretId, *EncryptionSecretKey);
				if (!bEncryptionSuccess)
				{
					UE_LOGF(LogMergeManifests, Error, "Systematic failure to encrypt new manifest");
				}
			}
			else
			{
				UE_LOGF(LogMergeManifests, Error, "Failed to encrypt new manifest, missing encryption key for %ls", *EncryptionSecretId.ToString());
				bEncryptionSuccess = false;
			}
		}
		if (bEncryptionSuccess)
		{
			TUniquePtr<IFileSystem> FileSystem(FFileSystemFactory::Create());
			const FString TmpManifestFilePathC = ManifestFilePathC + TEXT("tmp");
			if (!ResultRef->SaveToFile(TmpManifestFilePathC, ResultRef->ManifestMeta.FeatureLevel) || !FileSystem->MoveFile(*ManifestFilePathC, *TmpManifestFilePathC))
			{
				UE_LOGF(LogMergeManifests, Error, "Failed to save new manifest %ls", *ManifestFilePathC);
				Result.Reset();
			}
		}
		else
		{
				Result.Reset();
		}
	}
	else
	{
		UE_LOGF(LogMergeManifests, Error, "Not saving new manifest due to previous errors.");
	}

	return Result.IsValid();
}

IBuildManifestPtr FBuildMergeManifests::MergeManifests(const IBuildManifestRef& InManifestA, const IBuildManifestRef& InManifestB, const FString& NewVersionString, const TSet<FString>& FilesFromA, const TSet<FString>& FilesFromB)
{
	using namespace BuildPatchServices;
	bool bSuccess = true;
	FBuildPatchAppManifestRef ManifestA = StaticCastSharedRef<FBuildPatchAppManifest>(InManifestA);
	FBuildPatchAppManifestRef ManifestB = StaticCastSharedRef<FBuildPatchAppManifest>(InManifestB);

	// We can't do this if the manifests are currently encrypted.
	if (ManifestA->IsManifestEncrypted() || ManifestB->IsManifestEncrypted())
	{
		UE_LOGF(LogMergeManifests, Error, "Cannot merge encrypted manifests, make sure the secrets were provided.");
		return nullptr;
	}

	// Create the new manifest
	FBuildPatchAppManifestRef MergedManifest = MakeShareable(new FBuildPatchAppManifest());

	// Copy basic info from B, preserving the generated build ID
	{
		FString NewBuildId = MoveTemp(MergedManifest->ManifestMeta.BuildId);
		MergedManifest->ManifestMeta = ManifestB->ManifestMeta;
		MergedManifest->CustomFields = ManifestB->CustomFields;
		MergedManifest->ManifestMeta.BuildId = MoveTemp(NewBuildId);
	}

	// Set the new version string
	MergedManifest->ManifestMeta.BuildVersion = NewVersionString;

	// Copy the file manifests required from A
	bSuccess = MergeHelpers::CopyFileDataFromManifestToArray(FilesFromA, ManifestA, MergedManifest->FileManifestList.FileList) && bSuccess;

	// Copy the file manifests required from B
	bSuccess = MergeHelpers::CopyFileDataFromManifestToArray(FilesFromB, ManifestB, MergedManifest->FileManifestList.FileList) && bSuccess;

	// Call OnPostLoad for the file manifest list before entering chunk info.
	MergedManifest->FileManifestList.OnPostLoad();

	// Fill out the chunk list in order of reference
	bSuccess = MergeHelpers::ReinitialiseChunkInfoList(MergedManifest->FileManifestList.FileList, ManifestA.Get(), ManifestB.Get(), MergedManifest->ChunkDataList.ChunkList) && bSuccess;

	// Return invalid if there were any issues.
	if (!bSuccess)
	{
		return nullptr;
	}

	MergedManifest->InitLookups();
	return MergedManifest;
}

FBuildPatchAppManifestPtr FBuildMergeManifests::MergeDeltaManifest(const FBuildPatchAppManifest& Manifest, const FBuildPatchAppManifest& Delta)
{
	using namespace BuildPatchServices;
	FBuildPatchAppManifestRef MergedManifest = StaticCastSharedRef<FBuildPatchAppManifest>(Manifest.Duplicate());
	for (FFileManifest& FileManifest : MergedManifest->FileManifestList.FileList)
	{
		const FFileManifest* DeltaFileManifest = Delta.GetFileManifest(FileManifest.Filename);
		if (DeltaFileManifest != nullptr)
		{
			FileManifest.ChunkParts = DeltaFileManifest->ChunkParts;
			for (auto& Chunk : FileManifest.ChunkParts)
			{
				Chunk.bChunkFromDelta = true;
			}
		}
	}
	if (MergeHelpers::ReinitialiseChunkInfoList(MergedManifest->FileManifestList.FileList, Delta, Manifest, MergedManifest->ChunkDataList.ChunkList))
	{
		MergedManifest->InitLookups();
		return MergedManifest;
	}
	return nullptr;
}
