// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BuildPatchManifest.h"

class FBuildMergeManifests
{
public:
	static bool MergeManifests(const FString& ManifestFilePathA, const FString& ManifestFilePathB, const FString& ManifestFilePathC, const FString& NewVersionString, const FString& SelectionDetailFilePath, const TMap<FGuid, TArray<uint8>>& AvailableEncryptionSecrets);

	static IBuildManifestPtr MergeManifests(const IBuildManifestRef& ManifestA, const IBuildManifestRef& ManifestB, const FString& NewVersionString, const TSet<FString>& FilesFromA, const TSet<FString>& FilesFromB);

	static FBuildPatchAppManifestPtr MergeDeltaManifest(const FBuildPatchAppManifest& Manifest, const FBuildPatchAppManifest& Delta);
};
