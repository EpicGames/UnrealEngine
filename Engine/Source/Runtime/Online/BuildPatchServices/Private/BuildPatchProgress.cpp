// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BuildPatchProgress.cpp: Implements classes involved with tracking the patch
	progress information.
=============================================================================*/

#include "BuildPatchProgress.h"
#include "Misc/ScopeLock.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"

namespace BuildPatchServices
{

	/* FBuildPatchProgress implementation
	*****************************************************************************/
	FBuildPatchProgress::FBuildPatchProgress()
	{
		TotalWeight = 0.0f;
		CurrentState = EBuildPatchState::Queued;
		CurrentProgress = 0.0f;
		bIsDownloading = false;
		bShouldAbort = false;

		// Initialize array data
		for (uint32 idx = 0; idx < static_cast<uint32>(EBuildPatchState::NUM_PROGRESS_STATES); ++idx)
		{
			StateProgressValues[idx] = 0.0f;
			StateProgressWeights[idx] = 1.0f;
		}

		// Update for initial values
		UpdateCachedValues();
		UpdateProgressInfo();
	}

	void FBuildPatchProgress::SetPaused(bool bIsPaused)
	{
		FScopeLock ScopeLock(&ThreadLock);
		if (GetPauseState() != bIsPaused)
		{
			TogglePauseState();
		}
	}

	void FBuildPatchProgress::Abort()
	{
		bShouldAbort = true;
	}

	void FBuildPatchProgress::Reset()
	{
		bShouldAbort = false;
	}

	void FBuildPatchProgress::SetStateProgress(const EBuildPatchState& State, const float& Value)
	{
		check(!FMath::IsNaN(Value));
		FScopeLock ScopeLock(&ThreadLock);
		if (StateProgressValues[static_cast<uint32>(State)] != Value)
		{
			StateProgressValues[static_cast<uint32>(State)] = Value;
			UpdateProgressInfo();
		}
	}

	void FBuildPatchProgress::SetStateWeight(const EBuildPatchState& State, const float& Value)
	{
		check(!FMath::IsNaN(Value));
		FScopeLock ScopeLock(&ThreadLock);
		if (bCountsTowardsProgress[static_cast<uint32>(State)])
		{
			TotalWeight += Value - StateProgressWeights[static_cast<uint32>(State)];
			StateProgressWeights[static_cast<uint32>(State)] = Value;
		}
	}

	EBuildPatchState FBuildPatchProgress::GetState() const
	{
		FScopeLock ScopeLock(&ThreadLock);
		return CurrentState;
	}

	float FBuildPatchProgress::GetProgress() const
	{
		FScopeLock ScopeLock(&ThreadLock);
		if (bHasProgressValue[static_cast<uint32>(CurrentState)])
		{
			return CurrentProgress;
		}
		// We return -1.0f to indicate a marquee style progress should be displayed
		return -1.0f;
	}

	float FBuildPatchProgress::GetProgressNoMarquee() const
	{
		FScopeLock ScopeLock(&ThreadLock);
		return CurrentProgress;
	}

	float FBuildPatchProgress::GetStateProgress(const EBuildPatchState& State) const
	{
		FScopeLock ScopeLock(&ThreadLock);
		return StateProgressValues[static_cast<uint32>(State)];
	}

	float FBuildPatchProgress::GetStateWeight(const EBuildPatchState& State) const
	{
		FScopeLock ScopeLock(&ThreadLock);
		return StateProgressWeights[static_cast<uint32>(State)];
	}

	bool FBuildPatchProgress::TogglePauseState()
	{
		FScopeLock ScopeLock(&ThreadLock);

		// If in pause state revert to no state and recalculate progress
		if (CurrentState == EBuildPatchState::Paused)
		{
			CurrentState = EBuildPatchState::NUM_PROGRESS_STATES;
			UpdateProgressInfo();
			return false;
		}
		else
		{
			// Else we just set paused
			CurrentState = EBuildPatchState::Paused;
			return true;
		}
	}

	bool FBuildPatchProgress::GetPauseState() const
	{
		FScopeLock ScopeLock(&ThreadLock);
		return CurrentState == EBuildPatchState::Paused;
	}

	double FBuildPatchProgress::WaitWhilePaused() const
	{
		// Pause if necessary
		const double PrePauseTime = FPlatformTime::Seconds();
		double PostPauseTime = PrePauseTime;
		while (GetPauseState())
		{
			FPlatformProcess::Sleep(0.1f);
			PostPauseTime = FPlatformTime::Seconds();
		}
		// Return pause time
		return PostPauseTime - PrePauseTime;
	}

	void FBuildPatchProgress::SetIsDownloading(bool bInIsDownloading)
	{
		FScopeLock ScopeLock(&ThreadLock);
		if (bIsDownloading != bInIsDownloading)
		{
			bIsDownloading = bInIsDownloading;
			UpdateProgressInfo();
		}
	}

	void FBuildPatchProgress::UpdateProgressInfo()
	{
		FScopeLock ScopeLock(&ThreadLock);

		// If we are paused or there's an error, we don't change the state or progress value
		if (bShouldAbort || CurrentState == EBuildPatchState::Paused)
		{
			return;
		}

		// Reset values
		CurrentState = EBuildPatchState::NUM_PROGRESS_STATES;
		CurrentProgress = 0.0f;

		// Loop through each progress value to find total progress and current state
		for (uint32 idx = 0; idx < static_cast<uint32>(EBuildPatchState::NUM_PROGRESS_STATES); ++idx)
		{
			// Is this the current state
			if ((CurrentState == EBuildPatchState::NUM_PROGRESS_STATES) && StateProgressValues[idx] < 1.0f)
			{
				CurrentState = static_cast<EBuildPatchState>(idx);
			}

			// Do we count the progress
			if (bCountsTowardsProgress[idx])
			{
				const float StateProgress = StateProgressValues[idx] * (StateProgressWeights[idx] / TotalWeight);
				CurrentProgress += StateProgress;
				check(!FMath::IsNaN(CurrentProgress));
			}
		}

		// Ensure Sanity
		CurrentProgress = FMath::Clamp(CurrentProgress, 0.0f, 1.0f);

		// Switch between Downloading and Installing depending on bIsDownloading
		// This avoids reporting a Downloading state while the download system is idle during long periods of
		// mainly hard disk activity, before all downloadable chunks have been required.
		if (CurrentState == EBuildPatchState::Downloading && !bIsDownloading)
		{
			CurrentState = EBuildPatchState::Installing;
		}
	}

	void FBuildPatchProgress::UpdateCachedValues()
	{
		TotalWeight = 0.0f;
		for (uint32 idx = 0; idx < static_cast<uint32>(EBuildPatchState::NUM_PROGRESS_STATES); ++idx)
		{
			if (bCountsTowardsProgress[idx])
			{
				TotalWeight += StateProgressWeights[idx];
			}
		}
		// If you assert here you're going to cause a divide by 0.0f
		check(TotalWeight != 0.0f);
	}

	/* FBuildPatchProgress static constants
	*****************************************************************************/
	static_assert((uint32)BuildPatchServices::EBuildPatchState::NUM_PROGRESS_STATES == 13, "Please add support for the extra EBuildPatchState values to bHasProgressValue.");
	const bool FBuildPatchProgress::bHasProgressValue[static_cast<int32>(EBuildPatchState::NUM_PROGRESS_STATES)] =
	{
		false, // Queued
		false, // Initializing
		true,  // Resuming
		true,  // Downloading
		true,  // Installing
		true,  // RunningCustomUninstallAction
		true,  // MovingToInstall
		true,  // SettingAttributes
		true,  // BuildVerification
		false, // CleanUp
		false, // PrerequisitesInstall
		true,  // Completed
		true   // Paused
	};


	static_assert((uint32)BuildPatchServices::EBuildPatchState::NUM_PROGRESS_STATES == 13, "Please add support for the extra EBuildPatchState values to bCountsTowardsProgress.");
	const bool FBuildPatchProgress::bCountsTowardsProgress[static_cast<int32>(EBuildPatchState::NUM_PROGRESS_STATES)] =
	{
		false, // Queued
		false, // Initializing
		false, // Resuming
		true,  // Downloading
		true,  // Installing
		false, // RunningCustomUninstallAction
		true,  // MovingToInstall
		true,  // SettingAttributes
		true,  // BuildVerification
		false, // CleanUp
		false, // PrerequisitesInstall
		false, // Completed
		false  // Paused
	};
}
