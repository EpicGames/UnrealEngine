// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BuildPatchProgress.h: Declares classes involved with tracking the patch
	progress information.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeBool.h"
#include "Installer/Controllable.h"
#include "BuildPatchState.h"

namespace BuildPatchServices
{
	/**
	 * A struct to hold patch progress tracking
	 */
	struct FBuildPatchProgress
		: public IControllable
	{
	public:

		/**
		 * Default constructor
		 */
		FBuildPatchProgress();

		// IControllable interface begin.
		virtual void SetPaused(bool bIsPaused) override;
		virtual void Abort() override;
		virtual void Reset() override;
		// IControllable interface end.

		/**
		 * Sets the progress value for a particular state
		 * @param State		The state to set progress for
		 * @param Value		The progress value
		 */
		virtual void SetStateProgress(const EBuildPatchState& State, const float& Value);

		/**
		 * Sets the progress weight for a particular state
		 * @param State		The state to set weight for
		 * @param Value		The weight value
		 */
		virtual void SetStateWeight(const EBuildPatchState& State, const float& Value);

		/**
		 * Gets the current progress state
		 * @return The current progress state
		 */
		virtual EBuildPatchState GetState() const;

		/**
		 * Gets the current overall progress
		 * @return The current progress value. Range 0 to 1. -1 indicates undetermined, i.e. show a marquee style bar.
		 */
		virtual float GetProgress() const;

		/**
		 * Gets the current overall progress regardless of current state using marquee
		 * @return The current progress value. Range 0 to 1.
		 */
		virtual float GetProgressNoMarquee() const;

		/**
		 * Gets the progress value for a particular state
		 * @param State		The state to get progress for
		 * @return The state progress value. Range 0 to 1.
		 */
		virtual float GetStateProgress(const EBuildPatchState& State) const;

		/**
		 * Gets the weight value for a particular state
		 * @param State		The state to get weight for
		 * @return The state weight value.
		 */
		virtual float GetStateWeight(const EBuildPatchState& State) const;

		/**
		 * Toggles the pause state
		 * @return Whether the current state is now paused
		 */
		virtual bool TogglePauseState();

		/**
		 * Blocks calling thread while the progress is paused
		 * @return How long we paused for, in seconds
		 */
		virtual double WaitWhilePaused() const;

		/**
		 * Gets the pause state
		 * @return Whether the current state is paused
		 */
		virtual bool GetPauseState() const;

		/**
		 * Set the set state of whether the system is currently downloading data.
		 * @param bIsDownloading	Whether the system is currently downloading data.
		 */
		virtual void SetIsDownloading(bool bIsDownloading);

	private:

		/**
		 * Updates the current state and progress values
		 */
		void UpdateProgressInfo();

		/**
		 * Updates the cached values
		 */
		void UpdateCachedValues();

	private:

		// Defines whether each state displays progress percent or is designed for a "please wait" or marquee style progress bar
		// This is predefined and constant.
		static const bool bHasProgressValue[static_cast<int32>(EBuildPatchState::NUM_PROGRESS_STATES)];

		// Defines whether each state should count towards the overall progress
		// This is predefined and constant.
		static const bool bCountsTowardsProgress[static_cast<int32>(EBuildPatchState::NUM_PROGRESS_STATES)];

		// Holds the current percentage complete for each state, this will decide the "current" state, being the first that is not complete.
		// Range 0 to 1.
		float StateProgressValues[static_cast<int32>(EBuildPatchState::NUM_PROGRESS_STATES)];

		// Holds the weight that each stage has on overall progress. 
		// Range 0 to 1.
		float StateProgressWeights[static_cast<int32>(EBuildPatchState::NUM_PROGRESS_STATES)];

		// Cached total weight value for progress calculation
		float TotalWeight;

		// Externally set variable to say if the system is currently making download requests
		bool bIsDownloading;

		// The current state value for UI polling
		EBuildPatchState CurrentState;

		// The current progress value for UI polling
		float CurrentProgress;

		// Thread safe bool for whether the process should abort
		FThreadSafeBool bShouldAbort;

		// Critical section to protect variable access
		mutable FCriticalSection ThreadLock;

	};
}
