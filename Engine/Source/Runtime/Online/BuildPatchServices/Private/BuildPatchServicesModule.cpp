// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchServicesModule.h"

#include "Containers/Ticker.h"
#include "Algo/Transform.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Modules/ModuleManager.h"
#include "HttpModule.h"
#include "HttpManager.h"

#include "Compactify/PatchDataCompactifier.h"
#include "Data/ManifestData.h"
#include "Diffing/DiffManifests.h"
#include "Enumeration/PatchDataEnumeration.h"
#include "Generation/ChunkDeltaOptimiser.h"
#include "Generation/PackageChunkData.h"
#include "Generation/ChunkHarvester.h"
#include "Installer/BuildStatistics.h"
#include "Installer/InstallerSharedContext.h"
#include "Installer/MachineConfig.h"
#include "BuildPatchMergeManifests.h"
#include "BuildPatchHash.h"
#include "BuildPatchGeneration.h"
#include "BuildPatchServicesPrivate.h"
#include "BuildPatchVerifyChunkData.h"
#include "BuildPatchServicesSingleton.h"

using namespace BuildPatchServices;

DEFINE_LOG_CATEGORY(LogBuildPatchServices);
IMPLEMENT_MODULE(FBuildPatchServicesModule, BuildPatchServices);

/* FBuildPatchServicesModule implementation
 *****************************************************************************/

void FBuildPatchServicesModule::StartupModule()
{
	// Debug sanity checks
	static_assert(UE_ARRAY_COUNT(InstallErrorPrefixes::ErrorTypeStrings) == (uint64)EBuildPatchInstallError::InvalidOrMax, "Please add support for the extra values to the InstallErrorPrefixes::ErrorTypeStrings.");
#if UE_BUILD_DEBUG
	TSet<FString> NoDupes;
	bool bWasDupe = false;
	for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(InstallErrorPrefixes::ErrorTypeStrings); ++Idx)
	{
		NoDupes.Add(FString(InstallErrorPrefixes::ErrorTypeStrings[Idx]), &bWasDupe);
		check(bWasDupe == false);
	}
#endif

	// We need to initialize the lookup for our hashing functions
	FRollingHashConst::Init();

	const FBuildPatchServicesInitSettings& InitSettings = FBuildPatchServices::GetSettings();
	// Set the local machine config filename
	LocalMachineConfigFile = FPaths::Combine(InitSettings.ApplicationSettingsDir, InitSettings.ProjectName, InitSettings.LocalMachineConfigFileName);

	// Fix up any legacy configuration data
	FixupLegacyConfig();

	// Check if the user has opted to force skip prerequisites install
	bool bForceSkipPrereqsCmdline = FParse::Param(FCommandLine::Get(), TEXT("skipbuildpatchprereq"));
	bool bForceSkipPrereqsConfig = false;
	GConfig->GetBool(TEXT("Portal.BuildPatch"), TEXT("skipbuildpatchprereq"), bForceSkipPrereqsConfig, GEngineIni);

	if (bForceSkipPrereqsCmdline)
	{
		GLog->Log(TEXT("BuildPatchServicesModule: Setup to skip prerequisites install via commandline."));
	}

	if (bForceSkipPrereqsConfig)
	{
		GLog->Log( TEXT("BuildPatchServicesModule: Setup to skip prerequisites install via config."));
	}
	
	bForceSkipPrereqs = bForceSkipPrereqsCmdline || bForceSkipPrereqsConfig;

	// Add our ticker
	TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker( FTickerDelegate::CreateRaw( this, &FBuildPatchServicesModule::Tick ) );

	// Register core PreExit
	FCoreDelegates::OnPreExit.AddRaw(this, &FBuildPatchServicesModule::PreExit);

	// Test the rolling hash algorithm
	check( CheckRollingHashAlgorithm() );

	// Init Manifest serialization
	FManifestData::Init();

	// Create our installer start delegate
	InstallerStartDelegate = FBuildPatchInstallerDelegate::CreateLambda([this](const IBuildInstallerRef& StartedInstaller)
	{
		BuildPatchInstallers.Add(StaticCastSharedRef<FBuildPatchInstaller>(StartedInstaller));
		BuildPatchInstallerInterfaces.Add(StartedInstaller);
		OnStartBuildInstallEvent.Broadcast();
	});
}

void FBuildPatchServicesModule::ShutdownModule()
{
	GWarn->Logf( TEXT( "BuildPatchServicesModule: Shutting Down" ) );

	checkf(BuildPatchInstallers.Num() == 0, TEXT("BuildPatchServicesModule: FATAL ERROR: Core PreExit not called, or installer created during shutdown!"));

	// Remove our ticker
	GLog->Log(ELogVerbosity::VeryVerbose, TEXT( "BuildPatchServicesModule: Removing Ticker" ) );
	FTSTicker::RemoveTicker( TickDelegateHandle );

	GLog->Log(ELogVerbosity::VeryVerbose, TEXT( "BuildPatchServicesModule: Finished shutting down" ) );
}

IBuildInstallerRef FBuildPatchServicesModule::CreateBuildInstaller(BuildPatchServices::FBuildInstallerConfiguration Configuration, FBuildPatchInstallerDelegate CompleteDelegate) const
{
	// Override prereq install using the config/commandline value to force skip them.
	if (bForceSkipPrereqs)
	{
		Configuration.bRunRequiredPrereqs = false;
	}
	TMap<FGuid, TArray<uint8>> InstallerSecrets = GetEncryptionSecrets();
	InstallerSecrets.Append(MoveTemp(Configuration.EncryptionSecrets));
	Configuration.EncryptionSecrets = MoveTemp(InstallerSecrets);
	FBuildPatchInstallerRef Installer = MakeShared<FBuildPatchInstaller>(MoveTemp(Configuration), AvailableInstallations, LocalMachineConfigFile, Analytics, InstallerStartDelegate, MoveTemp(CompleteDelegate));
	return Installer;
}

IBuildInstallerSharedContextRef FBuildPatchServicesModule::CreateBuildInstallerSharedContext(const TCHAR* DebugName) const
{
	return BuildPatchServices::FBuildInstallerSharedContextFactory::Create(DebugName);
}

IBuildStatisticsRef FBuildPatchServicesModule::CreateBuildStatistics(const IBuildInstallerRef& Installer) const
{
	checkSlow(IsInGameThread());
	return MakeShareable(FBuildStatisticsFactory::Create(StaticCastSharedRef<FBuildPatchInstaller>(Installer)));
}

IDirectoryChunkerRef FBuildPatchServicesModule::CreateDirectoryChunker(BuildPatchServices::FDirectoryChunkerConfiguration Configuration) const
{
	using namespace BuildPatchServices;
	return MakeShareable(FDirectoryChunkerFactory::Create(MoveTemp(Configuration)));
}

IPatchDataEnumerationRef FBuildPatchServicesModule::CreatePatchDataEnumeration(BuildPatchServices::FPatchDataEnumerationConfiguration Configuration) const
{
	using namespace BuildPatchServices;
	return MakeShareable(FPatchDataEnumerationFactory::Create(MoveTemp(Configuration)));
}

IChunkDeltaOptimiserRef FBuildPatchServicesModule::CreateChunkDeltaOptimiser(BuildPatchServices::FChunkDeltaOptimiserConfiguration Configuration) const
{
	using namespace BuildPatchServices;
	TMap<FGuid, TArray<uint8>> AllSecrets = GetEncryptionSecrets();
	AllSecrets.Append(MoveTemp(Configuration.EncryptionSecrets));
	Configuration.EncryptionSecrets = MoveTemp(AllSecrets);
	return MakeShareable(FChunkDeltaOptimiserFactory::Create(MoveTemp(Configuration)));
}

IChunkHarvesterRef FBuildPatchServicesModule::CreateChunkHarvester(BuildPatchServices::FChunkHarvesterConfiguration Configuration) const
{
	using namespace BuildPatchServices;
	return MakeShareable(FChunkHarvesterFactory::Create(MoveTemp(Configuration)));
}

void FBuildPatchServicesModule::DecryptManifestIfNecessary(FBuildPatchAppManifest& Manifest) const
{
	bool bIsManifestEncrypted = false;
	const FGuid EncryptionSecretId = Manifest.GetEncryptionSecretId(&bIsManifestEncrypted);
	if (!bIsManifestEncrypted)
	{
		return;
	}

	const TOptional<TArray<uint8>> EncryptionSecret = FindEncryptionSecret(EncryptionSecretId);
	if (!EncryptionSecret.IsSet())
	{
		return;
	}

	const TMap<FGuid, TArray<uint8>> EncryptionSecrets = { {EncryptionSecretId, *EncryptionSecret} };
	Manifest.DecryptData(EncryptionSecrets);
}

IBuildManifestPtr FBuildPatchServicesModule::LoadManifestFromFile( const FString& Filename )
{
	FBuildPatchAppManifestRef Manifest = MakeShareable( new FBuildPatchAppManifest() );
	if( Manifest->LoadFromFile( Filename ) )
	{
		DecryptManifestIfNecessary(*Manifest);
		return Manifest;
	}
	else
	{
		GLog->Log(FString::Printf(TEXT("BuildPatchServicesModule: '%s' file is absent or can't be loaded."), *Filename));
		return nullptr;
	}
}

IBuildManifestPtr FBuildPatchServicesModule::MakeManifestFromData(const TArray<uint8>& ManifestData)
{
	FBuildPatchAppManifestRef Manifest = MakeShareable(new FBuildPatchAppManifest());
	if (Manifest->DeserializeFromData(ManifestData))
	{
		DecryptManifestIfNecessary(*Manifest);
		return Manifest;
	}
	return nullptr;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
IBuildManifestPtr FBuildPatchServicesModule::MakeManifestFromJSON( const FString& ManifestJSON )
{
	FBuildPatchAppManifestRef Manifest = MakeShareable( new FBuildPatchAppManifest() );
	if( Manifest->DeserializeFromJSON( ManifestJSON ) )
	{
		return Manifest;
	}
	return NULL;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FBuildPatchServicesModule::SaveManifestToFile(const FString& Filename, IBuildManifestRef Manifest)
{
	return StaticCastSharedRef<FBuildPatchAppManifest>(Manifest)->SaveToFile(Filename, Manifest->GetFeatureLevel());
}

TSet<FString> FBuildPatchServicesModule::GetInstalledPrereqIds() const
{
	const bool bAlwaysFlushChanges = true;
	TUniquePtr<IMachineConfig> MachineConfig(FMachineConfigFactory::Create(LocalMachineConfigFile, bAlwaysFlushChanges));
	return MachineConfig->LoadInstalledPrereqIds();
}

const TArray<IBuildInstallerRef>& FBuildPatchServicesModule::GetInstallers() const
{
	checkSlow(IsInGameThread());
	return BuildPatchInstallerInterfaces;
}

bool FBuildPatchServicesModule::Tick(float Delta)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FBuildPatchServicesModule_Tick);
	checkSlow(IsInGameThread());

	// Tick running installers.
	for (auto InstallerIter = BuildPatchInstallers.CreateIterator(); InstallerIter; ++InstallerIter)
	{
		const FBuildPatchInstallerRef& Installer = *InstallerIter;
		if (!Installer->Tick())
		{
			InstallerIter.RemoveCurrent();
		}
	}

	// Check for resetting the BuildPatchInstallerInterfaces array.
	if (BuildPatchInstallers.Num() != BuildPatchInstallerInterfaces.Num())
	{
		BuildPatchInstallerInterfaces.Empty(BuildPatchInstallers.Num());
		for (const FBuildPatchInstallerRef& Installer : BuildPatchInstallers)
		{
			BuildPatchInstallerInterfaces.Add(Installer);
		}
	}

	// More ticks.
	return true;
}

bool FBuildPatchServicesModule::CompactifyCloudDirectory(const BuildPatchServices::FCompactifyConfiguration& Configuration)
{
	using namespace BuildPatchServices;
	TUniquePtr<IPatchDataCompactifier> PatchDataCompactifier(FPatchDataCompactifierFactory::Create(Configuration));
	return PatchDataCompactifier->Run();
}

bool FBuildPatchServicesModule::VerifyChunkData(const FString& SearchPath, const FString& OutputFile)
{
	return FBuildVerifyChunkData::VerifyChunkData(SearchPath, OutputFile);
}

bool FBuildPatchServicesModule::PackageChunkData(const BuildPatchServices::FPackageChunksConfiguration& Configuration)
{
	using namespace BuildPatchServices;
	TUniquePtr<IPackageChunks> PackageChunks(FPackageChunksFactory::Create(Configuration));
	return PackageChunks->Run();
}

bool FBuildPatchServicesModule::MergeManifests(const FString& ManifestFilePathA, const FString& ManifestFilePathB, const FString& ManifestFilePathC, const FString& NewVersionString, const FString& SelectionDetailFilePath)
{
	return FBuildMergeManifests::MergeManifests(ManifestFilePathA, ManifestFilePathB, ManifestFilePathC, NewVersionString, SelectionDetailFilePath, GetEncryptionSecrets());
}

IBuildManifestPtr FBuildPatchServicesModule::MergeManifests(const IBuildManifestRef& ManifestA, const IBuildManifestRef& ManifestB, const FString& NewVersionString, const TSet<FString>& FilesFromA, const TSet<FString>& FilesFromB)
{
	return FBuildMergeManifests::MergeManifests(ManifestA, ManifestB, NewVersionString, FilesFromA, FilesFromB);
}

bool FBuildPatchServicesModule::DiffManifests(const BuildPatchServices::FDiffManifestsConfiguration& Configuration)
{
	using namespace BuildPatchServices;
	TUniquePtr<IDiffManifests> DiffManifests(FDiffManifestsFactory::Create(Configuration));
	return DiffManifests->Run();
}

FBuildPatchServicesModule::FSimpleEvent& FBuildPatchServicesModule::OnStartBuildInstall()
{
	return OnStartBuildInstallEvent;
}

void FBuildPatchServicesModule::SetStagingDirectory( const FString& StagingDir )
{
	StagingDirectory = StagingDir;
}

void FBuildPatchServicesModule::SetCloudDirectory(FString CloudDir)
{
	TArray<FString> CloudDirs;
	CloudDirs.Add(MoveTemp(CloudDir));
	SetCloudDirectories(MoveTemp(CloudDirs));
}

void FBuildPatchServicesModule::SetCloudDirectories(TArray<FString> CloudDirs)
{
	check(IsInGameThread());
	CloudDirectories = MoveTemp(CloudDirs);
	NormalizeCloudPaths(CloudDirectories);
}

void FBuildPatchServicesModule::NormalizeCloudPaths(TArray<FString>& InOutCloudPaths)
{
	for (FString& CloudPath : InOutCloudPaths)
	{
		// Ensure that we remove any double-slash characters apart from:
		//   1. A double slash following the URI schema
		//   2. A double slash at the start of the path, indicating a network share
		CloudPath.ReplaceInline(TEXT("\\"), TEXT("/"));
		bool bIsNetworkPath = CloudPath.StartsWith(TEXT("//"));
		CloudPath.ReplaceInline(TEXT("://"), TEXT(":////"));
		CloudPath.ReplaceInline(TEXT("//"), TEXT("/"));
		if (bIsNetworkPath)
		{
			CloudPath.InsertAt(0, TEXT("/"));
		}
	}
}

void FBuildPatchServicesModule::SetBackupDirectory( const FString& BackupDir )
{
	BackupDirectory = BackupDir;
}

void FBuildPatchServicesModule::SetAnalyticsProvider( TSharedPtr<IAnalyticsProvider> InAnalyticsProvider )
{
	Analytics = InAnalyticsProvider;
}

void FBuildPatchServicesModule::RegisterAppInstallation(IBuildManifestRef AppManifest, FString AppInstallDirectory)
{
	FPaths::NormalizeDirectoryName(AppInstallDirectory);
	FPaths::CollapseRelativeDirectories(AppInstallDirectory);
	FBuildPatchAppManifestRef InternalRef = StaticCastSharedRef<FBuildPatchAppManifest>(MoveTemp(AppManifest));
	AvailableInstallations.Add(MoveTemp(AppInstallDirectory), MoveTemp(InternalRef));
}

bool FBuildPatchServicesModule::UnregisterAppInstallation(FString AppInstallDirectory)
{
	FPaths::NormalizeDirectoryName(AppInstallDirectory);
	FPaths::CollapseRelativeDirectories(AppInstallDirectory);
	if (AvailableInstallations.Remove(AppInstallDirectory) > 0)
	{
		return true;
	}
	return false;
}

void FBuildPatchServicesModule::RegisterEncryptionSecret(const FGuid& EncryptionSecretId, TArray<uint8> EncryptionSecretKey)
{
	FScopeLock ScopeLock(&AvailableEncryptionSecretsCS);
	AvailableEncryptionSecrets.Add(EncryptionSecretId, MoveTemp(EncryptionSecretKey));
}

bool FBuildPatchServicesModule::UnregisterEncryptionSecret(const FGuid& EncryptionSecretId)
{
	FScopeLock ScopeLock(&AvailableEncryptionSecretsCS);
	if (AvailableEncryptionSecrets.Remove(EncryptionSecretId) > 0)
	{
		return true;
	}
	return false;
}

TOptional<TArray<uint8>> FBuildPatchServicesModule::FindEncryptionSecret(const FGuid& EncryptionSecretId) const
{
	FScopeLock ScopeLock(&AvailableEncryptionSecretsCS);
	const TArray<uint8>* EncryptionSecret = AvailableEncryptionSecrets.Find(EncryptionSecretId);
	return EncryptionSecret ? TOptional<TArray<uint8>>{*EncryptionSecret} : TOptional<TArray<uint8>>{};
}

TMap<FGuid, TArray<uint8>> FBuildPatchServicesModule::GetEncryptionSecrets() const
{
	FScopeLock ScopeLock(&AvailableEncryptionSecretsCS);
	return AvailableEncryptionSecrets;
}


bool FBuildPatchServicesModule::CreateRandomBytes(const TArrayView<uint8> OutData)
{
	TUniquePtr<ICrypto> Crypto{ FCryptoFactory::Create() };

	return Crypto->CreateRandomBytes(OutData);
}

void FBuildPatchServicesModule::CancelAllInstallers(bool WaitForThreads)
{
	// Using a local bool for this check will improve the assert message that gets displayed
	const bool bIsCalledFromMainThread = IsInGameThread();
	check(bIsCalledFromMainThread);

	// Loop each installer, cancel it.
	for (auto InstallerIter = BuildPatchInstallers.CreateIterator(); InstallerIter; ++InstallerIter)
	{
		const FBuildPatchInstallerRef& Installer = *InstallerIter;
		Installer->CancelInstall();
	}
	// Optionally wait for each installer to finish.
	if (WaitForThreads)
	{
		while (BuildPatchInstallers.Num() > 0)
		{
			FPlatformProcess::Sleep(1.0f / 60.0f);
			for (auto InstallerIter = BuildPatchInstallers.CreateIterator(); InstallerIter; ++InstallerIter)
			{
				const FBuildPatchInstallerRef& Installer = *InstallerIter;
				if (!Installer->Tick())
				{
					InstallerIter.RemoveCurrent();
				}
			}
		}
	}
	BuildPatchInstallers.Empty();
	BuildPatchInstallerInterfaces.Empty();
}

void FBuildPatchServicesModule::PreExit()
{
	// Inform installers
	for (auto InstallerIter = BuildPatchInstallers.CreateIterator(); InstallerIter; ++InstallerIter)
	{
		const FBuildPatchInstallerRef& Installer = *InstallerIter;
		Installer->PreExit();
	}

	// Release our ptr to analytics
	Analytics.Reset();
}

void FBuildPatchServicesModule::FixupLegacyConfig()
{
	// Check for old prerequisite installation values to bring in from user configuration
	TArray<FString> OldInstalledPrereqs;
	if (GConfig->GetArray(TEXT("Portal.BuildPatch"), TEXT("InstalledPrereqs"), OldInstalledPrereqs, GEngineIni) && OldInstalledPrereqs.Num() > 0)
	{
		bool bShouldSaveOut = false;
		TArray<FString> InstalledPrereqs;
		if (GConfig->GetArray(TEXT("Portal.BuildPatch"), TEXT("InstalledPrereqs"), InstalledPrereqs, LocalMachineConfigFile) && InstalledPrereqs.Num() > 0)
		{
			// Add old values to the new array
			for (const FString& OldInstalledPrereq : OldInstalledPrereqs)
			{
				int32 PrevNum = InstalledPrereqs.Num();
				bool bAlreadyInArray = InstalledPrereqs.AddUnique(OldInstalledPrereq) < PrevNum;
				bShouldSaveOut = bShouldSaveOut || !bAlreadyInArray;
			}
		}
		else
		{
			// Just use the old array
			InstalledPrereqs = MoveTemp(OldInstalledPrereqs);
			bShouldSaveOut = true;
		}
		// If we added extra then save new config
		if (bShouldSaveOut)
		{
			GConfig->SetArray(TEXT("Portal.BuildPatch"), TEXT("InstalledPrereqs"), InstalledPrereqs, LocalMachineConfigFile);
		}
		// Clear out the old config
		GConfig->RemoveKey(TEXT("Portal.BuildPatch"), TEXT("InstalledPrereqs"), GEngineIni);
	}
}

const FString& FBuildPatchServicesModule::GetStagingDirectory()
{
	// Default staging directory
	if( StagingDirectory.IsEmpty() )
	{
		StagingDirectory = FPaths::ProjectDir() + TEXT( "BuildStaging/" );
	}
	return StagingDirectory;
}

FString FBuildPatchServicesModule::GetCloudDirectory(int32 CloudIdx)
{
	FString RtnValue;
	if (CloudDirectories.Num())
	{
		RtnValue = CloudDirectories[CloudIdx % CloudDirectories.Num()];
	}
	else
	{
		// Default cloud directory
		RtnValue = FPaths::CloudDir();
	}
	return RtnValue;
}

TArray<FString> FBuildPatchServicesModule::GetCloudDirectories()
{
	TArray<FString> RtnValue;
	if (CloudDirectories.Num() > 0)
	{
		RtnValue = CloudDirectories;
	}
	else
	{
		// Singular function controls the default when none provided
		RtnValue.Add(GetCloudDirectory(0));
	}
	return RtnValue;
}

const FString& FBuildPatchServicesModule::GetBackupDirectory()
{
	// Default backup directory stays empty which simply doesn't backup
	return BackupDirectory;
}

/* Static variables
 *****************************************************************************/
TSharedPtr<IAnalyticsProvider> FBuildPatchServicesModule::Analytics;
TArray<FString> FBuildPatchServicesModule::CloudDirectories;
FString FBuildPatchServicesModule::StagingDirectory;
FString FBuildPatchServicesModule::BackupDirectory;
