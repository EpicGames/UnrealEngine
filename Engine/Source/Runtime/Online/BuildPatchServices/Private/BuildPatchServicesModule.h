// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "BuildPatchInstaller.h"
#include "Containers/Ticker.h"

class IAnalyticsProvider;

/**
 * Constant values and typedefs
 */
enum
{
	// Sizes
	FileBufferSize		= 1024*1024*4,		// When reading from files, how much to buffer
	StreamBufferSize	= FileBufferSize*4,	// When reading from build data stream, how much to buffer.
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/**
 * Implements the BuildPatchServicesModule.
 */
class FBuildPatchServicesModule
	: public IBuildPatchServicesModule
{
public:

	// IModuleInterface interface begin.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// IModuleInterface interface end.

	// IBuildPatchServicesModule interface begin.
	virtual IBuildInstallerRef CreateBuildInstaller(BuildPatchServices::FBuildInstallerConfiguration Configuration, FBuildPatchInstallerDelegate OnComplete) const override;
	virtual IBuildInstallerSharedContextRef CreateBuildInstallerSharedContext(const TCHAR* DebugName) const override;
	virtual BuildPatchServices::IBuildStatisticsRef CreateBuildStatistics(const IBuildInstallerRef& Installer) const override;
	virtual BuildPatchServices::IDirectoryChunkerRef CreateDirectoryChunker(BuildPatchServices::FDirectoryChunkerConfiguration Configuration) const override;
	virtual BuildPatchServices::IPatchDataEnumerationRef CreatePatchDataEnumeration(BuildPatchServices::FPatchDataEnumerationConfiguration Configuration) const override;
	virtual BuildPatchServices::IChunkDeltaOptimiserRef CreateChunkDeltaOptimiser(BuildPatchServices::FChunkDeltaOptimiserConfiguration Configuration) const override;
	virtual BuildPatchServices::IChunkHarvesterRef CreateChunkHarvester(BuildPatchServices::FChunkHarvesterConfiguration Configuration) const override;
	virtual IBuildManifestPtr LoadManifestFromFile(const FString& Filename) override;
	virtual IBuildManifestPtr MakeManifestFromData(const TArray<uint8>& ManifestData) override;
	virtual bool SaveManifestToFile(const FString& Filename, IBuildManifestRef Manifest) override;
	virtual TSet<FString> GetInstalledPrereqIds() const override;
	virtual const TArray<IBuildInstallerRef>& GetInstallers() const override;
	virtual void SetStagingDirectory(const FString& StagingDir) override;
	virtual void SetCloudDirectory(FString CloudDir) override;
	virtual void SetCloudDirectories(TArray<FString> CloudDirs) override;
	virtual void SetBackupDirectory(const FString& BackupDir) override;
	virtual void SetAnalyticsProvider(TSharedPtr< IAnalyticsProvider > AnalyticsProvider) override;
	virtual void RegisterAppInstallation(IBuildManifestRef AppManifest, const FString AppInstallDirectory) override;
	virtual bool UnregisterAppInstallation(const FString AppInstallDirectory) override;
	virtual void RegisterEncryptionSecret(const FGuid& EncryptionSecretId, TArray<uint8> EncryptionSecretKey) override;
	virtual bool UnregisterEncryptionSecret(const FGuid& EncryptionSecretId) override;
	virtual bool CreateRandomBytes(const TArrayView<uint8> OutData) override;
	virtual void CancelAllInstallers(bool WaitForThreads) override;
	virtual bool CompactifyCloudDirectory(const BuildPatchServices::FCompactifyConfiguration& Configuration) override;
	virtual bool VerifyChunkData(const FString& SearchPath, const FString& OutputFile) override;
	virtual bool PackageChunkData(const BuildPatchServices::FPackageChunksConfiguration& Configuration) override;
	virtual bool MergeManifests(const FString& ManifestFilePathA, const FString& ManifestFilePathB, const FString& ManifestFilePathC, const FString& NewVersionString, const FString& SelectionDetailFilePath) override;
	virtual IBuildManifestPtr MergeManifests(const IBuildManifestRef& ManifestA, const IBuildManifestRef& ManifestB, const FString& NewVersionString, const TSet<FString>& FilesFromA, const TSet<FString>& FilesFromB) override;
	virtual bool DiffManifests(const BuildPatchServices::FDiffManifestsConfiguration& Configuration) override;
	virtual FSimpleEvent& OnStartBuildInstall() override;
	virtual IBuildManifestPtr MakeManifestFromJSON(const FString& ManifestJSON) override;
	// IBuildPatchServicesModule interface end.

	/**
	 * Gets the directory used for staging intermediate files.
	 * @return	The staging directory
	 */
	static const FString& GetStagingDirectory();

	/**
	 * Gets the cloud directory where chunks and manifests will be pulled from.
	 * @param CloudIdx    Optional override for which cloud directory to get. This value will wrap within the range of available cloud directories.
	 * @return	The cloud directory
	 */
	static FString GetCloudDirectory(int32 CloudIdx = 0);

	/**
	 * Gets the cloud directories where chunks and manifests will be pulled from.
	 * @return	The cloud directories
	 */
	static TArray<FString> GetCloudDirectories();

	/**
	 * Gets the backup directory for saving files clobbered by repair/patch.
	 * @return	The backup directory
	 */
	static const FString& GetBackupDirectory();

private:
	/**
	 * Tick function to monitor installers for completion, so that we can call delegates on the main thread
	 * @param		Delta	Time since last tick
	 * @return	Whether to continue ticking
	 */
	bool Tick(float Delta);

	/**
	 * This will get called when core PreExits. Make sure any running installers are canceled out.
	 */
	void PreExit();

	/**
	 * Called during init to perform any fix up required to new configuration.
	 */
	void FixupLegacyConfig();

	/**
	 * Helper function to normalize the provided directory list.
	 */
	void NormalizeCloudPaths(TArray<FString>& InOutCloudPaths);

	// Get the encryption secret by the specified secret ID.
	TOptional<TArray<uint8>> FindEncryptionSecret(const FGuid& EncryptionSecretId) const;

	// Get all registered encryption secrets.
	TMap<FGuid, TArray<uint8>> GetEncryptionSecrets() const;

	// Decrypt manifest when the manifest's encryption secret is available
	void DecryptManifestIfNecessary(FBuildPatchAppManifest& Manifest) const;

private:
	// The analytics provider interface
	static TSharedPtr<IAnalyticsProvider> Analytics;

	// Holds the cloud directories where chunks should belong
	static TArray<FString> CloudDirectories;

	// Holds the staging directory where we can perform any temporary work
	static FString StagingDirectory;

	// Holds the backup directory where we can move files that will be clobbered by repair or patch
	static FString BackupDirectory;

	// Holds the filename for local machine config. This is instead of shipped or user config, to track machine installation config.
	FString LocalMachineConfigFile;

	// A flag specifying whether prerequisites install should be skipped
	bool bForceSkipPrereqs;

	// Array of running installers
	TArray<FBuildPatchInstallerRef> BuildPatchInstallers;

	// Array of running installers as exposable interface refs
	TArray<IBuildInstallerRef> BuildPatchInstallerInterfaces;

	// Holds available installations used for recycling install data
	TMultiMap<FString, FBuildPatchAppManifestRef> AvailableInstallations;

	// Critical section for multi-threaded access to AvailableEncryptionSecrets
	mutable FCriticalSection AvailableEncryptionSecretsCS;

	// Holds available encryption secrets used for decrypting installation data
	TMap<FGuid, TArray<uint8>> AvailableEncryptionSecrets;

	// Handle to the registered Tick delegate
	FTSTicker::FDelegateHandle TickDelegateHandle;

	// Delegate to give to installers so we know when they have been started.
	FBuildPatchInstallerDelegate InstallerStartDelegate;

	// Event broadcast upon a new build install
	FSimpleEvent OnStartBuildInstallEvent;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
