// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

/* Dependencies
*****************************************************************************/

/* Public includes
*****************************************************************************/

/* Private includes
*****************************************************************************/

DECLARE_LOG_CATEGORY_EXTERN(LogBuildPatchServices, Log, All);
