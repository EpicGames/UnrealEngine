// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchSettings.h"

#include "Misc/App.h"
#include "HAL/PlatformProcess.h"

namespace BuildPatchServices
{
	FBuildPatchServicesInitSettings::FBuildPatchServicesInitSettings()
		: ApplicationSettingsDir(FPlatformProcess::ApplicationSettingsDir())
		, ProjectName(FApp::GetProjectName())
		, LocalMachineConfigFileName(TEXT("BuildPatchServicesLocal.ini"))
	{
	}

	enum class EInstallActionIntent : int32
	{
		Install,
		Update,
		Repair,
		VerifyOnly,
		Uninstall,

		Invalid
	};

	FInstallerAction FInstallerAction::MakeInstall(const IBuildManifestRef& Manifest, TSet<FString> InstallTags /*= TSet<FString>()*/, FString InstallSubdirectory /*= FString()*/, FString CloudSubdirectory /*= FString()*/)
	{
		FInstallerAction InstallerAction;
		InstallerAction.InstallManifest = Manifest;
		InstallerAction.InstallTags = MoveTemp(InstallTags);
		InstallerAction.InstallSubdirectory = MoveTemp(InstallSubdirectory);
		InstallerAction.CloudSubdirectory = MoveTemp(CloudSubdirectory);
		InstallerAction.ActionIntent = EInstallActionIntent::Install;
		return InstallerAction;
	}

	FInstallerAction FInstallerAction::MakeUpdate(const IBuildManifestRef& CurrentManifest, const IBuildManifestRef& InstallManifest, TSet<FString> InstallTags /*= TSet<FString>()*/, FString InstallSubdirectory /*= FString()*/, FString CloudSubdirectory /*= FString()*/)
	{
		FInstallerAction InstallerAction;
		InstallerAction.CurrentManifest = CurrentManifest;
		InstallerAction.InstallManifest = InstallManifest;
		InstallerAction.InstallTags = MoveTemp(InstallTags);
		InstallerAction.InstallSubdirectory = MoveTemp(InstallSubdirectory);
		InstallerAction.CloudSubdirectory = MoveTemp(CloudSubdirectory);
		InstallerAction.ActionIntent = EInstallActionIntent::Update;
		return InstallerAction;
	}

	FInstallerAction FInstallerAction::MakeRepair(const IBuildManifestRef& Manifest, TSet<FString> InstallTags /*= TSet<FString>()*/, FString InstallSubdirectory /*= FString()*/, FString CloudSubdirectory /*= FString()*/)
	{
		FInstallerAction InstallerAction;
		InstallerAction.CurrentManifest = Manifest;
		InstallerAction.InstallManifest = Manifest;
		InstallerAction.InstallTags = MoveTemp(InstallTags);
		InstallerAction.InstallSubdirectory = MoveTemp(InstallSubdirectory);
		InstallerAction.CloudSubdirectory = MoveTemp(CloudSubdirectory);
		InstallerAction.ActionIntent = EInstallActionIntent::Repair;
		return InstallerAction;
	}

	FInstallerAction FInstallerAction::MakeVerifyOnly(const IBuildManifestRef& Manifest, TSet<FString> InstallTags /*= TSet<FString>()*/, FString InstallSubdirectory /*= FString()*/, FString CloudSubdirectory /*= FString()*/)
	{
		FInstallerAction InstallerAction;
		InstallerAction.CurrentManifest = Manifest;
		InstallerAction.InstallManifest = Manifest;
		InstallerAction.InstallTags = MoveTemp(InstallTags);
		InstallerAction.InstallSubdirectory = MoveTemp(InstallSubdirectory);
		InstallerAction.CloudSubdirectory = MoveTemp(CloudSubdirectory);
		InstallerAction.ActionIntent = EInstallActionIntent::VerifyOnly;
		return InstallerAction;
	}

	FInstallerAction FInstallerAction::MakeUninstall(const IBuildManifestRef& Manifest, FString InstallSubdirectory /*= FString()*/, FString CloudSubdirectory /*= FString()*/)
	{
		FInstallerAction InstallerAction;
		InstallerAction.CurrentManifest = Manifest;
		InstallerAction.InstallSubdirectory = MoveTemp(InstallSubdirectory);
		InstallerAction.CloudSubdirectory = MoveTemp(CloudSubdirectory);
		InstallerAction.ActionIntent = EInstallActionIntent::Uninstall;
		return InstallerAction;
	}

	FInstallerAction::FInstallerAction()
		: ActionIntent(EInstallActionIntent::Invalid)
	{
	}

	bool FInstallerAction::IsInstall() const
	{
		return ActionIntent == EInstallActionIntent::Install;
	}

	bool FInstallerAction::IsUpdate() const
	{
		return ActionIntent == EInstallActionIntent::Update;
	}

	bool FInstallerAction::IsRepair() const
	{
		return ActionIntent == EInstallActionIntent::Repair;
	}

	bool FInstallerAction::IsVerifyOnly() const
	{
		return ActionIntent == EInstallActionIntent::VerifyOnly;
	}

	bool FInstallerAction::IsUninstall() const
	{
		return ActionIntent == EInstallActionIntent::Uninstall;
	}

	const TSet<FString>& FInstallerAction::GetInstallTags() const
	{
		return InstallTags;
	}

	const FString& FInstallerAction::GetInstallSubdirectory() const
	{
		return InstallSubdirectory;
	}

	const FString& FInstallerAction::GetCloudSubdirectory() const
	{
		return CloudSubdirectory;
	}

	IBuildManifestRef FInstallerAction::GetInstallManifest() const
	{
		return InstallManifest.ToSharedRef();
	}

	IBuildManifestRef FInstallerAction::GetCurrentManifest() const
	{
		return CurrentManifest.ToSharedRef();
	}

	FBuildInstallerConfiguration::FBuildInstallerConfiguration(TArray<FInstallerAction> InInstallerActions)
		: InstallerActions(MoveTemp(InInstallerActions))
	{
	}

	FChunkDeltaOptimiserConfiguration::FChunkDeltaOptimiserConfiguration()
		: ScanWindowSize(8191)
		, OutputChunkSize(1024 * 1024)
		, bResaveExistingChunks(false)
	{
	}

	FPatchDataEnumerationConfiguration::FPatchDataEnumerationConfiguration()
		: bIncludeSizes(false)
	{
	}

	FCompactifyConfiguration::FCompactifyConfiguration()
		: DataAgeThreshold(7.0f)
		, bRunPreview(true)
	{
	}

	FPackageChunksConfiguration::FPackageChunksConfiguration()
		: FeatureLevel(EFeatureLevel::Latest)
		, MaxOutputFileSize(TNumericLimits<uint64>::Max())
	{
	}

	FInstallerConfiguration::FInstallerConfiguration(const IBuildManifestRef& InInstallManifest)
		: InstallManifest(InInstallManifest)
	{
	}

	FChunkHarvesterConfiguration::FChunkHarvesterConfiguration()
		: FeatureLevelOverride(EFeatureLevel::Invalid)
		, bResaveExistingChunks(false)
	{
	}
}

