// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchState.h"

#define LOCTEXT_NAMESPACE "BuildPatchInstallProgress"

// Deprecated in header
const FText& BuildPatchServices::StateToText(const BuildPatchServices::EBuildPatchState& State)
{
	// Static const fixed FText values so that they are not constantly constructed
	static const FText Queued = LOCTEXT("EBuildPatchProgress_Queued", "Queued");
	static const FText Initializing = LOCTEXT("EBuildPatchProgress_Initialising", "Initializing");
	static const FText Resuming = LOCTEXT("EBuildPatchProgress_Resuming", "Resuming");
	static const FText Downloading = LOCTEXT("EBuildPatchProgress_Downloading", "Downloading");
	static const FText Installing = LOCTEXT("EBuildPatchProgress_Installing", "Installing");
	static const FText BuildVerification = LOCTEXT("EBuildPatchProgress_BuildVerification", "Verifying");
	static const FText CleanUp = LOCTEXT("EBuildPatchProgress_CleanUp", "Cleaning up");
	static const FText PrerequisitesInstall = LOCTEXT("EBuildPatchProgress_PrerequisitesInstall", "Prerequisites");
	static const FText Completed = LOCTEXT("EBuildPatchProgress_Complete", "Complete");
	static const FText Paused = LOCTEXT("EBuildPatchProgress_Paused", "Paused");
	static const FText Empty = FText::GetEmpty();

	switch (State)
	{
	case BuildPatchServices::EBuildPatchState::Queued:
		return Queued;
	case BuildPatchServices::EBuildPatchState::Initializing:
		return Initializing;
	case BuildPatchServices::EBuildPatchState::Resuming:
		return Resuming;
	case BuildPatchServices::EBuildPatchState::Downloading:
		return Downloading;
	case BuildPatchServices::EBuildPatchState::Installing:
	case BuildPatchServices::EBuildPatchState::RunningCustomUninstallAction:
	case BuildPatchServices::EBuildPatchState::MovingToInstall:
	case BuildPatchServices::EBuildPatchState::SettingAttributes:
		return Installing;
	case BuildPatchServices::EBuildPatchState::BuildVerification:
		return BuildVerification;
	case BuildPatchServices::EBuildPatchState::CleanUp:
		return CleanUp;
	case BuildPatchServices::EBuildPatchState::PrerequisitesInstall:
		return PrerequisitesInstall;
	case BuildPatchServices::EBuildPatchState::Completed:
		return Completed;
	case BuildPatchServices::EBuildPatchState::Paused:
		return Paused;
	default:
		return Empty;
	}
}

#undef LOCTEXT_NAMESPACE
