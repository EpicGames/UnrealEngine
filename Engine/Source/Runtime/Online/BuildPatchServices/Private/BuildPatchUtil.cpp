// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchUtil.h"

#include "Misc/Base64.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceRedirector.h"

#include "Data/ChunkData.h"
#include "Data/ManifestData.h"
#include "Common/FileSystem.h"
#include "BuildPatchHash.h"
#include "BuildPatchServicesModule.h"

using namespace BuildPatchServices;

namespace BuildPatchUtils
{
	void B64UriSafe(FString& B64)
	{
		// Make URI safe.
		B64.ReplaceInline(TEXT("+"), TEXT("-"), ESearchCase::CaseSensitive);
		B64.ReplaceInline(TEXT("/"), TEXT("_"), ESearchCase::CaseSensitive);
		// Trim = characters.
		B64.ReplaceInline(TEXT("="), TEXT(""), ESearchCase::CaseSensitive);
	}

	FString ToMiniUriString(const FGuid& Val)
	{
		FString Mini = FBase64::Encode((uint8*)&Val, sizeof(Val));
		B64UriSafe(Mini);
		return Mini;
	}

	bool FromMiniUriString(const FString& String, FGuid& Val)
	{
		if (String.Len() == 22)
		{
			FString Mini = String.Replace(TEXT("-"), TEXT("+"), ESearchCase::CaseSensitive)
			                     .Replace(TEXT("_"), TEXT("/"), ESearchCase::CaseSensitive);
			return FBase64::Decode(Mini.GetCharArray().GetData(), 22, (uint8*)&Val);
		}
		return false;
	}

	FString ToMiniUriString(const uint64& Val)
	{
		FString Mini = FBase64::Encode((uint8*)&Val, sizeof(Val));
		B64UriSafe(Mini);
		return Mini;
	}

	int32 DataIdToGroupId(const FGuid& DataId)
	{
		return FCrc::MemCrc32(&DataId, sizeof(FGuid)) % 100;
	}

	FString GetChunkNewFilename(BuildPatchServices::EFeatureLevel FeatureLevel, const FGuid& ChunkId, const uint64& RollingHash, const FGuid& EncryptionSecretId)
	{
		if (FeatureLevel < EFeatureLevel::DataFileRenames)
		{
			return FBuildPatchUtils::GetChunkOldFilename(ChunkId);
		}
		else if (FeatureLevel < EFeatureLevel::ChunksStoredBySecret)
		{
			return FString::Printf(TEXT("%s/%02d/%016llX_%s.chunk"), ManifestVersionHelpers::GetChunkSubdir(FeatureLevel), DataIdToGroupId(ChunkId), RollingHash, *ChunkId.ToString());
		}
		else
		{
			const FString SecretPathComponent = EncryptionSecretId.IsValid() ? ToMiniUriString(EncryptionSecretId) : TEXT("plain");
			return FString::Printf(TEXT("%s/%s/%02d/%s_%s.chunk"), ManifestVersionHelpers::GetChunkSubdir(FeatureLevel), *SecretPathComponent, DataIdToGroupId(ChunkId), *ToMiniUriString(RollingHash), *ToMiniUriString(ChunkId));
		}
	}
}

/* FBuildPatchUtils implementation
*****************************************************************************/
FString FBuildPatchUtils::GetChunkNewFilename(BuildPatchServices::EFeatureLevel FeatureLevel, const BuildPatchServices::FChunkInfo& Chunk)
{
	return BuildPatchUtils::GetChunkNewFilename(FeatureLevel, Chunk.Guid, Chunk.Hash, Chunk.EncryptionSecretId);
}

FString FBuildPatchUtils::GetChunkNewFilename(BuildPatchServices::EFeatureLevel FeatureLevel, const BuildPatchServices::FChunkHeader& Chunk)
{
	return BuildPatchUtils::GetChunkNewFilename(FeatureLevel, Chunk.Guid, Chunk.RollingHash, Chunk.EncryptionSecretId);
}

FString FBuildPatchUtils::GetFileNewFilename(const EFeatureLevel FeatureLevel, const FGuid& FileGUID, const FSHAHash& FileHash)
{
	check(FileGUID.IsValid());
	return FString::Printf(TEXT("%s/%02d/%s_%s.file"), ManifestVersionHelpers::GetFileSubdir(FeatureLevel), BuildPatchUtils::DataIdToGroupId(FileGUID), *FileHash.ToString(), *FileGUID.ToString());
}

FString FBuildPatchUtils::GetFileNewFilename(const EFeatureLevel FeatureLevel, const FGuid& FileGUID, const uint64& FileHash)
{
	check(FileGUID.IsValid());
	return FString::Printf(TEXT("%s/%02d/%016llX_%s.file"), ManifestVersionHelpers::GetFileSubdir(FeatureLevel), BuildPatchUtils::DataIdToGroupId(FileGUID), FileHash, *FileGUID.ToString());
}

FString FBuildPatchUtils::GetChunkOldFilename(const FGuid& ChunkGUID)
{
	check(ChunkGUID.IsValid());
	return FString::Printf(TEXT("Chunks/%02d/%s.chunk"), FCrc::MemCrc_DEPRECATED(&ChunkGUID, sizeof(FGuid)) % 100, *ChunkGUID.ToString());
}

FString FBuildPatchUtils::GetFileOldFilename(const FGuid& FileGUID)
{
	check(FileGUID.IsValid());
	return FString::Printf(TEXT("Files/%02d/%s.file"), FCrc::MemCrc_DEPRECATED(&FileGUID, sizeof(FGuid)) % 100, *FileGUID.ToString());
}

FString FBuildPatchUtils::GetDataTypeOldFilename(EBuildPatchDataType DataType, const FGuid& Guid)
{
	check(Guid.IsValid());

	switch (DataType)
	{
	case EBuildPatchDataType::ChunkData:
		return GetChunkOldFilename(Guid);
	case EBuildPatchDataType::FileData:
		return GetFileOldFilename(Guid);
	}

	// Error, didn't case type
	check(false);
	return TEXT("");
}

FString FBuildPatchUtils::GetDataFilename(const FBuildPatchAppManifestRef& Manifest, const FGuid& DataGUID)
{
	return GetDataFilename(Manifest.Get(), DataGUID);
}

FString FBuildPatchUtils::GetDataFilename(const FBuildPatchAppManifest&    Manifest, const FGuid& DataGUID)
{
	const EBuildPatchDataType DataType = Manifest.IsFileDataManifest() ? EBuildPatchDataType::FileData : EBuildPatchDataType::ChunkData;
	if (Manifest.GetFeatureLevel() < EFeatureLevel::DataFileRenames)
	{
		return FBuildPatchUtils::GetDataTypeOldFilename(DataType, DataGUID);
	}
	else if (DataType == EBuildPatchDataType::ChunkData)
	{
		const FChunkInfo* ChunkInfo = Manifest.GetChunkInfo(DataGUID);
		// Should be impossible to not exist
		check(ChunkInfo != nullptr);
		return FBuildPatchUtils::GetChunkNewFilename(Manifest.GetFeatureLevel(), *ChunkInfo);
	}
	else if (Manifest.GetFeatureLevel() <= EFeatureLevel::StoredAsCompressedUClass)
	{
		FSHAHash FileHash;
		const bool bFound = Manifest.GetFileHash(DataGUID, FileHash);
		// Should be impossible to not exist
		check(bFound);
		return FBuildPatchUtils::GetFileNewFilename(Manifest.GetFeatureLevel(), DataGUID, FileHash);
	}
	else
	{
		uint64 FileHash;
		const bool bFound = Manifest.GetFilePartHash(DataGUID, FileHash);
		// Should be impossible to not exist
		check(bFound);
		return FBuildPatchUtils::GetFileNewFilename(Manifest.GetFeatureLevel(), DataGUID, FileHash);
	}
}

bool FBuildPatchUtils::GetGUIDFromFilename(const FString& DataFilename, FGuid& DataGUID)
{
	const FString DataBaseFilename = FPaths::GetBaseFilename(DataFilename);
	FString GuidString;
	if (DataBaseFilename.Len() == 49)
	{
		GuidString = DataBaseFilename.Right(32);
		return FGuid::Parse(GuidString, DataGUID);
	}
	else if (DataBaseFilename.Len() == 34)
	{
		GuidString = DataBaseFilename.Right(22);
		return BuildPatchUtils::FromMiniUriString(GuidString, DataGUID);
	}
	return false;
}

FString FBuildPatchUtils::GenerateNewBuildId()
{
	FGuid NewGuid = FGuid::NewGuid();
	// Minimise string length using base 64 string encode.
	FString BuildId = FBase64::Encode((const uint8*)&NewGuid, sizeof(FGuid));
	// Make URI safe.
	BuildId.ReplaceInline(TEXT("+"), TEXT("-"), ESearchCase::CaseSensitive);
	BuildId.ReplaceInline(TEXT("/"), TEXT("_"), ESearchCase::CaseSensitive);
	// Trim = characters.
	BuildId.ReplaceInline(TEXT("="), TEXT(""), ESearchCase::CaseSensitive);
	return BuildId;
}

FString FBuildPatchUtils::GetBackwardsCompatibleBuildId(const FManifestMeta& ManifestMeta)
{
	// Use an SHA to generate a fixed length unique identifier referring to some of the meta values.
	FSHA1 Sha;
	FSHAHash Hash;
	Sha.Update((const uint8*)&ManifestMeta.AppID, sizeof(ManifestMeta.AppID));
	// For platform agnostic result, we must use UTF8. TCHAR can be 16b, or 32b etc.
	FTCHARToUTF8 UTF8AppName(*ManifestMeta.AppName);
	FTCHARToUTF8 UTF8BuildVersion(*ManifestMeta.BuildVersion);
	FTCHARToUTF8 UTF8LaunchExe(*ManifestMeta.LaunchExe);
	FTCHARToUTF8 UTF8LaunchCommand(*ManifestMeta.LaunchCommand);
	Sha.Update((const uint8*)UTF8AppName.Get(), sizeof(ANSICHAR) * UTF8AppName.Length());
	Sha.Update((const uint8*)UTF8BuildVersion.Get(), sizeof(ANSICHAR) * UTF8BuildVersion.Length());
	Sha.Update((const uint8*)UTF8LaunchExe.Get(), sizeof(ANSICHAR) * UTF8LaunchExe.Length());
	Sha.Update((const uint8*)UTF8LaunchCommand.Get(), sizeof(ANSICHAR) * UTF8LaunchCommand.Length());
	Sha.Final();
	Sha.GetHash(Hash.Hash);

	// Minimise string length using base 64 string encode.
	FString BuildId = FBase64::Encode(Hash.Hash, FSHA1::DigestSize);
	// Make URI safe.
	BuildId.ReplaceInline(TEXT("+"), TEXT("-"), ESearchCase::CaseSensitive);
	BuildId.ReplaceInline(TEXT("/"), TEXT("_"), ESearchCase::CaseSensitive);
	// Trim = characters.
	BuildId.ReplaceInline(TEXT("="), TEXT(""), ESearchCase::CaseSensitive);
	return BuildId;
}

FString FBuildPatchUtils::GetChunkDeltaDirectory(const FBuildPatchAppManifest& DestinationManifest)
{
	return TEXT("Deltas") / DestinationManifest.GetBuildId();
}

FString FBuildPatchUtils::GetChunkDeltaFilename(const FBuildPatchAppManifest& SourceManifest, const FBuildPatchAppManifest& DestinationManifest, const FString& FilenameTrailer)
{
	if (FilenameTrailer.Len())
	{
		return GetChunkDeltaDirectory(DestinationManifest) / SourceManifest.GetBuildId() + TEXT("-") + FilenameTrailer + TEXT(".delta");
	}
	return GetChunkDeltaDirectory(DestinationManifest) / SourceManifest.GetBuildId() + TEXT(".delta");
}

uint8 FBuildPatchUtils::VerifyFile(IFileSystem* FileSystem, const FString& FileToVerify, const FSHAHash& Hash1, const FSHAHash& Hash2)
{
	FBuildPatchFloatDelegate NoProgressDelegate;
	FBuildPatchBoolRetDelegate NoPauseDelegate;
	FBuildPatchBoolRetDelegate NoAbortDelegate;
	return VerifyFile(FileSystem, FileToVerify, Hash1, Hash2, NoProgressDelegate, NoPauseDelegate, NoAbortDelegate);
}

uint8 FBuildPatchUtils::VerifyFile(IFileSystem* FileSystem, const FString& FileToVerify, const FSHAHash& Hash1, const FSHAHash& Hash2, FBuildPatchFloatDelegate ProgressDelegate, FBuildPatchBoolRetDelegate ShouldPauseDelegate, FBuildPatchBoolRetDelegate ShouldAbortDelegate)
{
	uint8 ReturnValue = 0;
	TUniquePtr<FArchive> FileReader = FileSystem->CreateFileReader(*FileToVerify);
	ProgressDelegate.ExecuteIfBound(0.0f);
	if (FileReader.IsValid())
	{
		FSHA1 HashState;
		FSHAHash HashValue;
		const int64 FileSize = FileReader->TotalSize();
		uint8* FileReadBuffer = new uint8[FileBufferSize];
		while (!FileReader->AtEnd() && (!ShouldAbortDelegate.IsBound() || !ShouldAbortDelegate.Execute()))
		{
			// Pause if necessary
			while ((ShouldPauseDelegate.IsBound() && ShouldPauseDelegate.Execute())
			   && (!ShouldAbortDelegate.IsBound() || !ShouldAbortDelegate.Execute()))
			{
				FPlatformProcess::Sleep(0.1f);
			}
			// Read file and update hash state
			const int64 SizeLeft = FileSize - FileReader->Tell();
			const uint32 ReadLen = FMath::Min< int64 >(FileBufferSize, SizeLeft);
			FileReader->Serialize(FileReadBuffer, ReadLen);
			HashState.Update(FileReadBuffer, ReadLen);
			const double FileSizeTemp = FileSize;
			const float Progress = 1.0f - ((SizeLeft - ReadLen) / FileSizeTemp);
			ProgressDelegate.ExecuteIfBound(Progress);
		}
		delete[] FileReadBuffer;
		HashState.Final();
		HashState.GetHash(HashValue.Hash);
		ReturnValue = (HashValue == Hash1) ? 1 : (HashValue == Hash2) ? 2 : 0;
		if (ReturnValue == 0)
		{
			GLog->Logf(TEXT("BuildDataGenerator: Verify failed on %s"), *FileToVerify);
		}
		FileReader->Close();
	}
	else
	{
		GLog->Logf(TEXT("BuildDataGenerator: ERROR VerifyFile cannot open %s"), *FileToVerify);
	}
	ProgressDelegate.ExecuteIfBound(1.0f);
	return ReturnValue;
}

void FBuildPatchUtils::SHAToBase32(const FSHAHash& SHA, FString& OutString)
{
	// For alphabet we'll use the "Extended hex" set.
	static const TCHAR* Alphabet = TEXT("0123456789ABCDEFGHIJKLMNOPQRSTUV");
	OutString.Empty(32);
	const uint8* Bytes = SHA.Hash;
	for (int32 Idx = 0; (Idx + 4) < 20; Idx += 5)
	{
		const uint64 Data =
			((uint64)Bytes[Idx]) << 32 |
			((uint64)Bytes[Idx + 1]) << 24 |
			((uint64)Bytes[Idx + 2]) << 16 |
			((uint64)Bytes[Idx + 3]) << 8 |
			((uint64)Bytes[Idx + 4]);

		// 5 bytes produce 8 chars
		// We shift 35, 30, 25 and so on to isolate each 5bit group starting from highest.
		for (int32 Shift = 35; Shift >= 0; Shift -= 5)
		{
			uint64 ThisRange = Data >> Shift;
			OutString += Alphabet[ThisRange & 0x1F];
		}
	}
}

FString FBuildPatchUtils::ResolveInstallationFileName(const FString& BaseInstallationDirectory, const TMap<FString, FString>& PerFileSubdirectories, const FString& BuildFilename)
{
	const FString* PerFileSubdir = PerFileSubdirectories.Find(BuildFilename);
	if (PerFileSubdir)
	{		
		// Build a relative path from install directory to subdirectory, then resolve it back to absolute path
		FString CombinedPath = FPaths::Combine(BaseInstallationDirectory, *PerFileSubdir, BuildFilename);
		FPaths::NormalizeFilename(CombinedPath);
		FPaths::CollapseRelativeDirectories(CombinedPath, true);
		return CombinedPath;
	}
	return BaseInstallationDirectory / BuildFilename;
}

uint64 FBuildPatchUtils::CalculateDiskSpaceRequirementsWithDeleteDuringInstall(
	const TArray<FString>& InFilesToConstruct, 
	int32 InCompletedFileCount, 
	int64 InProgressFileSize,
	IBuildManifestSet* InManifestSet,
	const TArray<uint64>& InChunkDbSizesAtPosition,
	const FString& InstallDirectory,
	EInstallMode InstallMode)
{
	uint64 TotalDeletedSize = 0;
	uint64 TotalWrittenSize = 0;

	int64 MaxDiskSize = 0;
	int32 MaxDiskSizeFileIndex = 0;

	TSet<FString> MissedCurrentFiles; // Files that were not installed, possibly due to filtering by tags.
	if (InstallMode == EInstallMode::DestructiveInstall)
	{
		// This data is used only when we overwrite existing files on the fly.
		InManifestSet->GetMissedFiles(InstallDirectory, MissedCurrentFiles);
	}

	for (int32 FileIndex = InCompletedFileCount; FileIndex < InFilesToConstruct.Num(); FileIndex++)
	{
		const FString& Filename = InFilesToConstruct[FileIndex];
		// We've completed this file
		if (const FFileManifest* IncomingFileManifest = InManifestSet->GetNewFileManifest(Filename))
		{
			TotalWrittenSize += IncomingFileManifest->FileSize;
			if (FileIndex == InCompletedFileCount && InProgressFileSize > 0)
			{
				// If the first file in the list is only partially completed, adjust TotalWrittenSize accordingly.
				TotalWrittenSize -= FMath::Min<uint64>(InProgressFileSize, IncomingFileManifest->FileSize);
			}
		}
		// We delete the chunkdbs _after_ we write the output so we can't use this size until 
		// the next file gets done. Be sure to handle the case where we've deleted so much data
		// we are below the waterline.
		const uint64 TotalChunkDbSizeAtLastFile = InChunkDbSizesAtPosition[FileIndex];
		const int64 CurrentDiskSize = static_cast<int64>(TotalChunkDbSizeAtLastFile + TotalWrittenSize) - static_cast<int64>(TotalDeletedSize);
		if (CurrentDiskSize > MaxDiskSize)
		{
			MaxDiskSize = CurrentDiskSize;
			MaxDiskSizeFileIndex = FileIndex;
		}

		//UE_LOGF(LogTemp, Display, "...@ file %d chunks = %llu, install = %llu", FileIndex, TotalChunkDbSizeAtLastFile, TotalWrittenSize);

		// We should take old files into account only if we will overwrite them on the fly.
		if (InstallMode == EInstallMode::DestructiveInstall)
		{
			// If we are patching, we can now delete the output file, which decreases our disk presence, however
			// we update this after the check because we can't delete until after we have the file fully constructed.
			const FFileManifest* OnDiskFileManifest = InManifestSet->GetCurrentFileManifest(Filename);
			// If old manifest contains file and file was not missed from disk due to filtering by tags
			if (OnDiskFileManifest && !MissedCurrentFiles.Contains(Filename))
			{
				TotalDeletedSize += OnDiskFileManifest->FileSize;
			}
		}
	}

	//UE_LOGF(LogTemp, Display, "Max disk use %llu (+%llu after d/l) (install size: %llu: +%llu) after file %d", MaxDiskSize, PostDlSize, TotalWrittenSize, MaxDiskSize - TotalWrittenSize, MaxDiskSizeFileIndex);

	return static_cast<uint64>(MaxDiskSize);
}