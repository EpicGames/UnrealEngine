// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BuildPatchUtil.h: Declares miscellaneous utility functions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "BuildPatchManifest.h"
#include "BuildPatchInstall.h"

namespace BuildPatchServices
{
	struct FChunkHeader;
	class IFileSystem;
	struct FManifestMeta;
	class IBuildManifestSet;
}

enum class EBuildPatchDataType
{
	// Represents data produced by the chunked patch generation mode.
	ChunkData   = 0,
	// Represents data produced by the nochunks patch generation mode, which has deprecated.
	FileData    = 1,
};


// A delegate taking a float. Used to receive progress.
DECLARE_DELEGATE_OneParam(FBuildPatchFloatDelegate, float);

// A delegate returning a bool. Used to pass a paused state.
DECLARE_DELEGATE_RetVal(bool, FBuildPatchBoolRetDelegate);

/**
 * Some constants
 */
namespace BuildPatchConstants
{
	// Result of converting SHA1 to base32 string value length
	static const int32 SHAToBase32StringLength = 32;
}

/**
 * Some utility functions
 */
struct FBuildPatchUtils
{
	/**
	 * Gets the relative cloud filename for a chunk in the newest format.
	 * @param FeatureLevel   The manifest version that references this chunk.
	 * @param Chunk          The chunk we need a path for
	 * @return the chunk path.
	 */
	static FString GetChunkNewFilename(BuildPatchServices::EFeatureLevel FeatureLevel, const BuildPatchServices::FChunkInfo& Chunk);
	static FString GetChunkNewFilename(BuildPatchServices::EFeatureLevel FeatureLevel, const BuildPatchServices::FChunkHeader& Chunk);

	/**
	 * Gets the the relative cloud filename for a file chunk generated from it's GUID and Hash, which is the new format.
	 * @param FeatureLevel   The manifest version that references this file.
	 * @param FileGUID       The file chunk Guid.
	 * @param FileHash       The file hash value.
	 * @return the file chunk path.
	 */
	static FString GetFileNewFilename(BuildPatchServices::EFeatureLevel FeatureLevel, const FGuid& FileGUID, const FSHAHash& FileHash);
	static FString GetFileNewFilename(BuildPatchServices::EFeatureLevel FeatureLevel, const FGuid& FileGUID, const uint64& FilePartHash);

	/**
	 * Gets the the relative cloud filename for a chunk generated from it's GUID
	 * @param ChunkGuid			The chunk Guid
	 * @return	the chunk path
	 */
	static FString GetChunkOldFilename(const FGuid& ChunkGUID);

	/**
	 * Gets the the relative cloud filename for a file data part generated from it's GUID
	 * @param FileGUID			The file part Guid
	 * @return	The file data path
	 */
	static FString GetFileOldFilename(const FGuid& FileGUID);

	/**
	 * Gets the filename for a specific data part type from it's GUID
	 * @param DataType			The type of data
	 * @param DataGUID			The data Guid
	 * @return	the data part path
	 */
	static FString GetDataTypeOldFilename(EBuildPatchDataType DataType, const FGuid& Guid);

	/**
	 * Gets the relative cloud filename for any data part. Wraps the choice between all of the above
	 * @param Manifest			The manifest referencing this data
	 * @param DataGUID			The data Guid
	 * @return	the data part path
	 */
	static FString GetDataFilename(const FBuildPatchAppManifestRef& Manifest, const FGuid& DataGUID);
	static FString GetDataFilename(const FBuildPatchAppManifest&    Manifest, const FGuid& DataGUID);

	/**
	 * Gets the GUID for a data file according to it's filename (new or old)
	 * @param DataFilename		IN		The data filename, or URL
	 * @param DataGUID			OUT		Receives the GUID of the data
	 * @return  True if successful, false otherwise
	 */
	static bool GetGUIDFromFilename(const FString& DataFilename, FGuid& DataGUID);

	/**
	 * Generates a new BuildId for a manifest. This should be used only when creating new builds, and thus saving out brand new manifests rather than copies of manifests.
	 * @return the generated id.
	 */
	static FString GenerateNewBuildId();

	/**
	 * Creates a deterministic BuildId for use with a manifest that is at EFeatureLevel::UsesRuntimeGeneratedBuildId or older.
	 * The id is created based on the meta data, which itself should be unique per build created.
	 * @param ManifestMeta     The meta for the old manifest.
	 * @return the id for this manifest.
	 */
	static FString GetBackwardsCompatibleBuildId(const BuildPatchServices::FManifestMeta& ManifestMeta);

	/**
	 * Based on the destination manifest, get the directory that will contains the deltas for getting to it from other builds.
	 * @param DestinationManifest   The destination manifest.
	 * @return the CloudDir relative delta directory.
	 */
	static FString GetChunkDeltaDirectory(const FBuildPatchAppManifest& DestinationManifest);

	/**
	 * Based on the source and destination manifests, get the filename for the delta that optimises patching from source to destination.
	 * @param SourceManifest        The source manifest.
	 * @param DestinationManifest   The destination manifest.
	 * @param FilenameTrailer       This is an optional parameter to allow for multiple optimized deltas for the same build pairing. If unneeded, leave as "".
	 * @return the CloudDir relative delta filename.
	 */
	static FString GetChunkDeltaFilename(const FBuildPatchAppManifest& SourceManifest, const FBuildPatchAppManifest& DestinationManifest, const FString& FilenameTrailer);

	/**
	 * Checks a file against SHA1 hashes. The function takes two so that it can return no match, match with Hash1, or match with Hash2, that way we can check the file for being the same as an old manifest or new manifest
	 * NOTE: This function is blocking and will not return until finished. Don't run on main thread.
	 * @param FileSystem			IN		Interface to disk access.
	 * @param FileToVerify			IN		The file to analyze.
	 * @param Hash1					IN		A Hash to match against the file
	 * @param Hash2					IN		A second Hash to match against the file
	 * @param ProgressDelegate		IN		Delegate to receive progress updates in the form of a float range 0.0f to 1.0f
	 * @param ShouldPauseDelegate	IN		Delegate that returns a bool, which if true will pause the process
	 * @param ShouldAbortDelegate	IN		Delegate that returns a bool, which if true will abort the process
	 * @return		0 if no match, 1 for match with Hash1, and 2 for match with Hash2
	 */
	static uint8 VerifyFile(BuildPatchServices::IFileSystem* FileSystem, const FString& FileToVerify, const FSHAHash& Hash1, const FSHAHash& Hash2, FBuildPatchFloatDelegate ProgressDelegate, FBuildPatchBoolRetDelegate ShouldPauseDelegate, FBuildPatchBoolRetDelegate ShouldAbortDelegate);

	/**
	 * Checks a file against SHA1 hashes. The function takes two so that it can return no match, match with Hash1, or match with Hash2, that way we can check the file for being the same as an old manifest or new manifest
	 * NOTE: This function is blocking and will not return until finished. Don't run on main thread. This allows the above function to be easily called without delegates
	 * @param FileSystem			IN		Interface to disk access.
	 * @param FileToVerify			IN		The file to analyze.
	 * @param Hash1					IN		A Hash to match against the file
	 * @param Hash2					IN		A second Hash to match against the file
	 * @return		0 if no match, 1 for match with Hash1, and 2 for match with Hash2
	 */
	static uint8 VerifyFile(BuildPatchServices::IFileSystem* FileSystem, const FString& FileToVerify, const FSHAHash& Hash1, const FSHAHash& Hash2);
	
	/**
	 * Encodes SHA hash to base32 and convert its to out string
	 * @param SHA					IN		SHA hash to encode
	 * @param OutString				OUT		String with base32 encoded value
	 */
	static void SHAToBase32(const FSHAHash& SHA, FString& OutString);

	static uint64 CalculateDiskSpaceRequirementsWithDeleteDuringInstall(
		const TArray<FString>& InFilesToConstruct, 
		int32 InCompletedFileCount, 
		int64 InProgressFileSize,
		BuildPatchServices::IBuildManifestSet* InManifestSet, 
		const TArray<uint64>& InChunkDbSizesAtPosition, 
		const FString& InstallDirectory,
		BuildPatchServices::EInstallMode mode);

    /**
	* Generates the path to the file we're installing by adjusting the path for per file subdirectories. Currently this does not handle
	* installation subdirectories which are appended as part of the manifest set.
	*/
	static FString ResolveInstallationFileName(const FString& BaseInstallationDirectory, const TMap<FString, FString>& PerFileSubdirectories, const FString& BuildFilename);
};
