// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildPatchVerifyChunkData.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Common/Crypto.h"
#include "Common/FileSystem.h"
#include "Common/StatsCollector.h"
#include "Installer/Statistics/CloudChunkSourceStat.h"
#include "Data/ChunkData.h"
#include "Async/Async.h"
#include "Generation/ChunkDatabaseWriter.h"
#include "BuildPatchManifest.h"
#include "BuildPatchUtil.h"

DECLARE_LOG_CATEGORY_EXTERN(LogVerifyChunkData, Log, All);
DEFINE_LOG_CATEGORY(LogVerifyChunkData);

namespace VerifyHelpers
{
	FBuildPatchAppManifestPtr LoadManifestFile(const FString& ManifestFilePath)
	{
		static FCriticalSection UObjectAllocationCS;
		UObjectAllocationCS.Lock();
		FBuildPatchAppManifestPtr Manifest = MakeShareable(new FBuildPatchAppManifest());
		UObjectAllocationCS.Unlock();
		if (Manifest->LoadFromFile(ManifestFilePath))
		{
			return Manifest;
		}
		return FBuildPatchAppManifestPtr();
	}
}

bool FBuildVerifyChunkData::VerifyChunkData(const FString& SearchPath, const FString& OutputFile)
{
	using namespace BuildPatchServices;
	bool bSuccess = true;

	FString OutputText;

	UE_LOGF(LogVerifyChunkData, Display, "Searching for files..");
	TArray<FString> AllFiles;
	IFileManager::Get().FindFilesRecursive(AllFiles, *(SearchPath / TEXT("")), TEXT("*"), true, false);

	TSet<FGuid> BadChunkData;
	TSet<FGuid> GoodChunkData;
	TSet<FGuid> SkippedChunkData;
	TArray<FString> ChunkFiles;
	TArray<FString> ChunkDbFiles;
	TArray<FString> ManifestFiles;
	for (const FString& File : AllFiles)
	{
		if (File.EndsWith(TEXT(".chunk")))
		{
			ChunkFiles.Add(File);
		}
		else if (File.EndsWith(TEXT(".chunkdb")))
		{
			ChunkDbFiles.Add(File);
		}
		else if (File.EndsWith(TEXT(".manifest")))
		{
			ManifestFiles.Add(File);
		}
		// Register delta files as if they are manifests.
		else if (File.EndsWith(TEXT(".delta")))
		{
			ManifestFiles.Add(File);
		}
	}

	// Systems.
	TUniquePtr<IFileSystem> FileSystem(FFileSystemFactory::Create());
	TUniquePtr<ICrypto> Crypto(FCryptoFactory::Create());
	TUniquePtr<FStatsCollector> StatsCollector(FStatsCollectorFactory::Create());
	TUniquePtr<ICloudChunkSourceStat> CloudChunkSourceStat(FCloudChunkSourceStatFactory::Create(StatsCollector.Get()));
	FChunkDataSerializationConfig ChunkDataSerializationConfig;
	// UE5 MERGE TODO: ChunkDataSerializationConfig.CloudChunkSourceStat = CloudChunkSourceStat.Get();
	TUniquePtr<IChunkDataSerialization> ChunkDataSerialization(FChunkDataSerializationFactory::Create(FileSystem.Get(), Crypto.Get(), ChunkDataSerializationConfig));

	// Kick off a verify chunk files.
	typedef TTuple<FString, EChunkLoadResult, FGuid> FChunkFileResult;
	TArray<TFuture<FChunkFileResult>> ChunkFileResults;
	for (const FString& ChunkFile : ChunkFiles)
	{
		TFunction<FChunkFileResult()> Task = [ChunkFile, &ChunkDataSerialization, &SearchPath]()
		{
			FChunkFileResult Result;
			EChunkLoadResult LoadResult;
			FGuid ChunkId;
			FBuildPatchUtils::GetGUIDFromFilename(ChunkFile, ChunkId);
			TUniquePtr<IChunkDataAccess> ChunkDataAccess(ChunkDataSerialization->LoadFromFile(ChunkFile, LoadResult));
			Result.Get<0>() = ChunkFile;
			Result.Get<1>() = LoadResult;
			Result.Get<2>() = ChunkId;
			if (ChunkDataAccess)
			{
				// Ensure we can get GUID from readable chunks.
				FScopeLockedChunkData ScopeLockedChunkData(ChunkDataAccess.Get());
				if (ChunkId != ScopeLockedChunkData.GetHeader()->Guid)
				{
					Result.Get<1>() = EChunkLoadResult::CorruptHeader;
				}
			}
			return Result;
		};
		ChunkFileResults.Add(Async(EAsyncExecution::ThreadPool, Task));
	}

	// Kick off loading of manifest files.
	typedef TTuple<FBuildPatchAppManifestPtr, FString> FManifestFileResult;
	TArray<TFuture<FManifestFileResult>> ManifestFileResults;
	for (const FString& ManifestFile : ManifestFiles)
	{
		TFunction<FManifestFileResult()> Task = [ManifestFile]()
		{
			FManifestFileResult ManifestFileResult;
			ManifestFileResult.Get<0>() = VerifyHelpers::LoadManifestFile(ManifestFile);
			ManifestFileResult.Get<1>() = ManifestFile;
			return ManifestFileResult;
		};
		ManifestFileResults.Add(Async(EAsyncExecution::ThreadPool, Task));
	}

	// Collect all the chunk file verification results.
	const int32 ChunkNum = ChunkFileResults.Num();
	int32 ChunkCount = 1;
	for (const TFuture<FChunkFileResult>& ChunkFileResult : ChunkFileResults)
	{
		FChunkFileResult Result = ChunkFileResult.Get();
		if (Result.Get<1>() == EChunkLoadResult::Success)
		{
			UE_LOGF(LogVerifyChunkData, Display, "[%d/%d]: Chunk file good: %ls", ChunkCount++, ChunkNum, *Result.Get<0>());
			GoodChunkData.Add(Result.Get<2>());
		}
		else if (Result.Get<1>() == EChunkLoadResult::MissingSecretKey)
		{
			UE_LOGF(LogVerifyChunkData, Display, "[%d/%d]: Chunk file cannot be verified without secret key: %ls", ChunkCount++, ChunkNum, *Result.Get<0>());
			SkippedChunkData.Add(Result.Get<2>());
		}
		else
		{
			UE_LOGF(LogVerifyChunkData, Error, "[%d/%d]: Corrupt chunk file: %ls", ChunkCount++, ChunkNum, *Result.Get<0>());
			BadChunkData.Add(Result.Get<2>());
			OutputText += Result.Get<0>() + TEXT("\r\n");
			bSuccess = false;
		}
	}

	// Verify chunkdb files.
	const int32 ChunkDbNum = ChunkDbFiles.Num();
	int32 ChunkDbCount = 1;
	for (const FString& ChunkDbFile : ChunkDbFiles)
	{
		bool bDbGood = false;
		TUniquePtr<FArchive> File = FileSystem->CreateFileReader(*ChunkDbFile);
		if (File.IsValid())
		{
			// Load header.
			FChunkDatabaseHeader Header;
			*File << Header;
			if (!File->IsError())
			{
				const int64 TotalFileSize = File->TotalSize();
				bDbGood = true;
				// Now every chunk file.
				const int32 ChunkContentNum = Header.Contents.Num();
				int32 ChunkContentCount = 1;
				for (const FChunkLocation& Location : Header.Contents)
				{
					bool bChunkGood = false;
					int64 DataEndPoint = Location.ByteStart + Location.ByteSize;
					if (TotalFileSize >= DataEndPoint)
					{
						File->Seek(Location.ByteStart);
						EChunkLoadResult LoadResult;
						TUniquePtr<IChunkDataAccess> ChunkDataAccess(ChunkDataSerialization->LoadFromArchive(*File, LoadResult));
						int64 End = File->Tell();
						bChunkGood = LoadResult == EChunkLoadResult::Success && ChunkDataAccess.IsValid() && (End == DataEndPoint);
					}
					if (bChunkGood)
					{
						UE_LOGF(LogVerifyChunkData, Display, "[%d/%d]: Chunk inside of db good: %ls", ChunkContentCount++, ChunkContentNum, *Location.ChunkId.ToString());
					}
					else
					{
						UE_LOGF(LogVerifyChunkData, Error, "[%d/%d]: Corrupt Chunk inside of db: %ls", ChunkContentCount++, ChunkContentNum, *Location.ChunkId.ToString());
						bSuccess = false;
						bDbGood = false;
					}
				}
			}
		}
		if (bDbGood)
		{
			UE_LOGF(LogVerifyChunkData, Display, "[%d/%d]: Chunkdb file good: %ls", ChunkDbCount++, ChunkDbNum, *ChunkDbFile);
		}
		else
		{
			UE_LOGF(LogVerifyChunkData, Error, "[%d/%d]: Corrupt chunkdb file: %ls", ChunkDbCount++, ChunkDbNum, *ChunkDbFile);
			OutputText += ChunkDbFile + TEXT("\r\n");
			bSuccess = false;
		}
	}

	// Collect all the manifest file loads, and see if any are referencing bad data.
	const int32 ManifestNum = ManifestFileResults.Num();
	int32 ManifestCount = 0;
	for (const TFuture<FManifestFileResult>& ManifestFileResult : ManifestFileResults)
	{
		++ManifestCount;
		bool bManifestOk = true;
		FManifestFileResult ManifestFile = ManifestFileResult.Get();
		if (ManifestFile.Get<0>().IsValid())
		{
			if (!ManifestFile.Get<0>()->IsFileDataManifest())
			{
				TSet<FGuid> ReferencedData;
				ManifestFile.Get<0>()->GetDataList(ReferencedData);
				TSet<FGuid> ReferencedSkippedChunkData = ReferencedData.Intersect(SkippedChunkData);
				TSet<FGuid> ReferencedBadData = ReferencedData.Intersect(BadChunkData);
				TSet<FGuid> ReferencedMissingData = ReferencedData.Difference(BadChunkData).Difference(GoodChunkData);
				if (ReferencedSkippedChunkData.Num() > 0)
				{
					UE_LOGF(LogVerifyChunkData, Warning, "[%d/%d]: Skipped data referenced by %ls file: %ls", ManifestCount, ManifestNum, *FPaths::GetExtension(ManifestFile.Get<1>()), *ManifestFile.Get<1>());
				}
				if (ReferencedBadData.Num() > 0)
				{
					UE_LOGF(LogVerifyChunkData, Error, "[%d/%d]: Bad data referenced by %ls file: %ls", ManifestCount, ManifestNum, *FPaths::GetExtension(ManifestFile.Get<1>()), *ManifestFile.Get<1>());
					bManifestOk = false;
				}
				if (ReferencedMissingData.Num() > 0)
				{
					UE_LOGF(LogVerifyChunkData, Error, "[%d/%d]: Missing data referenced by %ls file: %ls", ManifestCount, ManifestNum, *FPaths::GetExtension(ManifestFile.Get<1>()), *ManifestFile.Get<1>());
					bManifestOk = false;
				}
			}
			else
			{
				UE_LOGF(LogVerifyChunkData, Display, "[%d/%d]: Skipping legacy file based %ls file: %ls", ManifestCount, ManifestNum, *FPaths::GetExtension(ManifestFile.Get<1>()), *ManifestFile.Get<1>());
			}
		}
		else
		{
			UE_LOGF(LogVerifyChunkData, Error, "[%d/%d]: Corrupt %ls file: %ls", ManifestCount, ManifestNum, *FPaths::GetExtension(ManifestFile.Get<1>()), *ManifestFile.Get<1>());
			bManifestOk = false;
		}

		if (bManifestOk)
		{
			UE_LOGF(LogVerifyChunkData, Display, "[%d/%d]: %ls file good: %ls", ManifestCount, ManifestNum, *FPaths::GetExtension(ManifestFile.Get<1>()), *ManifestFile.Get<1>());
		}
		else
		{
			OutputText += ManifestFile.Get<1>() + TEXT("\r\n");
			bSuccess = false;
		}
	}

	// Save the output if we were given a file.
	if (!OutputFile.IsEmpty())
	{
		FFileHelper::SaveStringToFile(OutputText, *OutputFile);
	}

	return bSuccess;
}
