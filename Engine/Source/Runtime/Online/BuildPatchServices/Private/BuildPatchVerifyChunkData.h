// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FBuildVerifyChunkData
{
public:
	static bool VerifyChunkData(const FString& SearchPath, const FString& OutputFile);
};
