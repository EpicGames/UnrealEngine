// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/ChunkDataSizeProvider.h"
#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"
#include "BuildPatchUtil.h"

namespace BuildPatchServices
{
	class FChunkDataSizeProvider
		: public IChunkDataSizeProvider
	{
	public:
		FChunkDataSizeProvider(const TArray<FBuildPatchAppManifestPtr>& InitialManifests)
		{
			for (const FBuildPatchAppManifestPtr& InitialManifest : InitialManifests)
			{
				AddManifestData(InitialManifest);
			}
		}

		// Begin IDataSizeProvider
		virtual uint64 GetDownloadSize(const FString& Identifier) const override
		{
			FGuid ChunkGUID;
			if (!FBuildPatchUtils::GetGUIDFromFilename(Identifier, ChunkGUID))
			{
				return INDEX_NONE;
			}

			FScopeLock Lock(&ManifestsCS);
			return GetDownloadSizeUnsafe(ChunkGUID);
		}

		virtual void GetDownloadSize(TConstArrayView<FString> Identifiers, TArray<uint64>& OutSizes) const override
		{
			TArray<FGuid> ChunkGUIDs;
			ChunkGUIDs.Reserve(Identifiers.Num());
			for (const FString& Identifier : Identifiers)
			{
				FGuid ChunkGUID;
				if (!FBuildPatchUtils::GetGUIDFromFilename(Identifier, ChunkGUID))
				{
					ChunkGUID.Invalidate();
				}
				ChunkGUIDs.Emplace(ChunkGUID);
			}
			OutSizes.Reserve(OutSizes.Num() + Identifiers.Num());

			FScopeLock Lock(&ManifestsCS);
			for (const FGuid& ChunkGUID : ChunkGUIDs)
			{
				if (!ChunkGUID.IsValid())
				{
					OutSizes.Emplace(INDEX_NONE);
					continue;
				}

				OutSizes.Emplace(GetDownloadSizeUnsafe(ChunkGUID));
			}
		}
		// End IDataSizeProvider

		// Begin IChunkDataSizeProvider
		virtual void AddManifestData(FBuildPatchAppManifestPtr Manifest) override
		{
			if (!Manifest)
			{
				return;
			}

			FScopeLock Lock(&ManifestsCS);
			AddManifestDataUnsafe(MoveTemp(Manifest));
		}

		virtual void AddManifestData(TConstArrayView<FBuildPatchAppManifestPtr> InManifests) override
		{
			FScopeLock Lock(&ManifestsCS);
			for (FBuildPatchAppManifestPtr Manifest : InManifests)
			{
				if (Manifest)
				{
					AddManifestDataUnsafe(MoveTemp(Manifest));
				}
			}
		}
		// End IChunkDataSizeProvider

		uint64 GetDownloadSizeUnsafe(const FGuid& ChunkGUID) const
		{
			checkSlow(ChunkGUID.IsValid());
			uint64 DownloadSize = INDEX_NONE;
			for (int32 i = Manifests.Num() - 1; i >= 0; --i)
			{
				const FBuildPatchAppManifestPtr& Manifest = Manifests[i];
				const BuildPatchServices::FChunkInfo* ChunkInfo = Manifest->GetChunkInfo(ChunkGUID);
				if (ChunkInfo)
				{
					DownloadSize = ChunkInfo->FileSize;
					break;
				}
			}
			return DownloadSize;
		}

		void AddManifestDataUnsafe(FBuildPatchAppManifestPtr Manifest)
		{
			checkSlow(Manifest);
			int32 Index = Manifests.Find(Manifest);
			if (Index != INDEX_NONE)
			{
				Manifests.RemoveAt(Index, EAllowShrinking::No);
			}

			Manifests.Add(MoveTemp(Manifest));  // Last manifest added always goes on the end
		}

	private:
		TArray<FBuildPatchAppManifestPtr> Manifests;
		mutable FCriticalSection ManifestsCS;
	};
	
	IChunkDataSizeProvider* FChunkDataSizeProviderFactory::Create(const TArray<FBuildPatchAppManifestPtr>& InitialManifests)
	{
		return new FChunkDataSizeProvider(InitialManifests);
	}
}
