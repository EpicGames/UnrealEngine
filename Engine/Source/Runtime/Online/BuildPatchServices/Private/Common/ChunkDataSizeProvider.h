// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Common/DataSizeProvider.h"
#include "BuildPatchManifest.h"
#include "Containers/ContainersFwd.h"

namespace BuildPatchServices
{
	class IChunkDataSizeProvider
		: public IDataSizeProvider
	{
	public:
		virtual void AddManifestData(FBuildPatchAppManifestPtr Manifest) = 0;
		virtual void AddManifestData(TConstArrayView<FBuildPatchAppManifestPtr> Manifests) = 0;
	};

	class FChunkDataSizeProviderFactory
	{
	public:
		static IChunkDataSizeProvider* Create(const TArray<FBuildPatchAppManifestPtr>& InitialManifests);
	};
}