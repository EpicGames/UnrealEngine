// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/Crypto.h"

#if BPS_WITH_OPENSSL
#define UI UI_ST
THIRD_PARTY_INCLUDES_START
#include <openssl/evp.h>
#include <openssl/rsa.h>
#include <openssl/rand.h>
THIRD_PARTY_INCLUDES_END
#undef UI
#endif // !BPS_WITH_OPENSSL

#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogBPSCryptoOpenSSL, Warning, All);

namespace BuildPatchServices
{
	constexpr int OpenSSLCipherSuccess = 1;

#if BPS_WITH_OPENSSL
	class FEncryptor_AES_256_GCM_OpenSSL
	{
	public:
		FEncryptor_AES_256_GCM_OpenSSL(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector)
			: Context(EVP_CIPHER_CTX_new())
		{
			const EVP_CIPHER* Cipher = EVP_aes_256_gcm();
			const int32 ExpectedKeyLength = EVP_CIPHER_key_length(Cipher);
			if (Key.Num() < ExpectedKeyLength)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "Invalid Key Size, failed to construct Encryptor. KeySize=[%d] Expected=[%d]", Key.Num(), ExpectedKeyLength);
				ensure(Key.Num() >= ExpectedKeyLength);
				return;
			}

			const int32 IVExpectedLength = EVP_CIPHER_iv_length(Cipher);
			if (InitializationVector.Num() < IVExpectedLength)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "Invalid InitializationVector Size, failed to create Encryptor. InitializationVectorSize=[%d] Expected=[%d]", InitializationVector.Num(), IVExpectedLength);
				ensure(InitializationVector.Num() >= IVExpectedLength);
				return;
			}

			const int InitResult = EVP_EncryptInit_ex(Context, Cipher, nullptr, Key.GetData(), InitializationVector.GetData());
			if (InitResult != OpenSSLCipherSuccess)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FEncryptor_AES_256_GCM_OpenSSL::constructor: EVP_EncryptInit_ex failed. Result=[%d]", InitResult);
				return;
			}

			const int CTXBlockSize = EVP_CIPHER_CTX_block_size(Context);
			ensure(CTXBlockSize == 1); // 1 means no change in data size
			bIsInitialized = true;
		}

		~FEncryptor_AES_256_GCM_OpenSSL()
		{
			EVP_CIPHER_CTX_free(Context);
		}

		bool GetCipherText(const TArrayView<const uint8> Plaintext, TArray<uint8>& OutCiphertext) const
		{
			if (!bIsInitialized)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FEncryptor_AES_256_GCM_OpenSSL::GetCipherText failed. Encryptor was not properly initialized");
				return false;
			}
			int CiphertextBytesWritten = 0;
			OutCiphertext.Empty(Plaintext.Num());
			OutCiphertext.AddUninitialized(Plaintext.Num());
			const int UpdateResult = EVP_EncryptUpdate(Context, OutCiphertext.GetData(), &CiphertextBytesWritten, Plaintext.GetData(), Plaintext.Num());
			if (UpdateResult != OpenSSLCipherSuccess)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FEncryptor_AES_256_GCM_OpenSSL::GetCipherText: EVP_EncryptUpdate failed. Result=[%d]", UpdateResult);
				return false;
			}
			ensure(CiphertextBytesWritten == Plaintext.Num());
			const int FinalizeResult = EVP_EncryptFinal_ex(Context, OutCiphertext.GetData(), &CiphertextBytesWritten);
			if (FinalizeResult != OpenSSLCipherSuccess)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FEncryptor_AES_256_GCM_OpenSSL::GetCipherText: EVP_EncryptFinal_ex failed. Result=[%d]", FinalizeResult);
				return false;
			}
			ensure(CiphertextBytesWritten == 0);
			return true;
		}

		bool GetAuthTag(TArray<uint8>& OutAuthTag) const
		{
			if (!bIsInitialized)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FEncryptor_AES_256_GCM_OpenSSL::GetAuthTag failed. Encryptor was not properly initialized");
				return false;
			}
			OutAuthTag.Empty(AES256_GCM_AuthTagSizeInBytes);
			OutAuthTag.AddUninitialized(AES256_GCM_AuthTagSizeInBytes);
			const int GenerateAuthTagResult = EVP_CIPHER_CTX_ctrl(Context, EVP_CTRL_GCM_GET_TAG, OutAuthTag.Num(), OutAuthTag.GetData());
			if (GenerateAuthTagResult != OpenSSLCipherSuccess)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FEncryptor_AES_256_GCM_OpenSSL::GetAuthTag: EVP_CIPHER_CTX_ctrl failed. Result=[%d]", GenerateAuthTagResult);
				return false;
			}
			return true;
		}

	private:
		EVP_CIPHER_CTX* Context;
		bool bIsInitialized = false;
	};

	class FDecryptor_AES_256_GCM_OpenSSL
	{
	public:
		FDecryptor_AES_256_GCM_OpenSSL(const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, const TArrayView<const uint8> AuthTag)
			: Context(EVP_CIPHER_CTX_new())
		{
			const EVP_CIPHER* Cipher = EVP_aes_256_gcm();
			const int32 ExpectedKeyLength = EVP_CIPHER_key_length(Cipher);
			if (Key.Num() < ExpectedKeyLength)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "Invalid Key Size, failed to construct Decriptor. KeySize=[%d] Expected=[%d]", Key.Num(), ExpectedKeyLength);
				ensure(Key.Num() >= ExpectedKeyLength);
				return;
			}

			const int32 IVExpectedLength = EVP_CIPHER_iv_length(Cipher);
			if (InitializationVector.Num() < IVExpectedLength)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "Invalid InitializationVector Size, failed to create Decriptor. InitializationVectorSize=[%d] Expected=[%d]", InitializationVector.Num(), IVExpectedLength);
				ensure(InitializationVector.Num() >= IVExpectedLength);
				return;
			}

			if (AuthTag.Num() < AES256_GCM_AuthTagSizeInBytes)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "Invalid AuthTag Size, failed to create Decriptor. AuthTag Size=[%d] Expected=[%d]", AuthTag.Num(), AES256_GCM_AuthTagSizeInBytes);
				ensure(AuthTag.Num() >= AES256_GCM_AuthTagSizeInBytes);
				return;
			}

			const int InitResult = EVP_DecryptInit_ex(Context, Cipher, nullptr, Key.GetData(), InitializationVector.GetData());
			if (InitResult != OpenSSLCipherSuccess)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FDecryptor_AES_256_GCM_OpenSSL::Constructor: EVP_DecryptInit_ex failed. Result=[%d]", InitResult);
				return;
			}
			const int CTXBlockSize = EVP_CIPHER_CTX_block_size(Context);
			ensure(CTXBlockSize == 1); // 1 means no change in data size

			const int SetTagResult = EVP_CIPHER_CTX_ctrl(Context, EVP_CTRL_GCM_SET_TAG, AuthTag.Num(), const_cast<uint8*>(AuthTag.GetData()));
			if (SetTagResult != OpenSSLCipherSuccess)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FDecryptor_AES_256_GCM_OpenSSL::Constructor: EVP_CIPHER_CTX_ctrl failed. Result=[%d]", SetTagResult);
				return;
			}
			bIsInitialized = true;
		}

		~FDecryptor_AES_256_GCM_OpenSSL()
		{
			EVP_CIPHER_CTX_free(Context);
		}

		bool GetPlainText(const TArrayView<const uint8> Ciphertext, TArray<uint8>& OutPlaintext) const
		{
			if (!bIsInitialized)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FEncryptor_AES_256_GCM_OpenSSL::GetPlainText failed. Decryptor was not properly initialized");
				return false;
			}
			int PlaintextBytesWritten = 0;
			OutPlaintext.Empty(Ciphertext.Num());
			OutPlaintext.AddUninitialized(Ciphertext.Num());
			const int UpdateResult = EVP_DecryptUpdate(Context, OutPlaintext.GetData(), &PlaintextBytesWritten, Ciphertext.GetData(), Ciphertext.Num());
			if (UpdateResult != OpenSSLCipherSuccess)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FDecryptor_AES_256_GCM_OpenSSL::GetPlainText: EVP_DecryptUpdate failed. Result=[%d]", UpdateResult);
				return false;
			}
			ensure(PlaintextBytesWritten == Ciphertext.Num());
			
			const int FinalizeResult = EVP_DecryptFinal_ex(Context, OutPlaintext.GetData(), &PlaintextBytesWritten);
			if (FinalizeResult != OpenSSLCipherSuccess)
			{
				UE_LOGF(LogBPSCryptoOpenSSL, Warning, "FDecryptor_AES_256_GCM_OpenSSL::GetPlainText: EVP_DecryptFinal_ex failed. Result=[%d]", FinalizeResult);
				return false;
			}
			ensure(PlaintextBytesWritten == 0);

			return true;
		}

	private:
		EVP_CIPHER_CTX* Context;
		bool bIsInitialized = false;
	};

	class FRand_OpenSSL
	{
	public:
		bool CreateRandomBytes(const TArrayView<uint8> OutData) const
		{
			return RAND_bytes(OutData.GetData(), OutData.Num()) == 1;
		}
	};

#else
	class FEncryptor_AES_256_GCM_OpenSSL
	{
	public:
		FEncryptor_AES_256_GCM_OpenSSL(const TArrayView<const uint8>, const TArrayView<const uint8>) { }
		~FEncryptor_AES_256_GCM_OpenSSL() { }
		bool GetCipherText(const TArrayView<const uint8>, TArray<uint8>&) const { return false; }
		bool GetAuthTag(TArray<uint8>&) const { return false; }
	};

	class FDecryptor_AES_256_GCM_OpenSSL
	{
	public:
		FDecryptor_AES_256_GCM_OpenSSL(const TArrayView<const uint8>, const TArrayView<const uint8>, const TArrayView<const uint8>) { }
		~FDecryptor_AES_256_GCM_OpenSSL() { }
		bool GetPlainText(const TArrayView<const uint8>, TArray<uint8>&) const { return false; }
	};

	class FRand_OpenSSL
	{
	public:
		bool CreateRandomBytes(const TArrayView<uint8> OutData) const { return false; }
	};

#endif // !BPS_WITH_OPENSSL

	class FCrypto
		: public ICrypto
	{
	public:
		// ICrypto interface begin.
		virtual bool IsValidKey_AES_256_GCM(const TArrayView<const uint8> Key) const override;
		virtual TArray<uint8> Encrypt_AES_256_GCM(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, TArray<uint8>& OutAuthTag, bool& OutResult) const override;
		virtual TArray<uint8> Decrypt_AES_256_GCM(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, const TArrayView<const uint8> AuthTag, bool& OutResult) const override;
		virtual bool CreateRandomBytes(const TArrayView<uint8> OutData) const override;
		// ICrypto interface end.
	};

	bool FCrypto::IsValidKey_AES_256_GCM(const TArrayView<const uint8> Key) const
	{
		if (Key.Num() == AES256_GCM_KeySizeInBytes)
		{
			return true;
		}
		return false;
	}

	TArray<uint8> FCrypto::Encrypt_AES_256_GCM(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, TArray<uint8>& OutAuthTag, bool& OutResult) const
	{
		OutResult = false;
		FEncryptor_AES_256_GCM_OpenSSL Encryptor(Key, InitializationVector);
		TArray<uint8> Ciphertext;
		if (!Encryptor.GetCipherText(Plaintext, Ciphertext))
		{
			return TArray<uint8>();
		}
		if (!Encryptor.GetAuthTag(OutAuthTag))
		{
			return TArray<uint8>();
		}
		OutResult = true;
		return Ciphertext;
	}

	TArray<uint8> FCrypto::Decrypt_AES_256_GCM(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, const TArrayView<const uint8> AuthTag, bool& OutResult) const
	{
		OutResult = false;
		FDecryptor_AES_256_GCM_OpenSSL Decryptor(Key, InitializationVector, AuthTag);
		TArray<uint8> Plaintext;
		if (!Decryptor.GetPlainText(Ciphertext, Plaintext))
		{
			return TArray<uint8>();
		}
		OutResult = true;
		return Plaintext;
	}

	bool FCrypto::CreateRandomBytes(const TArrayView<uint8> OutData) const
	{
		FRand_OpenSSL Rand;
		return Rand.CreateRandomBytes(OutData);
	}

	ICrypto* FCryptoFactory::Create()
	{
		return new FCrypto();
	}
}