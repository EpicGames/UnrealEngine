// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"

namespace BuildPatchServices
{
	constexpr int AES256_GCM_InitializationVectorSizeInBytes = 12;
	constexpr int AES256_GCM_AuthTagSizeInBytes = 16;
	constexpr int AES256_GCM_KeySizeInBytes = 32;

	struct FAESAuthTag
	{
	public:
		alignas(uint32) uint8 AuthTag[AES256_GCM_AuthTagSizeInBytes] = { 0 };
		void Reset()
		{
			FMemory::Memzero(AuthTag, AES256_GCM_AuthTagSizeInBytes);
		}
	};

	/**
	 * This Crypto class for the time being takes code from other engine locations such as plugins, which unfortunately cannot be relied on.
	 * We are going to entirely use OpenSSL implementations, so it will only be available on platforms supporting this.
	 */
	class ICrypto
	{
	public:
		virtual ~ICrypto() {}

#if BPS_WITH_OPENSSL
		virtual bool IsAvailable() const { return true; }
#else
		virtual bool IsAvailable() const { return false; }
#endif
		virtual bool IsValidKey_AES_256_GCM(const TArrayView<const uint8> Key) const = 0;

		virtual TArray<uint8> Encrypt_AES_256_GCM(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, TArray<uint8>& OutAuthTag, bool& OutResult) const = 0;

		virtual TArray<uint8> Decrypt_AES_256_GCM(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, const TArrayView<const uint8> AuthTag, bool& OutResult) const = 0;

		virtual bool CreateRandomBytes(const TArrayView<uint8> OutData) const = 0;
	};

	/**
	 * A factory for creating an ICrypto instance.
	 */
	class FCryptoFactory
	{
	public:
		/**
		 * Creates an implementation which wraps use of OpenSLL crypto functionality, or is not available on platforms without OpenSSL.
		 * @return the new ICrypto instance created.
		 */
		static ICrypto* Create();
	};
}