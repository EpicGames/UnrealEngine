// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainersFwd.h"

namespace BuildPatchServices
{
	class IDataSizeProvider
	{
	public:
		virtual ~IDataSizeProvider() = default;

		virtual uint64 GetDownloadSize(const FString& Uri) const = 0;
		virtual void GetDownloadSize(TConstArrayView<FString> InUris, TArray<uint64>& OutSizes) const = 0;
	};
}
