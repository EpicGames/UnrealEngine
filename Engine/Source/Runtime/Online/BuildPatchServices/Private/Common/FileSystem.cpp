// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/FileSystem.h"

#include "Async/Async.h"
#include "Containers/Queue.h"
#include "Containers/StringConv.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceRedirector.h"

#include "Core/AsyncHelpers.h"

#if PLATFORM_MAC
#include <sys/stat.h>
#endif

#if PLATFORM_WINDOWS
// Start of region that uses windows types.
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <wtypes.h>
#include <winbase.h>
#include <winioctl.h>
THIRD_PARTY_INCLUDES_END

namespace FileSystemHelpers
{
	bool PlatformGetAttributes(const TCHAR* Filename, BuildPatchServices::EAttributeFlags& OutFileAttributes)
	{
		OutFileAttributes = BuildPatchServices::EAttributeFlags::None;
		DWORD FileAttributes = ::GetFileAttributesW(Filename);
		DWORD Error = ::GetLastError();
		if (FileAttributes != INVALID_FILE_ATTRIBUTES)
		{
			OutFileAttributes |= BuildPatchServices::EAttributeFlags::Exists;
			if ((FileAttributes & FILE_ATTRIBUTE_READONLY) != 0)
			{
				OutFileAttributes |= BuildPatchServices::EAttributeFlags::ReadOnly;
			}
			if ((FileAttributes & FILE_ATTRIBUTE_COMPRESSED) != 0)
			{
				OutFileAttributes |= BuildPatchServices::EAttributeFlags::Compressed;
			}
			return true;
		}
		else if (Error == ERROR_PATH_NOT_FOUND || Error == ERROR_FILE_NOT_FOUND)
		{
			return true;
		}
		return false;
	}

	bool PlatformSetCompressed(const TCHAR* Filename, bool bIsCompressed)
	{
		// Get the file handle.
		HANDLE FileHandle = ::CreateFile(
			Filename,                               // Path to file.
			GENERIC_READ | GENERIC_WRITE,           // Read and write access.
			FILE_SHARE_READ | FILE_SHARE_WRITE,     // Share access for DeviceIoControl.
			NULL,                                   // Default security.
			OPEN_EXISTING,                          // Open existing file.
			FILE_ATTRIBUTE_NORMAL,                  // No specific attributes.
			NULL                                    // No template file handle.
		);
		uint32 Error = ::GetLastError();
		if (FileHandle == NULL || FileHandle == INVALID_HANDLE_VALUE)
		{
			return false;
		}

		// Send the compression control code to the device.
		uint16 Message = bIsCompressed ? COMPRESSION_FORMAT_DEFAULT : COMPRESSION_FORMAT_NONE;
		uint16* MessagePtr = &Message;
		DWORD Dummy = 0;
		LPDWORD DummyPtr = &Dummy;
		bool bSuccess = ::DeviceIoControl(
			FileHandle,                 // The file handle.
			FSCTL_SET_COMPRESSION,      // Control code.
			MessagePtr,                 // Our message.
			sizeof(uint16),             // Our message size.
			NULL,                       // Not used.
			0,                          // Not used.
			DummyPtr,                   // The value returned will be meaningless, but is required.
			NULL                        // No overlap structure, we a running this synchronously.
		) == TRUE;
		Error = ::GetLastError();

		// Close the open file handle.
		::CloseHandle(FileHandle);

		// We treat unsupported as not being a failure.
		const bool bFileSystemUnsupported = Error == ERROR_INVALID_FUNCTION;
		return bSuccess || bFileSystemUnsupported;
	}

	bool PlatformSetExecutable(const TCHAR* Filename, bool bIsExecutable)
	{
		// Not implemented.
		return true;
	}
}
// End of region that uses windows types.
#include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_MAC || PLATFORM_LINUX
namespace FileSystemHelpers
{
	bool PlatformGetAttributes(const TCHAR* Filename, BuildPatchServices::EAttributeFlags& OutFileAttributes)
	{
		struct stat FileInfo;
		const FTCHARToUTF8 FilenameUtf8(Filename);
		int32 Result = stat(FilenameUtf8.Get(), &FileInfo);
		int32 Error = errno;
		OutFileAttributes = BuildPatchServices::EAttributeFlags::None;
		if (Result == 0)
		{
			OutFileAttributes |= BuildPatchServices::EAttributeFlags::Exists;
			if ((FileInfo.st_mode & S_IWUSR) == 0)
			{
				OutFileAttributes |= BuildPatchServices::EAttributeFlags::ReadOnly;
			}
			const mode_t ExeFlags = S_IXUSR | S_IXGRP | S_IXOTH;
			if ((FileInfo.st_mode & ExeFlags) == ExeFlags)
			{
				OutFileAttributes |= BuildPatchServices::EAttributeFlags::Executable;
			}
			return true;
		}
		else if (Error == ENOTDIR || Error == ENOENT)
		{
			return true;
		}
		return false;
	}

	bool PlatformSetCompressed(const TCHAR* Filename, bool bIsCompressed)
	{
		// Not implemented
		return true;
	}

	bool PlatformSetExecutable(const TCHAR* Filename, bool bIsExecutable)
	{
		struct stat FileInfo;
		FTCHARToUTF8 FilenameUtf8(Filename);
		bool bSuccess = false;
		// Set executable permission bit
		if (stat(FilenameUtf8.Get(), &FileInfo) == 0)
		{
			mode_t ExeFlags = S_IXUSR | S_IXGRP | S_IXOTH;
			FileInfo.st_mode = bIsExecutable ? (FileInfo.st_mode | ExeFlags) : (FileInfo.st_mode & ~ExeFlags);
			bSuccess = chmod(FilenameUtf8.Get(), FileInfo.st_mode) == 0;
		}
		return bSuccess;
	}
}
#else
namespace FileSystemHelpers
{
	bool PlatformGetAttributes(const TCHAR* Filename, BuildPatchServices::EAttributeFlags& OutFileAttributes)
	{
		// Not implemented.
		return true;
	}

	bool PlatformSetCompressed(const TCHAR* Filename, bool bIsCompressed)
	{
		// Not implemented.
		return true;
	}

	bool PlatformSetExecutable(const TCHAR* Filename, bool bIsExecutable)
	{
		// Not implemented.
		return true;
	}
}
#endif

// We are forwarding flags, so assert they are all equal.
static_assert((uint32)BuildPatchServices::EWriteFlags::None == (uint32)::EFileWrite::FILEWRITE_None, "Please update FileSystem.h to match BuildPatchServices::EFileWrite::None with ::EFileWrite::FILEWRITE_None");
static_assert((uint32)BuildPatchServices::EWriteFlags::NoFail == (uint32)::EFileWrite::FILEWRITE_NoFail, "Please update FileSystem.h to match BuildPatchServices::EFileWrite::NoFail with ::EFileWrite::FILEWRITE_NoFail");
static_assert((uint32)BuildPatchServices::EWriteFlags::NoReplaceExisting == (uint32)::EFileWrite::FILEWRITE_NoReplaceExisting, "Please update FileSystem.h to match BuildPatchServices::EFileWrite::NoReplaceExisting with ::EFileWrite::FILEWRITE_NoReplaceExisting");
static_assert((uint32)BuildPatchServices::EWriteFlags::EvenIfReadOnly == (uint32)::EFileWrite::FILEWRITE_EvenIfReadOnly, "Please update FileSystem.h to match BuildPatchServices::EFileWrite::EvenIfReadOnly with ::EFileWrite::FILEWRITE_EvenIfReadOnly");
static_assert((uint32)BuildPatchServices::EWriteFlags::Append == (uint32)::EFileWrite::FILEWRITE_Append, "Please update FileSystem.h to match BuildPatchServices::EFileWrite::Append with ::EFileWrite::FILEWRITE_Append");
static_assert((uint32)BuildPatchServices::EWriteFlags::AllowRead == (uint32)::EFileWrite::FILEWRITE_AllowRead, "Please update FileSystem.h to match BuildPatchServices::EFileWrite::AllowRead with ::EFileWrite::FILEWRITE_AllowRead");
static_assert((uint32)BuildPatchServices::EReadFlags::None == (uint32)::EFileRead::FILEREAD_None, "Please update FileSystem.h to match BuildPatchServices::EFileRead::None with ::EFileRead::FILEREAD_None");
static_assert((uint32)BuildPatchServices::EReadFlags::NoFail == (uint32)::EFileRead::FILEREAD_NoFail, "Please update FileSystem.h to match BuildPatchServices::EFileRead::NoFail with ::EFileRead::FILEREAD_NoFail");
static_assert((uint32)BuildPatchServices::EReadFlags::Silent == (uint32)::EFileRead::FILEREAD_Silent, "Please update FileSystem.h to match BuildPatchServices::EFileRead::Silent with ::EFileRead::FILEREAD_Silent");
static_assert((uint32)BuildPatchServices::EReadFlags::AllowWrite == (uint32)::EFileRead::FILEREAD_AllowWrite, "Please update FileSystem.h to match BuildPatchServices::EFileRead::AllowWrite with ::EFileRead::FILEREAD_AllowWrite");

namespace BuildPatchServices
{
	TMap<FString, int64> ParallelGetFileSizes(const TSet<FString>& FilesToCheck, const FString& RootDirectory)
	{
		TArray<TTuple<FString, int64>> TargetFiles;
		TargetFiles.Reserve(FilesToCheck.Num());
		for (const FString& File : FilesToCheck)
		{
			TargetFiles.Add(TTuple<FString, int64>(File, INDEX_NONE));
		}
		auto GetFileSizesTasks = CreateTaskSchedulerLimitedByCoresNumber<int64>(EAsyncExecution::TaskGraph);
		for (TTuple<FString, int64>& FileInfo : TargetFiles)
		{
			GetFileSizesTasks.ScheduleTask(
				[&RootDirectory, &FileInfo]() { return IFileManager::Get().FileSize(*(RootDirectory / FileInfo.Get<0>())); },
				[&FileInfo](int64 FileSize) { FileInfo.Get<1>() = FileSize; }
			);
		}
		GetFileSizesTasks.WaitAll();
		TMap<FString, int64> Results;
		for (TTuple<FString, int64>& FileInfo : TargetFiles)
		{
			Results.Add(FileInfo.Get<0>(), FileInfo.Get<1>());
		}
		return Results;
	}


	class FDirectoryEnumeratorBase : public IPlatformFile::FDirectoryVisitor
	{
	protected:
		FDirectoryEnumeratorBase(const TCHAR* InFileExtension, bool bInFollowSymlinks)
			: bFollowSymlinks(bInFollowSymlinks)
			, FileExtension(InFileExtension)
		{
			if (FileExtension.Len() > 0 && !FileExtension.StartsWith(TEXT(".")))
			{
				FileExtension.InsertAt(0, TEXT("."));
			}
		}

		bool IsSymlink(const TCHAR* Filename)
		{
			bool bIsSymlink = false;
			if (!bFollowSymlinks)
			{
#if PLATFORM_MAC || PLATFORM_LINUX
				struct stat StatBuf = {0};
				if (lstat(TCHAR_TO_UTF8(Filename), &StatBuf) != -1)
				{
					bIsSymlink = S_ISLNK(StatBuf.st_mode);
				}
#endif
			}
			return bIsSymlink;
		}

		bool MatchesExtension(const TCHAR* Filename) const
		{
			if (FileExtension.Len() > 0)
			{
				const int32 FileNameLen = FCString::Strlen(Filename);
				if (FileNameLen < FileExtension.Len() || FCString::Strcmp(&Filename[FileNameLen - FileExtension.Len()], *FileExtension) != 0)
				{
					return false;
				}
			}
			return true;
		}

	private:
		bool bFollowSymlinks;
		FString FileExtension;
	};

	class FDirectoryEnumerator : public FDirectoryEnumeratorBase
	{
	public:
		FDirectoryEnumerator(IPlatformFile& InPlatformFile, const TCHAR* InFileExtension, bool bInFollowSymlinks)
			: FDirectoryEnumeratorBase(InFileExtension, bInFollowSymlinks)
			, PlatformFile(InPlatformFile)
		{  }

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			const bool bIsSymlink = IsSymlink(FilenameOrDirectory);
			if (bIsDirectory && !bIsSymlink)
			{
				FString Directory = FilenameOrDirectory;
				PlatformFile.IterateDirectory(*Directory, *this);
			}
			else if (MatchesExtension(FilenameOrDirectory))
			{
				FoundFiles.Add(FilenameOrDirectory);
			}
			return true;
		}

		void GetFiles(TArray<FString>& Results)
		{
			Results = FoundFiles;
		}

	private:
		IPlatformFile& PlatformFile;
		TArray<FString> FoundFiles;
	};

	class FParallelDirectoryEnumerator : public FDirectoryEnumeratorBase
	{
	public:
		FParallelDirectoryEnumerator(IPlatformFile& InPlatformFile, const TCHAR* InFileExtension, EAsyncExecution InAsyncExecution, bool bInFollowSymlinks)
			: FDirectoryEnumeratorBase(InFileExtension, bInFollowSymlinks)
			, PlatformFile(InPlatformFile)
			, AsyncExecution(InAsyncExecution)
		{  }

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			const bool bIsSymlink = IsSymlink(FilenameOrDirectory);
			if (bIsDirectory && !bIsSymlink)
			{
				FString Directory = FilenameOrDirectory;
				DirectoryFutures.Enqueue(Async(AsyncExecution, [this, Directory = MoveTemp(Directory)]() { PlatformFile.IterateDirectory(*Directory, *this); }));
			}
			else if (MatchesExtension(FilenameOrDirectory))
			{
				FoundFilesQueue.Enqueue(FilenameOrDirectory);
			}
			return true;
		}

		void GetFiles(TArray<FString>& Results)
		{
			TFuture<void> Future;
			while (DirectoryFutures.Dequeue(Future)) { Future.Wait(); }
			while (FoundFilesQueue.Dequeue(Results.AddDefaulted_GetRef())) {}
			Results.Pop(EAllowShrinking::Yes);
			Results.Sort();
		}

	private:
		IPlatformFile& PlatformFile;
		EAsyncExecution AsyncExecution;
		TQueue<FString, EQueueMode::Mpsc> FoundFilesQueue;
		TQueue<TFuture<void>, EQueueMode::Mpsc> DirectoryFutures;
	};

	class FFileSystem
		: public IFileSystem
	{
	public:
		FFileSystem();
		~FFileSystem();

		// IFileSystem interface begin.
		virtual bool DirectoryExists(const TCHAR* DirectoryPath) const override;
		virtual bool MakeDirectory(const TCHAR* DirectoryPath) const override;
		virtual bool DeleteDirectoryRecursively(const TCHAR* DirectoryPath) const override;
		virtual bool GetFileSize(const TCHAR* Filename, int64& FileSize) const override;
		virtual bool GetAttributes(const TCHAR* Filename, EAttributeFlags& Attributes) const override;
		virtual bool GetTimeStamp(const TCHAR* Path, FDateTime& TimeStamp) const override;
		virtual bool SetReadOnly(const TCHAR* Filename, bool bIsReadOnly) const override;
		virtual bool SetCompressed(const TCHAR* Filename, bool bIsCompressed) const override;
		virtual bool SetExecutable(const TCHAR* Filename, bool bIsExecutable) const override;
		virtual bool SetFileSize(const TCHAR* Filename, int64 FileSize) const;
		virtual TUniquePtr<FArchive> CreateFileReader(const TCHAR* Filename, EReadFlags ReadFlags = EReadFlags::None) const override;
		virtual TUniquePtr<FArchive> CreateFileReaderEx(const TCHAR* Filename, EReadFlags ReadFlags, IFileHandle** OutHandle) const override;
		virtual TUniquePtr<FArchive> CreateFileWriter(const TCHAR* Filename, EWriteFlags WriteFlags = EWriteFlags::None) const override;
		virtual bool LoadFileToString(const TCHAR* Filename, FString& Contents) const override;
		virtual bool SaveStringToFile(const TCHAR* Filename, const FString& Contents) const override;
		virtual bool LoadFileToArray(const TCHAR* Filename, TArray<uint8>& Contents) const override;
		virtual bool SaveArrayToFile(const TCHAR* Filename, const TArray<uint8>& Contents) const override;
		virtual bool DeleteFile(const TCHAR* Filename) const override;
		virtual bool MoveFile(const TCHAR* FileDest, const TCHAR* FileSource, bool bDoNotRetryOrError) const override;
		virtual bool CopyFile(const TCHAR* FileDest, const TCHAR* FileSource) const override;
		virtual bool FileExists(const TCHAR* Filename) const override;
		virtual void FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr) const override;
		virtual void FindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr, bool bFollowSymlinks = true) const override;
		virtual void ParallelFindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr, EAsyncExecution AsyncExecution = EAsyncExecution::ThreadPool, bool bFollowSymlinks = true) const override;
		virtual FFileStatData GetStatData(const TCHAR* Filename) const override;
		virtual int64 GetAllowedBytesToWriteThrottledStorage(const TCHAR* DestinationPath = nullptr) const override;
		// IFileSystem interface end.

	private:
		IFileManager& FileManager;
		IPlatformFile& PlatformFile;
	};

	FFileSystem::FFileSystem()
		: FileManager(IFileManager::Get())
		, PlatformFile(IPlatformFile::GetPlatformPhysical())
	{
	}

	FFileSystem::~FFileSystem()
	{
	}

	bool FFileSystem::DirectoryExists(const TCHAR* DirectoryPath) const
	{
		return PlatformFile.DirectoryExists(DirectoryPath);
	}

	bool FFileSystem::MakeDirectory(const TCHAR* DirectoryPath) const
	{
		return PlatformFile.CreateDirectoryTree(DirectoryPath);
	}

	bool FFileSystem::DeleteDirectoryRecursively(const TCHAR* DirectoryPath) const
	{
		return PlatformFile.DeleteDirectoryRecursively(DirectoryPath);
	}

	bool FFileSystem::GetFileSize(const TCHAR* Filename, int64& FileSize) const
	{
		FileSize = PlatformFile.FileSize(Filename);
		return FileSize >= 0;
	}

	bool FFileSystem::GetAttributes(const TCHAR* Filename, EAttributeFlags& Attributes) const
	{
		return FileSystemHelpers::PlatformGetAttributes(Filename, Attributes);
	}

	bool FFileSystem::GetTimeStamp(const TCHAR* Path, FDateTime& TimeStamp) const
	{
		static const FDateTime FailedLookup = FDateTime::MinValue();
		TimeStamp = PlatformFile.GetTimeStamp(Path);
		return TimeStamp != FailedLookup;
	}

	bool FFileSystem::SetReadOnly(const TCHAR* Filename, bool bIsReadOnly) const
	{
		return PlatformFile.SetReadOnly(Filename, bIsReadOnly);
	}

	bool FFileSystem::SetCompressed(const TCHAR* Filename, bool bIsCompressed) const
	{
		return FileSystemHelpers::PlatformSetExecutable(Filename, bIsCompressed);
	}

	bool FFileSystem::SetExecutable(const TCHAR* Filename, bool bIsExecutable) const
	{
		return FileSystemHelpers::PlatformSetExecutable(Filename, bIsExecutable);
	}

	bool FFileSystem::SetFileSize(const TCHAR* Filename, int64 FileSize) const
	{
		TUniquePtr<IFileHandle> Handle{ PlatformFile.OpenWrite(Filename, true, true) };
		if (Handle.IsValid())
		{
			return Handle->Truncate(FileSize);
		}
		return false;
	}

	TUniquePtr<FArchive> FFileSystem::CreateFileReader(const TCHAR* Filename, EReadFlags ReadFlags) const
	{
		return TUniquePtr<FArchive>(FileManager.CreateFileReader(Filename, static_cast<uint32>(ReadFlags)));
	}

	TUniquePtr<FArchive> FFileSystem::CreateFileReaderEx(const TCHAR* Filename, EReadFlags ReadFlags, IFileHandle** OutHandle) const
	{
		IFileHandle* Handle = FPlatformFileManager::Get().GetPlatformFile().OpenWrite(Filename, true, true);
		if (OutHandle)
		{
			*OutHandle = Handle;
		}

		if (!Handle)
		{
			if (static_cast<uint32>(ReadFlags) & FILEREAD_NoFail)
			{
				GLog->Logf(ELogVerbosity::Type::Fatal, TEXT("CreateFileReaderEx: Failed to read file : %s"), Filename);
			}
			return nullptr;
		}
		Handle->Seek(0); // Manually move the position to the beginning because Append moved it to EOF.
		return MakeUnique<FArchiveFileReaderGeneric>(Handle, Filename, Handle->Size(), PLATFORM_FILE_READER_BUFFER_SIZE, static_cast<uint32>(ReadFlags));
	}

	TUniquePtr<FArchive> FFileSystem::CreateFileWriter(const TCHAR* Filename, EWriteFlags WriteFlags) const
	{
		return TUniquePtr<FArchive>(FileManager.CreateFileWriter(Filename, static_cast<uint32>(WriteFlags)));
	}

	bool FFileSystem::LoadFileToString(const TCHAR* Filename, FString& Contents) const
	{
		return FFileHelper::LoadFileToString(Contents, Filename);
	}

	bool FFileSystem::SaveStringToFile(const TCHAR* Filename, const FString& Contents) const
	{
		return FFileHelper::SaveStringToFile(Contents, Filename);
	}

	bool FFileSystem::LoadFileToArray(const TCHAR* Filename, TArray<uint8>& Contents) const
	{
		return FFileHelper::LoadFileToArray(Contents, Filename);
	}

	bool FFileSystem::SaveArrayToFile(const TCHAR* Filename, const TArray<uint8>& Contents) const
	{
		return FFileHelper::SaveArrayToFile(Contents, Filename);
	}

	bool FFileSystem::DeleteFile(const TCHAR* Filename) const
	{
		if (FileExists(Filename))
		{
			SetReadOnly(Filename, false);
			return PlatformFile.DeleteFile(Filename);
		}
		return true;
	}

	bool FFileSystem::MoveFile(const TCHAR* FileDest, const TCHAR* FileSource, bool bDoNotRetryOrError) const
	{
		return FileManager.Move(FileDest, FileSource, true, true, true, bDoNotRetryOrError);
	}

	bool FFileSystem::CopyFile(const TCHAR* FileDest, const TCHAR* FileSource) const
	{
		return FileManager.Copy(FileDest, FileSource, true, true, true) == COPY_OK;
	}

	bool FFileSystem::FileExists(const TCHAR* Filename) const
	{
#if PLATFORM_MAC || PLATFORM_LINUX
		struct stat StatBuf = {0};
		if (lstat(TCHAR_TO_UTF8(Filename), &StatBuf) != -1)
		{
			return S_ISREG(StatBuf.st_mode) || S_ISLNK(StatBuf.st_mode);
		}
		return false;
#else
		return FileManager.FileExists(Filename);
#endif
	}

	void FFileSystem::FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension) const
	{
		PlatformFile.FindFiles(FoundFiles, Directory, FileExtension);
	}

	void FFileSystem::FindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension, bool bFollowSymlinks) const
	{
		FDirectoryEnumerator DirectoryEnumerator(PlatformFile, FileExtension, bFollowSymlinks);
		PlatformFile.IterateDirectory(Directory, DirectoryEnumerator);
		DirectoryEnumerator.GetFiles(FoundFiles);
	}

	void FFileSystem::ParallelFindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension, EAsyncExecution AsyncExecution, bool bFollowSymlinks) const
	{
		FParallelDirectoryEnumerator DirectoryEnumerator(PlatformFile, FileExtension, AsyncExecution, bFollowSymlinks);
		PlatformFile.IterateDirectory(Directory, DirectoryEnumerator);
		DirectoryEnumerator.GetFiles(FoundFiles);
	}

	FFileStatData FFileSystem::GetStatData(const TCHAR* FilenameOrDirectory) const
	{
		return PlatformFile.GetStatData(FilenameOrDirectory);
	}

	int64 FFileSystem::GetAllowedBytesToWriteThrottledStorage(const TCHAR* DestinationPath) const
	{
		return PlatformFile.GetAllowedBytesToWriteThrottledStorage(DestinationPath);
	}

	IFileSystem* FFileSystemFactory::Create()
	{
		return new FFileSystem();
	}
}