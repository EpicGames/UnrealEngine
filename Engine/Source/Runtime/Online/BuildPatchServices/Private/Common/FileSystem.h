// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Async.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/UniquePtr.h"

namespace BuildPatchServices
{
	enum class EAttributeFlags : uint32
	{
		// Value for no attributes.
		None        = 0,
		// Whether the file exists on the storage volume.
		Exists      = 1 << 0,
		// Whether the file is readonly.
		ReadOnly    = 1 << 1,
		// Whether the file is compressed.
		Compressed  = 1 << 2,
		// Whether the file is executable.
		Executable  = 1 << 3
	};
	ENUM_CLASS_FLAGS(EAttributeFlags);

	enum class EWriteFlags : uint32
	{
		None                = 0,
		NoFail              = 1 << 0,
		NoReplaceExisting   = 1 << 1,
		EvenIfReadOnly      = 1 << 2,
		Append              = 1 << 3,
		AllowRead           = 1 << 4
	};
	ENUM_CLASS_FLAGS(EWriteFlags);

	enum class EReadFlags : uint32
	{
		None                = 0,
		NoFail              = 1 << 0,
		Silent              = 1 << 1,
		AllowWrite          = 1 << 2
	};
	ENUM_CLASS_FLAGS(EReadFlags);

	/**
	 * A static helper function to parallelise retrieving file sizes for performance on large sets of files.
	 * @param FilesToCheck    The set of files to lookup the filesize for
	 * @param RootDirectory   The root directory of the files. Empty string can be passed if FilesToCheck has full file paths.
	 * @return a map of string from FilesToCheck to size on disk. INDEX_NONE if the file does not exist.
	 */
	TMap<FString, int64> ParallelGetFileSizes(const TSet<FString>& FilesToCheck, const FString& RootDirectory);

	/**
	 * The File System class is used for classes which require file access. It wraps Core IFileManager, and IPlatformFile. Also provides additional
	 * functionality missing from these classes at the time of writing.
	 * Using this wrapper allows dependants to be easily testable.
	 */
	class IFileSystem
	{
	public:
		virtual ~IFileSystem() = default;

		/**
		 * Checks whether a directory exists.
		 * @param DirectoryPath     The directory to check.
		 * @return true if directory exists.
		 */
		virtual bool DirectoryExists(const TCHAR* DirectoryPath) const = 0;

		/**
		 * Create a directory path.
		 * @param DirectoryPath     The directory path to create.
		 * @return true if successful.
		 */
		virtual bool MakeDirectory(const TCHAR* DirectoryPath) const = 0;

		/**
		 * Delete a directory path including all containing files and folders.
		 * @param DirectoryPath     The directory path to delete.
		 * @return true if successful.
		 */
		virtual bool DeleteDirectoryRecursively(const TCHAR* DirectoryPath) const = 0;

		/**
		 * Get the size of a file.
		 * @param Filename          The filename for the request.
		 * @param FileSize          Receives the file size in bytes or INDEX_NONE if the file didn't exist.
		 * @return true if successful.
		 */
		virtual bool GetFileSize(const TCHAR* Filename, int64& FileSize) const = 0;

		/**
		 * Get the attributes for a file.
		 * @param Filename          The filename for the request.
		 * @param Attributes        Receives the attribute flags if successful.
		 * @return true if successful.
		 */
		virtual bool GetAttributes(const TCHAR* Filename, EAttributeFlags& Attributes) const = 0;

		/**
		 * Get modification time of a file.
		 * @param Filename          The filename for the request.
		 * @param TimeStamp         Receives the time stamp if successful.
		 * @return true if successful.
		 */
		virtual bool GetTimeStamp(const TCHAR* Filename, FDateTime& TimeStamp) const = 0;

		/**
		 * Set whether the file is readonly.
		 * @param Filename          The filename for the request.
		 * @param bIsReadOnly       The state to set.
		 * @return true if successful.
		 */
		virtual bool SetReadOnly(const TCHAR* Filename, bool bIsReadOnly) const = 0;

		/**
		 * Set whether the file is compressed.
		 * @param Filename          The filename for the request.
		 * @param bIsCompressed     The state to set.
		 * @return true if successful.
		 */
		virtual bool SetCompressed(const TCHAR* Filename, bool bIsCompressed) const = 0;

		/**
		 * Set whether the file is executable.
		 * @param Filename          The filename for the request.
		 * @param bIsExecutable     The state to set.
		 * @return true if successful.
		 */
		virtual bool SetExecutable(const TCHAR* Filename, bool bIsExecutable) const = 0;

		/**
		 * Sets the size of a file, can increase or decrease, and will also create a file that does not exist..
		 * @param Filename          The filename for the request.
		 * @param FileSize          The desired size in bytes.
		 * @return true if successful.
		 */
		virtual bool SetFileSize(const TCHAR* Filename, int64 FileSize) const = 0;

		/**
		 * Creates file reader archive.
		 * @param Filename          The filename for the request.
		 * @param ReadFlags         The file open flags.
		 * @return unique pointer to the created archive, invalid if failed to open the file.
		 */
		virtual TUniquePtr<FArchive> CreateFileReader(const TCHAR* Filename, EReadFlags ReadFlags = EReadFlags::None) const = 0;

		/**
		 * Creates file reader archive.
		 * @param Filename          The filename for the request.
		 * @param ReadFlags         The file open flags.
		 * @param OutHandle			Pointer to the file handler used by the archive. The returned FArchive owns the instance of OutHandle and controls its lifetime.
		 * @return unique pointer to the created archive, invalid if failed to open the file.
		 */
		virtual TUniquePtr<FArchive> CreateFileReaderEx(const TCHAR* Filename, EReadFlags ReadFlags, IFileHandle** OutHandle) const = 0;

		/**
		 * Creates file writer archive.
		 * @param Filename          The filename for the request.
		 * @param WriteFlags        The file open flags.
		 * @return unique pointer to the created archive, invalid if failed to open the file.
		 */
		virtual TUniquePtr<FArchive> CreateFileWriter(const TCHAR* Filename, EWriteFlags WriteFlags = EWriteFlags::None) const = 0;

		/**
		 * Load a file on disk into string.
		 * @param Filename          The filename for the request.
		 * @param Contents          The contents of the file if successful.
		 * @return true if successful.
		 */
		virtual bool LoadFileToString(const TCHAR* Filename, FString& Contents) const = 0;

		/**
		 * Save a string to file on disk.
		 * @param Filename          The filename for the request.
		 * @param Contents          The string to save.
		 * @return true if successful.
		 */
		virtual bool SaveStringToFile(const TCHAR* Filename, const FString& Contents) const = 0;

		/**
		 * Load a file on disk into a byte array.
		 * @param Filename          The filename for the request.
		 * @param Contents          The contents of the file if successful.
		 * @return true if successful.
		 */
		virtual bool LoadFileToArray(const TCHAR* Filename, TArray<uint8>& Contents) const = 0;

		/**
		 * Save a byte array to file on disk.
		 * @param Filename          The filename for the request.
		 * @param Contents          The string to save.
		 * @return true if successful.
		 */
		virtual bool SaveArrayToFile(const TCHAR* Filename, const TArray<uint8>& Contents) const = 0;

		/**
		 * Delete a file.
		 * @param Filename          The file to delete.
		 * @return true if the file was deleted or did not exist.
		 */
		virtual bool DeleteFile(const TCHAR* Filename) const = 0;

		/**
		 * Move or rename a file.
		 * @param FileDest          The destination file path.
		 * @param FileSource        The source file path.
		 * @param bDoNotRetryOrError Do not try to repeat failed move operation. 
		 * @return true if the file was moved/renamed successfully.
		 */
		virtual bool MoveFile(const TCHAR* FileDest, const TCHAR* FileSource, bool bDoNotRetryOrError = true) const = 0;

		/**
		 * Copy a file.
		 * @param FileDest          The destination file path.
		 * @param FileSource        The source file path.
		 * @return true if the file was copied successfully.
		 */
		virtual bool CopyFile(const TCHAR* FileDest, const TCHAR* FileSource) const = 0;

		/**
		 * Checks whether a file exists.
		 * @param Filename          The file to check.
		 * @return true if file exists.
		 */
		virtual bool FileExists(const TCHAR* Filename) const = 0;

		/**
		 * Finds all the files within the given directory, with optional file extension filter.
		 * @param FoundFiles        Receives the files that matched the optional FileExtension filter, or all files if none was specified.
		 * @param Directory         The directory to iterate the contents of.
		 * @param FileExtension     The extension to filter by, in the form of TEXT(".ext"). If null or empty string, all files are found.
		 */
		virtual void FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr) const = 0;

		/**
		 * Finds all the files within the given directory tree, with optional file extension filter.
		 * @param FoundFiles        Receives the files that matched the optional FileExtension filter, or all files if none was specified.
		 * @param Directory         The directory to iterate the contents of. This function explores subdirectories.
		 * @param FileExtension     The extension to filter by, in the form of TEXT(".ext"). If null or empty string, all files are found.
		 * @param bFollowSymlinks   Whether symlinks to directories should be followed and enumerated, or added as a file.
		 */
		virtual void FindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr, bool bFollowSymlinks = true) const = 0;

		/**
		 * Finds all the files within the given directory tree, with optional file extension filter. This version places each directory found
		 * on separate threads to improve large enumeration times.
		 * @param FoundFiles        Receives the files that matched the optional FileExtension filter, or all files if none was specified.
		 * @param Directory         The directory to iterate the contents of. This function explores subdirectories.
		 * @param FileExtension     The extension to filter by, in the form of TEXT(".ext"). If null or empty string, all files are found.
		 * @param AsyncExecution    The threaded execution method to use, i.e. on Task Graph, Thread Pool, or a new thread per directory.
		 * @param bFollowSymlinks   Whether symlinks to directories should be followed and enumerated, or added as a file.
		 */
		virtual void ParallelFindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr, EAsyncExecution AsyncExecution = EAsyncExecution::ThreadPool, bool bFollowSymlinks = true) const = 0;

		/**
		 * Return the stat data for the given file or directory. Check the FFileStatData::bIsValid member before using the returned data.
		 * @param Filename          The filename or directory for the request.
		 * @return resulted stat data of the requested file or dir.
		 */
		virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) const = 0;

		/**
		 * Returns the number of bytes that are currently allowed to be written to throttled write storage (if the platform
		 * has such restrictions)
		 * @param DestinationPath	If specified, the file system can optionally take into account the destination of
		 *							the file to determine the current limit
		 * @returns					The number of bytes that are allowed to be written to write throttled storage.
		 *							If there is no limit, INT64_MAX is returned
		 */
		virtual int64 GetAllowedBytesToWriteThrottledStorage(const TCHAR* DestinationPath = nullptr) const = 0;
	};

	/**
	 * A factory for creating an IFileSystem instance.
	 */
	class FFileSystemFactory
	{
	public:
		/**
		 * Creates an implementation which wraps use of IFileManager, and implements additional functionality.
		 * @return the new IFileSystem instance created.
		 */
		static IFileSystem* Create();
	};
}