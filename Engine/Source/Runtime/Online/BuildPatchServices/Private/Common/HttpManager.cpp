// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/HttpManager.h"
#include "HttpModule.h"

namespace BuildPatchServices
{
	class FBPSHttpManager
		: public IHttpManager
	{
	public:
		FBPSHttpManager();
		~FBPSHttpManager();

		// IHttpManager interface begin.
		virtual TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateRequest() override;
		// IHttpManager interface end.

	private:
		FHttpModule& HttpModule;
	};

	FBPSHttpManager::FBPSHttpManager()
		: HttpModule(FHttpModule::Get())
	{
	}

	FBPSHttpManager::~FBPSHttpManager()
	{
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FBPSHttpManager::CreateRequest()
	{
		return HttpModule.CreateRequest();
	}

	IHttpManager* FHttpManagerFactory::Create()
	{
		return new FBPSHttpManager();
	}
}