// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IHttpRequest;

namespace BuildPatchServices
{
	/**
	 * The HTTP manager is used for classes which require HTTP access. Using this wrapper
	 * allows those classes to be easily testable.
	 */
	class IHttpManager
	{
	public:
		virtual ~IHttpManager() = default;

		/**
		 * Instantiates a new Http request for the current platform
		 *
		 * @return new Http request instance
		 */
		virtual TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateRequest() = 0;
	};

	/**
	 * A factory for creating an IHttpManager instance.
	 */
	class FHttpManagerFactory
	{
	public:
		/**
		 * Creates an implementation which wraps use of FHttpModule.
		 * @return the new IHttpManager instance created.
		 */
		static IHttpManager* Create();
	};
}
