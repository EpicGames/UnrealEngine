// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/SpeedRecorder.h"
#include "Algo/Sort.h"
#include "Containers/Ticker.h"
#include "Containers/Queue.h"
#include "Math/UnitConversion.h"
#include "Common/StatsCollector.h"
#include "Stats/Stats.h"

namespace BuildPatchServices
{
	ISpeedRecorder::FRecord::FRecord()
		: CyclesStart(0)
		, CyclesEnd(0)
		, Size(0)
	{
	}

	struct FPeakSpeed
	{
	public:
		FPeakSpeed(const uint64& CyclesConfidenceFactor, const uint64& SizeConfidenceFactor, const TArray<ISpeedRecorder::FRecord>& Records);
		void Update();
		double Get() const;

	private:
		const uint64 CyclesConfidenceFactor;
		const uint64 SizeConfidenceFactor;
		const TArray<ISpeedRecorder::FRecord>& Records;
		bool bIsConfident;
		double PeakSpeed;
	};

	FPeakSpeed::FPeakSpeed(const uint64& InCyclesConfidenceFactor, const uint64& InSizeConfidenceFactor, const TArray<ISpeedRecorder::FRecord>& InRecords)
		: CyclesConfidenceFactor(InCyclesConfidenceFactor)
		, SizeConfidenceFactor(InSizeConfidenceFactor)
		, Records(InRecords)
		, bIsConfident(false)
		, PeakSpeed(0)
	{
	}

	void FPeakSpeed::Update()
	{
		// Process records for peak.
		uint64 NewPeakStart = TNumericLimits<uint64>::Max();
		uint64 NewPeakRangeCycles = 0;
		uint64 NewPeakRangeSize = 0;
		bool bNewConfident = false;
		for (int32 RecordIdx = Records.Num() - 1; RecordIdx >= 0 && !bNewConfident; --RecordIdx)
		{
			const ISpeedRecorder::FRecord& Record = Records[RecordIdx];
			// Do we have some time to count.
			if (NewPeakStart > Record.CyclesStart)
			{
				// Don't count time overlap.
				NewPeakRangeCycles += FMath::Min(NewPeakStart, Record.CyclesEnd) - Record.CyclesStart;
				NewPeakStart = Record.CyclesStart;
			}
			// Count size.
			NewPeakRangeSize += Record.Size;
			// Are we within confidence ranges?
			if (NewPeakRangeCycles >= CyclesConfidenceFactor && NewPeakRangeSize >= SizeConfidenceFactor)
			{
				bNewConfident = true;
			}
		}
		// If we are not fully confident then we just calculated the full average in the loop and so can use that as the peak
		// until we get a confident value which will then only be allowed to rise.
		if (bNewConfident || !bIsConfident)
		{
			const double NewSpeed = NewPeakRangeSize / FStatsCollector::CyclesToSeconds(NewPeakRangeCycles);
			if (NewSpeed > PeakSpeed || !bIsConfident)
			{
				PeakSpeed = NewSpeed;
				bIsConfident = bNewConfident;
			}
		}
	}

	double FPeakSpeed::Get() const
	{
		return PeakSpeed;
	}

	class FSpeedRecorder
		: public ISpeedRecorder
		, public FTSTickerObjectBase
	{
	public:
		FSpeedRecorder();
		virtual ~FSpeedRecorder();

		// ISpeedRecorder interface begin.
		virtual void AddRecord(const FRecord& Record);
		virtual double GetAverageSpeed(float Seconds, bool bIncludeInactiveTime) const;
		virtual double GetPeakSpeed() const;
		// ISpeedRecorder interface end.

		// FTSTickerObjectBase interface begin.
		virtual bool Tick(float DeltaTime);
		// FTSTickerObjectBase interface end.

	private:
		void PutRecordsInTemp(float OverSeconds) const;

	private:
		const uint64 CyclesConfidenceFactor;
		const uint64 SizeConfidenceFactor;
		TQueue<ISpeedRecorder::FRecord, EQueueMode::Mpsc> RecordsQueue;
		TArray<ISpeedRecorder::FRecord> Records;
		mutable TArray<ISpeedRecorder::FRecord> Temp;

		// Tracking peak speed.
		FPeakSpeed PeakSpeed;
	};

	FSpeedRecorder::FSpeedRecorder()
		: CyclesConfidenceFactor(FStatsCollector::SecondsToCycles(5.0))
		, SizeConfidenceFactor(FUnitConversion::Convert(10, EUnit::Megabytes, EUnit::Bytes))
		, RecordsQueue()
		, Records()
		, Temp()
		, PeakSpeed(CyclesConfidenceFactor, SizeConfidenceFactor, Records)
	{
		checkSlow(IsInGameThread());
	}

	FSpeedRecorder::~FSpeedRecorder()
	{
		checkSlow(IsInGameThread());
	}

	void FSpeedRecorder::AddRecord(const FRecord& Record)
	{
		RecordsQueue.Enqueue(Record);
	}

	double FSpeedRecorder::GetAverageSpeed(float Seconds, bool bIncludeInactiveTime) const
	{
		checkSlow(IsInGameThread());
		// Fill Temp with correct range data.
		PutRecordsInTemp(Seconds);
		// Calculate from Temp collection.
		uint64 TotalCycles = 0;
		uint64 TotalSize = 0;
		uint64 RecordCyclesEnd = 0;
		for (const FRecord& Record : Temp)
		{
			// Do we have some time to count.
			if (RecordCyclesEnd < Record.CyclesEnd)
			{
				// Don't count time overlap.
				TotalCycles += Record.CyclesEnd - FMath::Max(Record.CyclesStart, RecordCyclesEnd);
				RecordCyclesEnd = Record.CyclesEnd;
			}
			TotalSize += Record.Size;
		}
		double SecondsDivisor = bIncludeInactiveTime ? Seconds : FStatsCollector::CyclesToSeconds(TotalCycles);
		return TotalCycles > 0 ? TotalSize / SecondsDivisor : 0.0;
	}

	double FSpeedRecorder::GetPeakSpeed() const
	{
		return PeakSpeed.Get();
	}

	bool FSpeedRecorder::Tick(float DeltaTime)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSpeedRecorder_Tick);

		// Pull in queued records.
		const int32 StartIdx = Records.Num();
		do { Records.AddUninitialized(); }
		while (RecordsQueue.Dequeue(Records.Last()));
		Records.Pop(EAllowShrinking::No);
		const int32 EndIdx = Records.Num() - 1;
		// If we pulled more data.
		if (EndIdx >= StartIdx)
		{
			// Sort.
			Algo::SortBy(Records, &FRecord::CyclesStart);
			// Update peak.
			PeakSpeed.Update();
		}
		return true;
	}

	void FSpeedRecorder::PutRecordsInTemp(float OverSeconds) const
	{
		Temp.Reset();
		if (Records.Num())
		{
			const uint64 OverCycles = FStatsCollector::SecondsToCycles(OverSeconds);
			const uint64 RangeEnd = FStatsCollector::GetCycles();
			const uint64 RangeBegin = RangeEnd > OverCycles ? RangeEnd - OverCycles : 0;
			const int32 LastIdxInRange = Records.FindLastByPredicate([&RangeBegin](const FRecord& Entry) { return Entry.CyclesEnd <= RangeBegin; }) + 1;
			for (int32 RecordIdx = LastIdxInRange; RecordIdx >= 0 && RecordIdx < Records.Num(); ++RecordIdx)
			{
				const FRecord& Record = Records[RecordIdx];
				checkSlow(Record.CyclesEnd >= Record.CyclesStart);
				checkSlow(Record.CyclesEnd >= RangeBegin);
				if (Record.CyclesStart >= RangeBegin)
				{
					Temp.Add(Record);
				}
				else
				{
					// Interpolate
					Temp.Add(Record);
					FRecord& Last = Temp.Last();
					Last.CyclesStart = RangeBegin;
					if (Record.CyclesEnd == Record.CyclesStart)
					{
						Last.Size = 0;
					}
					else
					{
						Last.Size *= (Last.CyclesEnd - Last.CyclesStart) / (Record.CyclesEnd - Record.CyclesStart);
					}
				}
			}
		}
	}

	BuildPatchServices::ISpeedRecorder* FSpeedRecorderFactory::Create()
	{
		return new FSpeedRecorder();
	}
}
