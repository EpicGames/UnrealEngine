// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace BuildPatchServices
{
	/**
	 * A helper implementation for recording unit speeds.
	 */
	class ISpeedRecorder
	{
	public:
		/**
		 * A struct for holding a single activity record.
		 */
		struct FRecord
		{
		public:
			FRecord();

		public:
			uint64 CyclesStart;
			uint64 CyclesEnd;
			uint64 Size;
		};

	public:
		virtual ~ISpeedRecorder() = default;

		/**
		 * Adds a record to the recorder.
		 * @param Record    The record to add.
		 */
		virtual void AddRecord(const FRecord& Record) = 0;

		/**
		 * Get the current average speed achieved over the last X seconds.
		 * @param Seconds   The time in seconds to take the reading over. Use TNumericLimits<float>::Max() to get the average over all readings.
		 * @param bIncludeInactiveTime   If True speed will be averaged over the entire Seconds period. Default is to only average over periods of active download within the Seconds period.
		 * @return the average speed over the past given seconds.
		 */
		virtual double GetAverageSpeed(float Seconds, bool bIncludeInactiveTime = false) const = 0;

		/**
		 * Get the peak speed achieved so far.
		 * @return the peak speed achieved so far.
		 */
		virtual double GetPeakSpeed() const = 0;
	};

	/**
	 * A factory for creating an ISpeedRecorder instance.
	 */
	class FSpeedRecorderFactory
	{
	public:
		static ISpeedRecorder* Create();
	};
}
