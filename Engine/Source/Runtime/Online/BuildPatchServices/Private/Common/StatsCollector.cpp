// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/StatsCollector.h"
#include "HAL/PlatformTime.h"
#include "Misc/ScopeLock.h"
#include "Misc/OutputDeviceRedirector.h"

namespace BuildPatchServices
{
	const double ToPercentage = 10000;
	const double FromPercentage = 1 / ToPercentage;

	uint64 FStatsCollector::GetCycles()
	{
		return FPlatformTime::Cycles64();
	}

	double FStatsCollector::GetSeconds()
	{
		return CyclesToSeconds(GetCycles());
	}

	double FStatsCollector::CyclesToSeconds(uint64 Cycles)
	{
		return FPlatformTime::GetSecondsPerCycle64() * double(Cycles);
	}

	uint64 FStatsCollector::SecondsToCycles(double Seconds)
	{
		const long double Result = Seconds / FPlatformTime::GetSecondsPerCycle64();
		// We upper clamp to cover cases where Seconds is too high to represent the resulting cycles as uint64.
		if (Result >= double(TNumericLimits<uint64>::Max()))
		{
			return TNumericLimits<uint64>::Max();
		}
		else
		{
			return Result;
		}
	}

	void FStatsCollector::AccumulateTimeBegin(uint64& TempValue)
	{
		TempValue = FStatsCollector::GetCycles();
	}

	void FStatsCollector::AccumulateTimeEnd(volatile FAtomicValue* Stat, uint64& TempValue)
	{
		FPlatformAtomics::InterlockedAdd(Stat, FAtomicValue(FStatsCollector::GetCycles() - TempValue));
	}

	void FStatsCollector::Accumulate(volatile FAtomicValue* Stat, int64 Amount)
	{
		FPlatformAtomics::InterlockedAdd(Stat, FAtomicValue(Amount));
	}

	void FStatsCollector::Set(volatile FAtomicValue* Stat, int64 Value)
	{
		FPlatformAtomics::InterlockedExchange(Stat, FAtomicValue(Value));
	}

	void FStatsCollector::SetAsPercentage(volatile FAtomicValue* Stat, double Value)
	{
		FPlatformAtomics::InterlockedExchange(Stat, FAtomicValue(Value * ToPercentage));
	}

	double FStatsCollector::GetAsPercentage(volatile FAtomicValue* Stat)
	{
		return *Stat * FromPercentage;
	}

	class FStatsCollectorImpl
		: public FStatsCollector
	{
	public:
		FStatsCollectorImpl();
		virtual ~FStatsCollectorImpl();

		virtual volatile FAtomicValue* CreateStat(const FString& Name, EStatFormat Type, FAtomicValue InitialValue = 0) override;
		virtual bool LogStats(float TimeBetweenLogs = 0.0f) override;

	private:
		FCriticalSection DataCS;
		TMap<FString, volatile FAtomicValue*> AddedStats;
		TMap<FAtomicValue*, FString> AtomicNameMap;
		TMap<FAtomicValue*, EStatFormat> AtomicFormatMap;
		uint64 LastLogged;
		int32 LongestName;
		FNumberFormattingOptions PercentageFormattingOptions;
	};

	FStatsCollectorImpl::FStatsCollectorImpl()
		: LastLogged(FStatsCollector::GetCycles())
		, LongestName(0)
	{
		PercentageFormattingOptions.MinimumFractionalDigits = 2;
		PercentageFormattingOptions.MaximumFractionalDigits = 2;
	}

	FStatsCollectorImpl::~FStatsCollectorImpl()
	{
		FScopeLock ScopeLock(&DataCS);
		for (auto& Stat : AddedStats)
		{
			delete Stat.Value;
		}
		AddedStats.Empty();
		AtomicNameMap.Empty();
		AtomicFormatMap.Empty();
	}

	volatile FStatsCollector::FAtomicValue* FStatsCollectorImpl::CreateStat(const FString& Name, EStatFormat Type, FAtomicValue InitialValue)
	{
		FScopeLock ScopeLock(&DataCS);
		if (AddedStats.Contains(Name) == false)
		{
			volatile FAtomicValue* NewInteger = new volatile FAtomicValue(InitialValue);
			AddedStats.Add(Name, NewInteger);
			AtomicNameMap.Add((FAtomicValue*)NewInteger, Name + TEXT(": "));
			AtomicFormatMap.Add((FAtomicValue*)NewInteger, Type);
			LongestName = FMath::Max<uint32>(LongestName, Name.Len() + 2);
			for (auto& Stat : AtomicNameMap)
			{
				while (Stat.Value.Len() < LongestName)
				{
					Stat.Value += TEXT(" ");
				}
			}
		}
		return AddedStats[Name];
	}

	bool FStatsCollectorImpl::LogStats(float TimeBetweenLogs)
	{
		const uint64 Cycles = FStatsCollector::GetCycles();
		if (FStatsCollector::CyclesToSeconds(Cycles - LastLogged) >= TimeBetweenLogs)
		{
			LastLogged = Cycles;
			FScopeLock ScopeLock(&DataCS);
			GLog->Log(TEXT("/-------- FStatsCollector Log ---------------------"));
			for (const auto& Stat : AddedStats)
			{
				FAtomicValue* NameLookup = (FAtomicValue*)Stat.Value;
				switch (AtomicFormatMap[NameLookup])
				{
				case EStatFormat::Timer:
					GLog->Logf(TEXT("| %s%s"), *AtomicNameMap[NameLookup], *FPlatformTime::PrettyTime(FStatsCollector::CyclesToSeconds(*Stat.Value)));
					break;
				case EStatFormat::DataSize:
					GLog->Logf(TEXT("| %s%s, %s"), *AtomicNameMap[NameLookup], *FText::AsMemory(*Stat.Value, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(*Stat.Value, EMemoryUnitStandard::IEC).ToString());
					break;
				case EStatFormat::DataSpeed:
					GLog->Logf(TEXT("| %s%s/s, %s/s"), *AtomicNameMap[NameLookup], *FText::AsMemory(*Stat.Value, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(*Stat.Value, EMemoryUnitStandard::IEC).ToString());
					break;
				case EStatFormat::Percentage:
					GLog->Logf(TEXT("| %s%s"), *AtomicNameMap[NameLookup], *FText::AsPercent(GetAsPercentage(Stat.Value), &PercentageFormattingOptions).ToString());
					break;
				default:
					GLog->Logf(TEXT("| %s%lld"), *AtomicNameMap[NameLookup], *Stat.Value);
					break;
				}
			}
			GLog->Log(TEXT("\\--------------------------------------------------"));
			return true;
		}
		return false;
	}

	FStatsCollector* FStatsCollectorFactory::Create()
	{
		return new FStatsCollectorImpl();
	}
}
