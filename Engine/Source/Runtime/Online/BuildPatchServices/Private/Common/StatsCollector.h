// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace BuildPatchServices
{
	enum class EStatFormat : uint8
	{
		// Using the AccumulateTimeBegin and End functions, measured in cycles
		Timer,
		// Value measured in Bytes
		DataSize,
		// Value measured in Bytes per second
		DataSpeed,
		// Uses percentage printing, the returned stat should only be used with the percentage helpers
		Percentage,
		// Generic int64 output
		Value
	};

	class FStatsCollector
	{
	public:
		typedef int64 FAtomicValue;

	public:
		virtual ~FStatsCollector() = default;
		virtual volatile FAtomicValue* CreateStat(const FString& Name, EStatFormat Type, FAtomicValue InitialValue = 0) = 0;
		virtual bool LogStats(float TimeBetweenLogs = 0.0f) = 0;

	public:
		static uint64 GetCycles();
		static double GetSeconds();
		static double CyclesToSeconds(uint64 Cycles);
		static uint64 SecondsToCycles(double Seconds);
		static void AccumulateTimeBegin(uint64& TempValue);
		static void AccumulateTimeEnd(volatile FAtomicValue* Stat, uint64& TempValue);
		static void Accumulate(volatile FAtomicValue* Stat, int64 Amount);
		static void Set(volatile FAtomicValue* Stat, int64 Value);
		static void SetAsPercentage(volatile FAtomicValue* Stat, double Value);
		static double GetAsPercentage(volatile FAtomicValue* Stat);
	};

	class FStatsCollectorFactory
	{
	public:
		static FStatsCollector* Create();
	};

	class FStatsScopedCounter
	{
	public:
		FStatsScopedCounter(volatile FStatsCollector::FAtomicValue* InStat)
			: Stat(InStat)
		{
			FStatsCollector::Accumulate(Stat, 1);
		}
		~FStatsScopedCounter()
		{
			FStatsCollector::Accumulate(Stat, -1);
		}

	private:
		volatile FStatsCollector::FAtomicValue* Stat;
	};

	class FStatsScopedTimer
	{
	public:
		FStatsScopedTimer(volatile FStatsCollector::FAtomicValue* InStat)
			: Stat(InStat)
		{
			FStatsCollector::AccumulateTimeBegin(TempTime);
		}
		~FStatsScopedTimer()
		{
			FStatsCollector::AccumulateTimeEnd(Stat, TempTime);
		}

	private:
		uint64 TempTime;
		volatile FStatsCollector::FAtomicValue* Stat;
	};

	class FStatsParallelScopeTimer
	{
	public:
		FStatsParallelScopeTimer(volatile FStatsCollector::FAtomicValue* InStaticTempValue, volatile FStatsCollector::FAtomicValue* InTimerStat, volatile FStatsCollector::FAtomicValue* InCounterStat)
			: TempTime(InStaticTempValue)
			, TimerStat(InTimerStat)
			, CounterStat(InCounterStat)
		{
			FStatsCollector::FAtomicValue OldValue = FPlatformAtomics::InterlockedAdd(CounterStat, 1);
			if (OldValue == 0)
			{
				FPlatformAtomics::InterlockedExchange(TempTime, FStatsCollector::FAtomicValue(FStatsCollector::GetCycles()));
			}
		}
		~FStatsParallelScopeTimer()
		{
			FStatsCollector::FAtomicValue CurrTempTime = *TempTime;
			FStatsCollector::FAtomicValue OldValue = FPlatformAtomics::InterlockedAdd(CounterStat, -1);
			if (OldValue == 1)
			{
				FPlatformAtomics::InterlockedAdd(TimerStat, FStatsCollector::FAtomicValue(FStatsCollector::GetCycles() - CurrTempTime));
			}
		}
		FStatsCollector::FAtomicValue GetCurrentTime() const
		{
			return (*TimerStat) + ((*CounterStat) > 0 ? FStatsCollector::FAtomicValue(FStatsCollector::GetCycles() - (*TempTime)) : 0);
		}

	private:
		volatile FStatsCollector::FAtomicValue* TempTime;
		volatile FStatsCollector::FAtomicValue* TimerStat;
		volatile FStatsCollector::FAtomicValue* CounterStat;
	};
}
