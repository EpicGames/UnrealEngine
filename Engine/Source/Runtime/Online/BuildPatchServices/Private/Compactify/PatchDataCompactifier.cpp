// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compactify/PatchDataCompactifier.h"

#include "Algo/Accumulate.h"
#include "Algo/Partition.h"
#include "Algo/Transform.h"
#include "Async/Async.h"
#include "Common/FileSystem.h"
#include "Common/StatsCollector.h"
#include "Core/AsyncHelpers.h"
#include "Core/ProcessTimer.h"
#include "Enumeration/PatchDataEnumeration.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceRedirector.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDataCompactifier, Log, All);
DEFINE_LOG_CATEGORY(LogDataCompactifier);

namespace BuildPatchServices
{
	constexpr int MaxManifestHandlingThreads = 150;
	constexpr int MaxFileHandlingThreads = 500;

	struct FReferencedFileData
	{
	public:
		FString Name;
		int64 Size = 0;

		bool operator==(const FString& Rhs) const
		{
			return Name == Rhs;
		}
		bool operator==(const FReferencedFileData& Rhs) const
		{
			return Name == Rhs.Name;
		}
	};

	FORCEINLINE uint32 GetTypeHash(const FReferencedFileData& Data)
	{
		return GetTypeHash(Data.Name);
	}

	struct FManifestResult
	{
	public:
		FManifestResult() = default;

		FManifestResult(bool bInSuccess, int64 InFileSize, TSet<FReferencedFileData>&& InReferences)
			: bSuccess(bInSuccess)
			, FileSize(InFileSize)
			, References(MoveTemp(InReferences))
		{ }

	public:
		bool bSuccess = false;
		int64 FileSize = 0;
		TSet<FReferencedFileData> References;
	};

	struct FStatsTimeGuard
	{
	public:
		FStatsTimeGuard(FStatsCollector& StatsCollector, const TCHAR* Type)
		{
			TotalTime = StatsCollector.CreateStat(Type, EStatFormat::Timer);
		}

		~FStatsTimeGuard()
		{
			Refresh();
		}

		FStatsTimeGuard& Refresh()
		{
			FStatsCollector::Set(TotalTime, FStatsCollector::GetCycles() - StartTime);
			return *this;
		}

		FString GetAsString()
		{
			return FPlatformTime::PrettyTime(FStatsCollector::CyclesToSeconds(*TotalTime));
		}

	private:
		volatile int64* TotalTime;
		const uint64 StartTime = FStatsCollector::GetCycles();
	};

	struct FProcessingStats
	{
	public:
		FProcessingStats(FStatsCollector& StatsCollector, const TCHAR* Type)
		{
			Num = StatsCollector.CreateStat(FString::Printf(TEXT("Data Compactifier: %s Num"), Type), EStatFormat::Value);
			Bytes = StatsCollector.CreateStat(FString::Printf(TEXT("Data Compactifier: %s Bytes"), Type), EStatFormat::DataSize);
		}

		void Accumulate(uint64 ProcessedBytes)
		{
			Accumulate(1, ProcessedBytes);
		}

		void Accumulate(uint64 FilesNum, uint64 ProcessedBytes)
		{
			FStatsCollector::Accumulate(Num, FilesNum);
			FStatsCollector::Accumulate(Bytes, ProcessedBytes);
		}

		void Accumulate(const FProcessingStats& Stats)
		{
			Accumulate(*Stats.Num, *Stats.Bytes);
		}

	public:
		volatile FStatsCollector::FAtomicValue* Num;
		volatile FStatsCollector::FAtomicValue* Bytes;
	};	

	struct FDataFileResult
	{
	public:
		FDataFileResult() = default;

		FDataFileResult(bool bInIsOldEnough, bool bInIsRecognisedFileType, bool bInShouldDelete, int64 InFileSize)
			: bIsOldEnough(bInIsOldEnough)
			, bIsRecognisedFileType(bInIsRecognisedFileType)
			, bShouldDelete(bInShouldDelete)
			, FileSize(InFileSize)
		{ }

	public:
		bool bIsOldEnough = false;
		bool bIsRecognisedFileType = false;
		bool bShouldDelete = false;
		int64 FileSize = 0;
	};

	class FPatchDataCompactifier
		: public IPatchDataCompactifier
	{
	public:
		FPatchDataCompactifier(const FCompactifyConfiguration& InConfiguration);
		~FPatchDataCompactifier();

		// IPatchDataCompactifier interface begin.
		virtual bool Run() override;
		// IPatchDataCompactifier interface end.

	private:
		void LogStats(float TimeBetweenLogs = 0.f);
		TArray<FString> FindAllFiles();
		bool FindAllReferencedFiles(float StatsLoggerTimeSeconds, int32 FirstNonManifest, const TArray<FString>& AllFiles, FProcessingStats& ProcessedFilesStat, TSet<FReferencedFileData>& ReferencedFileSet);
		void DeleteFile(const FString& FilePath) const;
		bool IsPatchData(const FString& FilePath) const;

	private:
		const FCompactifyConfiguration Configuration;
		TUniquePtr<IFileSystem> FileSystem;
		TUniquePtr<FStatsCollector> StatsCollector{FStatsCollectorFactory::Create()};
		volatile int64* StatTotalTime = StatsCollector->CreateStat(TEXT("Data Compactifier: Total Time"), EStatFormat::Timer);
		uint64 StartCompactifierTime = FStatsCollector::GetCycles();
		volatile FStatsCollector::FAtomicValue* AllFilesProgress = StatsCollector->CreateStat(TEXT("Data Compactifier: All Files Progress"), EStatFormat::Percentage);
	};

	FPatchDataCompactifier::FPatchDataCompactifier(const FCompactifyConfiguration& InConfiguration)
		: Configuration(InConfiguration)
		, FileSystem(FFileSystemFactory::Create())
	{
	}

	FPatchDataCompactifier::~FPatchDataCompactifier()
	{
	}

	void FPatchDataCompactifier::LogStats(float TimeBetweenLogs)
	{
		FStatsCollector::Set(StatTotalTime, FStatsCollector::GetCycles() - StartCompactifierTime);
		if (StatsCollector->LogStats(TimeBetweenLogs))
		{
			GLog->Flush();
		}
	}

	TArray<FString> FPatchDataCompactifier::FindAllFiles()
	{
		FStatsTimeGuard FindFilesTotalTimeStat(*StatsCollector, TEXT("Data Compactifier: Find All Files Time"));
		TArray<FString> AllFiles;
		FileSystem->ParallelFindFilesRecursively(AllFiles, *Configuration.CloudDirectory, nullptr, EAsyncExecution::Thread);
		UE_LOGF(LogDataCompactifier, Log, "Found %d files in %ls", AllFiles.Num(), *FindFilesTotalTimeStat.Refresh().GetAsString());
		return AllFiles;
	}

	bool FPatchDataCompactifier::FindAllReferencedFiles(float StatsLoggerTimeSeconds, int32 FirstNonManifest, const TArray<FString>& AllFiles, FProcessingStats &ProcessedFilesStat, TSet<FReferencedFileData>& ReferencedFileSet)
	{
		// Track some statistics.
		FProcessingStats ManifestFilesStat(*StatsCollector, TEXT("Manifest Files"));
		FProcessingStats ReferencedFilesStat(*StatsCollector, TEXT("Referenced Files"));

		FStatsTimeGuard FindAllReferencedFilesTimeStat(*StatsCollector, TEXT("Data Compactifier: Handle Manifests Time"));

		bool bSuccess = true;
		FTaskScheduler<FManifestResult, MaxManifestHandlingThreads> ManifestTasks;
		for (int32 FileIdx = 0; FileIdx < FirstNonManifest; ++FileIdx)
		{
			const FString& Filename = AllFiles[FileIdx];
			auto TaskFunc = [&]()
			{
				bool bEnumerateSuccess = true;
				TSet<FReferencedFileData> ManifestReferences;
				FFileStatData FileStatData = FileSystem->GetStatData(*Filename);
				const bool bGotFileSize = FileStatData.FileSize >= 0;
				const bool bFileStillExists = FileStatData.bIsValid;
				if (bGotFileSize)
				{
					BuildPatchServices::FPatchDataEnumerationConfiguration EnumerationConfig;
					EnumerationConfig.InputFile = Filename;
					EnumerationConfig.bIncludeSizes = true;
					TUniquePtr<IPatchDataEnumeration> PatchDataEnumeration(FPatchDataEnumerationFactory::Create(EnumerationConfig));
					TArray<FString> ManifestReferencesArray;
					bEnumerateSuccess = PatchDataEnumeration->Run(ManifestReferencesArray);
					if (bEnumerateSuccess)
					{
						ManifestReferences.Reserve(ManifestReferencesArray.Num());
						for (FString& Elem : ManifestReferencesArray)
						{
							FReferencedFileData FileData;
							int32 BytesPos = Elem.Find("\t");
							if (BytesPos > 0)
							{
								FileData.Size = FCString::Atoi(*Elem + BytesPos + 1); // skip \t
								Elem.LeftInline(BytesPos);
							}
							else
							{
								UE_LOGF(LogDataCompactifier, Warning, "Could not find a size position of %ls. Assuming 0 bytes.", *Elem);
							}
							FileData.Name = Configuration.CloudDirectory / Elem;
							ManifestReferences.Add(MoveTemp(FileData));
						}
					}
				}
				else if (bFileStillExists)
				{
					UE_LOGF(LogDataCompactifier, Warning, "Could not determine size of %ls. Assuming 0 bytes.", *Filename);
					FileStatData.FileSize = 0;
				}
				else
				{
					UE_LOGF(LogDataCompactifier, Log, "File removed since enumeration %ls. Using 0 bytes.", *Filename);
					FileStatData.FileSize = 0;
				}
				return FManifestResult(bEnumerateSuccess, FileStatData.FileSize, MoveTemp(ManifestReferences));
			};
			auto CompleteFunc = [&](FManifestResult Result)
			{
				ManifestFilesStat.Accumulate(Result.FileSize);
				if (!Result.bSuccess)
				{
					bSuccess = false;
					UE_LOGF(LogDataCompactifier, Error, "Failed to extract references from %ls.", *Filename);
				}
				else
				{
					const int32 NumReferences = Result.References.Num();
					ReferencedFileSet.Append(MoveTemp(Result.References));
					UE_LOGF(LogDataCompactifier, Display, "Extracted %d references from %ls. Unioning with existing files, new count of %d.", NumReferences, *Filename, ReferencedFileSet.Num());
				}
			};

			ManifestTasks.ScheduleTask(MoveTemp(TaskFunc), MoveTemp(CompleteFunc));
			FindAllReferencedFilesTimeStat.Refresh();
			LogStats(StatsLoggerTimeSeconds);
		}
		ManifestTasks.WaitAll();

		uint64 FilesSize = Algo::Accumulate(ReferencedFileSet, uint64(0), [](uint64 Result, const FReferencedFileData& FileData) { return Result + FileData.Size; });
		ReferencedFilesStat.Accumulate(ReferencedFileSet.Num(), FilesSize);
		ProcessedFilesStat.Accumulate(ReferencedFilesStat);
		FStatsCollector::SetAsPercentage(AllFilesProgress, (double)FirstNonManifest / (double)AllFiles.Num());

		LogStats();
		UE_LOGF(LogDataCompactifier, Display, "Manifests handling completed in %ls. Found %u references in manifests.", *FindAllReferencedFilesTimeStat.Refresh().GetAsString(), ReferencedFileSet.Num());

		return bSuccess;
	}

	bool FPatchDataCompactifier::Run()
	{
		StartCompactifierTime = FStatsCollector::GetCycles();
		bool bSuccess = true;

		// We output filenames deleted if requested.
		TUniquePtr<FArchive> DeletedChunkOutput;
		if (!Configuration.DeletedChunkLogFile.IsEmpty())
		{
			DeletedChunkOutput = FileSystem->CreateFileWriter(*Configuration.DeletedChunkLogFile);
			if (!DeletedChunkOutput.IsValid())
			{
				UE_LOGF(LogDataCompactifier, Error, "Could not open output file for writing %ls.", *Configuration.DeletedChunkLogFile);
				bSuccess = false;
			}
		}

		if (bSuccess)
		{
			// We'll work out the date of the oldest unreferenced file we'll keep
			FDateTime Cutoff = FDateTime::UtcNow() - FTimespan::FromDays(Configuration.DataAgeThreshold);

			// List all files first, and then we'll work with the list.
			TArray<FString> AllFiles = FindAllFiles();
			if (AllFiles.Num() == 0)
			{
				UE_LOGF(LogDataCompactifier, Log, "Could not find any file in the cloud directory %ls.", *Configuration.CloudDirectory);
				return true;
			}

			volatile FStatsCollector::FAtomicValue* AllFilesNum = StatsCollector->CreateStat(TEXT("Data Compactifier: All Files Num"), EStatFormat::Value);
			FStatsCollector::Accumulate(AllFilesNum, AllFiles.Num());
			FStatsCollector::SetAsPercentage(AllFilesProgress, 0);

			// Filter for manifest files.
			int32 FirstNonManifest = Algo::Partition(AllFiles.GetData(), AllFiles.Num(), [&](const FString& Elem) { return Elem.EndsWith(TEXT(".manifest"), ESearchCase::IgnoreCase); });

			// If we don't have any manifest files, notify that we'll continue to delete all mature chunks.
			if (FirstNonManifest == 0)
			{
				UE_LOGF(LogDataCompactifier, Log, "Could not find any manifest files. Proceeding to delete all mature data.");
			}

			float StatsLoggerTimeSeconds = 60.0f;
			GConfig->GetFloat(TEXT("BuildPatchServices"), TEXT("StatsLoggerTimeSeconds"), StatsLoggerTimeSeconds, GEngineIni);

			// Handle all manifests first.
			FProcessingStats ProcessedFilesStat(*StatsCollector, TEXT("Processed Files"));
			TSet<FReferencedFileData> ReferencedFileSet;
			bSuccess = FindAllReferencedFiles(StatsLoggerTimeSeconds, FirstNonManifest, AllFiles, ProcessedFilesStat, ReferencedFileSet);
			if (bSuccess)
			{
				FProcessingStats UnreferencedFilesStat(*StatsCollector, TEXT("Unreferenced Files"));
				FProcessingStats SkippedFilesStat(*StatsCollector, TEXT("Skipped Files"));
				FProcessingStats NonPatchFilesStat(*StatsCollector, TEXT("Unknown Files"));
				FProcessingStats DeletedFilesStat(*StatsCollector, TEXT("Deleted Files"));

				FStatsTimeGuard HandleAllCloudFilesTimeStat(*StatsCollector, TEXT("Data Compactifier: Handle Cloud Files Time"));

				// Now handle all CloudDir files.
				FTaskScheduler<FDataFileResult, MaxFileHandlingThreads> DataFileTasks;
				UE_LOGF(LogDataCompactifier, Log, "Using %d queue depth for file handling operations.", MaxFileHandlingThreads);
				for (int32 FileIdx = FirstNonManifest; FileIdx < AllFiles.Num(); ++FileIdx)
				{
					const FString& Filename = AllFiles[FileIdx];
					if (ReferencedFileSet.ContainsByHash(GetTypeHash(Filename), Filename))
					{
						continue;
					}
					auto TaskFunc = [&]()
					{
						FFileStatData FileStatData = FileSystem->GetStatData(*Filename);
						const bool bGotFileSize = FileStatData.FileSize >= 0;
						const bool bFileStillExists = FileStatData.bIsValid;
						if (bGotFileSize)
						{
							bool bIsOldEnough = FileStatData.ModificationTime < Cutoff;
							bool bIsRecognisedFileType = IsPatchData(Filename);
							bool bShouldDelete = bIsOldEnough && bIsRecognisedFileType;
							if (bShouldDelete)
							{
								DeleteFile(Filename);
							}
							return FDataFileResult(bIsOldEnough, bIsRecognisedFileType, bShouldDelete, FileStatData.FileSize);
						}
						else if (bFileStillExists)
						{
							UE_LOGF(LogDataCompactifier, Warning, "Could not determine size of %ls. Assuming 0 bytes.", *Filename);
						}
						else
						{
							UE_LOGF(LogDataCompactifier, Log, "File removed since enumeration %ls. Using 0 bytes.", *Filename);
						}
						return FDataFileResult();
					};
					auto CompleteFunc = [&](FDataFileResult Result)
					{
						ProcessedFilesStat.Accumulate(Result.FileSize);
						UnreferencedFilesStat.Accumulate(Result.FileSize);
						if (!Result.bIsOldEnough)
						{
							SkippedFilesStat.Accumulate(Result.FileSize);
						}
						else if (!Result.bIsRecognisedFileType)
						{
							NonPatchFilesStat.Accumulate(Result.FileSize);
						}
						else if (Result.bShouldDelete)
						{
							DeletedFilesStat.Accumulate(Result.FileSize);
							if (DeletedChunkOutput.IsValid())
							{
								FString OutputLine = Filename + TEXT("\r\n");
								FTCHARToUTF8 UTF8String(*OutputLine);
								DeletedChunkOutput->Serialize((UTF8CHAR*)UTF8String.Get(), UTF8String.Length() * sizeof(UTF8CHAR));
							}
							UE_LOGF(LogDataCompactifier, Log, "Deprecated data %ls%ls.", *Filename, Configuration.bRunPreview ? TEXT("") : TEXT(" deleted"));
						}
					};

					DataFileTasks.ScheduleTask(MoveTemp(TaskFunc), MoveTemp(CompleteFunc));
					HandleAllCloudFilesTimeStat.Refresh();
					FStatsCollector::SetAsPercentage(AllFilesProgress, (double)FileIdx / (double)AllFiles.Num());
					LogStats(StatsLoggerTimeSeconds);
				}
				DataFileTasks.WaitAll();
				FStatsCollector::SetAsPercentage(AllFilesProgress, 1);
				UE_LOGF(LogDataCompactifier, Display, "Cloud files handling completed in %ls.", *HandleAllCloudFilesTimeStat.Refresh().GetAsString());
				LogStats();
			}
		}

		return bSuccess;
	}

	void FPatchDataCompactifier::DeleteFile(const FString& FilePath) const
	{
		if (!Configuration.bRunPreview)
		{
			FileSystem->DeleteFile(*FilePath);
		}
	}

	bool FPatchDataCompactifier::IsPatchData(const FString& FilePath) const
	{
		static const TCHAR* ChunkFile = TEXT(".chunk");
		static const TCHAR* DeltaFile = TEXT(".delta");
		static const TCHAR* LegacyFile = TEXT(".file");
		return FilePath.EndsWith(ChunkFile) || FilePath.EndsWith(DeltaFile) || FilePath.EndsWith(LegacyFile);
	}

	IPatchDataCompactifier* FPatchDataCompactifierFactory::Create(const FCompactifyConfiguration& Configuration)
	{
		return new FPatchDataCompactifier(Configuration);
	}
}
