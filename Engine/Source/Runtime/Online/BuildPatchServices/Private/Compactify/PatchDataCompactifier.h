// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "BuildPatchSettings.h"

namespace BuildPatchServices
{
	class IPatchDataCompactifier
	{
	public:
		virtual ~IPatchDataCompactifier() = default;
		virtual bool Run() = 0;
	};

	class FPatchDataCompactifierFactory
	{
	public:
		static IPatchDataCompactifier* Create(const FCompactifyConfiguration& Configuration);
	};
}
