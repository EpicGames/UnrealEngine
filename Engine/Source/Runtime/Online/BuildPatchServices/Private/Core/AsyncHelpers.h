// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Async/Async.h"
#include "Containers/Queue.h"
#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSafeCounter64.h"

namespace BuildPatchServices
{
	/**
	 * Helper functions for wrapping async functionality.
	 */
	namespace AsyncHelpers
	{
		template<typename ResultType, typename... Args>
		static TFunction<void()> MakePromiseKeeper(const TSharedRef<TPromise<ResultType>, ESPMode::ThreadSafe>& Promise, const TFunction<ResultType(Args...)>& Function, Args... FuncArgs)
		{
			return [Promise, Function, FuncArgs...]()
			{
				Promise->SetValue(Function(FuncArgs...));
			};
		}

		template<typename... Args>
		static TFunction<void()> MakePromiseKeeper(const TSharedRef<TPromise<void>, ESPMode::ThreadSafe>& Promise, const TFunction<void(Args...)>& Function, Args... FuncArgs)
		{
			return [Promise, Function, FuncArgs...]()
			{
				Function(FuncArgs...);
				Promise->SetValue();
			};
		}

		template<typename ResultType, typename... Args>
		static TFuture<ResultType> ExecuteOnGameThread(const TFunction<ResultType(Args...)>& Function, Args... FuncArgs)
		{
			TSharedRef<TPromise<ResultType>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<ResultType>());
			TFunction<void()> PromiseKeeper = MakePromiseKeeper(Promise, Function, FuncArgs...);
			if (!IsInGameThread())
			{
				AsyncTask(ENamedThreads::GameThread, MoveTemp(PromiseKeeper));
			}
			else
			{
				PromiseKeeper();
			}
			return Promise->GetFuture();
		}

		template<typename ResultType>
		static TFuture<ResultType> ExecuteOnGameThread(const TFunction<ResultType()>& Function)
		{
			TSharedRef<TPromise<ResultType>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<ResultType>());
			TFunction<void()> PromiseKeeper = MakePromiseKeeper(Promise, Function);
			if (!IsInGameThread())
			{
				AsyncTask(ENamedThreads::GameThread, MoveTemp(PromiseKeeper));
			}
			else
			{
				PromiseKeeper();
			}
			return Promise->GetFuture();
		}

		template<typename ResultType>
		static TFuture<ResultType> ExecuteOnCustomThread(const TFunction<ResultType()>& Function, TQueue<TFunction<void()>, EQueueMode::Spsc>& CustomThreadQueue)
		{
			TSharedRef<TPromise<ResultType>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<ResultType>());
			TFunction<void()> PromiseKeeper = MakePromiseKeeper(Promise, Function);
			if (!IsInGameThread())
			{
				CustomThreadQueue.Enqueue(MoveTemp(PromiseKeeper));
			}
			else
			{
				PromiseKeeper();
			}
			return Promise->GetFuture();
		}
	}

	/**
	 * Additional atomic functionality
	 */
	namespace AsyncHelpers
	{
		/**
		 * LockFreePeak will set the destination value, to NewSample, if NewSample is higher.
		 * This works by spinning on InterlockedCompareExchange. For other usage examples see reference code in GenericPlatformAtomics.h
		 * @param PeakValue     The destination variable to set.
		 * @param NewSample     The sample to set with if higher.
		 */
		template<typename IntegerType>
		void LockFreePeak(volatile IntegerType* PeakValue, IntegerType NewSample)
		{
			IntegerType CurrentPeak;
			do
			{
				// Read the current value.
				CurrentPeak = *PeakValue;
			}
			// If the value was lower than the sample, try to update it to NewSample if the current value has not been set by another thread.
			// If the current value change since we read it, try again to see if our sample is still higher.
			while (CurrentPeak < NewSample && FPlatformAtomics::InterlockedCompareExchange(PeakValue, NewSample, CurrentPeak) != CurrentPeak);
		}
	}

	template<typename TResultType>
	struct TTaskSchedulerTraits
	{
		using FOnCompleteFunction = TFunction<void(TResultType)>;
	};

	template<>
	struct TTaskSchedulerTraits<void>
	{
		using FOnCompleteFunction = TFunction<void()>;
	};

	/**
	 * A helper class for scheduling a large amount of tasks on the thread pool while conserving event handles.
	 * The class itself should be called into entirely from one single thread only. For multiple threads with tasks, use
	 * separate FTaskScheduler instances.
	 */
	template<typename TResultType, int32 CompileTimeMaxTasks = 64>
	class FTaskScheduler
	{
	public:
		using FOnCompleteCb = typename TTaskSchedulerTraits<TResultType>::FOnCompleteFunction;

		FTaskScheduler() = default;

		/**
		* @param InRuntimeLimitation Runtime limitaion on a number of concurrent tasks
		*/
		explicit FTaskScheduler(int32 InRuntimeLimitation):
			FTaskScheduler(EAsyncExecution::ThreadPool, InRuntimeLimitation)
		{
		}

		/**
		* @param InRuntimeLimitation Runtime limitaion on a number of concurrent tasks 
		* @param InExecutionType	 The execution method to use, i.e. on Task Graph or in a separate thread.
		*/
		FTaskScheduler(EAsyncExecution InExecutionType, int32 InRuntimeLimitation = CompileTimeMaxTasks)
			: ExecutionType(InExecutionType)
			, MaxScheduledTasks(FMath::Clamp(InRuntimeLimitation, 1, CompileTimeMaxTasks))
		{
		}

		/**
		 * Adds a scheduled task to the queue. If the queue is full, will wait for a slot to become available, calling the OnComplete for
		 * the task being replaced.
		 * @param Task        The function to be ran asynchronously on the thread pool.
		 * @param OnComplete  The function to be called with completed task result.
		 *                    OnComplete will be executed from within ScheduleTask(..) or WaitAll().
		 */
		void ScheduleTask(TFunction<TResultType()>&& Task, FOnCompleteCb&& OnComplete = nullptr)
		{
			if (TaskFutures.Num() < MaxScheduledTasks)
			{
				// Space for new slot.
				TaskFutures.Add(Async(ExecutionType, MoveTemp(Task)));
				TaskCompleteFuncs.Add(MoveTemp(OnComplete));
			}
			else
			{
				// Find a finished slot, or complete the next from last search.
				int32 NewTaskIndex = SlotSearchCount;
				for (int32 i = 0; i < TaskFutures.Num(); i++)
				{
					const int32 TaskIdx = (SlotSearchCount + i) % TaskFutures.Num();
					if (TaskFutures[TaskIdx].IsReady())
					{
						NewTaskIndex = TaskIdx;
						break;
					}
				}
				SlotSearchCount = (SlotSearchCount + 1) % TaskFutures.Num();
				// Complete the slot we will be assigning to.
				CompleteTask(*this, NewTaskIndex);
				// Overwrite slot.
				TaskFutures[NewTaskIndex] = Async(ExecutionType, MoveTemp(Task));
				TaskCompleteFuncs[NewTaskIndex] = MoveTemp(OnComplete);
			}
		}

		/**
		 * Waits and completes all remaining tasks in the queue. Must always be called to ensure that no OnComplete calls are missed.
		 */
		void WaitAll()
		{
			for (int32 i = 0; i < TaskFutures.Num(); i++)
			{
				CompleteTask(*this, i);
			}
			TaskFutures.Empty();
			TaskCompleteFuncs.Empty();
		}

	private:

		template<typename TTaskResultType, int32 Tasks>
		void CompleteTask(FTaskScheduler<TTaskResultType, Tasks>& Scheduler, int32 Idx)
		{
			TTaskResultType Result = Scheduler.TaskFutures[Idx].Get();
			if (Scheduler.TaskCompleteFuncs[Idx])
			{
				Scheduler.TaskCompleteFuncs[Idx](MoveTemp(Result));
			}
		}

		template<int32 Tasks>
		void CompleteTask(FTaskScheduler<void, Tasks>& Scheduler, int32 Idx)
		{
			Scheduler.TaskFutures[Idx].Wait();
			if (Scheduler.TaskCompleteFuncs[Idx])
			{
				Scheduler.TaskCompleteFuncs[Idx]();
			}
		}

	private:

		EAsyncExecution ExecutionType = EAsyncExecution::ThreadPool;
		int32 MaxScheduledTasks = CompileTimeMaxTasks;
		TArray<TFuture<TResultType>, TFixedAllocator<CompileTimeMaxTasks>> TaskFutures;
		TArray<FOnCompleteCb, TFixedAllocator<CompileTimeMaxTasks>> TaskCompleteFuncs;
		int32 SlotSearchCount = 0;
	};

	constexpr int32 TaskCoresSchedulerThreadsLimitation = 128;

	template<typename ResultType>
	using FTaskCoresScheduler = FTaskScheduler<ResultType, TaskCoresSchedulerThreadsLimitation>;

	template<typename ResultType = void, int32 MaxThreads = TaskCoresSchedulerThreadsLimitation>
	static FTaskCoresScheduler<ResultType> CreateTaskSchedulerLimitedByCoresNumber(EAsyncExecution ExecutionType = EAsyncExecution::ThreadPool)
	{
		static const int32 NumberOfCores = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
		return FTaskScheduler<ResultType, MaxThreads>(ExecutionType, NumberOfCores - 1);
	}

	typedef FThreadSafeCounter64 FThreadSafeInt64;
	typedef FThreadSafeCounter FThreadSafeInt32;
}
