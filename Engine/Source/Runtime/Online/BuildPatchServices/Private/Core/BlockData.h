// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Templates/Tuple.h"
#include "Core/BlockRange.h"
#include "Core/BlockStructure.h"

namespace BuildPatchServices
{
	template<typename ElementType>
	struct TBlockData
	{
	private:
		struct FBlockToCopy
		{
		public:
			uint64 SortOffset;
			FBlockRange CopyBlockRange;
			const ElementType* DataPtr;
		};

	public:

		int32 GetDataCount() const
		{
			return Data.Num();
		}

		uint32 GetAllocatedSize() const
		{
			return Data.GetAllocatedSize();
		}

		void CopyTo(TArray<ElementType>& OutData, const FBlockStructure& Structure) const
		{
			// Fully intersects!
			check(BlockStructureHelpers::CountSize(DataStructure.Intersect(Structure)) == BlockStructureHelpers::CountSize(Structure));

			// Add data for each piece of the provided structure.
			const FBlockStructure LocalSpaceStructure = BlockStructureHelpers::SerializeIntersection(DataStructure, Structure);
			const FBlockEntry* LocalSpaceBlock = LocalSpaceStructure.GetHead();
			while (LocalSpaceBlock != nullptr)
			{
				OutData.Append(Data.GetData() + LocalSpaceBlock->GetOffset(), LocalSpaceBlock->GetSize());
				LocalSpaceBlock = LocalSpaceBlock->GetNext();
			}
		}

		void AddData(const FBlockStructure& NewStructure, const TArray<ElementType>& NewData)
		{
			AddData(NewStructure, NewData.GetData(), NewData.Num());
		}

		void AddData(const FBlockStructure& NewStructure, const ElementType* NewData, const int32 NewDataCount)
		{
			// No intersections!
			check(BlockStructureHelpers::CountSize(DataStructure.Intersect(NewStructure)) == 0);
			// Correct input
			check(BlockStructureHelpers::CountSize(NewStructure) == NewDataCount);

			// Construct the new array.
			const uint64 NewStructureFirst = NewStructure.GetHead()->GetOffset();
			const uint64 ThisStructureLast = DataStructure.GetTail() == nullptr ? 0 : DataStructure.GetTail()->GetOffset();
			const bool bAppendOnly = NewStructureFirst >= ThisStructureLast;
			if (bAppendOnly)
			{
				Data.Append(NewData, NewDataCount);
			}
			else
			{
				TArray<ElementType> CombinedData;
				CombinedData.Reserve(NewDataCount + Data.Num());
				TArray<FBlockToCopy> BlocksToCopy;
				const FBlockEntry* Block = DataStructure.GetHead();
				uint64 FirstByte = 0;
				while (Block != nullptr)
				{
					BlocksToCopy.Add(FBlockToCopy{Block->GetOffset(), FBlockRange::FromFirstAndSize(FirstByte, Block->GetSize()), Data.GetData()});
					FirstByte += Block->GetSize();
					Block = Block->GetNext();
				}
				Block = NewStructure.GetHead();
				FirstByte = 0;
				while (Block != nullptr)
				{
					BlocksToCopy.Add(FBlockToCopy{Block->GetOffset(), FBlockRange::FromFirstAndSize(FirstByte, Block->GetSize()), NewData});
					FirstByte += Block->GetSize();
					Block = Block->GetNext();
				}
				Algo::SortBy(BlocksToCopy, [] (const FBlockToCopy& BlockToCopy) { return BlockToCopy.SortOffset; });
				for (const FBlockToCopy& BlockToCopy : BlocksToCopy)
				{
					CombinedData.Append(BlockToCopy.DataPtr + BlockToCopy.CopyBlockRange.GetFirst(), BlockToCopy.CopyBlockRange.GetSize());
				}
				Data = MoveTemp(CombinedData);
			}
			DataStructure.Add(NewStructure);
		}

		void RemoveData(const FBlockStructure& Structure)
		{
			const FBlockStructure LocalSpaceStructure = BlockStructureHelpers::SerializeIntersection(DataStructure, Structure);

			{
				const uint64 StructureSize = BlockStructureHelpers::CountSize(Structure);
				const uint64 DataStructureSize = BlockStructureHelpers::CountSize(DataStructure);
				const uint64 LocalSpaceStructureSize = BlockStructureHelpers::CountSize(LocalSpaceStructure);
				const uint64 IntersectSize = BlockStructureHelpers::CountSize(DataStructure.Intersect(Structure));
				check(IntersectSize == StructureSize);
				check(LocalSpaceStructureSize == StructureSize);
				check(DataStructureSize == Data.Num());
			}

			// Remove from data buffer each piece of the provided structure.
			const FBlockEntry* LocalSpaceBlock = LocalSpaceStructure.GetTail();
			while (LocalSpaceBlock != nullptr)
			{
				Data.RemoveAt(LocalSpaceBlock->GetOffset(), LocalSpaceBlock->GetSize(), EAllowShrinking::No);
				LocalSpaceBlock = LocalSpaceBlock->GetPrevious();
			}
			DataStructure.Remove(Structure);
		}

	private:
		FBlockStructure DataStructure;
		TArray<ElementType> Data;
	};
}
