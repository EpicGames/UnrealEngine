// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"

namespace BuildPatchServices
{
	struct FBlockEntry;

	struct FBlockRange
	{
	public:
		uint64 GetFirst() const
		{
			checkf(Size > 0, TEXT("Using GetFirst for a 0 size range is invalid."));
			return First;
		}

		uint64 GetLast() const
		{
			checkf(Size > 0, TEXT("Using GetLast for a 0 size range is invalid."));
			return First + (Size - 1);
		}

		uint64 GetSize() const
		{
			return Size;
		}

		bool Overlaps(const FBlockRange& Other) const
		{
			return GetSize() > 0 && Other.GetSize() > 0 && GetFirst() <= Other.GetLast() && GetLast() >= Other.GetFirst();
		}

		bool Touches(const FBlockRange& Other) const
		{
			return GetSize() > 0 && Other.GetSize() > 0 && GetFirst() <= (Other.GetLast() + 1) && (GetLast() + 1) >= Other.GetFirst();
		}

		static FBlockRange FromFirstAndSize(uint64 InFirst, uint64 InSize)
		{
			FBlockRange BlockRange;
			BlockRange.First = InFirst;
			BlockRange.Size = InSize;
			checkf((BlockRange.GetSize() == 0) || (BlockRange.GetLast() >= BlockRange.GetFirst()), TEXT("Byte range has uint64 overflow."));
			return BlockRange;
		}

		static FBlockRange FromFirstAndLast(uint64 InFirst, uint64 InLast)
		{
			checkf(InFirst <= InLast, TEXT("Invalid args, first must <= last."));
			return FromFirstAndSize(InFirst, (InLast - InFirst) + 1);
		}

		static FBlockRange FromIntersection(const FBlockRange& RangeA, const FBlockRange& RangeB)
		{
			checkf(RangeA.Overlaps(RangeB), TEXT("Invalid args, ranges must overlap."));
			return FromFirstAndLast(FMath::Max<uint64>(RangeA.GetFirst(), RangeB.GetFirst()), FMath::Min<uint64>(RangeA.GetLast(), RangeB.GetLast()));
		}

		static FBlockRange FromMerge(const FBlockRange& RangeA, const FBlockRange& RangeB)
		{
			checkf(RangeA.Touches(RangeB), TEXT("Invalid args, ranges must overlap or touch."));
			return FromFirstAndLast(FMath::Min<uint64>(RangeA.GetFirst(), RangeB.GetFirst()), FMath::Max<uint64>(RangeA.GetLast(), RangeB.GetLast()));
		}

		friend bool operator==(const FBlockRange& Lhs, const FBlockRange& Rhs)
		{
			return Lhs.First == Rhs.First && Lhs.Size == Rhs.Size;
		}

	private:
		FBlockRange()
			: First(0)
			, Size(0)
		{
		}

	private:
		uint64 First;
		uint64 Size;
	};
}
