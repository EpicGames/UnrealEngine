// Copyright Epic Games, Inc. All Rights Reserved.
#include "Core/BlockStructure.h"

namespace BuildPatchServices
{
	FBlockEntry::FBlockEntry(uint64 InOffset, uint64 InSize)
		: Offset(InOffset)
		, Size(InSize)
		, Prev(nullptr)
		, Next(nullptr)
	{}

	FBlockEntry::~FBlockEntry()
	{
		Prev = nullptr;
		Next = nullptr;
	}

	void FBlockEntry::InsertBefore(FBlockEntry* NewEntry, FBlockEntry** Head)
	{
		NewEntry->Prev = Prev;
		NewEntry->Next = this;
		if (Prev != nullptr)
		{
			Prev->Next = NewEntry;
		}
		else
		{
			(*Head) = NewEntry;
		}
		Prev = NewEntry;
	}

	void FBlockEntry::InsertAfter(FBlockEntry* NewEntry, FBlockEntry** Tail)
	{
		NewEntry->Prev = this;
		NewEntry->Next = Next;
		if (Next != nullptr)
		{
			Next->Prev = NewEntry;
		}
		else
		{
			(*Tail) = NewEntry;
		}
		Next = NewEntry;
	}

	void FBlockEntry::Unlink(FBlockEntry** Head, FBlockEntry** Tail)
	{
		if (Prev != nullptr)
		{
			Prev->Next = Next;
		}
		else
		{
			(*Head) = Next;
		}
		if (Next != nullptr)
		{
			Next->Prev = Prev;
		}
		else
		{
			(*Tail) = Prev;
		}
		delete this;
	}

	void FBlockEntry::Merge(uint64 InOffset, uint64 InSize)
	{
		checkSlow(InOffset <= (Offset + Size));
		checkSlow((InOffset + InSize) >= Offset);
		uint64 NewOffset = FMath::Min(Offset, InOffset);
		uint64 NewEnd = FMath::Max(InOffset + InSize, Offset + Size);
		Offset = NewOffset;
		Size = NewEnd - NewOffset;
	}

	void FBlockEntry::Chop(uint64 InOffset, uint64 InSize, FBlockEntry** Head, FBlockEntry** Tail)
	{
		const uint64 End = Offset + Size;
		const uint64 InEnd = InOffset + InSize;
		checkSlow(InOffset < End);
		checkSlow(InEnd > Offset);

		// Complete overlap.
		if (InOffset <= Offset && InEnd >= End)
		{
			Unlink(Head, Tail);
		}
		// Mid overlap
		else if (InOffset > Offset && InEnd < End)
		{
			InsertAfter(new FBlockEntry(InEnd, End - InEnd), Tail);
			Size = InOffset - Offset;
		}
		// Trim start
		else if (InOffset <= Offset && InEnd < End)
		{
			Offset = InEnd;
			Size = End - Offset;
		}
		// Trim end
		else
		{
			// If the others are false this one must be true.
			checkSlow(InOffset > Offset && InEnd >= End);
			Size = InOffset - Offset;
		}
	}

	uint64 FBlockEntry::GetOffset() const
	{
		return Offset;
	}

	uint64 FBlockEntry::GetSize() const
	{
		return Size;
	}

	const FBlockEntry* FBlockEntry::GetNext() const
	{
		return Next;
	} 

	const FBlockEntry* FBlockEntry::GetPrevious() const
	{
		return Prev;
	}

	FBlockStructure::FBlockStructure()
		: Head(nullptr)
		, Tail(nullptr)
	{}

	FBlockStructure::FBlockStructure(uint64 Offset, uint64 Size)
		: FBlockStructure()
	{
		Add(Offset, Size);
	}

	FBlockStructure::FBlockStructure(const FBlockStructure& CopyFrom)
		: Head(nullptr)
		, Tail(nullptr)
	{
		Add(CopyFrom, ESearchDir::FromEnd);
	}

	FBlockStructure::FBlockStructure(FBlockStructure&& MoveFrom)
		: Head(MoveFrom.Head)
		, Tail(MoveFrom.Tail)
	{
		MoveFrom.Head = nullptr;
		MoveFrom.Tail = nullptr;
	}

	FBlockStructure::~FBlockStructure()
	{
		Empty();
	}

	FBlockStructure& FBlockStructure::operator=(const FBlockStructure& CopyFrom)
	{
		Empty();
		Add(CopyFrom, ESearchDir::FromEnd);
		return *this;
	}

	FBlockStructure& FBlockStructure::operator=(FBlockStructure&& MoveFrom)
	{
		Empty();
		Head = MoveFrom.Head;
		Tail = MoveFrom.Tail;
		MoveFrom.Head = nullptr;
		MoveFrom.Tail = nullptr;
		return *this;
	}

	const FBlockEntry* FBlockStructure::GetHead() const
	{
		return Head;
	}

	const FBlockEntry* FBlockStructure::GetTail() const
	{
		return Tail;
	}

	void FBlockStructure::Empty()
	{
		// Delete all entries, Tail can be used as temp
		while (Head != nullptr)
		{
			Tail = Head->Next;
			delete Head;
			Head = Tail;
		}
	}

	void FBlockStructure::Add(uint64 Offset, uint64 Size, ESearchDir::Type SearchDir /*= ESearchDir::FromStart*/)
	{
		if (Size > 0)
		{
			if (Head != nullptr)
			{
				if (SearchDir == ESearchDir::FromStart)
				{
					FBlockEntry* Entry = Head;
					while (Entry != nullptr)
					{
						// New block before this one?
						if (Entry->Offset > (Offset + Size))
						{
							Entry->InsertBefore(new FBlockEntry(Offset, Size), &Head);
							Entry = nullptr;
						}
						// New block after this one?
						else if (Offset > (Entry->Offset + Entry->Size))
						{
							Entry = Entry->Next;
							// New block is last?
							if (Entry == nullptr)
							{
								Tail->InsertAfter(new FBlockEntry(Offset, Size), &Tail);
							}
						}
						// New block overlaps?
						else
						{
							Entry->Merge(Offset, Size);
							CollectOverlaps(Entry, SearchDir);
							Entry = nullptr;
						}
					}
				}
				else
				{
					FBlockEntry* Entry = Tail;
					while (Entry != nullptr)
					{
						// New block after this one?
						if (Offset > (Entry->Offset + Entry->Size))
						{
							Entry->InsertAfter(new FBlockEntry(Offset, Size), &Tail);
							Entry = nullptr;
						}
						// New block before this one?
						else if (Entry->Offset > (Offset + Size))
						{
							Entry = Entry->Prev;
							// New block is first?
							if (Entry == nullptr)
							{
								Head->InsertBefore(new FBlockEntry(Offset, Size), &Head);
							}
						}
						// New block overlaps?
						else
						{
							Entry->Merge(Offset, Size);
							CollectOverlaps(Entry, SearchDir);
							Entry = nullptr;
						}
					}
				}
			}
			else
			{
				// If we are headless we have no data
				Head = new FBlockEntry(Offset, Size);
				Tail = Head;
			}
		}
	}

	void FBlockStructure::Add(const FBlockStructure& OtherStructure, ESearchDir::Type SearchDir /*= ESearchDir::FromStart*/)
	{
		const FBlockEntry* Block = OtherStructure.GetHead();
		while (Block != nullptr)
		{
			Add(Block->GetOffset(), Block->GetSize(), SearchDir);
			Block = Block->GetNext();
		}
	}

	void FBlockStructure::Add(const FBlockRange& BlockRange, ESearchDir::Type SearchDir /*= ESearchDir::FromStart*/)
	{
		Add(BlockRange.GetFirst(), BlockRange.GetSize(), SearchDir);
	}

	void FBlockStructure::Remove(uint64 Offset, uint64 Size, ESearchDir::Type SearchDir /*= ESearchDir::FromStart*/)
	{
		if (Size > 0)
		{
			if (Head != nullptr)
			{
				FBlockEntry* LastTest = nullptr;
				FBlockEntry* Entry = (SearchDir == ESearchDir::FromStart) ? Head : Tail;
				while (Entry != nullptr)
				{
					// New block before this one?
					if (Entry->Offset >= (Offset + Size))
					{
						if (LastTest == Entry->Prev)
						{
							return;
						}
						LastTest = Entry;
						Entry = Entry->Prev;
					}
					// New block after this one?
					else if (Offset >= (Entry->Offset + Entry->Size))
					{
						if (LastTest == Entry->Next)
						{
							return;
						}
						LastTest = Entry;
						Entry = Entry->Next;
					}
					// New block overlaps?
					else
					{
						FBlockEntry* Next = (SearchDir == ESearchDir::FromStart) ? Entry->Next : Entry->Prev;
						Entry->Chop(Offset, Size, &Head, &Tail);
						Entry = Next;
					}
				}
			}
		}
	}

	void FBlockStructure::Remove(const FBlockStructure& OtherStructure, ESearchDir::Type SearchDir /*= ESearchDir::FromStart*/)
	{
		const FBlockEntry* Block = OtherStructure.GetHead();
		while (Block != nullptr)
		{
			Remove(Block->GetOffset(), Block->GetSize(), SearchDir);
			Block = Block->GetNext();
		}
	}

	void FBlockStructure::Remove(const FBlockRange& BlockRange, ESearchDir::Type SearchDir /*= ESearchDir::FromStart*/)
	{
		Remove(BlockRange.GetFirst(), BlockRange.GetSize(), SearchDir);
	}

	uint64 FBlockStructure::SelectSerialBytes(uint64 FirstByteIdx, uint64 Count, FBlockStructure& OutputStructure) const
	{
		uint64 StartByte = 0;
		uint64 EndByte = 0;
		uint64 OutputCount = 0;
		const FBlockEntry* InputBlock = Head;
		while (InputBlock != nullptr && OutputCount < Count)
		{
			EndByte += InputBlock->GetSize();
			if (EndByte > FirstByteIdx)
			{
				// Use block
				const uint64 SelectOffset = FirstByteIdx > StartByte ? FirstByteIdx - StartByte : 0;
				const uint64 SelectStart = InputBlock->GetOffset() + SelectOffset;
				const uint64 SelectSize = FMath::Min<uint64>(InputBlock->GetSize() - SelectOffset, Count - OutputCount);
				OutputStructure.Add(SelectStart, SelectSize, ESearchDir::FromEnd);
				OutputCount += SelectSize;
			}
			StartByte += InputBlock->GetSize();
			InputBlock = InputBlock->GetNext();
		}
		return OutputCount;
	}

	FBlockStructure FBlockStructure::Intersect(const FBlockStructure& OtherStructure) const
	{
		FBlockStructure Result;
		if (Head && OtherStructure.Head)
		{
			Result.Add(OtherStructure);
			const FBlockEntry* Block = Head;
			while (Block != nullptr)
			{
				uint64 Offset = Block->Prev ? Block->Prev->Offset + Block->Prev->Size : 0;
				uint64 Size = Block->Offset - Offset;
				Result.Remove(Offset, Size);
				Block = Block->Next;
			}
			uint64 EndA = Tail->Offset + Tail->Size;
			uint64 EndB = OtherStructure.Tail->Offset + OtherStructure.Tail->Size;
			if (EndA < EndB)
			{
				Result.Remove(EndA, EndB - EndA);
			}
		}
		return Result;
	}

	FString FBlockStructure::ToString(uint64 BlockCountLimit /*= 20*/) const
	{
		FString Output;
		if (Head != nullptr)
		{
			uint64 NumSkippedBlocks = 0;
			bool bIsFirst = true;
			const FBlockEntry* InputBlock = Head;
			while (InputBlock != nullptr)
			{
				if (BlockCountLimit > 0)
				{
					--BlockCountLimit;
					if (bIsFirst)
					{
						bIsFirst = false;
					}
					else
					{
						Output += TEXT("-");
					}
					Output += FString::Printf(TEXT("[%llu,%llu]"), InputBlock->GetOffset(), InputBlock->GetSize());
				}
				else
				{
					++NumSkippedBlocks;
				}
				InputBlock = InputBlock->GetNext();
			}
			if (NumSkippedBlocks > 0)
			{
				Output += FString::Printf(TEXT(".. %llu more"), NumSkippedBlocks);
			}
			Output += TEXT(".");
		}
		return Output;
	}

	void FBlockStructure::CollectOverlaps(FBlockEntry* From, ESearchDir::Type SearchDir)
	{
		if (SearchDir == ESearchDir::FromStart)
		{
			FBlockEntry* Entry = From->Next;
			while (Entry != nullptr)
			{
				checkSlow(Entry->Offset >= From->Offset);
				// Next block is mergeable?
				if (Entry->Offset <= (From->Offset + From->Size))
				{
					From->Merge(Entry->Offset, Entry->Size);
					Entry->Unlink(&Head, &Tail);
					Entry = From->Next;
				}
				else
				{
					Entry = nullptr;
				}
			}
		}
		else
		{
			FBlockEntry* Entry = From->Prev;
			while (Entry != nullptr)
			{
				checkSlow((Entry->Offset + Entry->Size) <= (From->Offset + From->Size));
				// Prev block is mergeable?
				if ((Entry->Offset + Entry->Size) >= From->Offset)
				{
					From->Merge(Entry->Offset, Entry->Size);
					Entry->Unlink(&Head, &Tail);
					Entry = From->Prev;
				}
				else
				{
					Entry = nullptr;
				}
			}
		}
	}

	uint64 BlockStructureHelpers::CountSize(const FBlockStructure& Structure)
	{
		uint64 Count = 0;
		const FBlockEntry* Block = Structure.GetHead();
		while (Block != nullptr)
		{
			Count += Block->GetSize();
			Block = Block->GetNext();
		}
		return Count;
	}

	bool BlockStructureHelpers::HasIntersection(const FBlockStructure& ByteStructure, const FBlockStructure& Intersection)
	{
		return ByteStructure.Intersect(Intersection).GetHead() != nullptr;
	}

	FBlockStructure BlockStructureHelpers::SerializeIntersection(const FBlockStructure& ByteStructure, const FBlockStructure& Intersection)
	{
		FBlockStructure SerialByteRanges;
		const FBlockStructure ActualIntersection = ByteStructure.Intersect(Intersection);
		const FBlockEntry* ByteBlock = ByteStructure.GetHead();
		const FBlockEntry* IntersectionBlock = ActualIntersection.GetHead();
		uint64 ByteCount = 0;
		while (ByteBlock != nullptr && IntersectionBlock != nullptr)
		{
			const FBlockRange& ByteBlockRange = ByteBlock->AsRange();
			const FBlockRange& IntersectionBlockRange = IntersectionBlock->AsRange();
			check(IntersectionBlockRange.GetFirst() >= ByteBlockRange.GetFirst());
			if (ByteBlockRange.Overlaps(IntersectionBlockRange))
			{
				// Add the overlap.
				const FBlockRange SerialByteRange = FBlockRange::FromFirstAndSize(ByteCount + (IntersectionBlockRange.GetFirst() - ByteBlockRange.GetFirst()), IntersectionBlockRange.GetSize());
				SerialByteRanges.Add(SerialByteRange);

				// Next intersection.
				IntersectionBlock = IntersectionBlock->GetNext();
			}
			else
			{
				// Count bytes.
				ByteCount += ByteBlockRange.GetSize();
				ByteBlock = ByteBlock->GetNext();
			}
		}
		return SerialByteRanges;
	}
}
