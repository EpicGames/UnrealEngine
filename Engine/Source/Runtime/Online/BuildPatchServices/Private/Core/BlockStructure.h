// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Core/BlockRange.h"

namespace BuildPatchServices
{
	struct FBlockEntry
	{
		friend class FBlockStructure;
	public:
		FBlockEntry(uint64 InOffset, uint64 InSize);
		~FBlockEntry();

		uint64 GetOffset() const;
		uint64 GetSize() const;
		const FBlockEntry* GetNext() const;
		const FBlockEntry* GetPrevious() const;
		FBlockRange AsRange() const { return FBlockRange::FromFirstAndSize(Offset, Size); }

	private:
		void InsertBefore(FBlockEntry* NewEntry, FBlockEntry** Head);
		void InsertAfter(FBlockEntry* NewEntry, FBlockEntry** Tail);
		void Unlink(FBlockEntry** Head, FBlockEntry** Tail);
		void Merge(uint64 InOffset, uint64 InSize);
		void Chop(uint64 InOffset, uint64 InSize, FBlockEntry** Head, FBlockEntry** Tail);

	private:
		uint64 Offset;
		uint64 Size;
		FBlockEntry* Prev;
		FBlockEntry* Next;
	};

	class FBlockStructure
	{
	public:
		FBlockStructure();
		FBlockStructure(uint64 Offset, uint64 Size);
		FBlockStructure(const FBlockStructure&);
		FBlockStructure(FBlockStructure&&);
		~FBlockStructure();
		FBlockStructure& operator=(const FBlockStructure&);
		FBlockStructure& operator=(FBlockStructure&&);

		const FBlockEntry* GetHead() const;
		const FBlockEntry* GetTail() const;

		/**
		 * Empty the structure of all blocks.
		 */
		void Empty();

		/**
		 * Add a block to this structure. Any overlap will be merged, growing existing blocks where necessary.
		 * @param   Offset      The offset of the block.
		 * @param   Size        The size of the block.
		 * @param   SearchDir   The direction in which to search, default FromStart. If you know which is faster then
		 *                      specify this explicitly.
		 */
		void Add(uint64 Offset, uint64 Size, ESearchDir::Type SearchDir = ESearchDir::FromStart);

		/**
		 * Add a block to this structure. Any overlap will be merged, growing existing blocks where necessary.
		 * @param   BlockRange  The range of the block.
		 * @param   SearchDir   The direction in which to search, default FromStart. If you know which is faster then
		 *                      specify this explicitly.
		 */
		void Add(const FBlockRange& BlockRange, ESearchDir::Type SearchDir = ESearchDir::FromStart);

		/**
		 * Add another structure to this structure. Any overlap will be merged, growing existing blocks where necessary.
		 * @param   OtherStructure      The other structure.
		 * @param   SearchDir           The direction in which to search, default FromStart. If you know which is faster then
		 *                              specify this explicitly.
		 */
		void Add(const FBlockStructure& OtherStructure, ESearchDir::Type SearchDir = ESearchDir::FromStart);

		/**
		 * Remove a block from this structure. Any overlap will shrink existing blocks, or remove where necessary.
		 * @param   Offset      The offset of the block.
		 * @param   Size        The size of the block.
		 * @param   SearchDir   The direction in which to search, default FromStart. If you know which is faster then
		 *                      specify this explicitly.
		 */
		void Remove(uint64 Offset, uint64 Size, ESearchDir::Type SearchDir = ESearchDir::FromStart);

		/**
		 * Remove a block from this structure. Any overlap will shrink existing blocks, or remove where necessary.
		 * @param   BlockRange  The range of the block.
		 * @param   SearchDir   The direction in which to search, default FromStart. If you know which is faster then
		 *                      specify this explicitly.
		 */
		void Remove(const FBlockRange& BlockRange, ESearchDir::Type SearchDir = ESearchDir::FromStart);

		/**
		 * Remove another structure from this structure. Any overlap will shrink existing blocks, or remove where necessary.
		 * @param   OtherStructure      The other structure.
		 * @param   SearchDir           The direction in which to search, default FromStart. If you know which is faster then
		 *                              specify this explicitly.
		 */
		void Remove(const FBlockStructure& OtherStructure, ESearchDir::Type SearchDir = ESearchDir::FromStart);

		/**
		 * Starting from the nth byte in the structure, not including gaps, select a number of bytes into the provided structure.
		 * e.g for the structure [Offset, Size] MyStructure: [ 0,10]-[20,10]-[40,10]
		 *     MyStructure.SelectSerialBytes(15, 10, SerialStruct);
		 *     would result in SerialStruct representing [25, 5]-[40, 5]
		 *
		 * @param   FirstByte           The first byte to select, zero indexed.
		 * @param   Count               The number of bytes to provide.
		 * @param   OutputStructure     The structure receiving the blocks.
		 * @return the number of bytes selected.
		 */
		uint64 SelectSerialBytes(uint64 FirstByte, uint64 Count, FBlockStructure& OutputStructure) const;

		/**
		 * Get the intersection of this block structure and another.
		 * e.g for the structures [Offset, Size] MyStructure: [ 0,10]-[20,10]-[40,10]
		 *                                and OtherStructure: [25,10]-[45,10]-[50,10]
		 *     Intersection = MyStructure.Intersect(OtherStructure);
		 *     would result in Intersection representing [25, 5]-[45, 5]
		 *
		 * @param   OtherStructure      The structure to intersect with.
		 * @return structure of blocks representing the overlap between this and other.
		 */
		FBlockStructure Intersect(const FBlockStructure& OtherStructure) const;

		/**
		 * Get a string representation of this block structure, with ability to limit the number of blocks to stringify.
		 * The string is formatted "[Offset,Size]-[Offset,Size].. %d more."
		 * e.g. "[0,10]-[20,10]-[40,10]."
		 *      "[0,10]-[20,10]-[40,10].. 300 more."
		 *
		 * @param   BlockCountLimit     The maximum number of blocks to include in the string.
		 * @return a string representation of this block structure.
		 */
		FString ToString(uint64 BlockCountLimit = 20) const;

	private:
		FBlockEntry* Head;
		FBlockEntry* Tail;
		void CollectOverlaps(FBlockEntry* First, ESearchDir::Type SearchDir);
	};

	namespace BlockStructureHelpers
	{
		uint64 CountSize(const FBlockStructure& Structure);
		bool HasIntersection(const FBlockStructure& ByteStructure, const FBlockStructure& Intersection);
		FBlockStructure SerializeIntersection(const FBlockStructure& ByteStructure, const FBlockStructure& Intersection);
	}
}
