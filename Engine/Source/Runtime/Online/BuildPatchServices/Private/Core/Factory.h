// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace BuildPatchServices
{
	/**
	 * Templated factory interface declaration.
	 * This allows a system to declare the factory it requires a consumer to provide.
	 * For code readability, you should only use well defined types. For example, declare a config struct, use other IClassDependancies, or typedef a generic tye to name it.
	 * Example use:
	 *	struct FClassConfig
	 *	{
	 *		uint32 RetryCount;
	 *		uint32 TimeoutSeconds;
	 *	};
	 * typedef TArray<FString> FUriListToLoad;
	 * typedef TFactory<IClass, const FClassConfig&, FUriListToLoad, IUriHandlerClass*> IMyClassFactory;
	 * A consumer would then implement the factory:
	 *	class FMyClassFactory : public IMyClassFactory
	 *	{
	 *		virtual IClass* Create(const FClassConfig& ClassConfig, FUriListToLoad UriListToLoad, IUriHandlerClass* UriHandlerClass) override
	 *		{
	 *			FClass* Class = new FClass(UriHandlerClass);
	 *			Class->SetRetires(ClassConfig.RetryCount);
	 *			Class->SetTimeout(ClassConfig.TimeoutSeconds);
	 *			Class->LoadAll(UriListToLoad);
	 *			return Class;
	 *		}
	 *	};
	 */
	template<typename Product, typename... DependencyTypes>
	class TFactory
	{
	public:
		virtual Product* Create(DependencyTypes... Dependencies) = 0;
	};
}
