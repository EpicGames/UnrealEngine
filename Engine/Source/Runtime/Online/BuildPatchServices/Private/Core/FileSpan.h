// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Misc/SecureHash.h"

namespace BuildPatchServices
{
	struct FFileSpan
	{
	public:
		FFileSpan(const FString& InFilename, uint64 InSize, uint64 InStartIdx, bool InIsUnixExecutable, const FString& InSymlinkTarget)
			: Filename(InFilename)
			, Size(InSize)
			, StartIdx(InStartIdx)
			, IsUnixExecutable(InIsUnixExecutable)
			, SymlinkTarget(InSymlinkTarget)
		{
		}

		FFileSpan(const FFileSpan& CopyFrom)
			: Filename(CopyFrom.Filename)
			, Size(CopyFrom.Size)
			, StartIdx(CopyFrom.StartIdx)
			, MD5Hash(CopyFrom.MD5Hash)
			, SHA1Hash(CopyFrom.SHA1Hash)
			, SHA256Hash(CopyFrom.SHA256Hash)
			, MIMEType(CopyFrom.MIMEType)
			, IsUnixExecutable(CopyFrom.IsUnixExecutable)
			, SymlinkTarget(CopyFrom.SymlinkTarget)
		{
		}

		FFileSpan(FFileSpan&& MoveFrom)
			: Filename(MoveTemp(MoveFrom.Filename))
			, Size(MoveTemp(MoveFrom.Size))
			, StartIdx(MoveTemp(MoveFrom.StartIdx))
			, MD5Hash(MoveTemp(MoveFrom.MD5Hash))
			, SHA1Hash(MoveTemp(MoveFrom.SHA1Hash))
			, SHA256Hash(MoveTemp(MoveFrom.SHA256Hash))
			, MIMEType(MoveTemp(MoveFrom.MIMEType))
			, IsUnixExecutable(MoveTemp(MoveFrom.IsUnixExecutable))
			, SymlinkTarget(MoveTemp(MoveFrom.SymlinkTarget))
		{
		}

		FFileSpan()
			: Size(0)
			, StartIdx(0)
			, IsUnixExecutable(false)
		{
		}

		FORCEINLINE FFileSpan& operator=(const FFileSpan& CopyFrom)
		{
			Filename = CopyFrom.Filename;
			Size = CopyFrom.Size;
			StartIdx = CopyFrom.StartIdx;
			MD5Hash = CopyFrom.MD5Hash;
			SHA1Hash = CopyFrom.SHA1Hash;
			SHA256Hash = CopyFrom.SHA256Hash;
			MIMEType = CopyFrom.MIMEType;
			IsUnixExecutable = CopyFrom.IsUnixExecutable;
			SymlinkTarget = CopyFrom.SymlinkTarget;
			return *this;
		}

		FORCEINLINE FFileSpan& operator=(FFileSpan&& MoveFrom)
		{
			Filename = MoveTemp(MoveFrom.Filename);
			Size = MoveTemp(MoveFrom.Size);
			StartIdx = MoveTemp(MoveFrom.StartIdx);
			MD5Hash = MoveTemp(MoveFrom.MD5Hash);
			SHA1Hash = MoveTemp(MoveFrom.SHA1Hash);
			SHA256Hash = MoveTemp(MoveFrom.SHA256Hash);
			MIMEType = MoveTemp(MoveFrom.MIMEType);
			IsUnixExecutable = MoveTemp(MoveFrom.IsUnixExecutable);
			SymlinkTarget = MoveTemp(MoveFrom.SymlinkTarget);
			return *this;
		}

	public:
		FString Filename;
		uint64 Size;
		uint64 StartIdx;
		FMD5Hash MD5Hash;
		FSHAHash SHA1Hash;
		FSHA256Signature SHA256Hash;
		FString MIMEType;
		bool IsUnixExecutable;
		FString SymlinkTarget;
	};
}
