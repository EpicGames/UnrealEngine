// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"

namespace BuildPatchServices
{
	/**
	 * A class that can be used to monitor the average value and standard deviation, without storing full history.
	 */
	class FMeanValue
	{
	public:

		FMeanValue()
			: ReliabilityCount(10)
			, Count(0)
			, Total(0)
			, TotalSqs(0)
		{ }

		FMeanValue(uint64 InReliabilityCount)
			: ReliabilityCount(InReliabilityCount)
			, Count(0)
			, Total(0)
			, TotalSqs(0)
		{ }

		void Reset()
		{
			Count = 0;
			Total = 0;
			TotalSqs = 0;
		}

		bool IsReliable() const
		{
			return Count > ReliabilityCount;
		}

		void GetValues(double& Mean, double& Std) const
		{
			Mean = GetMean();
			Std = GetStd(Mean);
		}

		void AddSample(double Sample)
		{
			Total += Sample;
			TotalSqs += Sample * Sample;
			++Count;
		}

	private:
		double GetMean() const
		{
			checkSlow(Count > 0);
			return Total / Count;
		}

		double GetStd(double Mean) const
		{
			return FMath::Sqrt((TotalSqs / Count) - (Mean * Mean));
		}

	private:
		const uint64 ReliabilityCount;
		uint64 Count;
		double Total;
		double TotalSqs;
	};
}
