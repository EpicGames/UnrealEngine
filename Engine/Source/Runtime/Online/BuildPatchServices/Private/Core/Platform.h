// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"

namespace BuildPatchServices
{
	/**
	 * Class for wrapping platform functionality.
	 * We implement this using a template class, so that the class with platform dependency can easily be provided
	 * an implementation, and can still be nicely tested.
	 * Under normal circumstances, use FPlatform, which is a typedef below, and/or FPlatformFactory::Create.
	 */
	class IPlatform
	{
	public:
		virtual ~IPlatform() = default;

		/**
		 * Executes a process as administrator, requesting elevation as necessary. This
		 * call blocks until the process has returned.
		 * @param URL             IN  The URL of the process to execute, typically the path to an executable file
		 * @param Params          IN  The params to pass to the invoked process
		 * @param OutReturnCode   OUT Will be set to the return code of the executed process
		 * @return true if the execution was successful.
		 */
		virtual bool ExecElevatedProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode) = 0;

		/**
		 * Sleep this thread for Seconds. 0.0 means release the current time slice to let other threads get some attention.
		 * @param Seconds         The time in seconds to sleep for.
		 */
		virtual void Sleep(float Seconds) = 0;

		/**
		 * @return The error value for last platform error that occurred.
		 */
		virtual uint32 GetLastError() = 0;
	};

	template <typename FPlatformProcessImpl, typename FPlatformMiscImpl>
	class TPlatform
		: public IPlatform
	{
	public:

		// IPlatform interface begin.
		virtual bool ExecElevatedProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode) override
		{
			return FPlatformProcessImpl::ExecElevatedProcess(URL, Params, OutReturnCode);
		}

		virtual void Sleep(float Seconds) override
		{
			FPlatformProcessImpl::SleepNoStats(Seconds);
		}

		virtual uint32 GetLastError() override
		{
			return FPlatformMiscImpl::GetLastError();
		}
		// IPlatform interface end.
	};

	// A typedef for the standard use platform type instantiation.
	typedef TPlatform<FPlatformProcess, FPlatformMisc> FPlatform;

	/**
	 * A factory for creating an IPlatform instance.
	 */
	class FPlatformFactory
	{
	public:
		/**
		 * Creates an implementation of IPlatform.
		 * @return the new IPlatform instance created.
		 */
		static IPlatform* Create()
		{
			return new FPlatform();
		}
	};
}
