// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/ScopeLock.h"
#include "Common/StatsCollector.h"

namespace BuildPatchServices
{
	/**
	 * Class for wrapping timing functionality with pause feature. The class is thread safe.
	 * We template the dependency for cycles so that the class can be nicely tested.
	 * Under normal circumstances, use FProcessTimer, which is a typedef below.
	 */
	template<typename FCyclesProvider, bool bThreadSafe = true>
	class TProcessTimer
	{
	public:
		/**
		 * Default constructor zeros members.
		 */
		TProcessTimer()
			: ThreadLock()
			, StartCycles(0)
			, Cycles(0)
			, bIsRunning(false)
			, bIsPaused(false)
		{
		}

		/**
		 * Gets the currently accumulated time in seconds, accounting for any time paused.
		 * @return the seconds currently accumulated by the timer.
		 */
		double GetSeconds()
		{
			if (bThreadSafe)
			{
				ThreadLock.Lock();
			}
			double Seconds = FCyclesProvider::CyclesToSeconds(Cycles);
			if (bIsRunning && !bIsPaused)
			{
				Seconds += FCyclesProvider::CyclesToSeconds(FCyclesProvider::GetCycles() - StartCycles);
			}
			if (bThreadSafe)
			{
				ThreadLock.Unlock();
			}
			return Seconds;
		}

		/**
		 * Start timing. Repeated calls before a Stop() are ignored.
		 */
		void Start()
		{
			if (bThreadSafe)
			{
				ThreadLock.Lock();
			}
			if (!bIsRunning)
			{
				bIsRunning = true;
				if (!bIsPaused)
				{
					StartCycles = FCyclesProvider::GetCycles();
				}
			}
			if (bThreadSafe)
			{
				ThreadLock.Unlock();
			}
		}

		/**
		 * Stop timing and accumulate the recording. Repeated calls before a Start() are ignored.
		 */
		void Stop()
		{
			if (bThreadSafe)
			{
				ThreadLock.Lock();
			}
			if (bIsRunning)
			{
				bIsRunning = false;
				if (!bIsPaused)
				{
					Cycles += FCyclesProvider::GetCycles() - StartCycles;
				}
			}
			if (bThreadSafe)
			{
				ThreadLock.Unlock();
			}
		}

		/**
		 * Set whether the process being timed is paused. Automatically adjusts timer results for paused durations.
		 * @param bPause    Whether the process is currently paused.
		 */
		void SetPause(bool bPause)
		{
			if (bThreadSafe)
			{
				ThreadLock.Lock();
			}
			if (bIsPaused != bPause)
			{
				bIsPaused = bPause;
				if (bIsPaused)
				{
					if (bIsRunning)
					{
						Cycles += FCyclesProvider::GetCycles() - StartCycles;
						StartCycles = 0;
					}
				}
				else
				{
					if (bIsRunning)
					{
						StartCycles = FCyclesProvider::GetCycles();
					}
				}
			}
			if (bThreadSafe)
			{
				ThreadLock.Unlock();
			}
		}

		void Reset()
		{
			if (bThreadSafe)
			{
				ThreadLock.Lock();
			}
			StartCycles = 0;
			Cycles = 0;
			bIsRunning = false;
			bIsPaused = false;
			if (bThreadSafe)
			{
				ThreadLock.Unlock();
			}
		}

	private:
		// Thread lock for multi-threaded use.
		FCriticalSection ThreadLock;
		// The cycle count when we started timing.
		uint64 StartCycles;
		// The total accumulated cycles between Start and Stop calls.
		uint64 Cycles;
		// Whether the timer is currently running.
		bool bIsRunning;
		// Whether the timer is in pause state.
		bool bIsPaused;
	};
	typedef TProcessTimer<class FStatsCollector> FProcessTimer;
}
