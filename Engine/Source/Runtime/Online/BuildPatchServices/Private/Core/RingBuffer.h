// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/SecureHash.h"

template< typename DataType >
class TRingBuffer
{
public:
	/**
	 * Constructor
	 */
	TRingBuffer(uint32 BufferDataSize);

	/**
	 * Default Destructor
	 */
	~TRingBuffer();

	/**
	 * Clears memory and indexes
	 */
	void Empty();

	/**
	 * Pushes a data word to the end of the FIFO. WILL OVERWRITE OLDEST if full.
	 * @param Val	The data word to push
	 */
	void Enqueue(const DataType& Val);

	/**
	 * Pushes a buffer of data words to the end of the FIFO. WILL OVERWRITE OLDEST if full.
	 * @param ValBuf		The buffer pointer
	 * @param BufLen		The length to copy
	 */
	void Enqueue(const DataType* ValBuf, const uint32& BufLen);

	/**
	 * Take the next data word from the FIFO buffer.
	 * @param OUT	ValOut	The variable to receive the data word
	 * @return	true if the buffer was not empty, false otherwise.
	 */
	bool Dequeue(DataType& ValOut);

	/**
	 * Take the next set of data words from the FIFO buffer.
	 * @param ValBuf		The buffer to receive the data
	 * @param BufLen		The number of words requested
	 * @return	the number of words actually copied
	 */
	uint32 Dequeue(DataType* ValBuf, const uint32& BufLen);

	/**
	 * Get the next data word from the FIFO buffer without removing it.
	 * @param OUT	ValOut	The variable to receive the data word
	 * @return	true if the buffer was not empty, false otherwise.
	 */
	bool Peek(DataType& ValOut) const;

	/**
	 * Get the next set of data words from the FIFO buffer without removing them.
	 * @param ValBuf		The buffer to receive the data
	 * @param BufLen		The number of words requested
	 * @return	the number of words actually copied
	 */
	uint32 Peek(DataType* ValBuf, const uint32& BufLen) const;

	/**
	 * Compare the memory in the FIFO to the memory in the given buffer
	 * @param SerialBuffer	The buffer containing data to compare
	 * @param CompareLen		The number of words to compare
	 * @return	< 0 if data < SerialBuffer.. 0 if data == SerialBuffer... > 0 if data > SerialBuffer
	 */
	int32 SerialCompare(const DataType* SerialBuffer, uint32 CompareLen) const;

	/**
	 * Get the SHA1 hash for the data currently in the FIFO
	 * @param OutHash	Receives the SHA hash
	 */
	void GetShaHash(FSHAHash& OutHash) const;

	/**
	 * Serializes the internal buffer into the given buffer
	 * @param SerialBuffer	A preallocated buffer to copy data into
	 */
	void Serialize(DataType* SerialBuffer) const;

	/**
	 * Square bracket operators for accessing data in the buffer by FIFO index. [0] returns the next entry in the FIFO that would get provided by Dequeue or Peek.
	 * @param Index	The index into the FIFO buffer
	 * @return	The data word
	 */
	FORCEINLINE const DataType& operator[](const int32& Index) const;
	FORCEINLINE DataType& operator[](const int32& Index);

	/**
	 * Gets the last data word in the FIFO (i.e. most recently pushed).
	 * @return	The last data word
	 */
	FORCEINLINE const DataType& Top() const;
	FORCEINLINE DataType& Top();

	/**
	 * Gets the first data word in the FIFO (i.e. oldest).
	 * @return	The first data word
	 */
	FORCEINLINE const DataType& Bottom() const;
	FORCEINLINE DataType& Bottom();

	/**
	 * Gets the buffer index that the last data word is stored in.
	 * @return	The index of Top()
	 */
	FORCEINLINE uint32 TopIndex() const;

	/**
	 * Gets the buffer index that the first data word is stored in.
	 * @return	The index of Bottom()
	 */
	FORCEINLINE uint32 BottomIndex() const;

	/**
	 * Gets the buffer index that the next enqueued word will get stored in.
	 * @return	The index that the next enqueued word will have
	 */
	FORCEINLINE uint32 NextIndex() const;

	/**
	 * Gets the size of the data buffer.
	 * @return	Template arg BufferDataSize
	 */
	FORCEINLINE uint32 RingDataSize() const;

	/**
	 * Gets the number of words currently in the FIFO.
	 * @return	data size
	 */
	FORCEINLINE uint32 RingDataUsage() const;

	/**
	 * Gets the total number of words that have been pushed through this buffer since clearing
	 * @return	total length of data passed through
	 */
	FORCEINLINE uint64 TotalDataPushed() const;


private:
	TRingBuffer() = delete;

	// Buffer size
	const uint32 BufferDataSize;

	// The data memory
	DataType* Data;

	// Value keeping track of free space
	uint32 NumDataAvailable;

	// Value keeping track of the next data index
	uint32 DataIndex;

	// Value to keep track of total amount of data Enqueued
	uint64 TotalNumDataPushed;
};

/* TRingBuffer implementation
*****************************************************************************/
template< typename DataType >
TRingBuffer< DataType >::TRingBuffer(uint32 InBufferDataSize)
	: BufferDataSize(InBufferDataSize)
{
	Data = new DataType[BufferDataSize];
	Empty();
}

template< typename DataType >
TRingBuffer< DataType >::~TRingBuffer()
{
	delete[] Data;
}

template< typename DataType >
void TRingBuffer< DataType >::Empty()
{
	FMemory::Memzero(Data, sizeof(DataType)* BufferDataSize);
	NumDataAvailable = 0;
	DataIndex = 0;
	TotalNumDataPushed = 0;
}

template< typename DataType >
void TRingBuffer< DataType >::Enqueue(const DataType& Val)
{
	Data[DataIndex++] = Val;

	if (DataIndex == BufferDataSize)
	{
		DataIndex = 0;
	}

	++TotalNumDataPushed;
	NumDataAvailable = FMath::Clamp< uint32 >(NumDataAvailable + 1, 0, BufferDataSize);
}

template< typename DataType >
void TRingBuffer< DataType >::Enqueue(const DataType* ValBuf, const uint32& BufLen)
{
	check(BufLen <= BufferDataSize);
	const uint32 FirstPartLen = BufferDataSize - DataIndex;
	FMemory::Memcpy(Data + DataIndex, ValBuf, sizeof(DataType)* FMath::Min(FirstPartLen, BufLen));
	if (FirstPartLen < BufLen)
	{
		FMemory::Memcpy(Data, ValBuf + FirstPartLen, sizeof(DataType)* (BufLen - FirstPartLen));
	}
	DataIndex += BufLen;
	if (DataIndex >= BufferDataSize)
	{
		DataIndex -= BufferDataSize;
	}
	TotalNumDataPushed += BufLen;
	NumDataAvailable = FMath::Clamp<uint32>(NumDataAvailable + BufLen, 0, BufferDataSize);
}

template< typename DataType >
bool TRingBuffer< DataType >::Dequeue(DataType& ValOut)
{
	if (Peek(ValOut))
	{
		NumDataAvailable = FMath::Clamp< uint32 >(NumDataAvailable - 1, 0, BufferDataSize);
		return true;
	}
	else
	{
		return false;
	}
}

template< typename DataType >
uint32 TRingBuffer< DataType >::Dequeue(DataType* ValBuf, const uint32& BufLen)
{
	const uint32 DataProvided = Peek(ValBuf, BufLen);
	NumDataAvailable -= DataProvided;
	return DataProvided;
}

template< typename DataType >
bool TRingBuffer< DataType >::Peek(DataType& ValOut) const
{
	if (NumDataAvailable > 0)
	{
		ValOut = Bottom();
		return true;
	}
	else
	{
		return false;
	}
}

template< typename DataType >
uint32 TRingBuffer< DataType >::Peek(DataType* ValBuf, const uint32& BufLen) const
{
	const uint32 DataProvided = FMath::Min(BufLen, NumDataAvailable);
	const uint32 BottomIdx = BottomIndex();
	const uint32 BottomPartLen = BufferDataSize - BottomIdx;
	FMemory::Memcpy(ValBuf, Data + BottomIdx, sizeof(DataType)* FMath::Min(BottomPartLen, DataProvided));
	if (BottomPartLen < DataProvided)
	{
		FMemory::Memcpy(ValBuf + BottomPartLen, Data, sizeof(DataType)* (DataProvided - BottomPartLen));
	}
	return DataProvided;
}

template< typename DataType >
int32 TRingBuffer< DataType >::SerialCompare(const DataType* SerialBuffer, uint32 CompareLen) const
{
	check(CompareLen <= BufferDataSize);
	const uint32 FirstPartLen = BufferDataSize - DataIndex;
	int32 FirstCmp = FMemory::Memcmp(Data + BottomIndex(), SerialBuffer, sizeof(DataType)* FMath::Min(FirstPartLen, CompareLen));
	if (FirstCmp != 0 || FirstPartLen >= CompareLen)
	{
		return FirstCmp;
	}
	return FMemory::Memcmp(Data, &SerialBuffer[FirstPartLen], sizeof(DataType)* (CompareLen - FirstPartLen));
}

template< typename DataType >
void TRingBuffer< DataType >::GetShaHash(FSHAHash& OutHash) const
{
	const uint32 FirstPartLen = FMath::Min(BufferDataSize - DataIndex, NumDataAvailable);
	FSHA1 Sha;
	Sha.Update(Data + BottomIndex(), sizeof(DataType) * FirstPartLen);
	if (FirstPartLen < NumDataAvailable)
	{
		Sha.Update(Data, sizeof(DataType) * (NumDataAvailable - FirstPartLen));
	}
	Sha.Final();
	Sha.GetHash(OutHash.Hash);
}

template< typename DataType >
void TRingBuffer< DataType >::Serialize(DataType* SerialBuffer) const
{
	const uint32 BottomIdx = BottomIndex();
	const uint32 BottomPartLen = BufferDataSize - BottomIdx;
	FMemory::Memcpy(SerialBuffer, Data + BottomIdx, sizeof(DataType)* BottomPartLen);
	FMemory::Memcpy(SerialBuffer + BottomPartLen, Data, sizeof(DataType)* (BufferDataSize - BottomPartLen));
}

template< typename DataType >
FORCEINLINE const DataType& TRingBuffer< DataType >::operator[](const int32& Index) const
{
	return Data[(BottomIndex() + Index) % BufferDataSize];
}

template< typename DataType >
FORCEINLINE DataType& TRingBuffer< DataType >::operator[](const int32& Index)
{
	return Data[(BottomIndex() + Index) % BufferDataSize];
}

template< typename DataType >
FORCEINLINE const DataType& TRingBuffer< DataType >::Top() const
{
	return Data[TopIndex()];
}

template< typename DataType >
FORCEINLINE DataType& TRingBuffer< DataType >::Top()
{
	return Data[TopIndex()];
}

template< typename DataType >
FORCEINLINE const DataType& TRingBuffer< DataType >::Bottom() const
{
	return Data[BottomIndex()];
}

template< typename DataType >
FORCEINLINE DataType& TRingBuffer< DataType >::Bottom()
{
	return Data[BottomIndex()];
}

template< typename DataType >
FORCEINLINE uint32 TRingBuffer< DataType >::TopIndex() const
{
	return (DataIndex + BufferDataSize - 1) % BufferDataSize;
}

template< typename DataType >
FORCEINLINE uint32 TRingBuffer< DataType >::BottomIndex() const
{
	return (DataIndex + BufferDataSize - NumDataAvailable) % BufferDataSize;
}

template< typename DataType >
FORCEINLINE uint32 TRingBuffer< DataType >::NextIndex() const
{
	return DataIndex;
}

template< typename DataType >
FORCEINLINE uint32 TRingBuffer< DataType >::RingDataSize() const
{
	return BufferDataSize;
}

template< typename DataType >
FORCEINLINE uint32 TRingBuffer< DataType >::RingDataUsage() const
{
	return NumDataAvailable;
}

template< typename DataType >
FORCEINLINE uint64 TRingBuffer< DataType >::TotalDataPushed() const
{
	return TotalNumDataPushed;
}
