// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/ChunkData.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Memory/SharedBuffer.h"
#include "Misc/CommandLine.h"
#include "Misc/Compression.h"
#include "Misc/ConfigCacheIni.h"
#include "Common/Crypto.h"
#include "Common/FileSystem.h"
#include "Common/StatsCollector.h"
#include "BuildPatchHash.h"
#include "BuildPatchManifest.h"
#include "BuildPatchServicesPrivate.h"
#include "Installer/CloudChunkSource.h"

// The chunk header magic codeword, for quick checking that the opened file is a chunk file.
#define CHUNK_HEADER_MAGIC      0xB1FE3AA2

// The chunkdb header magic codeword, for quick checking that the opened file is a chunkdb file.
#define CHUNKDB_HEADER_MAGIC    0xB1FE3AA3

// Some logs are spammy, but used in automated tests, so default to suppress them in shipping.
#if UE_BUILD_SHIPPING
DEFINE_LOG_CATEGORY_STATIC(LogChunkData, Log, All);
#else
DEFINE_LOG_CATEGORY_STATIC(LogChunkData, Verbose, All);
#endif // UE_BUILD_SHIPPING


namespace BuildPatchServices
{
	const TCHAR* ToString(const EChunkLoadResult& ChunkLoadResult)
	{
		switch (ChunkLoadResult)
		{
			case EChunkLoadResult::Success:
				return TEXT("Success");
			case EChunkLoadResult::OpenFileFail:
				return TEXT("OpenFileFail");
			case EChunkLoadResult::BadArchive:
				return TEXT("BadArchive");
			case EChunkLoadResult::CorruptHeader:
				return TEXT("CorruptHeader");
			case EChunkLoadResult::IncorrectFileSize:
				return TEXT("IncorrectFileSize");
			case EChunkLoadResult::UnsupportedStorage:
				return TEXT("UnsupportedStorage");
			case EChunkLoadResult::MissingHashInfo:
				return TEXT("MissingHashInfo");
			case EChunkLoadResult::SerializationError:
				return TEXT("SerializationError");
			case EChunkLoadResult::DecompressFailure:
				return TEXT("DecompressFailure");
			case EChunkLoadResult::HashCheckFailed:
				return TEXT("HashCheckFailed");
			case EChunkLoadResult::Aborted:
				return TEXT("Aborted");
			default:
				return TEXT("Unknown");
		}
	}

	const TCHAR* ToString(const EChunkSaveResult& ChunkSaveResult)
	{
		static_assert((uint32)EChunkSaveResult::InvalidOrMax == 6, "Please adjust the switch below for new values.");
		switch (ChunkSaveResult)
		{
			case EChunkSaveResult::Success:
				return TEXT("Success");
			case EChunkSaveResult::FileCreateFail:
				return TEXT("FileCreateFail");
			case EChunkSaveResult::BadArchive:
				return TEXT("BadArchive");
			case EChunkSaveResult::SerializationError:
				return TEXT("SerializationError");
			case EChunkSaveResult::MissingEncryptionKey:
				return TEXT("MissingEncryptionKey");
			case EChunkSaveResult::EncryptionFail:
				return TEXT("EncryptionFail");
			default:
				return TEXT("Unknown");
		}
	}

	bool ConsideredFailure(const EChunkLoadResult& ChunkLoadResult)
	{
		return ChunkLoadResult != EChunkLoadResult::Success && ChunkLoadResult != EChunkLoadResult::MissingSecretKey;
	}

	bool ConsideredFailure(const EChunkSaveResult& ChunkSaveResult)
	{
		return ChunkSaveResult != EChunkSaveResult::Success;
	}

	/**
	 * Enum which describes the chunk header version.
	 */
	enum class EChunkVersion : uint32
	{
		Invalid = 0,
		Original,
		StoresShaAndHashType,
		StoresDataSizeUncompressed,
		StoresEncryptionData,

		// Always after the latest version, signifies the latest version plus 1 to allow initialization simplicity.
		LatestPlusOne,
		Latest = (LatestPlusOne - 1)
	};

	// The constant minimum sizes for each version of a header struct. Must be updated.
	// If new member variables are added the version MUST be bumped and handled properly here,
	// and these values must never change.
	static const uint32 ChunkHeaderVersionSizes[(uint32)EChunkVersion::LatestPlusOne] =
	{
		// Dummy for indexing.
		0,
		// Original is 41 bytes (32b Magic, 32b Version, 32b HeaderSize, 32b DataSizeCompressed, 4x32b GUID, 64b Hash, 8b StoredAs).
		41,
		// StoresShaAndHashType is 62 bytes (328b Original, 160b SHA1, 8b HashType).
		62,
		// StoresDataSizeUncompressed is 66 bytes (496b StoresShaAndHashType, 32b DataSizeUncompressed).
		66,
		// StoresEncryptionData is 98 bytes (496b StoresDataSizeUncompressed, 4x32b GUID, 128b AuthTag).
		98
	};
	static_assert((uint32)EChunkVersion::LatestPlusOne == 5, "Please adjust ChunkHeaderVersionSizes values accordingly.");

	namespace HeaderHelpers
	{
		void ZeroHeader(FChunkHeader& Header)
		{
			FMemory::Memzero(Header);
		}

		void ZeroHeader(FChunkDatabaseHeader& Header)
		{
			Header.Version = 0;
			Header.HeaderSize = 0;
			Header.DataSize = 0;
			Header.Contents.Empty();
		}

		EChunkVersion FeatureLevelToChunkVersion(EFeatureLevel FeatureLevel)
		{
			static_assert((uint32)EFeatureLevel::Latest == 24, "Please adjust the switch below for new feature levels.");
			switch (FeatureLevel)
			{
				case BuildPatchServices::EFeatureLevel::Original:
				case BuildPatchServices::EFeatureLevel::CustomFields:
				case BuildPatchServices::EFeatureLevel::StartStoringVersion:
				case BuildPatchServices::EFeatureLevel::DataFileRenames:
				case BuildPatchServices::EFeatureLevel::StoresIfChunkOrFileData:
				case BuildPatchServices::EFeatureLevel::StoresDataGroupNumbers:
				case BuildPatchServices::EFeatureLevel::ChunkCompressionSupport:
				case BuildPatchServices::EFeatureLevel::StoresPrerequisitesInfo:
				case BuildPatchServices::EFeatureLevel::StoresChunkFileSizes:
				case BuildPatchServices::EFeatureLevel::StoredAsCompressedUClass:
				case BuildPatchServices::EFeatureLevel::UNUSED_0:
				case BuildPatchServices::EFeatureLevel::UNUSED_1:
					return EChunkVersion::Original;
				case BuildPatchServices::EFeatureLevel::StoresChunkDataShaHashes:
				case BuildPatchServices::EFeatureLevel::StoresPrerequisiteIds:
					return EChunkVersion::StoresShaAndHashType;
				case BuildPatchServices::EFeatureLevel::StoredAsBinaryData:
				case BuildPatchServices::EFeatureLevel::VariableSizeChunksWithoutWindowSizeChunkInfo:
				case BuildPatchServices::EFeatureLevel::VariableSizeChunks:
				case BuildPatchServices::EFeatureLevel::UsesRuntimeGeneratedBuildId:
				case BuildPatchServices::EFeatureLevel::UsesBuildTimeGeneratedBuildId:
				case BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType:
				case BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes:
				case BuildPatchServices::EFeatureLevel::StoresUninstallActions:
					return EChunkVersion::StoresDataSizeUncompressed;
				case BuildPatchServices::EFeatureLevel::ChunkEncryptionSupport:
				case BuildPatchServices::EFeatureLevel::ChunksStoredBySecret:
				case BuildPatchServices::EFeatureLevel::ManifestEncryptionSupport:
					return EChunkVersion::StoresEncryptionData;
			}
			checkf(false, TEXT("Unhandled FeatureLevel %s"), LexToString(FeatureLevel));
			return EChunkVersion::Invalid;
		}
	}

	FChunkHeader::FChunkHeader()
		: Version((uint32)EChunkVersion::Latest)
		, HeaderSize(ChunkHeaderVersionSizes[(uint32)EChunkVersion::Latest])
		, DataSizeCompressed(0)
		, DataSizeUncompressed(1024 * 1024)
		, StoredAs(EChunkStorageFlags::RawData)
		, HashType(EChunkHashFlags::RollingPoly64)
		, RollingHash(0)
	{
	}

	FArchive& operator<<(FArchive& Ar, FChunkHeader& Header)
	{
		if (Ar.IsError())
		{
			return Ar;
		}
		// Calculate how much space left in the archive for reading data ( will be 0 when writing ).
		const int64 StartPos = Ar.Tell();
		const int64 ArchiveSizeLeft = Ar.TotalSize() - StartPos;
		uint32 ExpectedSerializedBytes = 0;
		// Make sure the archive has enough data to read from, or we are saving instead.
		bool bSuccess = Ar.IsSaving() || (ArchiveSizeLeft >= ChunkHeaderVersionSizes[(uint32)EChunkVersion::Original]);
		if (bSuccess)
		{
			Header.HeaderSize = ChunkHeaderVersionSizes[Header.Version];
			uint32 Magic = CHUNK_HEADER_MAGIC;
			uint8 StoredAs = (uint8)Header.StoredAs;
			Ar << Magic
			   << Header.Version
			   << Header.HeaderSize
			   << Header.DataSizeCompressed
			   << Header.Guid
			   << Header.RollingHash
			   << StoredAs;
			Header.StoredAs = (EChunkStorageFlags)StoredAs;
			bSuccess = Magic == CHUNK_HEADER_MAGIC && !Ar.IsError();
			ExpectedSerializedBytes = ChunkHeaderVersionSizes[(uint32)EChunkVersion::Original];

			// From version 2, we have a hash type choice. Previous versions default as only rolling.
			if (bSuccess && Header.Version >= (uint32)EChunkVersion::StoresShaAndHashType)
			{
				bSuccess = Ar.IsSaving() || (ArchiveSizeLeft >= ChunkHeaderVersionSizes[(uint32)EChunkVersion::StoresShaAndHashType]);
				if (bSuccess)
				{
					uint8 HashType = (uint8)Header.HashType;
					Ar.Serialize(Header.SHAHash.Hash, FSHA1::DigestSize);
					Ar << HashType;
					Header.HashType = (EChunkHashFlags)HashType;
					bSuccess = !Ar.IsError();
				}
				ExpectedSerializedBytes = ChunkHeaderVersionSizes[(uint32)EChunkVersion::StoresShaAndHashType];
			}

			// From version 3, we have an uncompressed data size. Previous versions default to 1 MiB (1048576 B).
			if (bSuccess && Header.Version >= (uint32)EChunkVersion::StoresDataSizeUncompressed)
			{
				bSuccess = Ar.IsSaving() || (ArchiveSizeLeft >= ChunkHeaderVersionSizes[(uint32)EChunkVersion::StoresDataSizeUncompressed]);
				if (bSuccess)
				{
					Ar << Header.DataSizeUncompressed;
					bSuccess = !Ar.IsError();
				}
				ExpectedSerializedBytes = ChunkHeaderVersionSizes[(uint32)EChunkVersion::StoresDataSizeUncompressed];
			}

			// Serialse encryption ID and auth tag required by this chunk
			if (bSuccess && Header.Version >= (uint32)EChunkVersion::StoresEncryptionData)
			{
				bSuccess = Ar.IsSaving() || (ArchiveSizeLeft >= ChunkHeaderVersionSizes[(uint32)EChunkVersion::StoresEncryptionData]);
				if (bSuccess)
				{
					Ar << Header.EncryptionSecretId;
					Ar.Serialize(Header.AESAuthTag.AuthTag, AES256_GCM_AuthTagSizeInBytes);
					bSuccess = !Ar.IsError();
				}
				ExpectedSerializedBytes = ChunkHeaderVersionSizes[(uint32)EChunkVersion::StoresEncryptionData];
			}
		}

		// Make sure the expected number of bytes were serialized. In practice this will catch errors where type
		// serialization operators changed their format and that will need investigating.
		bSuccess = bSuccess && (Ar.Tell() - StartPos) == ExpectedSerializedBytes;

		if (bSuccess)
		{
			// Make sure the archive now points to data location. Only seek if we must, to avoid a flush.
			const int64 DataLocation = StartPos + Header.HeaderSize;
			if (Ar.Tell() != DataLocation)
			{
				Ar.Seek(DataLocation);
			}
		}
		else
		{
			// If we had a serialization error when loading, zero out the header values.
			if (Ar.IsLoading())
			{
				HeaderHelpers::ZeroHeader(Header);
			}
			Ar.SetError();
		}

		return Ar;
	}

	/* FChunkPart implementation
	*****************************************************************************/
	FChunkPart::FChunkPart()
		: Guid()
		, Offset(0)
		, Size(0)
	{
	}

	FChunkPart::FChunkPart(const FGuid& InGuid, const uint32 InOffset, const uint32 InSize)
		: Guid(InGuid)
		, Offset(InOffset)
		, Size(InSize)
	{
	}

	FArchive& operator<<(FArchive& Ar, FChunkPart& ChunkPart)
	{
		if (Ar.IsError())
		{
			return Ar;
		}

		const int64 StartPos = Ar.Tell();
		uint32 DataSize = 0;

		Ar << DataSize;
		Ar << ChunkPart.Guid;
		Ar << ChunkPart.Offset;
		Ar << ChunkPart.Size;

		// If saving, we need to go back and set the data size.
		if (!Ar.IsError() && Ar.IsSaving())
		{
			const int64 EndPos = Ar.Tell();
			DataSize = EndPos - StartPos;
			Ar.Seek(StartPos);
			Ar << DataSize;
			Ar.Seek(EndPos);
		}

		// We must always make sure to seek the archive to the correct end location. Only seek if we must, to avoid a flush.
		const int64 DataLocation = StartPos + DataSize;
		if (Ar.Tell() != DataLocation)
		{
			Ar.Seek(DataLocation);
		}
		return Ar;
	}

	/* FFileChunkPart implementation
	*****************************************************************************/
	FFileChunkPart::FFileChunkPart()
		: Filename()
		, FileOffset(0)
		, ChunkPart()
	{
	}

	/**
	 * Enum which describes the chunk database header version.
	 */
	enum class EChunkDatabaseVersion : uint32
	{
		Invalid = 0,
		Original,

		// Always after the latest version, signifies the latest version plus 1 to allow initialization simplicity
		LatestPlusOne,
		Latest = (LatestPlusOne - 1)
	};

	// The constant minimum sizes for each version of a header struct. Must be updated.
	// If new member variables are added the version MUST be bumped and handled properly here,
	// and these values must never change.
	static const uint32 ChunkDatabaseHeaderVersionSizes[(uint32)EChunkDatabaseVersion::LatestPlusOne] =
	{
		// Dummy for indexing.
		0,
		// Version 1 is 24 bytes (32b Magic, 32b Version, 32b HeaderSize, 64b DataSize, 32b ChunkCount).
		24
	};

	FChunkDatabaseHeader::FChunkDatabaseHeader()
		: Version((uint32)EChunkDatabaseVersion::Latest)
		, HeaderSize(ChunkDatabaseHeaderVersionSizes[(uint32)EChunkDatabaseVersion::Latest])
		, Contents()
	{
	}

	FArchive& operator<<(FArchive& Ar, FChunkDatabaseHeader& Header)
	{
		if (Ar.IsError())
		{
			return Ar;
		}
		// Calculate how much space left in the archive for reading data (will be 0 when writing).
		const int64 StartPos = Ar.Tell();
		const int64 ArchiveSizeLeft = Ar.TotalSize() - StartPos;
		uint32 ExpectedSerializedBytes = 0;
		// Make sure the archive has enough data to read from, or we are saving instead.
		bool bSuccess = Ar.IsSaving() || (ArchiveSizeLeft >= ChunkDatabaseHeaderVersionSizes[(uint32)EChunkDatabaseVersion::Original]);
		if (bSuccess)
		{
			uint32 Magic = CHUNKDB_HEADER_MAGIC;
			// Chunk entry is 28 bytes (4x32b GUID, 64b FileStart, 32b FileSize).
			static const uint32 ChunkEntrySize = 28;
			int32 ChunkCount = Header.Contents.Num();
			Header.HeaderSize = ChunkDatabaseHeaderVersionSizes[(uint32)EChunkDatabaseVersion::Original] + (ChunkCount * ChunkEntrySize);
			Ar << Magic
			   << Header.Version
			   << Header.HeaderSize
			   << Header.DataSize
			   << ChunkCount;
			bSuccess = Magic == CHUNKDB_HEADER_MAGIC && !Ar.IsError();
			ExpectedSerializedBytes = ChunkDatabaseHeaderVersionSizes[(uint32)EChunkDatabaseVersion::Original];

			// Serialize all chunk info.
			if (bSuccess)
			{
				Header.Contents.SetNumZeroed(ChunkCount);
				for (int32 ChunkIdx = 0; ChunkIdx < ChunkCount; ++ChunkIdx)
				{
					FChunkLocation& Location = Header.Contents[ChunkIdx];
					Ar << Location.ChunkId
					   << Location.ByteStart
					   << Location.ByteSize;
				}
				ExpectedSerializedBytes += ChunkCount * ChunkEntrySize;
			}
		}

		// Make sure the expected number of bytes were serialized. In practice this will catch errors where type
		// serialization operators changed their format and that will need investigating.
		bSuccess = bSuccess && (Ar.Tell() - StartPos) == ExpectedSerializedBytes;

		if (bSuccess)
		{
			// Make sure the archive now points to data location. Only seek if we must, to avoid a flush.
			const int64 DataLocation = StartPos + Header.HeaderSize;
			if (Ar.Tell() != DataLocation)
			{
				Ar.Seek(DataLocation);
			}
		}
		else
		{
			// If we had a serialization error, zero out the header values.
			if (Ar.IsLoading())
			{
				HeaderHelpers::ZeroHeader(Header);
			}
		}

		return Ar;
	}


	FScopeLockedChunkData::FScopeLockedChunkData(IChunkDataAccess* InChunkDataAccess)
		: ChunkDataAccess(InChunkDataAccess)
	{
		ChunkDataAccess->GetDataLock(&ChunkData, &ChunkHeader);
	}

	FScopeLockedChunkData::~FScopeLockedChunkData()
	{
		ChunkDataAccess->ReleaseDataLock();
	}

	BuildPatchServices::FChunkHeader* FScopeLockedChunkData::GetHeader() const
	{
		return ChunkHeader;
	}

	uint8* FScopeLockedChunkData::GetData() const
	{
		return ChunkData;
	}


	class FChunkDataAccess : public IChunkDataAccess
	{
	public:
		FChunkDataAccess(uint32 DataSize)
		{
			ChunkData.Reserve(DataSize);
			ChunkData.SetNumUninitialized(DataSize);
		}

		~FChunkDataAccess()
		{
		}

		// IChunkDataAccess interface begin.
		virtual void GetDataLock(uint8** OutChunkData, FChunkHeader** OutChunkHeader) override
		{
			ThreadLock.Lock();
			if (OutChunkData)
			{
				(*OutChunkData) = ChunkData.GetData();
			}
			if (OutChunkHeader)
			{
				(*OutChunkHeader) = &ChunkHeader;
			}
		}
		virtual void GetDataLock(const uint8** OutChunkData, const FChunkHeader** OutChunkHeader) const override
		{
			ThreadLock.Lock();
			if (OutChunkData)
			{
				(*OutChunkData) = ChunkData.GetData();
			}
			if (OutChunkHeader)
			{
				(*OutChunkHeader) = &ChunkHeader;
			}
		}
		virtual void ReleaseDataLock() const override
		{
			ThreadLock.Unlock();
		}
		// IChunkDataAccess interface end.

	private:
		FChunkHeader ChunkHeader;
		TArray<uint8> ChunkData;
		mutable FCriticalSection ThreadLock;
	};

	IChunkDataAccess* FChunkDataAccessFactory::Create(uint32 DataSize)
	{
		return new FChunkDataAccess(DataSize);
	}


	class FChunkDataSerialization : public IChunkDataSerialization
	{
		int64 MinCompressionBytes = 64 * 1024;
		int32 MinCompressionPct = 10;

	public:
		FChunkDataSerialization(IFileSystem* InFileSystem, ICrypto* InCrypto, FChunkDataSerializationConfig InConfig)
			: FileSystem(InFileSystem)
			, Crypto(InCrypto)
			, Config(InConfig)
			, WriteEncryptionSecretKey(Config.EncryptionSecrets.Find(Config.WriteEncryptionSecretId))
			// UE5 MERGE TODO: , CloudChunkSourceStat(Config.CloudChunkSourceStat)
		{
			// UE5 MERGE TODO: check(CloudChunkSourceStat);
			GConfig->GetInt64(TEXT("BuildPatchTool"), TEXT("MinCompressionBytes"), MinCompressionBytes, GEngineIni);
			GConfig->GetInt(TEXT("BuildPatchTool"), TEXT("MinCompressionPct"), MinCompressionPct, GEngineIni);

			FParse::Value(FCommandLine::Get(), TEXT("MinCompressionPct="), MinCompressionPct);
			FParse::Value(FCommandLine::Get(), TEXT("MinCompressionBytes="), MinCompressionBytes);

			UE_LOGF(LogChunkData, Display, "Using MinCompressionPct=%d", MinCompressionPct);
			UE_LOGF(LogChunkData, Display, "Using MinCompressionBytes=%lld", MinCompressionBytes);
		}

		~FChunkDataSerialization()
		{
		}

		static bool CheckChunkHash(const FChunkHeader& Header, const void* Data, uint64 DataLen)
		{
			// If we have both hashes, only check Sha1 instead of both.
			if (EnumHasAnyFlags(Header.HashType, EChunkHashFlags::Sha1))
			{
				FSHAHash ShaHashCheck;
				TRACE_CPUPROFILER_EVENT_SCOPE(Chunk_ShaHash);
				FSHA1::HashBuffer(Data, DataLen, ShaHashCheck.Hash);
				return ShaHashCheck == Header.SHAHash;
			}

			if (EnumHasAnyFlags(Header.HashType, EChunkHashFlags::RollingPoly64))
			{
				return Header.RollingHash == FRollingHash::GetHashForDataSet((const uint8*)Data, DataLen);
			}

			return false;
		}

		// IChunkDataAccess interface begin.
		virtual bool ValidateAndRead(FArchive& InArchive, FMutableMemoryView InDestinationBuffer, FChunkHeader& OutHeader, FUniqueBuffer& OutCompressedBuffer) const override
		{
			// Read and check the header.
			InArchive << OutHeader;

			if (!OutHeader.Guid.IsValid() || // Must have a valid guid
				(!EnumHasAnyFlags(OutHeader.HashType, EChunkHashFlags::RollingPoly64) && !EnumHasAnyFlags(OutHeader.HashType, EChunkHashFlags::Sha1)) || // Must have a valid hash
				(InDestinationBuffer.GetSize() > OutHeader.DataSizeUncompressed)) // Unexpected buffer size.
			{
				UE_LOGF(LogBuildPatchServices, Error, "Failed to validate chunk. Header reported Guid=%ls HashType=%d StoredAs=%d UncompressedSize=%d DestinationSize=%llu",
					*WriteToString<64>(OutHeader.Guid),
					EnumToUnderlyingType(OutHeader.HashType),
					EnumToUnderlyingType(OutHeader.StoredAs),
					OutHeader.DataSizeUncompressed,
					InDestinationBuffer.GetSize());
				return false;
			}

			// If the chunk isn't compressed and is the entire block, we can read directly in to the destination buffer.
			// We seem to be getting chunkinfos that report window sizes that are not the full chunk size, so we need to be
			// able to catch these subchunks and copy them out as needed.
			uint64 ReadSize;
			if (OutHeader.StoredAs == EChunkStorageFlags::RawData &&
				InDestinationBuffer.GetSize() == OutHeader.DataSizeUncompressed)
			{
				ReadSize = InDestinationBuffer.GetSize();
				InArchive.Serialize(InDestinationBuffer.GetData(), InDestinationBuffer.GetSize());
			}
			else
			{
				// We need to allocate and read in to another buffer for decompression/decryption.
				ReadSize = OutHeader.DataSizeCompressed;
				OutCompressedBuffer = FUniqueBuffer::Alloc(OutHeader.DataSizeCompressed);
				InArchive.Serialize(OutCompressedBuffer.GetData(), OutCompressedBuffer.GetSize());
			}

			if (InArchive.IsError())
			{
				UE_LOGF(LogBuildPatchServices, Error, "Archive error while deserializing chunk %ls, ReadSize = %llu", *WriteToString<64>(OutHeader.Guid), ReadSize);
				return false;
			}
			return true;
		}

		virtual bool DecompressAndDecryptValidatedRead(const FChunkHeader& InHeader, FMutableMemoryView InDestinationBuffer, const FUniqueBuffer& InCompressedBuffer) const override
		{
			FMutableMemoryView UseDestinationBuffer = InDestinationBuffer;
			FMemoryView UseCompressedBuffer = InCompressedBuffer.GetView();
			TArray<uint8> DecryptedData;

			FUniqueBuffer TempDestinationBuffer;
			if (InHeader.DataSizeUncompressed != InDestinationBuffer.GetSize())
			{
				TempDestinationBuffer = FUniqueBuffer::Alloc(InHeader.DataSizeUncompressed);
				UseDestinationBuffer = TempDestinationBuffer.GetView();
			}

			// Decrypt if necessary	
			if (EnumHasAllFlags(InHeader.StoredAs, EChunkStorageFlags::Encrypted))
			{
				const TArray<uint8>* EncryptionSecretKey = Config.EncryptionSecrets.Find(InHeader.EncryptionSecretId);

				if (EncryptionSecretKey == nullptr)
				{
					UE_LOGF(LogChunkData, Error, "Decryption failed for chunk: %ls EncryptionSecretKey is empty.", *InHeader.Guid.ToString());
					return false;
				}

				// Attempt to decrypt the data
				const uint8* CompressedData = (const uint8*)InCompressedBuffer.GetData();
				TArrayView<const uint8> Ciphertext(CompressedData, InCompressedBuffer.GetSize());
				TArrayView<const uint8> InitializationVector(InHeader.SHAHash.Hash, AES256_GCM_InitializationVectorSizeInBytes);
				TArrayView<const uint8> AESAuthTag(InHeader.AESAuthTag.AuthTag, AES256_GCM_AuthTagSizeInBytes);
				bool bDecryptResult = false;
				uint64 DecryptStartCycles = FStatsCollector::GetCycles();
				DecryptedData = Crypto->Decrypt_AES_256_GCM(Ciphertext, *EncryptionSecretKey, InitializationVector, AESAuthTag, bDecryptResult);
				// UE5 MERGE TODO: CloudChunkSourceStat->OnChunkDecrypted(FStatsCollector::GetCycles() - DecryptStartCycles);
				if (bDecryptResult)
				{
					UseCompressedBuffer = FMemoryView(DecryptedData.GetData(), DecryptedData.Num());
				}
				else
				{
					UE_LOGF(LogChunkData, Error, "Decryption failed for chunk: %ls (EncryptionSecretId=\"%ls\").", *InHeader.Guid.ToString(), *InHeader.EncryptionSecretId.ToString());
					return false;
				}
			}

			if (EnumHasAnyFlags(InHeader.StoredAs, EChunkStorageFlags::Compressed))
			{
				if (!FCompression::UncompressMemory(
					NAME_Zlib,
					UseDestinationBuffer.GetData(),
					UseDestinationBuffer.GetSize(),
					UseCompressedBuffer.GetData(),
					UseCompressedBuffer.GetSize()))
				{
					return false;
				}
			}
			else if (EnumHasAnyFlags(InHeader.StoredAs, EChunkStorageFlags::Encrypted))
			{
				// Encrypted but not compressed, copy before checking hashes
				FMemory::Memcpy(UseDestinationBuffer.GetData(), UseCompressedBuffer.GetData(), FMath::Min(UseDestinationBuffer.GetSize(), UseCompressedBuffer.GetSize()));
			}
			
			// Uncompressed - where ValidateAndRead put the data depends on whether the destination buffer
			// was big enough for the full chunk.
			FMemoryView HashCheckBuffer;
			if ((InHeader.StoredAs == EChunkStorageFlags::RawData) &&
			    (InHeader.DataSizeUncompressed != InDestinationBuffer.GetSize()))
			{
				HashCheckBuffer = InCompressedBuffer.GetView();
			}
			else
			{
				HashCheckBuffer = UseDestinationBuffer;
			}

			bool bHashMatches = CheckChunkHash(InHeader, HashCheckBuffer.GetData(), HashCheckBuffer.GetSize());
			if (bHashMatches && InHeader.DataSizeUncompressed != InDestinationBuffer.GetSize())
			{
				// Destination buffer should always be <= the chunk size.
				if (InDestinationBuffer.GetSize() > TempDestinationBuffer.GetSize())
				{
					UE_LOGF(LogChunkData, Error, "Invalid destination buffer size for chunk %ls", *WriteToString<64>(InHeader.Guid));
					return false;
				}
				// We routed through a temp buffer, copy out as necessary.
				FMemory::Memcpy(InDestinationBuffer.GetData(), HashCheckBuffer.GetData(), InDestinationBuffer.GetSize());
			}

			return bHashMatches;
		}

		virtual IChunkDataAccess* LoadFromFile(const FString& Filename, EChunkLoadResult& OutLoadResult) const override
		{
			IChunkDataAccess* ChunkData = nullptr;
			// Read the chunk file.
			TUniquePtr<FArchive> FileReader(FileSystem->CreateFileReader(*Filename));
			bool bSuccess = FileReader.IsValid();
			if (bSuccess)
			{
				ChunkData = Load(*FileReader, OutLoadResult);
				// Close the file.
				FileReader->Close();
			}
			else
			{
				OutLoadResult = EChunkLoadResult::OpenFileFail;
			}

			return ChunkData;
		}

		virtual IChunkDataAccess* LoadFromMemory(const TArray<uint8>& Memory, EChunkLoadResult& OutLoadResult) const override
		{
			FMemoryReader MemoryReader(Memory);
			return Load(MemoryReader, OutLoadResult);
		}

		virtual IChunkDataAccess* LoadFromArchive(FArchive& Archive, EChunkLoadResult& OutLoadResult) const override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Chunk_LoadFromArchive);
			if (Archive.IsLoading())
			{
				return Load(Archive, OutLoadResult);
			}
			else
			{
				OutLoadResult = EChunkLoadResult::BadArchive;
				return nullptr;
			}
		}

		virtual EChunkSaveResult SaveToFile(const FString& Filename, IChunkDataAccess* ChunkDataAccess) const override
		{
			EChunkSaveResult SaveResult;
			TUniquePtr<FArchive> FileOut(FileSystem->CreateFileWriter(*Filename));
			if (FileOut.IsValid())
			{
				SaveResult = SaveToArchive(*FileOut, ChunkDataAccess);
			}
			else
			{
				SaveResult = EChunkSaveResult::FileCreateFail;
			}
			return SaveResult;
		}

		virtual EChunkSaveResult SaveToMemory(TArray<uint8>& Memory, IChunkDataAccess* ChunkDataAccess) const override
		{
			FMemoryWriter MemoryWriter(Memory);
			return Save(MemoryWriter, ChunkDataAccess, true);
		}

		virtual EChunkSaveResult SaveToArchive(FArchive& Archive, IChunkDataAccess* ChunkDataAccess) const override
		{
			if (Archive.IsSaving())
			{
				return Save(Archive, ChunkDataAccess, true);
			}
			else
			{
				return EChunkSaveResult::BadArchive;
			}
		}

		virtual EChunkSaveResult SaveToArchiveUncompressed(FArchive& Archive, IChunkDataAccess* ChunkDataAccess) const override
		{
			if (Archive.IsSaving())
			{
				return Save(Archive, ChunkDataAccess, false);
			}
			else
			{
				return EChunkSaveResult::BadArchive;
			}
		}

		virtual bool InjectShaToChunkData(TArray<uint8>& Memory, const FSHAHash& ShaHashData) const override
		{
			const uint32 StoresShaAndHashTypeUint = (uint32)EChunkVersion::StoresShaAndHashType;
			FMemoryReader MemoryReader(Memory);

			FChunkHeader Header;
			MemoryReader << Header;
			if (MemoryReader.IsError())
			{
				return false;
			}
			// Chunk already contains SHA by itself and doesn't need injection
			if (EnumHasAnyFlags(Header.HashType, EChunkHashFlags::Sha1))
			{
				return true;
			}

			FMemoryWriter MemoryWriter(Memory);
			EnumAddFlags(Header.HashType, EChunkHashFlags::Sha1);
			Header.SHAHash = ShaHashData;
			if (Header.Version < StoresShaAndHashTypeUint)
			{
				const uint32 StoresShaAndHashTypeHeaderSize = ChunkHeaderVersionSizes[StoresShaAndHashTypeUint];
				check(Header.HeaderSize <= StoresShaAndHashTypeHeaderSize);
				Header.Version = StoresShaAndHashTypeUint;
				Memory.InsertZeroed(0, StoresShaAndHashTypeHeaderSize - Header.HeaderSize);
			}
			MemoryWriter << Header;
			return !MemoryWriter.IsError();
		}

		// IChunkDataAccess interface end.

	private:
		IChunkDataAccess* Load(FArchive& Reader, EChunkLoadResult& OutLoadResult) const
		{
			IChunkDataAccess* ChunkData = nullptr;

			// Begin of read pos.
			const int64 StartPos = Reader.Tell();

			// Available read size.
			const int64 AvailableSize = Reader.TotalSize() - StartPos;

			// Read and check the header.
			FChunkHeader HeaderCheck;
			Reader << HeaderCheck;

			// Get file size.
			const uint64 FileSize = (uint64)HeaderCheck.HeaderSize + (uint64)HeaderCheck.DataSizeCompressed;

			// Check that Filesize can fit into a uint32 for FChunkDataAccessFactory::Create()
			if (IntFitsIn<uint32>(FileSize))
			{
				if (HeaderCheck.Guid.IsValid())
				{
					if (HeaderCheck.HashType != EChunkHashFlags::None)
					{
						if ((HeaderCheck.HeaderSize + HeaderCheck.DataSizeCompressed) <= AvailableSize)
						{
							constexpr EChunkStorageFlags AllSupportedStorage = EChunkStorageFlags::Compressed | EChunkStorageFlags::Encrypted;
							if ((HeaderCheck.StoredAs & AllSupportedStorage) == HeaderCheck.StoredAs)
							{
								// Create the data.
								ChunkData = FChunkDataAccessFactory::Create(FileSize);

								// Lock data.
								FChunkHeader* Header;
								uint8* Data;
								ChunkData->GetDataLock(&Data, &Header);
								*Header = HeaderCheck;

								// Read the data.
								{
									TRACE_CPUPROFILER_EVENT_SCOPE(Chunk_Serialize);
									Reader.Serialize(Data, Header->DataSizeCompressed);
								}
								if (Reader.IsError() == false)
								{
									OutLoadResult = EChunkLoadResult::Success;
									// Decrypt.
									if (EnumHasAllFlags(Header->StoredAs, EChunkStorageFlags::Encrypted))
									{
										const TArray<uint8>* EncryptionSecretKey = Config.EncryptionSecrets.Find(Header->EncryptionSecretId);
										if (EncryptionSecretKey != nullptr)
										{
											// Attempt to decrypt the data
											TArrayView<const uint8> Ciphertext(Data, Header->DataSizeCompressed);
											TArrayView<const uint8> InitializationVector(Header->SHAHash.Hash, AES256_GCM_InitializationVectorSizeInBytes);
											TArrayView<const uint8> AESAuthTag(Header->AESAuthTag.AuthTag, AES256_GCM_AuthTagSizeInBytes);
											bool bDecryptResult = false;
											uint64 DecryptStartCycles = FStatsCollector::GetCycles();
											TArray<uint8> DecryptedData = Crypto->Decrypt_AES_256_GCM(Ciphertext, *EncryptionSecretKey, InitializationVector, AESAuthTag, bDecryptResult);
											// UE5 MERGE TODO: CloudChunkSourceStat->OnChunkDecrypted(FStatsCollector::GetCycles() - DecryptStartCycles);
											if (bDecryptResult)
											{
												// If successful, remove the encryption flag
												EnumRemoveFlags(Header->StoredAs, EChunkStorageFlags::Encrypted);
												// Copy the decrypted data
												FMemory::Memcpy(Data, DecryptedData.GetData(), Header->DataSizeCompressed);
											}
											else
											{
												OutLoadResult = EChunkLoadResult::DecryptionFailure;
												UE_LOGF(LogChunkData, Error, "Decryption failed for chunk: %ls (EncryptionSecretId=\"%ls\").", *Header->Guid.ToString(), *Header->EncryptionSecretId.ToString());
											}
										}
										else
										{
											OutLoadResult = EChunkLoadResult::MissingSecretKey;
											UE_LOGF(LogChunkData, Verbose, "The security key is not ready for chunk: %ls (EncryptionSecretId=\"%ls\").", *Header->Guid.ToString(), *Header->EncryptionSecretId.ToString());
										}
									}
									// Decompress.
									if (OutLoadResult == EChunkLoadResult::Success && EnumHasAllFlags(Header->StoredAs, EChunkStorageFlags::Compressed))
									{
										// Create a new data instance.
										IChunkDataAccess* NewChunkData = FChunkDataAccessFactory::Create(Header->DataSizeUncompressed);
										// Lock data.
										FChunkHeader* NewHeader;
										uint8* NewData;
										NewChunkData->GetDataLock(&NewData, &NewHeader);
										// Uncompress the memory.
										bool bSuccess = FCompression::UncompressMemory(
											NAME_Zlib,
											NewData,
											Header->DataSizeUncompressed,
											Data,
											Header->DataSizeCompressed);
										// If successful, switch over to new data.
										if (bSuccess)
										{
											FMemory::Memcpy(*NewHeader, *Header);
											EnumRemoveFlags(NewHeader->StoredAs, EChunkStorageFlags::Compressed);
											NewHeader->DataSizeCompressed = Header->DataSizeUncompressed;
											ChunkData->ReleaseDataLock();
											delete ChunkData;
											ChunkData = NewChunkData;
											Header = NewHeader;
											Data = NewData;
										}
										// Otherwise delete new data.
										else
										{
											OutLoadResult = EChunkLoadResult::DecompressFailure;
											NewChunkData->ReleaseDataLock();
											delete NewChunkData;
											NewChunkData = nullptr;
										}
									}
									// Verify.
									// If we have both hashes, only check Sha1 instead of both.
									bool bHasRollingHash = EnumHasAnyFlags(Header->HashType, EChunkHashFlags::RollingPoly64);
									bool bHasSha1Hash = EnumHasAnyFlags(Header->HashType, EChunkHashFlags::Sha1);
									if (OutLoadResult == EChunkLoadResult::Success && bHasRollingHash && !bHasSha1Hash) // Don't bother verifying the rolling hash if we have a sha1
									{
										if (Header->DataSizeCompressed != Header->DataSizeUncompressed || Header->RollingHash != FRollingHash::GetHashForDataSet(Data, Header->DataSizeUncompressed))
										{
											OutLoadResult = EChunkLoadResult::HashCheckFailed;
										}
									}
									FSHAHash ShaHashCheck;
									if (OutLoadResult == EChunkLoadResult::Success && bHasSha1Hash)
									{
										TRACE_CPUPROFILER_EVENT_SCOPE(Chunk_ShaHash);
										FSHA1::HashBuffer(Data, Header->DataSizeUncompressed, ShaHashCheck.Hash);
										if (!(ShaHashCheck == Header->SHAHash))
										{
											OutLoadResult = EChunkLoadResult::HashCheckFailed;
										}
									}
								}
								else
								{
									OutLoadResult = EChunkLoadResult::SerializationError;
								}
							}
							else
							{
								OutLoadResult = EChunkLoadResult::UnsupportedStorage;
							}
						}
						else
						{
							OutLoadResult = EChunkLoadResult::IncorrectFileSize;
						}
					}
					else
					{
						OutLoadResult = EChunkLoadResult::MissingHashInfo;
					}
				}
				else
				{
					OutLoadResult = EChunkLoadResult::CorruptHeader;
				}
			}
			else
			{
				OutLoadResult = EChunkLoadResult::FileSizeTooBig;
			}

			if (ChunkData != nullptr)
			{
				// Release data.
				ChunkData->ReleaseDataLock();

				// Delete data if failed
				if (ConsideredFailure(OutLoadResult))
				{
					delete ChunkData;
					ChunkData = nullptr;
				}
			}

			return ChunkData;
		}

		EChunkSaveResult Save(FArchive& Writer, IChunkDataAccess* ChunkDataAccess, bool bTryCompress) const
		{
			EChunkSaveResult SaveResult = EChunkSaveResult::Success;
			uint8* ChunkDataSource;
			FChunkHeader* ChunkAccessHeader;
			ChunkDataAccess->GetDataLock(&ChunkDataSource, &ChunkAccessHeader);
			// Setup to handle compression.
			bool bDataIsCompressed = false;
			TArray<uint8> TempCompressedData;
			int64 CompressedSize = ChunkAccessHeader->DataSizeUncompressed;
			if (Config.FeatureLevel >= EFeatureLevel::ChunkCompressionSupport && bTryCompress)
			{
				TempCompressedData.Empty(ChunkAccessHeader->DataSizeUncompressed);
				TempCompressedData.AddUninitialized(ChunkAccessHeader->DataSizeUncompressed);
				// Compression can increase data size, too. This call will set bDataIsCompressed false in that case.
				const bool bCompressSuccess = FCompression::CompressMemoryIfWorthDecompressing
				(
					NAME_Zlib,
					bDataIsCompressed,
					MinCompressionBytes,
					MinCompressionPct,
					TempCompressedData.GetData(),
					CompressedSize,
					ChunkDataSource,
					ChunkAccessHeader->DataSizeUncompressed,
					COMPRESS_BiasMemory
				);
				bDataIsCompressed = bDataIsCompressed && bCompressSuccess;
			}

			// If compression succeeded, set data vars.
			if (bDataIsCompressed)
			{
				ChunkDataSource = TempCompressedData.GetData();
			}
			else
			{
				CompressedSize = ChunkAccessHeader->DataSizeUncompressed;
			}

			TArray<uint8> TempEncryptedData;
			TArray<uint8> TempEncryptionAuthTag;
			bool bDataIsEncrypted = Config.FeatureLevel >= EFeatureLevel::ChunkEncryptionSupport && Config.WriteEncryptionSecretId.IsValid();
			if (bDataIsEncrypted)
			{
				if (WriteEncryptionSecretKey != nullptr)
				{
					TArrayView<const uint8> SourceData(ChunkDataSource, CompressedSize);
					TArrayView<const uint8> InitializationVector(ChunkAccessHeader->SHAHash.Hash, AES256_GCM_InitializationVectorSizeInBytes);
					bool bEncryptResult = false;
					TempEncryptedData = Crypto->Encrypt_AES_256_GCM(SourceData, *WriteEncryptionSecretKey, InitializationVector, TempEncryptionAuthTag, bEncryptResult);
					if (bEncryptResult)
					{
						ChunkDataSource = TempEncryptedData.GetData();
					}
					else
					{
						SaveResult = EChunkSaveResult::EncryptionFail;
					}
				}
				else
				{
					SaveResult = EChunkSaveResult::MissingEncryptionKey;
				}
			}

			if (SaveResult == EChunkSaveResult::Success)
			{
				// Setup Header.
				ChunkAccessHeader->Version = (uint32)HeaderHelpers::FeatureLevelToChunkVersion(Config.FeatureLevel);
				const int64 StartPos = Writer.Tell();
				Writer << *ChunkAccessHeader;
				ChunkAccessHeader->HeaderSize = Writer.Tell() - StartPos;
				ChunkAccessHeader->StoredAs = bDataIsCompressed ? EChunkStorageFlags::Compressed : EChunkStorageFlags::RawData;
				if (bDataIsEncrypted)
				{
					ChunkAccessHeader->StoredAs |= EChunkStorageFlags::Encrypted;
					ChunkAccessHeader->EncryptionSecretId = Config.WriteEncryptionSecretId;
					FMemory::Memcpy(ChunkAccessHeader->AESAuthTag.AuthTag, TempEncryptionAuthTag.GetData(), AES256_GCM_AuthTagSizeInBytes);
				}
				ChunkAccessHeader->DataSizeCompressed = CompressedSize;
				// Make sure we at least have rolling hash.
				if ((ChunkAccessHeader->HashType & EChunkHashFlags::RollingPoly64) == EChunkHashFlags::None)
				{
					ChunkAccessHeader->HashType = EChunkHashFlags::RollingPoly64;
				}

				// Write out files.
				Writer.Seek(StartPos);
				Writer << *ChunkAccessHeader;
				Writer.Serialize(ChunkDataSource, CompressedSize);
				if (Writer.IsError())
				{
					SaveResult = EChunkSaveResult::SerializationError;
				}
			}
			ChunkDataAccess->ReleaseDataLock();
			return SaveResult;
		}

	private:
		IFileSystem* const FileSystem;
		ICrypto* const Crypto;
		const FChunkDataSerializationConfig Config;
		const TArray<uint8>* const WriteEncryptionSecretKey;
		// UE5 MERGE TODO: ICloudChunkSourceStat* CloudChunkSourceStat;
	};

	IChunkDataSerialization* FChunkDataSerializationFactory::Create(IFileSystem* FileSystem, ICrypto* Crypto, FChunkDataSerializationConfig Config)
	{
		check(FileSystem != nullptr);
		return new FChunkDataSerialization(FileSystem, Crypto, Config);
	}
}
