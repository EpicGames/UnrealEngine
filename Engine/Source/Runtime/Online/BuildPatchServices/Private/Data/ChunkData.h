// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Memory/MemoryFwd.h"
#include "Misc/SecureHash.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"

#include "Common/Crypto.h"
#include "BuildPatchFeatureLevel.h"

namespace BuildPatchServices
{
	class IFileSystem;
	class ICloudChunkSourceStat;

	// Constant for the legacy fixed chunk window size, which was 1MiB.
	static const uint32 LegacyFixedChunkWindow = 1024 * 1024;

	/**
	 * Declares flags for chunk headers which specify storage types.
	 */
	enum class EChunkStorageFlags : uint8
	{
		RawData = 0x00,

		// Flag for compressed data.
		Compressed = 0x01,

		// Flag for encrypted. If also compressed, decrypt first. Encryption will ruin compressibility.
		Encrypted = 0x02
	};
	ENUM_CLASS_FLAGS(EChunkStorageFlags);

	/**
	 * Declares flags for chunk headers which specify storage types.
	 */
	enum class EChunkHashFlags : uint8
	{
		None = 0x00,

		// Flag for FRollingHash class used, stored in RollingHash on header.
		RollingPoly64 = 0x01,

		// Flag for FSHA1 class used, stored in SHAHash on header.
		Sha1 = 0x02,

		// Flag for AES AuthTag being present in header header.
		AesAuthTag = 0x02,
	};
	ENUM_CLASS_FLAGS(EChunkHashFlags);

	/**
	 * Enum which describes success, or the reason for failure when loading a chunk.
	 */
	enum class EChunkLoadResult : uint8
	{
		Success = 0,

		// Failed to open the file to load the chunk.
		OpenFileFail,

		// Could not serialize due to wrong archive type.
		BadArchive,

		// The header in the loaded chunk was invalid.
		CorruptHeader,

		// The expected file size in the header did not match the size of the file.
		IncorrectFileSize,

		// The storage type of the chunk is not one which we support.
		UnsupportedStorage,

		// The hash information was missing.
		MissingHashInfo,

		// The serialized data was not successfully understood.
		SerializationError,

		// The data was saved compressed but decompression failed.
		DecompressFailure,

		// The expected data hash in the header did not match the hash of the data.
		HashCheckFailed,

		// The size of the file is too big to handle 
		FileSizeTooBig,

		// If the decryption key necessary has not been provided, this result is given.
		// It is up to the calling code to tell whether this is acceptable (expected preloading for example).
		MissingSecretKey,

		// The data was saved encrypted, and we had the secret key, but decryption failed.
		DecryptionFailure,

		// The operation was aborted.
		Aborted
	};

	/**
	 * A ToString implementation for EChunkLoadResult.
	 */
	const TCHAR* ToString(const EChunkLoadResult& ChunkLoadResult);

	/**
	 * Enum which describes success, or the reason for failure when saving a chunk.
	 */
	enum class EChunkSaveResult : uint8
	{
		Success = 0,

		// Failed to create the file for the chunk.
		FileCreateFail,

		// Could not serialize due to wrong archive type.
		BadArchive,

		// There was a serialization problem when writing to the chunk file.
		SerializationError,

		// If we were asked to encrypted saved out chunks but did not get the key, we cannot save at all.
		MissingEncryptionKey,

		// If the crypto interface returns an error.
		EncryptionFail,

		InvalidOrMax
	};

	/**
	 * A ToString implementation for EChunkSaveResult.
	 */
	const TCHAR* ToString(const EChunkSaveResult& ChunkSaveResult);

	/**
	 * Declares a struct to store the info for a chunk header.
	 */
	struct FChunkHeader
	{
		FChunkHeader();
		/**
		 * Serialization operator.
		 * @param Ar        Archive to serialize to.
		 * @param Header    Header to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<< (FArchive& Ar, FChunkHeader& Header);
		// The version of this header data.
		uint32 Version;
		// The size of this header.
		uint32 HeaderSize;
		// The GUID for this data.
		FGuid Guid;
		// An ID that identifies the encryption secret required to decrypt this chunk. This is blank if the chunk is not encrypted.
		FGuid EncryptionSecretId;
		// The size of this data compressed.
		uint32 DataSizeCompressed;
		// The size of this data uncompressed.
		uint32 DataSizeUncompressed;
		// How the chunk data is stored.
		EChunkStorageFlags StoredAs;
		// What type of hash we are using.
		EChunkHashFlags HashType;
		// The FRollingHash hashed value for this chunk data.
		uint64 RollingHash;
		// The FSHA hashed value for this chunk data.
		FSHAHash SHAHash;
		// The 16 byte AuthTag space used for AES encryption, which is an additional integrity check value.
		FAESAuthTag AESAuthTag;
	};

	// Some helper constants for dealing with chunks that are full of one single byte, usually padding.
	namespace PaddingChunk
	{
		// The A, B, and C components of a chunk Guid indicating that this is a padding chunk. D would be the actual byte padded with.
		static const int32 ChunkIdA = 0x00000001;
		static const int32 ChunkIdB = 0x00000000;
		static const int32 ChunkIdC = 0x00000000;
		// The size of the chunk we use to save out, which would allow a legacy client to actually use one.
		static const uint32 ChunkSize = LegacyFixedChunkWindow;
		/**
		 * Get whether this chunk part refers to a special cased padding chunk.
		 * @return true if this chunk is padding only.
		 */
		FORCEINLINE bool IsPadding(const FGuid& Guid)
		{
			return Guid.A == ChunkIdA && Guid.B == ChunkIdB && Guid.C == ChunkIdC && Guid.D >= 0 && Guid.D <= 255;
		}
		/**
		 * For padding chunks, returns the byte that is padded with.
		 * @return the byte used to pad data.
		 */
		FORCEINLINE uint8 GetPaddingByte(const FGuid& Guid)
		{
			check(IsPadding(Guid));
			return Guid.D;
		}
		/**
		 * For padding chunks, returns the byte that is padded with.
		 * @return the byte used to pad data.
		 */
		FORCEINLINE FGuid MakePaddingGuid(uint8 Byte)
		{
			return FGuid(ChunkIdA, ChunkIdB, ChunkIdC, Byte);
		}
	}

	/**
	 * A data structure describing the part of a chunk used to construct a file
	 */
	struct FChunkPart
	{
		FChunkPart();
		FChunkPart(const FGuid& Guid, const uint32 Offset, const uint32 Size);
		/**
		 * Serialization operator.
		 * @param Ar        Archive to serialize to.
		 * @param ChunkPart FChunkPart to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FChunkPart& ChunkPart);
		/**
		 * Get whether this chunk part refers to a special cased padding chunk.
		 * @return true if this chunk is padding only.
		 */
		bool IsPadding() const
		{
			return PaddingChunk::IsPadding(Guid);
		}
		/**
		 * For padding chunks, returns the byte that is padded with.
		 * @return the byte used to pad data.
		 */
		uint8 GetPaddingByte() const
		{
			return PaddingChunk::GetPaddingByte(Guid);
		}
		// The GUID of the chunk containing this part.
		FGuid Guid;
		// The offset of the first byte into the chunk.
		uint32 Offset;
		// The size of this part.
		uint32 Size;
		// Determine whether the chunk is from delta file or manifest (default).
		bool bChunkFromDelta = false;
	};

	/**
	 * Declares a struct to store the info about a piece of a chunk that is inside a file
	 */
	struct FFileChunkPart
	{
		FFileChunkPart();
		// The file containing this piece
		FString Filename;
		// The offset into the file of this piece
		uint64 FileOffset;
		// The FChunkPart that can be salvaged from this file
		FChunkPart ChunkPart;
	};

	/**
	 * A data structure describing a chunk file
	 */
	struct FChunkInfo
	{
		// The GUID for this data.
		FGuid Guid;
		// The FRollingHash hashed value for this chunk data.
		uint64 Hash = 0;
		// The FSHA hashed value for this chunk data.
		FSHAHash ShaHash;
		// The group number this chunk divides into.
		uint8 GroupNumber = 0;
		// The size of this data compressed.
		uint32 DataSizeCompressed = 0;
		// The size of this data uncompressed (also known as window size in older algorithms).
		uint32 DataSizeUncompressed = 1048576;
		// The file download size for this chunk.
		int64 FileSize = 0;
		// The ID of the encryption secret key used by this chunk.
		FGuid EncryptionSecretId;
		// The 16 byte AuthTag space used for AES encryption, which is an additional integrity check value.
		FAESAuthTag AESAuthTag;
		// Determine whether the chunk is from delta file or manifest (default).
		bool bChunkFromDelta = false;
	};

	/**
	 * Declares a struct holding variables to identify chunk and location.
	 */
	struct FChunkLocation
	{
		FGuid ChunkId;
		uint64 ByteStart;
		uint32 ByteSize;
	};

	/**
	 * Declares a struct to store the info for a chunk database header.
	 */
	struct FChunkDatabaseHeader
	{
		FChunkDatabaseHeader();
		/**
		 * Serialization operator.
		 * @param Ar        Archive to serialize to.
		 * @param Header    Header to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<< (FArchive& Ar, FChunkDatabaseHeader& Header);
		// The version of this header data.
		uint32 Version;
		// The size of this header.
		uint32 HeaderSize;
		// The size of the following data.
		uint64 DataSize;
		// The table of contents.
		TArray<FChunkLocation> Contents;
	};

	/**
	 * An interface providing locked access to chunk data.
	 */
	class IChunkDataAccess
	{
	public:
		virtual ~IChunkDataAccess() = default;

		/**
		 * Gets the thread lock on the data, must call ReleaseDataLock when finished with data.
		 * @param OutChunkData      Receives the pointer to chunk data.
		 * @param OutChunkHeader    Receives the pointer to header.
		 */
		virtual void GetDataLock(const uint8** OutChunkData, const FChunkHeader** OutChunkHeader) const = 0;
		virtual void GetDataLock(      uint8** OutChunkData,       FChunkHeader** OutChunkHeader) = 0;

		/**
		 * Releases access to the data to allow other threads to use.
		 */
		virtual void ReleaseDataLock() const = 0;
	};

	/**
	 * A factory for creating an IChunkDataAccess instance with allocated data.
	 */
	class FChunkDataAccessFactory
	{
	public:
		/**
		 * Creates a chunk data access class.
		 * @param DataSize  The size of the data to be held in bytes.
		 * @return the new IChunkDataAccess instance created.
		 */
		static IChunkDataAccess* Create(uint32 DataSize);
	};

	/**
	 * Provides simple access to the header and data in an IChunkDataAccess, whilst obtaining and releasing the data lock
	 * within the current scope.
	 */
	struct FScopeLockedChunkData
	{
	public:
		FScopeLockedChunkData(IChunkDataAccess* ChunkDataAccess);
		~FScopeLockedChunkData();

		/**
		 * @return the pointer to the chunk header.
		 */
		FChunkHeader* GetHeader() const;
		
		/**
		 * @return the pointer to the chunk data.
		 */
		uint8* GetData() const;

	private:
		// The provided IChunkDataAccess which we lock and release.
		IChunkDataAccess* ChunkDataAccess;
		// Pointer to the header data.
		FChunkHeader* ChunkHeader;
		// Pointer to the chunk data.
		uint8* ChunkData;
	};

	/**
	 * An interface providing serialization for chunk data.
	 */
	class IChunkDataSerialization
	{
	public:
		virtual ~IChunkDataSerialization() = default;

		/**
		 * Loads a chunk from a file on disk or network.
		 * @param Filename          The full file path to the file.
		 * @param OutLoadResult     Receives the result, indicating the error reason if return value is nullptr.
		 * @return ptr to an allocated IChunkDataAccess holding the data, nullptr if failed to load.
		 */
		virtual IChunkDataAccess* LoadFromFile(const FString& Filename, EChunkLoadResult& OutLoadResult) const = 0;

		/**
		 * Loads a chunk from memory.
		 * @param Memory            The memory array.
		 * @param OutLoadResult     Receives the result, indicating the error reason if return value is nullptr.
		 * @return ptr to an allocated IChunkDataAccess holding the data, nullptr if failed to load.
		 */
		virtual IChunkDataAccess* LoadFromMemory(const TArray<uint8>& Memory, EChunkLoadResult& OutLoadResult) const = 0;

		/**
		 * Loads a chunk from an archive.
		 * @param Archive           The archive.
		 * @param OutLoadResult     Receives the result, indicating the error reason if return value is nullptr.
		 * @return ptr to an allocated IChunkDataAccess holding the data, nullptr if failed to load.
		 */
		virtual IChunkDataAccess* LoadFromArchive(FArchive& Archive, EChunkLoadResult& OutLoadResult) const = 0;

		/**
		 * Saves a chunk to a file on disk or network.
		 * @param Filename          The full file path to the file.
		 * @param ChunkDataAccess   Ptr to the chunk data to save. The header will be updated with relevant changes.
		 * @return the result, EChunkSaveResult::Success if successful.
		 */
		virtual EChunkSaveResult SaveToFile(const FString& Filename, IChunkDataAccess* ChunkDataAccess) const = 0;

		/**
		 * Saves a chunk to memory.
		 * @param Memory            The memory array.
		 * @param ChunkDataAccess   Ptr to the chunk data to save. The header will be updated with relevant changes.
		 * @return the result, EChunkSaveResult::Success if successful.
		 */
		virtual EChunkSaveResult SaveToMemory(TArray<uint8>& Memory, IChunkDataAccess* ChunkDataAccess) const = 0;

		/**
		 * Saves a chunk to an archive.
		 * @param Archive           The archive.
		 * @param ChunkDataAccess   Ptr to the chunk data to save. The header will be updated with relevant changes.
		 * @return the result, EChunkSaveResult::Success if successful.
		 */
		virtual EChunkSaveResult SaveToArchive(FArchive& Archive, IChunkDataAccess* ChunkDataAccess) const = 0;

		// In performance sensitive situations, can request the serializer to save uncompressed if implemented
		virtual EChunkSaveResult SaveToArchiveUncompressed(FArchive& Archive, IChunkDataAccess* ChunkDataAccess) const
		{
			return SaveToArchive(Archive, ChunkDataAccess);
		}

		// Read the header from the archive, make sure it's sane, and set up to decompress/hash in a separate call. This
		// is to allow the IO to be on a separate thread from decompression/hashing.
		virtual bool ValidateAndRead(FArchive& InArchive, FMutableMemoryView InDestinationBuffer, FChunkHeader& OutHeader, FUniqueBuffer& OutCompressedBuffer) const = 0;
		
		// Take the output of ValidateAndRead and process it in to an actual output chunk.
		virtual bool DecompressAndDecryptValidatedRead(const FChunkHeader& InHeader, FMutableMemoryView InDestionationBuffer, const FUniqueBuffer& InCompressedBuffer) const = 0;

		/**
		 * Injects an SHA hash for the data into the structure of a serialized chunk.
		 * @param Memory            The memory array containing the serialized chunk.
		 * @param ShaHashData       The SHA hash to inject.
		 * @return true if ShaHashData was successfully injected into memory, false otherwise.
		 */
		virtual bool InjectShaToChunkData(TArray<uint8>& Memory, const FSHAHash& ShaHashData) const = 0;
	};

	struct FChunkDataSerializationConfig
	{
		EFeatureLevel FeatureLevel = EFeatureLevel::Latest;
		TMap<FGuid,TArray<uint8>> EncryptionSecrets;
		FGuid WriteEncryptionSecretId;
		// UE5 MERGE TODO: ICloudChunkSourceStat* CloudChunkSourceStat;
	};

	/**
	 * A factory for creating an IChunkDataSerialization instance.
	 */
	class FChunkDataSerializationFactory
	{
	public:
		static IChunkDataSerialization* Create(IFileSystem* FileSystem, ICrypto* Crypto, FChunkDataSerializationConfig Config);
	};
}