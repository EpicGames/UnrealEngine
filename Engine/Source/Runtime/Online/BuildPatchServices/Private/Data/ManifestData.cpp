// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/ManifestData.h"
#include "Misc/Base64.h"
#include "Misc/Compression.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EnumClassFlags.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Algo/Accumulate.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "Data/ManifestUObject.h"
#include "BuildPatchManifest.h"
#include "BuildPatchUtil.h"

DECLARE_LOG_CATEGORY_EXTERN(LogManifestData, Log, All);
DEFINE_LOG_CATEGORY(LogManifestData);

// The manifest header magic codeword, for quick checking that the opened file is probably a manifest file.
#define MANIFEST_HEADER_MAGIC 0x44BEC00C

namespace BuildPatchServices
{

	// The constant minimum sizes for each version of a header struct. Must be updated.
	// If new member variables are added the version MUST be bumped and handled properly here,
	// and these values must never change.
	static const uint32 ManifestHeaderVersionSizes[(int32)EFeatureLevel::LatestPlusOne] =
	{
		// EFeatureLevel::Original is 37B (32b Magic, 32b HeaderSize, 32b DataSizeUncompressed, 32b DataSizeCompressed, 160b SHA1, 8b StoredAs)
		// This remained the same all up to including EFeatureLevel::StoresPrerequisiteIds.
		37, 37, 37, 37, 37, 37, 37, 37, 37, 37, 37, 37, 37, 37,
		// EFeatureLevel::StoredAsBinaryData is 41B, (296b Original, 32b Version).
		// This remained the same all up to including EFeatureLevel::StoresUninstallActions.
		41, 41, 41, 41, 41, 41, 41, 41,
		// EFeatureLevel::ChunkEncryptionSupport added a 128b SecretId GUID, plus 128b AuthTag
		73, 73, 73
	};
	static_assert((int32)EFeatureLevel::Latest == 24, "Please adjust ManifestHeaderVersionSizes values accordingly.");

	/**
	 * Enum which describes the FManifestMeta data version.
	 */
	enum class EManifestMetaVersion : uint8
	{
		Original = 0,
		SerialisesBuildId,
		SerialisesUnistallActions,
		SerialisesManifestEncryptionSecretId,

		// Always after the latest version, signifies the latest version plus 1 to allow initialization simplicity.
		LatestPlusOne,
		Latest = (LatestPlusOne - 1)
	};

	/**
	 * Enum which describes the FChunkDataList data version.
	 */
	enum class EChunkDataListVersion : uint8
	{
		Original = 0,
		SerialisesEncryptionSecretId,
		SerialisesCompressesDataSize,
		SerialisesAESAuthTag,

		// Always after the latest version, signifies the latest version plus 1 to allow initialization simplicity.
		LatestPlusOne,
		Latest = (LatestPlusOne - 1)
	};

	/**
	 * Enum which describes the FFileManifestList data version.
	 */
	enum class EFileManifestListVersion : uint8
	{
		Original = 0,
		HasMD5AndMIMEType,
		HasSHA256,

		// Always after the latest version, signifies the latest version plus 1 to allow initialization simplicity.
		LatestPlusOne,
		Latest = (LatestPlusOne - 1)
	};

	/**
	 * Enum which describes the EEncryptedDataVersion data version.
	 */
	enum class EEncryptedDataVersion : uint8
	{
		Original = 0,

		// Always after the latest version, signifies the latest version plus 1 to allow initialization simplicity.
		LatestPlusOne,
		Latest = (LatestPlusOne - 1)
	};

	namespace ManifestVersionHelpers
	{
		const TCHAR* GetChunkSubdir(EFeatureLevel FeatureLevel)
		{
			return FeatureLevel < EFeatureLevel::DataFileRenames ? TEXT("Chunks")
				 : FeatureLevel < EFeatureLevel::ChunkCompressionSupport ? TEXT("ChunksV2")
				 : FeatureLevel < EFeatureLevel::VariableSizeChunksWithoutWindowSizeChunkInfo ? TEXT("ChunksV3")
				 : FeatureLevel < EFeatureLevel::ChunkEncryptionSupport ? TEXT("ChunksV4")
				 : TEXT("ChunksV5");
		}

		const TCHAR* GetFileSubdir(EFeatureLevel FeatureLevel)
		{
			return FeatureLevel < EFeatureLevel::DataFileRenames ? TEXT("Files")
				 : FeatureLevel < EFeatureLevel::StoresChunkDataShaHashes ? TEXT("FilesV2")
				 : TEXT("FilesV3");
		}

		EManifestMetaVersion FeatureLevelToManifestMetaVersion(EFeatureLevel FeatureLevel)
		{
			static_assert((uint32)EFeatureLevel::Latest == 24, "Please adjust the switch below for new feature levels.");
			switch (FeatureLevel)
			{
			case BuildPatchServices::EFeatureLevel::Original:
			case BuildPatchServices::EFeatureLevel::CustomFields:
			case BuildPatchServices::EFeatureLevel::StartStoringVersion:
			case BuildPatchServices::EFeatureLevel::DataFileRenames:
			case BuildPatchServices::EFeatureLevel::StoresIfChunkOrFileData:
			case BuildPatchServices::EFeatureLevel::StoresDataGroupNumbers:
			case BuildPatchServices::EFeatureLevel::ChunkCompressionSupport:
			case BuildPatchServices::EFeatureLevel::StoresPrerequisitesInfo:
			case BuildPatchServices::EFeatureLevel::StoresChunkFileSizes:
			case BuildPatchServices::EFeatureLevel::StoredAsCompressedUClass:
			case BuildPatchServices::EFeatureLevel::UNUSED_0:
			case BuildPatchServices::EFeatureLevel::UNUSED_1:
			case BuildPatchServices::EFeatureLevel::StoresChunkDataShaHashes:
			case BuildPatchServices::EFeatureLevel::StoresPrerequisiteIds:
			case BuildPatchServices::EFeatureLevel::StoredAsBinaryData:
			case BuildPatchServices::EFeatureLevel::VariableSizeChunksWithoutWindowSizeChunkInfo:
			case BuildPatchServices::EFeatureLevel::VariableSizeChunks:
			case BuildPatchServices::EFeatureLevel::UsesRuntimeGeneratedBuildId:
				return EManifestMetaVersion::Original;
			case BuildPatchServices::EFeatureLevel::UsesBuildTimeGeneratedBuildId:
			case BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType:
			case BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes:
				return EManifestMetaVersion::SerialisesBuildId;
			case BuildPatchServices::EFeatureLevel::StoresUninstallActions:
				return EManifestMetaVersion::SerialisesUnistallActions;
			case BuildPatchServices::EFeatureLevel::ChunkEncryptionSupport:
			case BuildPatchServices::EFeatureLevel::ChunksStoredBySecret:
			case BuildPatchServices::EFeatureLevel::ManifestEncryptionSupport:
				return EManifestMetaVersion::SerialisesManifestEncryptionSecretId;
			}
			checkf(false, TEXT("Unhandled FeatureLevel %s"), LexToString(FeatureLevel));
			return EManifestMetaVersion::Latest;
		}

		EChunkDataListVersion FeatureLevelToChunkDataListVersion(EFeatureLevel FeatureLevel)
		{
			static_assert((uint32)EFeatureLevel::Latest == 24, "Please adjust the switch below for new feature levels.");
			switch (FeatureLevel)
			{
			case BuildPatchServices::EFeatureLevel::Original:
			case BuildPatchServices::EFeatureLevel::CustomFields:
			case BuildPatchServices::EFeatureLevel::StartStoringVersion:
			case BuildPatchServices::EFeatureLevel::DataFileRenames:
			case BuildPatchServices::EFeatureLevel::StoresIfChunkOrFileData:
			case BuildPatchServices::EFeatureLevel::StoresDataGroupNumbers:
			case BuildPatchServices::EFeatureLevel::ChunkCompressionSupport:
			case BuildPatchServices::EFeatureLevel::StoresPrerequisitesInfo:
			case BuildPatchServices::EFeatureLevel::StoresChunkFileSizes:
			case BuildPatchServices::EFeatureLevel::StoredAsCompressedUClass:
			case BuildPatchServices::EFeatureLevel::UNUSED_0:
			case BuildPatchServices::EFeatureLevel::UNUSED_1:
			case BuildPatchServices::EFeatureLevel::StoresChunkDataShaHashes:
			case BuildPatchServices::EFeatureLevel::StoresPrerequisiteIds:
			case BuildPatchServices::EFeatureLevel::StoredAsBinaryData:
			case BuildPatchServices::EFeatureLevel::VariableSizeChunksWithoutWindowSizeChunkInfo:
			case BuildPatchServices::EFeatureLevel::VariableSizeChunks:
			case BuildPatchServices::EFeatureLevel::UsesRuntimeGeneratedBuildId:
			case BuildPatchServices::EFeatureLevel::UsesBuildTimeGeneratedBuildId:
			case BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType:
			case BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes:
			case BuildPatchServices::EFeatureLevel::StoresUninstallActions:
				return EChunkDataListVersion::Original;
			case BuildPatchServices::EFeatureLevel::ChunkEncryptionSupport:
			case BuildPatchServices::EFeatureLevel::ChunksStoredBySecret:
			case BuildPatchServices::EFeatureLevel::ManifestEncryptionSupport:
				return EChunkDataListVersion::SerialisesAESAuthTag;
			}
			checkf(false, TEXT("Unhandled FeatureLevel %s"), LexToString(FeatureLevel));
			return EChunkDataListVersion::Latest;
		}

		EFileManifestListVersion FeatureLevelToFileManifestListVersion(EFeatureLevel FeatureLevel)
		{
			static_assert((uint32)EFeatureLevel::Latest == 24, "Please adjust the switch below for new feature levels.");
			switch (FeatureLevel)
			{
				case BuildPatchServices::EFeatureLevel::Original:
				case BuildPatchServices::EFeatureLevel::CustomFields:
				case BuildPatchServices::EFeatureLevel::StartStoringVersion:
				case BuildPatchServices::EFeatureLevel::DataFileRenames:
				case BuildPatchServices::EFeatureLevel::StoresIfChunkOrFileData:
				case BuildPatchServices::EFeatureLevel::StoresDataGroupNumbers:
				case BuildPatchServices::EFeatureLevel::ChunkCompressionSupport:
				case BuildPatchServices::EFeatureLevel::StoresPrerequisitesInfo:
				case BuildPatchServices::EFeatureLevel::StoresChunkFileSizes:
				case BuildPatchServices::EFeatureLevel::StoredAsCompressedUClass:
				case BuildPatchServices::EFeatureLevel::UNUSED_0:
				case BuildPatchServices::EFeatureLevel::UNUSED_1:
				case BuildPatchServices::EFeatureLevel::StoresChunkDataShaHashes:
				case BuildPatchServices::EFeatureLevel::StoresPrerequisiteIds:
				case BuildPatchServices::EFeatureLevel::StoredAsBinaryData:
				case BuildPatchServices::EFeatureLevel::VariableSizeChunksWithoutWindowSizeChunkInfo:
				case BuildPatchServices::EFeatureLevel::VariableSizeChunks:
				case BuildPatchServices::EFeatureLevel::UsesRuntimeGeneratedBuildId:
				case BuildPatchServices::EFeatureLevel::UsesBuildTimeGeneratedBuildId:
					return EFileManifestListVersion::Original;
				case BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType:
					return EFileManifestListVersion::HasMD5AndMIMEType;
				case BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes:
				case BuildPatchServices::EFeatureLevel::StoresUninstallActions:
				case BuildPatchServices::EFeatureLevel::ChunkEncryptionSupport:
				case BuildPatchServices::EFeatureLevel::ChunksStoredBySecret:
				case BuildPatchServices::EFeatureLevel::ManifestEncryptionSupport:
					return EFileManifestListVersion::HasSHA256;
			}
			checkf(false, TEXT("Unhandled FeatureLevel %s"), LexToString(FeatureLevel));
			return EFileManifestListVersion::Latest;
		}

		EEncryptedDataVersion FeatureLevelToEncryptedDataVersion(EFeatureLevel FeatureLevel)
		{
			static_assert((uint32)EFeatureLevel::Latest == 24, "Please adjust the switch below for new feature levels.");
			switch (FeatureLevel)
			{
				case BuildPatchServices::EFeatureLevel::Original:
				case BuildPatchServices::EFeatureLevel::CustomFields:
				case BuildPatchServices::EFeatureLevel::StartStoringVersion:
				case BuildPatchServices::EFeatureLevel::DataFileRenames:
				case BuildPatchServices::EFeatureLevel::StoresIfChunkOrFileData:
				case BuildPatchServices::EFeatureLevel::StoresDataGroupNumbers:
				case BuildPatchServices::EFeatureLevel::ChunkCompressionSupport:
				case BuildPatchServices::EFeatureLevel::StoresPrerequisitesInfo:
				case BuildPatchServices::EFeatureLevel::StoresChunkFileSizes:
				case BuildPatchServices::EFeatureLevel::StoredAsCompressedUClass:
				case BuildPatchServices::EFeatureLevel::UNUSED_0:
				case BuildPatchServices::EFeatureLevel::UNUSED_1:
				case BuildPatchServices::EFeatureLevel::StoresChunkDataShaHashes:
				case BuildPatchServices::EFeatureLevel::StoresPrerequisiteIds:
				case BuildPatchServices::EFeatureLevel::StoredAsBinaryData:
				case BuildPatchServices::EFeatureLevel::VariableSizeChunksWithoutWindowSizeChunkInfo:
				case BuildPatchServices::EFeatureLevel::VariableSizeChunks:
				case BuildPatchServices::EFeatureLevel::UsesRuntimeGeneratedBuildId:
				case BuildPatchServices::EFeatureLevel::UsesBuildTimeGeneratedBuildId:
				case BuildPatchServices::EFeatureLevel::StoresFileMD5HashesAndMIMEType:
				case BuildPatchServices::EFeatureLevel::StoresFileSHA256Hashes:
				case BuildPatchServices::EFeatureLevel::StoresUninstallActions:
				case BuildPatchServices::EFeatureLevel::ChunkEncryptionSupport:
				case BuildPatchServices::EFeatureLevel::ChunksStoredBySecret:
				case BuildPatchServices::EFeatureLevel::ManifestEncryptionSupport:
					return EEncryptedDataVersion::Original;
			}
			checkf(false, TEXT("Unhandled FeatureLevel %s"), LexToString(FeatureLevel));
			return EEncryptedDataVersion::Latest;
		}
	}

	namespace ManifestDataHelpers
	{
		uint32 GetFullDataSize(const FManifestHeader& Header)
		{
			const bool bIsCompressed = EnumHasAllFlags(Header.StoredAs, EManifestStorageFlags::Compressed);
			return Header.HeaderSize + (bIsCompressed ? Header.DataSizeCompressed : Header.DataSizeUncompressed);
		}

		TUniquePtr<FArchive> CreateMemoryArchive(bool bIsLoading, TArray<uint8>& Memory)
		{
			if (bIsLoading)
			{
				return TUniquePtr<FArchive>(new FMemoryReader(Memory));
			}
			else
			{
				return TUniquePtr<FArchive>(new FMemoryWriter(Memory));
			}
		}

		EManifestStorageFlags CompressManifestMemory(TArray<uint8>& Data)
		{
			const uint32 DataSizeUncompressed = Data.Num();
			// Compression format selection - we only have one right now.
			const FName CompressionFormat = NAME_Zlib;
			const ECompressionFlags CompressionFlags = ECompressionFlags::COMPRESS_BiasMemory;

			// Sanely clamp compression effort.
			int MinCompressionBytes = 64 * 1024;
			int MinCompressionPct = 10;
			GConfig->GetInt(TEXT("BuildPatchTool"), TEXT("MinCompressionBytes"), MinCompressionBytes, GEngineIni);
			GConfig->GetInt(TEXT("BuildPatchTool"), TEXT("MinCompressionPct"), MinCompressionPct, GEngineIni);

			// Perform compression
			TArray<uint8> TempCompressed;
			uint32 DataSizeCompressed = FCompression::CompressMemoryBound(CompressionFormat, DataSizeUncompressed, CompressionFlags);
			TempCompressed.AddUninitialized(DataSizeCompressed);
			const bool bDataIsCompressed = FCompression::CompressMemoryIfWorthDecompressing(
				CompressionFormat,
				MinCompressionBytes, // bytes is the minimum worth compressing.
				MinCompressionPct, // percent is the minimum worth compressing.
				TempCompressed.GetData(),
				(int32&)DataSizeCompressed,
				Data.GetData(),
				Data.Num(),
				CompressionFlags);
			EManifestStorageFlags StoredAs;
			if (bDataIsCompressed)
			{
				TempCompressed.SetNum(DataSizeCompressed, EAllowShrinking::No);
				Data = MoveTemp(TempCompressed);
				StoredAs = EManifestStorageFlags::Compressed;
			}
			else
			{
				DataSizeCompressed = Data.Num();
				StoredAs = EManifestStorageFlags::RawData;
			}
			return StoredAs;
		}

		bool UncompressManifestMemory(TArray<uint8>& Data, uint32 DataSizeUncompressed)
		{
			// Compression format selection - we only have one right now.
			const FName CompressionFormat = NAME_Zlib;
			const ECompressionFlags CompressionFlags = ECompressionFlags::COMPRESS_BiasMemory;

			TArray<uint8> CompressedData = MoveTemp(Data);
			Data.AddUninitialized(DataSizeUncompressed);
			return FCompression::UncompressMemory(
				CompressionFormat,
				Data.GetData(),
				Data.Num(),
				CompressedData.GetData(),
				CompressedData.Num(),
				CompressionFlags);
		}

		FString ObfuscateString(const FString& String)
		{
			if (String.Len() > 0)
			{
				FSHA1 Sha;
				FSHAHash Hash;
				FTCHARToUTF8 UTF8String(*String);
				FSHA1::HashBuffer((const uint8*)UTF8String.Get(), UTF8String.Length(), Hash.Hash);
				FString ObfuscatedString;
				FBuildPatchUtils::SHAToBase32(Hash, ObfuscatedString);
				return ObfuscatedString;
			}
			return String;
		}

		TArray<FString> ObfuscateStrings(const TArray<FString>& Strings)
		{
			TArray<FString> Result = Strings;
			for (FString& Val : Result)
			{
				Val = ObfuscateString(Val);
			}
			return Result;
		}

		TSet<FString> ObfuscateStrings(const TSet<FString>& Strings)
		{
			TSet<FString> Result = Strings;
			for (FString& Val : Result)
			{
				Val = ObfuscateString(Val);
			}
			return Result;
		}
	}

	FArchive& operator<<(FArchive& Ar, FSHA256Signature& Hash)
	{
		Ar.Serialize(&Hash.Signature, sizeof(Hash.Signature));
		return Ar;
	}

	/* FManifestHeader - The header for a compressed/encoded manifest file.
	*****************************************************************************/

	FManifestHeader::FManifestHeader()
		: Version(EFeatureLevel::Latest)
		, HeaderSize(0)
		, DataSizeCompressed(0)
		, DataSizeUncompressed(0)
		, StoredAs(EManifestStorageFlags::RawData)
		, SHAHash()
	{
	}

	FArchive& operator<<(FArchive& Ar, FManifestHeader& Header)
	{
		if (Ar.IsError())
		{
			return Ar;
		}
		// Calculate how much space left in the archive for reading data ( will be 0 when writing ).
		const int64 StartPos = Ar.Tell();
		const int64 ArchiveSizeLeft = Ar.TotalSize() - StartPos;
		uint32 ExpectedSerializedBytes = 0;
		// Make sure the archive has enough data to read from, or we are saving instead.
		bool bSuccess = Ar.IsSaving() || (ArchiveSizeLeft >= ManifestHeaderVersionSizes[(int32)EFeatureLevel::Original]);
		if (bSuccess)
		{
			// Start by loading the first version we had.
			uint32 Magic = MANIFEST_HEADER_MAGIC;
			uint8 StoredAs = (uint8)Header.StoredAs;
			Header.HeaderSize = ManifestHeaderVersionSizes[(int32)Header.Version];
			Ar << Magic;
			Ar << Header.HeaderSize;
			Ar << Header.DataSizeUncompressed;
			Ar << Header.DataSizeCompressed;
			Ar.Serialize(Header.SHAHash.Hash, FSHA1::DigestSize);
			Ar << StoredAs;
			Header.StoredAs = (EManifestStorageFlags)StoredAs;
			bSuccess = Magic == MANIFEST_HEADER_MAGIC && !Ar.IsError();
			ExpectedSerializedBytes = ManifestHeaderVersionSizes[(int32)EFeatureLevel::Original];

			// After the Original with no specific version serialized, the header size increased and we had a version to load.
			if (bSuccess && Header.HeaderSize > ManifestHeaderVersionSizes[(int32)EFeatureLevel::Original])
			{
				int32 Version = (int32)Header.Version;
				Ar << Version;
				Header.Version = (EFeatureLevel)Version;
				bSuccess = !Ar.IsError();
				ExpectedSerializedBytes = ManifestHeaderVersionSizes[(int32)EFeatureLevel::StoredAsBinaryData];
			}
			// Otherwise, this header was at the version for a UObject class before this code refactor.
			else if (bSuccess && Ar.IsLoading())
			{
				Header.Version = EFeatureLevel::StoredAsCompressedUClass;
			}

			// With encryption support, came the encryption secretID in the manifest header
			// We also stored the AuthTag for the manifest.
			if (!Ar.IsError() && Header.Version >= EFeatureLevel::ChunkEncryptionSupport)
			{
				Ar << Header.EncryptionSecretId;
				Ar.Serialize(Header.EncryptionAuthTag, AES256_GCM_AuthTagSizeInBytes);
				ExpectedSerializedBytes = ManifestHeaderVersionSizes[(int32)EFeatureLevel::ChunkEncryptionSupport];
			}

			//// Here we would check for later data versions to serialise additional values.
			//if (!Ar.IsError() && Header.Version >= EFeatureLevel::SomeShinyNewVersion)
			//{
			//	Ar << Header.SomeShinyNewVariable;
			//	ExpectedSerializedBytes = ManifestHeaderVersionSizes[(int32)EFeatureLevel::SomeShinyNewVersion];
			//}
		}

		// Make sure the expected number of bytes were serialized. In practice this will catch errors where type
		// serialization operators changed their format and that will need investigating.
		bSuccess = bSuccess && (Ar.Tell() - StartPos) == ExpectedSerializedBytes;

		if (bSuccess)
		{
			// Make sure the archive now points to data location.
			Ar.Seek(StartPos + Header.HeaderSize);
		}
		else
		{
			// If we had a serialization error when loading, zero out the header values.
			if (Ar.IsLoading())
			{
				FMemory::Memzero(Header);
			}
			Ar.SetError();
		}

		return Ar;
	}

	/* FManifestMeta - The data implementation for a build meta data.
	*****************************************************************************/

	FManifestMeta::FManifestMeta()
		: FeatureLevel(BuildPatchServices::EFeatureLevel::Invalid)
		, bIsFileData(false)
		, AppID(INDEX_NONE)
		, BuildId(FBuildPatchUtils::GenerateNewBuildId())
	{
	}

	FArchive& operator<<(FArchive& Ar, FManifestMeta& Meta)
	{
		if (Ar.IsError())
		{
			return Ar;
		}

		// Serialise the data header type values.
		const int64 StartPos = Ar.Tell();
		uint32 DataSize = 0;
		EManifestMetaVersion DataVersion = Ar.IsSaving() ? ManifestVersionHelpers::FeatureLevelToManifestMetaVersion(Meta.FeatureLevel) : EManifestMetaVersion::Latest;
		{
			uint8 DataVersionInt = (uint8)DataVersion;
			Ar << DataSize;
			Ar << DataVersionInt;
			DataVersion = (EManifestMetaVersion)DataVersionInt;
		}

		// Serialise the ManifestMetaVersion::Original version variables.
		if (!Ar.IsError() && DataVersion >= EManifestMetaVersion::Original)
		{
			int32 FeatureLevelInt = (int32)Meta.FeatureLevel;
			uint8 IsFileDataInt = Meta.bIsFileData ? 1 : 0;
			Ar << FeatureLevelInt;
			Ar << IsFileDataInt;
			Ar << Meta.AppID;
			Ar << Meta.AppName;
			Ar << Meta.BuildVersion;
			Ar << Meta.LaunchExe;
			Ar << Meta.LaunchCommand;
			Ar << Meta.PrereqIds;
			Ar << Meta.PrereqName;
			Ar << Meta.PrereqPath;
			Ar << Meta.PrereqArgs;
			Meta.FeatureLevel = (EFeatureLevel)FeatureLevelInt;
			Meta.bIsFileData = IsFileDataInt == 1;
		}

		// Serialise the BuildId.
		if (!Ar.IsError() && DataVersion >= EManifestMetaVersion::SerialisesBuildId)
		{
			Ar << Meta.BuildId;
		}
		// Otherwise, initialise with backwards compatible default when loading.
		else if (!Ar.IsError() && Ar.IsLoading())
		{
			Meta.BuildId = FBuildPatchUtils::GetBackwardsCompatibleBuildId(Meta);
		}

		// Serialise the Uninstall Actions.
		if (!Ar.IsError() && DataVersion >= EManifestMetaVersion::SerialisesUnistallActions)
		{
			Ar << Meta.UninstallActionPath;
			Ar << Meta.UninstallActionArgs;
		}

		//// Here we would check for later data versions to serialise additional values.
		//if (!Ar.IsError() && DataVersion >= ManifestMetaVersion::SomeShinyNewVersion)
		//{
		//	Ar << Meta.SomeShinyNewVariable;
		//}

		// If saving, we need to go back and set the data size.
		if (!Ar.IsError() && Ar.IsSaving())
		{
			const int64 EndPos = Ar.Tell();
			DataSize = EndPos - StartPos;
			Ar.Seek(StartPos);
			Ar << DataSize;
			Ar.Seek(EndPos);
		}

		// We must always make sure to seek the archive to the correct end location.
		Ar.Seek(StartPos + DataSize);
		return Ar;
	}

	/* FChunkDataList - The data implementation for a list of referenced chunk data.
	*****************************************************************************/

	FChunkDataList::FChunkDataList()
		: FeatureLevel(BuildPatchServices::EFeatureLevel::Invalid)
	{
	}

	FArchive& operator<<(FArchive& Ar, FChunkDataList& ChunkDataList)
	{
		if (Ar.IsError())
		{
			return Ar;
		}

		// Serialise the data header type values.
		const int64 StartPos = Ar.Tell();
		uint32 DataSize = 0;
		EChunkDataListVersion DataVersion = Ar.IsSaving() ? ManifestVersionHelpers::FeatureLevelToChunkDataListVersion(ChunkDataList.FeatureLevel) : EChunkDataListVersion::Latest;
		int32 ElementCount = ChunkDataList.ChunkList.Num();
		{
			uint8 DataVersionInt = (uint8)DataVersion;
			Ar << DataSize;
			Ar << DataVersionInt;
			Ar << ElementCount;
			DataVersion = (EChunkDataListVersion)DataVersionInt;
		}

		// Make sure we have the right number of defaulted structs.
		ChunkDataList.ChunkList.AddDefaulted(ElementCount - ChunkDataList.ChunkList.Num());
		checkf(ElementCount == ChunkDataList.ChunkList.Num(), TEXT("Programmer error with count and array initialisation sync up."));

		// For a struct list type of data, we serialise every variable as it's own flat list.
		// This makes it very simple to handle or skip, extra variables added to the struct later.

		// Serialise the ManifestMetaVersion::Original version variables.
		if (!Ar.IsError() && DataVersion >= EChunkDataListVersion::Original)
		{
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.Guid; }
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.Hash; }
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.ShaHash; }
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.GroupNumber; }
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.DataSizeUncompressed; }
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.FileSize; }
		}

		// If we also have encryption, serialise these members.
		if (!Ar.IsError() && DataVersion >= EChunkDataListVersion::SerialisesEncryptionSecretId)
		{
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.EncryptionSecretId; }
		}
		if (!Ar.IsError() && DataVersion >= EChunkDataListVersion::SerialisesCompressesDataSize)
		{
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.DataSizeCompressed; }
		}
		if (!Ar.IsError() && DataVersion >= EChunkDataListVersion::SerialisesAESAuthTag)
		{
			for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar.Serialize(ChunkInfo.AESAuthTag.AuthTag, AES256_GCM_AuthTagSizeInBytes); }
		}

		//// Here we would check for later data versions to serialise additional values.
		//if (!Ar.IsError() && DataVersion >= EChunkDataListVersion::SomeShinyNewVersion)
		//{
		//	for (FChunkInfo& ChunkInfo : ChunkDataList.ChunkList) { Ar << ChunkInfo.SomeShinyNewVariable; }
		//}

		// If saving, we need to go back and set the data size.
		if (!Ar.IsError() && Ar.IsSaving())
		{
			const int64 EndPos = Ar.Tell();
			DataSize = EndPos - StartPos;
			Ar.Seek(StartPos);
			Ar << DataSize;
			Ar.Seek(EndPos);
		}

		// We must always make sure to seek the archive to the correct end location.
		Ar.Seek(StartPos + DataSize);
		return Ar;
	}

	/* FFileManifests - The data implementation for a list of file manifests.
	*****************************************************************************/

	FFileManifest::FFileManifest()
		: FileMetaFlags(EFileMetaFlags::None)
		, FileSize(0)
	{
	}

	/* FFileManifestList - The data implementation for a list of referenced files.
	*****************************************************************************/

	FFileManifestList::FFileManifestList()
		: FeatureLevel(BuildPatchServices::EFeatureLevel::Invalid)
	{
	}

	void FFileManifestList::OnPostLoad()
	{
		for (FFileManifest& FileManifest : FileList)
		{
			FileManifest.FileSize = Algo::Accumulate<int64>(FileManifest.ChunkParts, 0, [](int64 Count, const FChunkPart& ChunkPart){ return Count + ChunkPart.Size; });
		}
	}

	FArchive& operator<<(FArchive& Ar, FFileManifestList& FileDataList)
	{
		if (Ar.IsError())
		{
			return Ar;
		}

		// Serialise the data header type values.
		const int64 StartPos = Ar.Tell();
		uint32 DataSize = 0;
		EFileManifestListVersion DataVersion = Ar.IsSaving() ? ManifestVersionHelpers::FeatureLevelToFileManifestListVersion(FileDataList.FeatureLevel) : EFileManifestListVersion::Latest;
		int32 ElementCount = FileDataList.FileList.Num();
		{
			uint8 DataVersionInt = (uint8)DataVersion;
			Ar << DataSize;
			Ar << DataVersionInt;
			Ar << ElementCount;
			DataVersion = (EFileManifestListVersion)DataVersionInt;
		}

		// Make sure we have the right number of defaulted structs.
		FileDataList.FileList.AddDefaulted(ElementCount - FileDataList.FileList.Num());
		checkf(ElementCount == FileDataList.FileList.Num(), TEXT("Programmer error with count and array initialisation sync up."));

		// Serialise the ManifestMetaVersion::Original version variables.
		if (!Ar.IsError() && DataVersion >= EFileManifestListVersion::Original)
		{
			for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.Filename; }
			for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.SymlinkTarget; }
			for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.SHA1Hash; }
			for (FFileManifest& FileManifest : FileDataList.FileList)
			{
				uint8 FileMetaFlagsInt = (uint8)FileManifest.FileMetaFlags;
				Ar << FileMetaFlagsInt;
				FileManifest.FileMetaFlags = (EFileMetaFlags)FileMetaFlagsInt;
			}
			for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.InstallTags; }
			for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.ChunkParts; }
		}

		// Serialise the ManifestMetaVersion::HasMD5AndMIMEType version variables.
		if (!Ar.IsError() && DataVersion >= EFileManifestListVersion::HasMD5AndMIMEType)
		{
			for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.MD5Hash; }
			for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.MIMEType; }
		}

		// Serialise the ManifestMetaVersion::HasSHA256 version variables.
		if (!Ar.IsError() && DataVersion >= EFileManifestListVersion::HasSHA256)
		{
			for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.SHA256Hash; }
		}

		//// Here we would check for later data versions to serialise additional values.
		//if (!Ar.IsError() && DataVersion >= EFileManifestListVersion::SomeShinyNewVersion)
		//{
		//	for (FFileManifest& FileManifest : FileDataList.FileList) { Ar << FileManifest.SomeShinyNewVariable; }
		//}

		// If saving, we need to go back and set the data size.
		if (!Ar.IsError() && Ar.IsSaving())
		{
			const int64 EndPos = Ar.Tell();
			DataSize = EndPos - StartPos;
			Ar.Seek(StartPos);
			Ar << DataSize;
			Ar.Seek(EndPos);
		}

		// If loading call OnPostLoad to setup calculated values.
		if (!Ar.IsError() && Ar.IsLoading())
		{
			FileDataList.OnPostLoad();
		}

		// We must always make sure to seek the archive to the correct end location.
		Ar.Seek(StartPos + DataSize);
		return Ar;
	}

	/* FCustomFields - The data implementation for a list of custom fields.
	*****************************************************************************/

	FCustomFields::FCustomFields()
		: FeatureLevel(BuildPatchServices::EFeatureLevel::Invalid)
	{
	}

	FArchive& operator<<(FArchive& Ar, FCustomFields& CustomFields)
	{
		if (Ar.IsError())
		{
			return Ar;
		}

		// We have to convert a map to an array.
		TArray<TTuple<FString, FString>> ArrayFields;
		ArrayFields.Reserve(CustomFields.Fields.Num());
		for (TTuple<FString, FString>& Field : CustomFields.Fields)
		{
			ArrayFields.Emplace(MoveTemp(Field.Get<0>()), MoveTemp(Field.Get<1>()));
		}
		CustomFields.Fields.Empty();

		// Serialise the data header type values.
		const int64 StartPos = Ar.Tell();
		uint32 DataSize = 0;
		EChunkDataListVersion DataVersion = EChunkDataListVersion::Latest;
		int32 ElementCount = ArrayFields.Num();
		{
			uint8 DataVersionInt = (uint8)DataVersion;
			Ar << DataSize;
			Ar << DataVersionInt;
			Ar << ElementCount;
			DataVersion = (EChunkDataListVersion)DataVersionInt;
		}
		ArrayFields.AddDefaulted(ElementCount - ArrayFields.Num());
		checkf(ElementCount == ArrayFields.Num(), TEXT("Programmer error with count and array initialisation sync up."));

		// Serialise the ManifestMetaVersion::Original version variables.
		if (!Ar.IsError() && DataVersion >= EChunkDataListVersion::Original)
		{
			for (TTuple<FString, FString>& Field : ArrayFields) { Ar << Field.Get<0>(); }
			for (TTuple<FString, FString>& Field : ArrayFields) { Ar << Field.Get<1>(); }
		}

		//// Here we would check for later data versions to serialise additional values.
		//if (!Ar.IsError() && DataVersion >= EChunkDataListVersion::SomeShinyNewVersion)
		//{
		//	for (TTuple<FString, FString, FShinyNewType>& Field : ArrayFields) { Ar << Field.Get<2>(); }
		//}

		// If saving, we need to go back and set the data size.
		if (!Ar.IsError() && Ar.IsSaving())
		{
			const int64 EndPos = Ar.Tell();
			DataSize = EndPos - StartPos;
			Ar.Seek(StartPos);
			Ar << DataSize;
			Ar.Seek(EndPos);
		}

		// We convert the array back to a map.
		CustomFields.Fields.Empty(ArrayFields.Num());
		for (TTuple<FString, FString>& Field : ArrayFields)
		{
			CustomFields.Fields.Add(MoveTemp(Field.Get<0>()), MoveTemp(Field.Get<1>()));
		}
		ArrayFields.Empty();

		// We must always make sure to seek the archive to the correct end location.
		Ar.Seek(StartPos + DataSize);
		return Ar;
	}

	/* FEncryptedData - The data implementation for manifest ciphertext.
	*****************************************************************************/

	FEncryptedData::FEncryptedData()
		: FeatureLevel(BuildPatchServices::EFeatureLevel::Invalid)
	{
	}

	FArchive& operator<<(FArchive& Ar, FEncryptedData& EncryptedData)
	{
		if (Ar.IsError())
		{
			return Ar;
		}

		// We didn't start serialising any encrypted manifest data until EFeatureLevel::ManifestEncryptionSupport, so early out if necessary.
		if(EncryptedData.FeatureLevel < EFeatureLevel::ManifestEncryptionSupport)
		{
			return Ar;
		}

		// Serialise the data header type values.
		const int64 StartPos = Ar.Tell();
		uint32 DataSize = 0;
		EEncryptedDataVersion DataVersion = Ar.IsSaving() ? ManifestVersionHelpers::FeatureLevelToEncryptedDataVersion(EncryptedData.FeatureLevel) : EEncryptedDataVersion::Latest;
		{
			uint8 DataVersionInt = (uint8)DataVersion;
			Ar << DataSize;
			Ar << DataVersionInt;
			DataVersion = (EEncryptedDataVersion)DataVersionInt;
		}

		// Serialise the Original version, just the data itself.
		if (!Ar.IsError() && DataVersion >= EEncryptedDataVersion::Original)
		{
			Ar << EncryptedData.Data;
		}

		//// Here we would check for later data versions to serialise additional values.
		//if (!Ar.IsError() && DataVersion >= EEncryptedDataVersion::SomeShinyNewVersion)
		//{
		//	Ar << Meta.SomeShinyNewVariable;
		//}

		// If saving, we need to go back and set the data size.
		if (!Ar.IsError() && Ar.IsSaving())
		{
			const int64 EndPos = Ar.Tell();
			DataSize = EndPos - StartPos;
			Ar.Seek(StartPos);
			Ar << DataSize;
			Ar.Seek(EndPos);
		}

		// We must always make sure to seek the archive to the correct end location.
		Ar.Seek(StartPos + DataSize);
		return Ar;
	}

	/* FManifestData - The public interface to load/saving manifest files.
	*****************************************************************************/
	void FManifestData::Init()
	{
#if !BUILDPATCHSERVICES_NOUOBJECT
		FManifestUObject::Init();
#endif // !BUILDPATCHSERVICES_NOUOBJECT

#if DO_CHECK && !UE_BUILD_SHIPPING
		// Run tests to verify entered header sizes, asserting on failure.
		for (EFeatureLevel FeatureLevel : TEnumRange<EFeatureLevel>())
		{
			FManifestHeader Header;
			Header.Version = FeatureLevel;
			TArray<uint8> Data;
			FMemoryWriter Ar(Data);
			Ar << Header;
			check(Header.HeaderSize == Data.Num());
			check(Header.HeaderSize == ManifestHeaderVersionSizes[(int32)FeatureLevel]);
		}
#endif
	}

	bool FManifestData::Serialize(FArchive& Ar, FBuildPatchAppManifest& AppManifest, BuildPatchServices::EFeatureLevel SaveFormat)
	{
		const EFeatureLevel OriginalFormat = AppManifest.GetFeatureLevel();
		if (Ar.IsError())
		{
			return false;
		}

		// if we are trying to save a format we don't recognize, return with error
		if (Ar.IsSaving() && OriginalFormat > EFeatureLevel::Latest)
		{
			UE_LOGF(LogManifestData, Error, "FManifestData::Serialize: Trying to save an unrecognized format, Latest format %d, Original format %d",
				static_cast<uint32>(EFeatureLevel::Latest), static_cast<uint32>(OriginalFormat));
			Ar.SetError();
			return false;
		}
		bool bSuccess = true;
		// If we are saving an old format, defer to the old code!
		if (Ar.IsSaving() && SaveFormat < EFeatureLevel::StoredAsBinaryData)
		{
			bSuccess = FManifestUObject::SaveToArchive(Ar, AppManifest);
		}
		else
		{
			const int64 StartPos = Ar.Tell();
			FManifestHeader Header;
			Header.Version = SaveFormat;

			// Load header right away.
			if (Ar.IsLoading())
			{
				Ar << Header;
				bSuccess = !Ar.IsError();
				AppManifest.EncryptionSecretId = Header.EncryptionSecretId;
				FMemory::Memcpy(AppManifest.EncryptionAuthTag.AuthTag, Header.EncryptionAuthTag, AES256_GCM_AuthTagSizeInBytes);
			}
			// Otherwise set header values for saving
			else
			{
				Header.EncryptionSecretId = AppManifest.EncryptionSecretId;
				FMemory::Memcpy(Header.EncryptionAuthTag, AppManifest.EncryptionAuthTag.AuthTag, AES256_GCM_AuthTagSizeInBytes);
			}

			// If we are loading an old format, defer to the old code!
			if (Ar.IsLoading() && Header.Version < EFeatureLevel::StoredAsBinaryData)
			{
				const uint32 FullDataSize = ManifestDataHelpers::GetFullDataSize(Header);
				TArray<uint8> FullData;
				FullData.AddUninitialized(FullDataSize);
				Ar.Seek(StartPos);
				Ar.Serialize(FullData.GetData(), FullDataSize);
				bSuccess = FManifestUObject::LoadFromMemory(FullData, AppManifest);
				// Mark as should be re-saved, client that stores binary should stop using UObject class.
				AppManifest.bNeedsResaving = true;
			}
			else
			{
				TArray<uint8> ManifestRawData;
				// Fill the array with loaded data.
				if (bSuccess && Ar.IsLoading())
				{
					// DataSizeCompressed always equals the size of the data following the header, even if not compressed.
					ManifestRawData.AddUninitialized(Header.DataSizeCompressed);
					Ar.Serialize(ManifestRawData.GetData(), Header.DataSizeCompressed);
					bSuccess = !Ar.IsError();
				}
				// Uncompress from input archive.
				if (bSuccess && Ar.IsLoading() && EnumHasAllFlags(Header.StoredAs, EManifestStorageFlags::Compressed))
				{
					bSuccess = ManifestDataHelpers::UncompressManifestMemory(ManifestRawData, Header.DataSizeUncompressed);
				}
				// If loading, check the raw data SHA
				if (bSuccess && Ar.IsLoading())
				{
					FSHAHash DataHash;
					FSHA1::HashBuffer(ManifestRawData.GetData(), ManifestRawData.Num(), DataHash.Hash);
					bSuccess = DataHash == Header.SHAHash;
				}
				if (bSuccess)
				{
					// If we are saving, set the save format on each complex serialisable member
					if (Ar.IsSaving())
					{
						AppManifest.ManifestMeta.FeatureLevel = SaveFormat;
						AppManifest.ChunkDataList.FeatureLevel = SaveFormat;
						AppManifest.FileManifestList.FeatureLevel = SaveFormat;
						AppManifest.CustomFields.FeatureLevel = SaveFormat;
						AppManifest.EncryptedData.FeatureLevel = SaveFormat;
					}

					// Create the directional interface to the raw data array.
					TUniquePtr<FArchive> RawAr = ManifestDataHelpers::CreateMemoryArchive(Ar.IsLoading(), ManifestRawData);
					// Serialise each of the manifest's data members.
					*RawAr << AppManifest.ManifestMeta;

					// If we are loading, set the load format on each complex serialisable member, after getting it from the meta block.
					if (Ar.IsLoading())
					{
						AppManifest.ChunkDataList.FeatureLevel = AppManifest.ManifestMeta.FeatureLevel;
						AppManifest.FileManifestList.FeatureLevel = AppManifest.ManifestMeta.FeatureLevel;
						AppManifest.CustomFields.FeatureLevel = AppManifest.ManifestMeta.FeatureLevel;
						AppManifest.EncryptedData.FeatureLevel = AppManifest.ManifestMeta.FeatureLevel;
					}

					*RawAr << AppManifest.ChunkDataList;
					*RawAr << AppManifest.FileManifestList;
					*RawAr << AppManifest.CustomFields;
					bSuccess = !RawAr->IsError();

					// If we support encryption, serialise the encryption data if necessary.
					if (bSuccess && Header.Version >= EFeatureLevel::ManifestEncryptionSupport)
					{
						if (Header.EncryptionSecretId.IsValid())
						{
							*RawAr << AppManifest.EncryptedData;
							bSuccess = !RawAr->IsError();
							AppManifest.bIsManifestEncrypted = AppManifest.EncryptedData.Data.Num() > 0;
						}
					}

					//// Here we would check for later header versions to serialise additional structures.
					//if (bSuccess && Header.Version >= EFeatureLevel::SomeShinyNewVersion)
					//{
					//	*RawAr << AppManifest.CustomFields;
					//	bSuccess = !RawAr->IsError();
					//}

					// If we are saving, restore the original manifest format on each complex serialisable member
					if (Ar.IsSaving())
					{
						AppManifest.ManifestMeta.FeatureLevel = OriginalFormat;
						AppManifest.ChunkDataList.FeatureLevel = OriginalFormat;
						AppManifest.FileManifestList.FeatureLevel = OriginalFormat;
						AppManifest.CustomFields.FeatureLevel = OriginalFormat;
						AppManifest.EncryptedData.FeatureLevel = OriginalFormat;
					}
				}
				// If saving, calculate the raw data SHA.
				if (bSuccess && Ar.IsSaving())
				{
					FSHAHash DataHash;
					FSHA1::HashBuffer(ManifestRawData.GetData(), ManifestRawData.Num(), DataHash.Hash);
					Header.SHAHash = DataHash;
				}
				// Compress if saving.
				if (bSuccess && Ar.IsSaving())
				{
					Header.DataSizeUncompressed = ManifestRawData.Num();
					Header.StoredAs = ManifestDataHelpers::CompressManifestMemory(ManifestRawData);
					Header.DataSizeCompressed = ManifestRawData.Num();
				}
				// Set encryption info if saving.
				if (bSuccess && Ar.IsSaving() && AppManifest.EncryptionSecretId.IsValid())
				{
					Header.EncryptionSecretId = AppManifest.EncryptionSecretId;
					FMemory::Memcpy(Header.EncryptionAuthTag, AppManifest.EncryptionAuthTag.AuthTag, AES256_GCM_AuthTagSizeInBytes);
					if (AppManifest.bIsManifestEncrypted)
					{
						EnumAddFlags(Header.StoredAs, EManifestStorageFlags::Encrypted);
					}
				}
				// If saving, go back to fill out header and then write data.
				if (bSuccess && Ar.IsSaving())
				{
					Ar.Seek(StartPos);
					Ar << Header;
					Ar.Serialize(ManifestRawData.GetData(), ManifestRawData.Num());
					Ar.Flush();
					bSuccess = !Ar.IsError();
				}
				// If loading, setup manifest internal tracking.
				if (bSuccess && Ar.IsLoading())
				{
					AppManifest.FileManifestList.OnPostLoad();
					AppManifest.InitLookups();
				}
			}
			// We must always make sure to seek the archive to the correct end location, but only seek if we must, to avoid a flush.
			const int64 DataLocation = StartPos + Header.HeaderSize + Header.DataSizeCompressed;
			if (bSuccess && Ar.Tell() != DataLocation)
			{
				Ar.Seek(DataLocation);
			}
		}
		bSuccess = bSuccess && !Ar.IsError();
		if (!bSuccess)
		{
			Ar.SetError();
		}
		return bSuccess;
	}
}
