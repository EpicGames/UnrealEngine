// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/SecureHash.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/EnumRange.h"
#include "Serialization/Archive.h"

#include "Interfaces/IBuildManifest.h"
#include "Data/ChunkData.h"
#include "BuildPatchFeatureLevel.h"

class FBuildPatchAppManifest;

namespace BuildPatchServices
{
	/**
	 * Declares flags for manifest headers which specify storage types.
	 */
	enum class EManifestStorageFlags : uint8
	{
		RawData = 0x00,

		// Flag for compressed data.
		Compressed = 0x01,

		// Flag for encrypted. If also compressed, decrypt first. Encryption will ruin compressibility.
		Encrypted = 0x02
	};
	ENUM_CLASS_FLAGS(EManifestStorageFlags);

	/**
	 * Helpers for switching logic based on manifest feature version.
	 */
	namespace ManifestVersionHelpers
	{
		/**
		 * Get the chunk subdirectory for used for a specific manifest version, e.g. Chunks, ChunksV2 etc.
		 * @param FeatureLevel   The version of the manifest.
		 * @return the subdirectory name that this manifest version will access.
		 */
		const TCHAR* GetChunkSubdir(EFeatureLevel FeatureLevel);

		/**
		 * Get the file data subdirectory for used for a specific manifest version, e.g. Files, FilesV2 etc.
		 * @param FeatureLevel   The version of the manifest.
		 * @return the subdirectory name that this manifest version will access.
		 */
		const TCHAR* GetFileSubdir(EFeatureLevel FeatureLevel);
	}

	namespace ManifestDataHelpers
	{
		EManifestStorageFlags CompressManifestMemory(TArray<uint8>& Data);
		bool UncompressManifestMemory(TArray<uint8>& Data, uint32 DataSizeUncompressed);
		FString ObfuscateString(const FString& String);
		TArray<FString> ObfuscateStrings(const TArray<FString>& String);
		TSet<FString> ObfuscateStrings(const TSet<FString>& Strings);
	}


	struct FManifestHeader
	{
		FManifestHeader();
		/**
		 * Serialization operator.
		 * @param Ar        Archive to serialize to.
		 * @param Header    FManifestHeader to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FManifestHeader& Header);
		// The version of this header and manifest data format, driven by the feature level.
		EFeatureLevel Version;
		// The size of this header.
		uint32 HeaderSize;
		// The size of this data compressed.
		uint32 DataSizeCompressed;
		// The size of this data uncompressed.
		uint32 DataSizeUncompressed;
		// How the chunk data is stored.
		EManifestStorageFlags StoredAs;
		// The SHA1 hash for the manifest data that follows.
		FSHAHash SHAHash;
		// An ID that identifies the encryption secret required to decrypt certain manifest fields.
		// Without having the associated key, only chunk use, file hashes, and sizes can be understood.
		FGuid EncryptionSecretId;
		uint8 EncryptionAuthTag[AES256_GCM_AuthTagSizeInBytes];
	};

	struct FManifestMeta
	{
		FManifestMeta();
		/**
		 * Serialization operator.
		 * @param Ar        Archive to serialize to.
		 * @param Meta      FManifestMeta to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FManifestMeta& Meta);
		// The feature level support this build was created with, regardless of the serialised format.
		EFeatureLevel FeatureLevel;
		// Whether this is a legacy 'nochunks' build.
		bool bIsFileData;
		// The app id provided at generation.
		uint32 AppID;
		// The app name string provided at generation.
		FString AppName;
		// The build version string provided at generation.
		FString BuildVersion;
		// The file in this manifest designated the application executable of the build.
		// Can be an obfuscated string for encrypted manifests.
		FString LaunchExe;
		// The command line required when launching the application executable.
		// Can be an obfuscated string for encrypted manifests.
		FString LaunchCommand;
		// The set of prerequisite ids for dependencies that this build's prerequisite installer will apply.
		// Can be an obfuscated strings for encrypted manifests.
		TSet<FString> PrereqIds;
		// A display string for the prerequisite provided at generation.
		// Can be an obfuscated string for encrypted manifests.
		FString PrereqName;
		// The file in this manifest designated the launch executable of the prerequisite installer.
		// Can be an obfuscated string for encrypted manifests.
		FString PrereqPath;
		// The command line required when launching the prerequisite installer.
		// Can be an obfuscated string for encrypted manifests.
		FString PrereqArgs;
		// The path to the uninstall custom action executable.
		// Can be an obfuscated string for encrypted manifests.
		FString UninstallActionPath;
		// The arguments to the uninstall custom action executable.
		// Can be an obfuscated string for encrypted manifests.
		FString UninstallActionArgs;
		// A unique build id generated at original chunking time to identify an exact build.
		FString BuildId;
	};

	struct FChunkDataList
	{
		FChunkDataList();
		/**
		 * Serialization operator.
		 * @param Ar                Archive to serialize to.
		 * @param ChunkDataList     FChunkDataList to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FChunkDataList& ChunkDataList);
		// The feature level support this build was created with, regardless of the serialised format.
		EFeatureLevel FeatureLevel;
		// The list of chunks.
		TArray<FChunkInfo> ChunkList;
	};

	struct FFileManifest
	{
		FFileManifest();
		// The build relative filename.
		// Can be an obfuscated string for encrypted manifests.
		FString Filename;
		// Whether this is a symlink to another file.
		// Can be an obfuscated string for encrypted manifests.
		FString SymlinkTarget;
		// The file MD5.
		FMD5Hash MD5Hash;
		// The file SHA1.
		FSHAHash SHA1Hash;
		// The file SHA256.
		FSHA256Signature SHA256Hash;
		// The calculated MIME type for the file.
		FString MIMEType;
		// The flags for this file.
		EFileMetaFlags FileMetaFlags;
		// The install tags for this file.
		TArray<FString> InstallTags;
		// The list of chunk parts to stitch.
		TArray<FChunkPart> ChunkParts;
		// The size of this file.
		uint64 FileSize;
	};

	struct FFileManifestList
	{
		FFileManifestList();
		/**
		 * Serialization operator.
		 * @param Ar                Archive to serialize to.
		 * @param FileManifestList  FFileManifestList to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FFileManifestList& FileManifestList);
		// The feature level support this build was created with, regardless of the serialised format.
		EFeatureLevel FeatureLevel;
		/**
		 * Helper to sort and calculate file sizes after loading.
		 */
		void OnPostLoad();
		// The list of files.
		TArray<FFileManifest> FileList;
	};

	struct FCustomFields
	{
		FCustomFields();
		/**
		 * Serialization operator.
		 * @param Ar            Archive to serialize to.
		 * @param CustomFields  FCustomFields to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FCustomFields& CustomFields);
		// The feature level support this build was created with, regardless of the serialised format.
		EFeatureLevel FeatureLevel;
		// The map of field name to field data.
		TMap<FString, FString> Fields;
	};

	struct FEncryptedData
	{
		FEncryptedData();
		/**
		 * Serialization operator.
		 * @param Ar             Archive to serialize to.
		 * @param EncryptedData  FEncryptedData to serialize.
		 * @return Passed in archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FEncryptedData& EncryptedData);
		// The feature level support this build was created with, regardless of the serialised format.
		EFeatureLevel FeatureLevel;
		// The raw ciphertext.
		TArray<uint8> Data;
	};

	class FManifestData
	{
	public:
		static void Init();
		static bool Serialize(FArchive& Ar, FBuildPatchAppManifest& AppManifest, BuildPatchServices::EFeatureLevel SaveFormat /* Ignored for loading */ = BuildPatchServices::EFeatureLevel::Latest);
	};
}

ENUM_RANGE_BY_FIRST_AND_LAST(BuildPatchServices::EFeatureLevel, BuildPatchServices::EFeatureLevel::Original, BuildPatchServices::EFeatureLevel::Latest);
