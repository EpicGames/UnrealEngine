// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/ManifestUObject.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Algo/Accumulate.h"
#include "Algo/Transform.h"
#include "Data/ManifestData.h"
#include "BuildPatchManifest.h"
#include "Misc/Compression.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ManifestUObject)

DECLARE_LOG_CATEGORY_EXTERN(LogManifestUObject, Log, All);
DEFINE_LOG_CATEGORY(LogManifestUObject);


// The maximum number of FNames that we expect a manifest to generate. This is not a technical limitation, just a sanity check
// and can be increased if more properties are added to our manifest UObject. FNames are only used by the UOject serialization system.
#define MANIFEST_MAX_NAMES			50

/* FCustomFieldData implementation
*****************************************************************************/
FCustomFieldData::FCustomFieldData()
	: Key(TEXT(""))
	, Value(TEXT(""))
{
}

FCustomFieldData::FCustomFieldData(const FString& InKey, const FString& InValue)
	: Key(InKey)
	, Value(InValue)
{
}

/* FSHAHashData implementation
*****************************************************************************/
FSHAHashData::FSHAHashData()
{
	FMemory::Memset(Hash, 0, FSHA1::DigestSize);
}

/* FChunkInfoData implementation
*****************************************************************************/
FChunkInfoData::FChunkInfoData()
	: Guid()
	, Hash(0)
	, ShaHash()
	, FileSize(0)
	, GroupNumber(0)
{
}

/* FChunkPartData implementation
*****************************************************************************/
FChunkPartData::FChunkPartData()
	: Guid()
	, Offset(0)
	, Size(0)
{
}

/* FFileManifestData implementation
*****************************************************************************/
FFileManifestData::FFileManifestData()
	: Filename(TEXT(""))
	, bIsUnixExecutable(false)
	, SymlinkTarget(TEXT(""))
	, bIsReadOnly(false)
	, bIsCompressed(false)
{
}

/* UBuildPatchManifest implementation
*****************************************************************************/
UBuildPatchManifest::UBuildPatchManifest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ManifestFileVersion(static_cast<uint8>(BuildPatchServices::EFeatureLevel::Invalid))
	, bIsFileData(false)
	, AppID(INDEX_NONE)
	, AppName(TEXT(""))
	, BuildVersion(TEXT(""))
	, LaunchExe(TEXT(""))
	, LaunchCommand(TEXT(""))
	, PrereqIds()
	, PrereqName(TEXT(""))
	, PrereqPath(TEXT(""))
	, PrereqArgs(TEXT(""))
	, FileManifestList()
	, ChunkList()
	, CustomFields()
{
}

UBuildPatchManifest::~UBuildPatchManifest()
{
}

namespace ManifestUObjectHelpers
{
	UBuildPatchManifest* CreateUBuildPatchManifestFromAsync()
	{
		FGCScopeGuard GCGuard;
		UBuildPatchManifest* Data = NewObject<UBuildPatchManifest>();
		Data->AddToRoot();
		return Data;
	}

	BuildPatchServices::FChunkPart FromChunkPartData(const FChunkPartData& Input)
	{
		BuildPatchServices::FChunkPart Output;
		Output.Guid = Input.Guid;
		Output.Offset = Input.Offset;
		Output.Size = Input.Size;
		return Output;
	}

	BuildPatchServices::FFileManifest FromFileManifestData(const FFileManifestData& Input)
	{
		BuildPatchServices::FFileManifest Output;
		Output.Filename = Input.Filename;
		FMemory::Memcpy(Output.SHA1Hash.Hash, Input.FileHash.Hash, FSHA1::DigestSize);
		Output.ChunkParts.Empty(Input.FileChunkParts.Num());
		Algo::Transform(Input.FileChunkParts, Output.ChunkParts, &ManifestUObjectHelpers::FromChunkPartData);
		Output.InstallTags = Input.InstallTags;
		Output.SymlinkTarget = Input.SymlinkTarget;
		Output.FileMetaFlags |= Input.bIsReadOnly ? EFileMetaFlags::ReadOnly : EFileMetaFlags::None;
		Output.FileMetaFlags |= Input.bIsCompressed ? EFileMetaFlags::Compressed : EFileMetaFlags::None;
		Output.FileMetaFlags |= Input.bIsUnixExecutable ? EFileMetaFlags::UnixExecutable : EFileMetaFlags::None;
		return Output;
	}

	BuildPatchServices::FChunkInfo FromChunkInfoData(const FChunkInfoData& Input)
	{
		BuildPatchServices::FChunkInfo Output;
		Output.Guid = Input.Guid;
		Output.Hash = Input.Hash;
		FMemory::Memcpy(Output.ShaHash.Hash, Input.ShaHash.Hash, FSHA1::DigestSize);
		Output.FileSize = Input.FileSize;
		Output.GroupNumber = Input.GroupNumber;
		return Output;
	}

	FChunkPartData ToChunkPartData(const BuildPatchServices::FChunkPart& Input)
	{
		FChunkPartData Output;
		Output.Guid = Input.Guid;
		Output.Offset = Input.Offset;
		Output.Size = Input.Size;
		return Output;
	}

	FFileManifestData ToFileManifestData(const BuildPatchServices::FFileManifest& Input)
	{
		FFileManifestData Output;
		Output.Filename = Input.Filename;
		FMemory::Memcpy(Output.FileHash.Hash, Input.SHA1Hash.Hash, FSHA1::DigestSize);
		Output.FileChunkParts.Empty(Input.ChunkParts.Num());
		Algo::Transform(Input.ChunkParts, Output.FileChunkParts, &ManifestUObjectHelpers::ToChunkPartData);
		Output.InstallTags = Input.InstallTags;
		Output.bIsUnixExecutable = EnumHasAllFlags(Input.FileMetaFlags, EFileMetaFlags::UnixExecutable);
		Output.SymlinkTarget = Input.SymlinkTarget;
		Output.bIsReadOnly = EnumHasAllFlags(Input.FileMetaFlags, EFileMetaFlags::ReadOnly);
		Output.bIsCompressed = EnumHasAllFlags(Input.FileMetaFlags, EFileMetaFlags::Compressed);
		return Output;
	}

	FChunkInfoData ToChunkInfoData(const BuildPatchServices::FChunkInfo& Input)
	{
		FChunkInfoData Output;
		Output.Guid = Input.Guid;
		Output.Hash = Input.Hash;
		FMemory::Memcpy(Output.ShaHash.Hash, Input.ShaHash.Hash, FSHA1::DigestSize);
		Output.FileSize = Input.FileSize;
		Output.GroupNumber = Input.GroupNumber;
		return Output;
	}

	FCustomFieldData ToCustomFieldData(const TPair<FString, FString>& Input)
	{
		FCustomFieldData Output;
		Output.Key = Input.Key;
		Output.Value = Input.Value;
		return Output;
	}
}

/**
 * Archive for writing a manifest into memory
 */
class FManifestWriter : public FArchive
{
public:
	FManifestWriter()
		: FArchive()
		, Offset(0)
	{
		this->SetIsPersistent(false);
		this->SetIsSaving(true);
	}

	virtual void Seek(int64 InPos) override
	{
		Offset = InPos;
	}

	virtual int64 Tell() override
	{
		return Offset;
	}

	virtual FString GetArchiveName() const override
	{
		return TEXT("FManifestWriter");
	}

	virtual FArchive& operator<<(FName& N) override
	{
		if (!FNameIndexLookup.Contains(N))
		{
			const int32 ArNameIndex = FNameIndexLookup.Num();
			FNameIndexLookup.Add(N, ArNameIndex);
		}
		*this << FNameIndexLookup[N];
		return *this;
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		if (Num && !IsError())
		{
			const int64 NumBytesToAdd = Offset + Num - Bytes.Num();
			if (NumBytesToAdd > 0)
			{
				const int64 NewArrayCount = Bytes.Num() + NumBytesToAdd;
				if (NewArrayCount >= MAX_int32)
				{
					SetError();
					return;
				}
				Bytes.AddUninitialized((int32)NumBytesToAdd);
			}
			check((Offset + Num) <= Bytes.Num());
			FMemory::Memcpy(&Bytes[Offset], Data, Num);
			Offset += Num;
		}
	}

	virtual int64 TotalSize() override
	{
		return Bytes.Num();
	}

	void Finalize()
	{
		TArray<uint8> FinalData;
		FMemoryWriter NameTableWriter(FinalData);
		int32 NumNames = FNameIndexLookup.Num();
		check(NumNames <= MANIFEST_MAX_NAMES);
		NameTableWriter << NumNames;
		for (auto& MapEntry : FNameIndexLookup)
		{
			FName& Name = MapEntry.Key;
			int32& Index = MapEntry.Value;
			NameTableWriter << Name;
			NameTableWriter << Index;
		}
		FinalData.Append(Bytes);
		Bytes = MoveTemp(FinalData);
	}

	TArray<uint8>& GetBytes()
	{
		return Bytes;
	}

private:
	int64 Offset;
	TArray<uint8> Bytes;
	TMap<FName, int32> FNameIndexLookup;
};

/**
* Archive for reading a manifest from data in memory
*/
class FManifestReader : public FArchive
{
public:
	FManifestReader(const TArray<uint8>& InBytes)
		: FArchive()
		, Offset(0)
		, Bytes(InBytes)
	{
		this->SetIsPersistent(false);
		this->SetIsLoading(true);

		// Must load table immediately
		FMemoryReader NameTableReader(InBytes);
		int32 NumNames;
		NameTableReader << NumNames;

		// Check not insane, we know to expect a small number for a manifest
		if (NumNames < MANIFEST_MAX_NAMES)
		{
			FNameLookup.Empty(NumNames);
			for (; NumNames > 0; --NumNames)
			{
				FName Name;
				int32 Index;
				NameTableReader << Name;
				NameTableReader << Index;
				FNameLookup.Add(Index, Name);
			}
		}
		else
		{
			SetError();
		}
		Offset = NameTableReader.Tell();
	}

	virtual FString GetArchiveName() const override
	{
		return TEXT("FManifestReader");
	}

	virtual void Seek(int64 InPos) override
	{
		check(InPos <= Bytes.Num());
		Offset = InPos;
	}

	virtual int64 Tell() override
	{
		return Offset;
	}

	virtual FArchive& operator<<(FName& N) override
	{
		if (IsError())
		{
			N = NAME_None;
		}
		else
		{
			// Read index and lookup
			int32 ArNameIndex;
			*this << ArNameIndex;
			if (FNameLookup.Contains(ArNameIndex))
			{
				N = FNameLookup[ArNameIndex];
			}
			else
			{
				N = NAME_None;
				SetError();
			}
		}
		return *this;
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		if (Num && !IsError())
		{
			// Only serialize if we have the requested amount of data
			if (Offset + Num <= Bytes.Num())
			{
				FMemory::Memcpy(Data, &Bytes[Offset], Num);
				Offset += Num;
			}
			else
			{
				SetError();
			}
		}
	}

	virtual int64 TotalSize() override
	{
		return Bytes.Num();
	}

private:
	int64 Offset;
	const TArray<uint8>& Bytes;
	TMap<int32, FName> FNameLookup;
};

/* FManifestUObject implementation
*****************************************************************************/

void FManifestUObject::Init()
{
#if !BUILDPATCHSERVICES_NOUOBJECT
	// This fixes a potential crash if async loading manifests.
	// We make sure that NewObject<UBuildPatchManifest>() has been called from main thread before it can be called for the 'first time' concurrently
	// on multiple threads, otherwise a race condition can hit unprotected Emplace on UPackage::ClassUniqueNameIndexMap.
	// The object will be GC'd on next GC run.
	check(IsInGameThread());
	NewObject<UBuildPatchManifest>();
#endif // !BUILDPATCHSERVICES_NOUOBJECT
}

bool FManifestUObject::LoadFromMemory(const TArray<uint8>& DataInput, FBuildPatchAppManifest& AppManifest)
{
	using namespace BuildPatchServices;
#if !BUILDPATCHSERVICES_NOUOBJECT

	FMemoryReader ManifestFile(DataInput);
	FManifestHeader Header;
	ManifestFile << Header;
	const int32 SignedHeaderSize = Header.HeaderSize;
	if (!ManifestFile.IsError() && DataInput.Num() > SignedHeaderSize)
	{
		FSHAHash DataHash;
		FSHA1::HashBuffer(&DataInput[Header.HeaderSize], DataInput.Num() - Header.HeaderSize, DataHash.Hash);
		if (DataHash == Header.SHAHash)
		{
			const bool bIsCompressed = EnumHasAllFlags(Header.StoredAs, EManifestStorageFlags::Compressed);
			TArray<uint8> UncompressedData;
			if (bIsCompressed && (Header.DataSizeCompressed + Header.HeaderSize) == DataInput.Num())
			{
				UncompressedData.AddUninitialized(Header.DataSizeUncompressed);
				if (!FCompression::UncompressMemory(
					NAME_Zlib,
					UncompressedData.GetData(),
					Header.DataSizeUncompressed,
					&DataInput[Header.HeaderSize],
					DataInput.Num() - Header.HeaderSize,
					COMPRESS_BiasMemory))
				{
					return false;
				}
			}
			else if ((Header.DataSizeUncompressed + Header.HeaderSize) == DataInput.Num())
			{
				UncompressedData.Append(&DataInput[Header.HeaderSize], Header.DataSizeUncompressed);
			}
			else
			{
				return false;
			}
			FManifestReader ManifestData(UncompressedData);
			return LoadInternal(ManifestData, AppManifest);
		}
		else
		{
			return false;
		}
	}
	else
	{
		return false;
	}

#else // !BUILDPATCHSERVICES_NOUOBJECT

	UE_LOGF(LogManifestUObject, Error, "FManifestUObject::LoadFromMemory called but UObjects are disabled for BuildPatchServices");
	return false;

#endif // !BUILDPATCHSERVICES_NOUOBJECT
}

bool FManifestUObject::SaveToArchive(FArchive& Ar, const FBuildPatchAppManifest& AppManifest)
{
	using namespace BuildPatchServices;
#if !BUILDPATCHSERVICES_NOUOBJECT

	if (Ar.IsSaving())
	{
		FManifestWriter ManifestData;
		bool bSaveOk = SaveInternal(ManifestData, AppManifest);
		ManifestData.Finalize();
		if (bSaveOk && !ManifestData.IsError())
		{
			int32 DataSize = ManifestData.TotalSize();
			TArray<uint8> TempCompressed;
			TempCompressed.AddUninitialized(DataSize);
			int32 CompressedSize = DataSize;
			bool bDataIsCompressed = FCompression::CompressMemory(
				NAME_Zlib,
				TempCompressed.GetData(),
				CompressedSize,
				ManifestData.GetBytes().GetData(),
				DataSize,
				COMPRESS_BiasMemory);
			TempCompressed.SetNum(CompressedSize);

			TArray<uint8>& FileData = bDataIsCompressed ? TempCompressed : ManifestData.GetBytes();

			FManifestHeader Header;
			Header.Version = AppManifest.ManifestMeta.FeatureLevel;
			Header.StoredAs = bDataIsCompressed ? EManifestStorageFlags::Compressed : EManifestStorageFlags::RawData;
			Header.DataSizeUncompressed = DataSize;
			Header.DataSizeCompressed = bDataIsCompressed ? CompressedSize : Header.DataSizeUncompressed;
			FSHA1::HashBuffer(FileData.GetData(), FileData.Num(), Header.SHAHash.Hash);

			// Write to provided archive
			Ar << Header;
			Ar.Serialize(FileData.GetData(), FileData.Num());
		}
		else
		{
			Ar.SetError();
		}
		return !Ar.IsError();
	}
	Ar.SetError();
	return false;

#else // !BUILDPATCHSERVICES_NOUOBJECT

	UE_LOGF(LogManifestUObject, Error, "FManifestUObject::SaveToArchive called but UObjects are disabled for BuildPatchServices");
	Ar.SetError();
	return false;

#endif // !BUILDPATCHSERVICES_NOUOBJECT
}

bool FManifestUObject::LoadInternal(FArchive& Ar, FBuildPatchAppManifest& AppManifest)
{
	using namespace BuildPatchServices;
#if !BUILDPATCHSERVICES_NOUOBJECT

	UBuildPatchManifest* Data = ManifestUObjectHelpers::CreateUBuildPatchManifestFromAsync();

	// Make sure we use the correct serialization version, this is now fixed and must never use a newer version,
	// because the property tag has changed in structure meaning older clients would not read correctly.
	FPackageFileVersion Version = FPackageFileVersion::CreateUE4Version(VER_UE4_STRUCT_GUID_IN_PROPERTY_TAG - 1);
	Ar.SetUEVer(Version);

	if (Ar.IsLoading())
	{
		Data->Serialize(Ar);

		AppManifest.DestroyData();
		AppManifest.ManifestMeta.FeatureLevel = static_cast<EFeatureLevel>(Data->ManifestFileVersion);
		AppManifest.ManifestMeta.bIsFileData = MoveTemp(Data->bIsFileData);
		AppManifest.ManifestMeta.AppID = MoveTemp(Data->AppID);
		AppManifest.ManifestMeta.AppName = MoveTemp(Data->AppName);
		AppManifest.ManifestMeta.BuildVersion = MoveTemp(Data->BuildVersion);
		AppManifest.ManifestMeta.LaunchExe = MoveTemp(Data->LaunchExe);
		AppManifest.ManifestMeta.LaunchCommand = MoveTemp(Data->LaunchCommand);
		AppManifest.ManifestMeta.PrereqIds = MoveTemp(Data->PrereqIds);
		AppManifest.ManifestMeta.PrereqName = MoveTemp(Data->PrereqName);
		AppManifest.ManifestMeta.PrereqPath = MoveTemp(Data->PrereqPath);
		AppManifest.ManifestMeta.PrereqArgs = MoveTemp(Data->PrereqArgs);

		AppManifest.FileManifestList.FileList.Empty(Data->FileManifestList.Num());
		Algo::Transform(Data->FileManifestList, AppManifest.FileManifestList.FileList, &ManifestUObjectHelpers::FromFileManifestData);

		AppManifest.ChunkDataList.ChunkList.Empty(Data->ChunkList.Num());
		Algo::Transform(Data->ChunkList, AppManifest.ChunkDataList.ChunkList, &ManifestUObjectHelpers::FromChunkInfoData);

		AppManifest.CustomFields.Fields.Empty(Data->CustomFields.Num());
		for (const FCustomFieldData& CustomField : Data->CustomFields)
		{
			AppManifest.CustomFields.Fields.Add(CustomField.Key, CustomField.Value);
		}

		// If we didn't load the version number, we know it was skipped when saving therefore must be
		// the first UObject version
		if (AppManifest.ManifestMeta.FeatureLevel == EFeatureLevel::Invalid)
		{
			AppManifest.ManifestMeta.FeatureLevel = EFeatureLevel::StoredAsCompressedUClass;
		}

		// Call OnPostLoad for the file manifest list
		AppManifest.FileManifestList.OnPostLoad();

		// Setup internal lookups
		AppManifest.InitLookups();
	}
	else
	{
		Ar.SetError();
	}

	// Clear data to reduce memory usages before GC occurs
	Data->AppName.Empty();
	Data->BuildVersion.Empty();
	Data->LaunchExe.Empty();
	Data->LaunchCommand.Empty();
	Data->PrereqIds.Empty();
	Data->PrereqName.Empty();
	Data->PrereqPath.Empty();
	Data->PrereqArgs.Empty();
	Data->FileManifestList.Empty();
	Data->ChunkList.Empty();
	Data->CustomFields.Empty();

	Data->RemoveFromRoot();
	return !Ar.IsError();

#else // !BUILDPATCHSERVICES_NOUOBJECT

	UE_LOGF(LogManifestUObject, Error, "FManifestUObject::SerializeInternal called but UObjects are disabled for BuildPatchServices");
	return false;

#endif // !BUILDPATCHSERVICES_NOUOBJECT
}

bool FManifestUObject::SaveInternal(FArchive& Ar, const FBuildPatchAppManifest& AppManifest)
{
#if !BUILDPATCHSERVICES_NOUOBJECT

	UBuildPatchManifest* Data = ManifestUObjectHelpers::CreateUBuildPatchManifestFromAsync();

	// Make sure we use the correct serialization version, this is now fixed and must never use a newer version,
	// because the property tag has changed in structure meaning older clients would not read correctly.
	FPackageFileVersion Version = FPackageFileVersion::CreateUE4Version(VER_UE4_STRUCT_GUID_IN_PROPERTY_TAG - 1);
	Ar.SetUEVer(Version);

	if (Ar.IsLoading())
	{
		Ar.SetError();
	}
	else
	{
		Data->ManifestFileVersion = static_cast<uint8>(AppManifest.ManifestMeta.FeatureLevel);
		Data->bIsFileData = AppManifest.ManifestMeta.bIsFileData;
		Data->AppID = AppManifest.ManifestMeta.AppID;
		Data->AppName = AppManifest.ManifestMeta.AppName;
		Data->BuildVersion = AppManifest.ManifestMeta.BuildVersion;
		Data->LaunchExe = AppManifest.ManifestMeta.LaunchExe;
		Data->LaunchCommand = AppManifest.ManifestMeta.LaunchCommand;
		Data->PrereqIds = AppManifest.ManifestMeta.PrereqIds;
		Data->PrereqName = AppManifest.ManifestMeta.PrereqName;
		Data->PrereqPath = AppManifest.ManifestMeta.PrereqPath;
		Data->PrereqArgs = AppManifest.ManifestMeta.PrereqArgs;

		Data->FileManifestList.Empty(AppManifest.FileManifestList.FileList.Num());
		Algo::Transform(AppManifest.FileManifestList.FileList, Data->FileManifestList, &ManifestUObjectHelpers::ToFileManifestData);

		Data->ChunkList.Empty(AppManifest.ChunkDataList.ChunkList.Num());
		Algo::Transform(AppManifest.ChunkDataList.ChunkList, Data->ChunkList, &ManifestUObjectHelpers::ToChunkInfoData);

		Data->CustomFields.Empty(AppManifest.CustomFields.Fields.Num());
		for (const TPair<FString, FString>& CustomField : AppManifest.CustomFields.Fields)
		{
			Data->CustomFields.Add(ManifestUObjectHelpers::ToCustomFieldData(CustomField));
		}

		Data->Serialize(Ar);
	}

	// Clear data to reduce memory usages before GC occurs
	Data->AppName.Empty();
	Data->BuildVersion.Empty();
	Data->LaunchExe.Empty();
	Data->LaunchCommand.Empty();
	Data->PrereqIds.Empty();
	Data->PrereqName.Empty();
	Data->PrereqPath.Empty();
	Data->PrereqArgs.Empty();
	Data->FileManifestList.Empty();
	Data->ChunkList.Empty();
	Data->CustomFields.Empty();

	Data->RemoveFromRoot();
	return !Ar.IsError();

#else // !BUILDPATCHSERVICES_NOUOBJECT

	UE_LOGF(LogManifestUObject, Error, "FManifestUObject::SerializeInternal called but UObjects are disabled for BuildPatchServices");
	return false;

#endif // !BUILDPATCHSERVICES_NOUOBJECT
}

