// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Misc/SecureHash.h"
#include "ManifestUObject.generated.h"

/**
 * This file contains structures and classes specific for use with loading and saving manifests with UStruct serialization, and relies
 * on the ability to have a CoreUObject dependency.
 */

#ifndef BUILDPATCHSERVICES_NOUOBJECT
#define BUILDPATCHSERVICES_NOUOBJECT 0
#endif

class FBuildPatchAppManifest;

USTRUCT()
struct FCustomFieldData
{
public:
	GENERATED_USTRUCT_BODY()
	FCustomFieldData();
	FCustomFieldData(const FString& Key, const FString& Value);

public:
	UPROPERTY()
	FString Key;

	UPROPERTY()
	FString Value;
};

USTRUCT()
struct FSHAHashData
{
public:
	GENERATED_USTRUCT_BODY()
	FSHAHashData();

public:
	UPROPERTY()
	uint8 Hash[FSHA1::DigestSize];

};
static_assert(FSHA1::DigestSize == 20, "If this changes a lot of stuff here will break!");

USTRUCT()
struct FChunkInfoData
{
public:
	GENERATED_USTRUCT_BODY()
	FChunkInfoData();

public:
	UPROPERTY()
	FGuid Guid;

	UPROPERTY()
	uint64 Hash;

	UPROPERTY()
	FSHAHashData ShaHash;

	UPROPERTY()
	int64 FileSize;

	UPROPERTY()
	uint8 GroupNumber;
};

USTRUCT()
struct FChunkPartData
{
public:
	GENERATED_USTRUCT_BODY()
	FChunkPartData();

public:
	UPROPERTY()
	FGuid Guid;

	UPROPERTY()
	uint32 Offset;

	UPROPERTY()
	uint32 Size;
};

USTRUCT()
struct FFileManifestData
{
public:
	GENERATED_USTRUCT_BODY()
	FFileManifestData();

public:
	UPROPERTY()
	FString Filename;

	UPROPERTY()
	FSHAHashData FileHash;

	UPROPERTY()
	TArray<FChunkPartData> FileChunkParts;

	UPROPERTY()
	TArray<FString> InstallTags;

	UPROPERTY()
	bool bIsUnixExecutable;

	UPROPERTY()
	FString SymlinkTarget;

	UPROPERTY()
	bool bIsReadOnly;

	UPROPERTY()
	bool bIsCompressed;
};

UCLASS()
class UBuildPatchManifest : public UObject
{
public:
	GENERATED_UCLASS_BODY()
	~UBuildPatchManifest();

public:
	UPROPERTY()
	uint8 ManifestFileVersion;

	UPROPERTY()
	bool bIsFileData;

	UPROPERTY()
	uint32 AppID;

	UPROPERTY()
	FString AppName;

	UPROPERTY()
	FString BuildVersion;

	UPROPERTY()
	FString LaunchExe;

	UPROPERTY()
	FString LaunchCommand;

	UPROPERTY()
	TSet<FString> PrereqIds;

	UPROPERTY()
	FString PrereqName;

	UPROPERTY()
	FString PrereqPath;

	UPROPERTY()
	FString PrereqArgs;

	UPROPERTY()
	TArray<FFileManifestData> FileManifestList;

	UPROPERTY()
	TArray<FChunkInfoData> ChunkList;

	UPROPERTY()
	TArray<FCustomFieldData> CustomFields;
};

class FManifestUObject
{
public:
	static void Init();
	static bool LoadFromMemory(const TArray<uint8>& DataInput, FBuildPatchAppManifest& AppManifest);
	static bool SaveToArchive(FArchive& Ar, const FBuildPatchAppManifest& AppManifest);

private:
	static bool LoadInternal(FArchive& Ar, FBuildPatchAppManifest& AppManifest);
	static bool SaveInternal(FArchive& Ar, const FBuildPatchAppManifest& AppManifest);
};
