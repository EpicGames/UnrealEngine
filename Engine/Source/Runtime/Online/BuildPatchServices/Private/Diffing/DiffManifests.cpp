// Copyright Epic Games, Inc. All Rights Reserved.

#include "Diffing/DiffManifests.h"

#include "Algo/Transform.h"
#include "Async/Async.h"
#include "HAL/ThreadSafeBool.h"
#include "Logging/LogMacros.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"
#include "HttpModule.h"

#include "Common/ChunkDataSizeProvider.h"
#include "Common/FileSystem.h"
#include "Common/HttpManager.h"
#include "Common/SpeedRecorder.h"
#include "Common/StatsCollector.h"
#include "Installer/Statistics/DownloadServiceStatistics.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/DownloadService.h"
#include "Installer/InstallerAnalytics.h"
#include "Installer/OptimisedDelta.h"
#include "Installer/MessagePump.h"
#include "BuildPatchFileConstructor.h"
#include "BuildPatchManifest.h"
#include "BuildPatchUtil.h"
#include "Tasks/Task.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDiffManifests, Log, All);
DEFINE_LOG_CATEGORY(LogDiffManifests);

// For the output file we'll use pretty json in debug, otherwise condensed.
#if UE_BUILD_DEBUG
typedef  TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FDiffJsonWriter;
typedef  TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FDiffJsonWriterFactory;
#else
typedef  TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FDiffJsonWriter;
typedef  TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FDiffJsonWriterFactory;
#endif //UE_BUILD_DEBUG

namespace BuildPatchServices
{
	namespace DiffHelpers
	{
		struct FSimConfig
		{
		public:
			FSimConfig()
				: InstallMode(EInstallMode::DestructiveInstall)
				, DownloadSpeed(0)
				, DiskReadSpeed(0)
				, DiskWriteSpeed(0)
				, BackupSerialisationSpeed(0)
				, FileCreateTime(0)
			{ }

			BuildPatchServices::EInstallMode InstallMode;
			double DownloadSpeed;
			double DiskReadSpeed;
			double DiskWriteSpeed;
			double BackupSerialisationSpeed;
			double FileCreateTime;
		};

		TArray<double> CalculateInstallTimeCoefficient(const FBuildPatchAppManifestRef& CurrentManifest, const TSet<FString>& InCurrentTags, const FBuildPatchAppManifestRef& InstallManifest, const TSet<FString>& InInstallTags, const TArray<FSimConfig>& SimConfigs)
		{
			// Process tag setup.
			TSet<FString> CurrentTags = InCurrentTags;
			TSet<FString> InstallTags = InInstallTags;
			if (CurrentTags.Num() == 0)
			{
				CurrentManifest->GetFileTagList(CurrentTags);
			}
			if (InstallTags.Num() == 0)
			{
				InstallManifest->GetFileTagList(InstallTags);
			}
			CurrentTags.Add(TEXT(""));
			InstallTags.Add(TEXT(""));

			// Enumerate what is available in the current install.
			TSet<FString> FilesInstalled;
			TSet<FGuid>   ChunksInstalled;
			CurrentManifest->GetTaggedFileList(CurrentTags, FilesInstalled);
			{
				TSet<FGuid> ChunksReferenced;
				CurrentManifest->GetChunksRequiredForFiles(FilesInstalled, ChunksReferenced);
				CurrentManifest->EnumerateProducibleChunks(CurrentTags, ChunksReferenced, ChunksInstalled);
			}

			// Enumerate what is needed for the update.
			TSet<FString> FilesToBuild;
			TSet<FGuid>   ChunksNeeded;
			{
				TSet<FString> TaggedFiles;
				InstallManifest->GetTaggedFileList(InstallTags, TaggedFiles);
				for (FString& TaggedFile : TaggedFiles)
				{
					const FFileManifest* const OldFile = CurrentManifest->GetFileManifest(TaggedFile);
					const FFileManifest* const NewFile = InstallManifest->GetFileManifest(TaggedFile);
					if (!OldFile || OldFile->SHA1Hash != NewFile->SHA1Hash || !FilesInstalled.Contains(TaggedFile))
					{
						FilesToBuild.Add(MoveTemp(TaggedFile));
					}
				}
			}
			CurrentManifest->GetChunksRequiredForFiles(FilesToBuild, ChunksNeeded);

			// Setup a chunk reference tracker.
			TArray<FGuid> ChunkReferences;
			for (const FString& FileToBuild : FilesToBuild)
			{
				const FFileManifest* const FileManifest = InstallManifest->GetFileManifest(FileToBuild);
				for (const FChunkPart& ChunkPart : FileManifest->ChunkParts)
				{
					ChunkReferences.Add(ChunkPart.Guid);
				}
			}
			TUniquePtr<IChunkReferenceTracker> ChunkReferenceTracker(FChunkReferenceTrackerFactory::Create(ChunkReferences));

			// A private struct to simulate based on statistics configuration SimConfigs.
			struct FInstallTimeSim
			{
			public:
				FInstallTimeSim(const IChunkReferenceTracker& InChunkReferenceTracker, const FBuildPatchAppManifest& InInstallManifest, const TSet<FGuid>& InChunksInstalled, const FSimConfig& InConfig)
					: ChunkReferenceTracker(InChunkReferenceTracker)
					, InstallManifest(InInstallManifest)
					, ChunksInstalled(InChunksInstalled)
					, Config(InConfig)
					, Timer(0.0)
				{ }

				void CreateFile()
				{
					Timer += Config.FileCreateTime;
				}

				void TickDownloads()
				{
					// Complete downloads.
					while (DownloadChunks.Num() > 0 && DownloadChunks[0].Get<0>() <= Timer)
					{
						LoadedChunks.Add(DownloadChunks[0].Get<1>());
						DownloadChunks.RemoveAt(0);
					}
					// Queue up some more downloads once our in-flight list is getting emptied.
					if (DownloadChunks.Num() < 50)
					{
						TSet<FGuid> DownloadingChunks;
						Algo::Transform(DownloadChunks, DownloadingChunks, [](const TTuple<double, FGuid>& Elem) { return Elem.Get<1>(); });
						TFunction<bool(const FGuid&)> SelectPredicate = [&](const FGuid& ChunkId) { return !ChunksInstalled.Contains(ChunkId); };
						TArray<FGuid> BatchLoadChunks = ChunkReferenceTracker.SelectFromNextReferences(100, SelectPredicate);
						TFunction<bool(const FGuid&)> RemovePredicate = [&](const FGuid& ChunkId) { return DownloadingChunks.Contains(ChunkId) || LoadedChunks.Contains(ChunkId) || BackupChunks.Contains(ChunkId); };
						BatchLoadChunks.RemoveAll(RemovePredicate);
						double DownloadTime = FMath::Max(DownloadChunks.Num() > 0 ? DownloadChunks.Last().Get<0>() : 0, Timer);
						for (const FGuid& BatchLoadChunk : BatchLoadChunks)
						{
							DownloadTime += (double)InstallManifest.GetChunkInfo(BatchLoadChunk)->FileSize / Config.DownloadSpeed;
							DownloadChunks.Emplace(DownloadTime, BatchLoadChunk);
						}
					}
				}

				void GetChunk(const FChunkInfo& ChunkInfo)
				{
					if (!LoadedChunks.Contains(ChunkInfo.Guid))
					{
						if (BackupChunks.Contains(ChunkInfo.Guid))
						{
							Timer += (double)ChunkInfo.FileSize / Config.DiskReadSpeed;
							LoadedChunks.Add(ChunkInfo.Guid);
						}
						else if (ChunksInstalled.Contains(ChunkInfo.Guid))
						{
							Timer += (double)ChunkInfo.DataSizeUncompressed / Config.DiskReadSpeed;
							LoadedChunks.Add(ChunkInfo.Guid);
						}
						else
						{
							// figure out when this chunk will finish downloading
							for (int32 DownloadChunkIdx = 0; DownloadChunkIdx < DownloadChunks.Num(); ++DownloadChunkIdx)
							{
								const double& TimeDownloaded = DownloadChunks[DownloadChunkIdx].Get<0>();
								const FGuid& ChunkId = DownloadChunks[DownloadChunkIdx].Get<1>();
								if (ChunkInfo.Guid == ChunkId)
								{
									Timer = TimeDownloaded;
									LoadedChunks.Add(ChunkId);
									DownloadChunks.RemoveAt(DownloadChunkIdx);
									break;
								}
							}
						}
						checkf(LoadedChunks.Contains(ChunkInfo.Guid), TEXT("Logic error with timer simulation."));
					}
				}

				void WriteData(uint32 Size)
				{
					Timer += (double)Size / Config.DiskWriteSpeed;
				}

				void EvaluateDestruction(const FFileManifest& OldFile)
				{
					if (Config.InstallMode == EInstallMode::DestructiveInstall)
					{
						// Collect all chunks in this file.
						TSet<FGuid> FileManifestChunks;
						Algo::Transform(OldFile.ChunkParts, FileManifestChunks, &FChunkPart::Guid);
						FileManifestChunks = FileManifestChunks.Intersect(ChunksInstalled);
						// Select all chunks still required from this file.
						TFunction<bool(const FGuid&)> SelectPredicate = [&](const FGuid& ChunkId) { return !LoadedChunks.Contains(ChunkId) && FileManifestChunks.Contains(ChunkId); };
						TArray<FGuid> BatchLoadChunks = ChunkReferenceTracker.GetNextReferences(TNumericLimits<int32>::Max(), SelectPredicate);
						for (const FGuid& BatchLoadChunk : BatchLoadChunks)
						{
							// Load it from disk.
							Timer += (double)InstallManifest.GetChunkInfo(BatchLoadChunk)->DataSizeUncompressed / Config.BackupSerialisationSpeed;
							// Save it to backup.
							Timer += (double)InstallManifest.GetChunkInfo(BatchLoadChunk)->FileSize / Config.BackupSerialisationSpeed;
							BackupChunks.Add(BatchLoadChunk);
						}
					}
				}

				double GetTimer() const
				{
					return Timer;
				}

				// Dependencies
				const IChunkReferenceTracker& ChunkReferenceTracker;
				const FBuildPatchAppManifest& InstallManifest;
				const TSet<FGuid>& ChunksInstalled;
				const FSimConfig Config;

				// Tracking
				double Timer;
				TSet<FGuid> LoadedChunks;
				TSet<FGuid> BackupChunks;
				TArray<TTuple<double, FGuid>> DownloadChunks;
			};

			// Setup the simulators and run the process through them.
			TArray<FInstallTimeSim> TimeSims;
			for (const FSimConfig& SimConfig : SimConfigs)
			{
				TimeSims.Emplace(*ChunkReferenceTracker.Get(), InstallManifest.Get(), ChunksInstalled, SimConfig);
			}
			for (const FString& FileToBuild : FilesToBuild)
			{
				// Create a new file.
				for (FInstallTimeSim& TimeSim : TimeSims) { TimeSim.CreateFile(); }

				// For each required chunk.
				const FFileManifest& NewFileManifest = *InstallManifest->GetFileManifest(FileToBuild);
				for (const FChunkPart& ChunkPart : NewFileManifest.ChunkParts)
				{
					// Process completed downloads.
					for (FInstallTimeSim& TimeSim : TimeSims) { TimeSim.TickDownloads(); }

					// Get the chunk.
					const FChunkInfo& ChunkInfo = *InstallManifest->GetChunkInfo(ChunkPart.Guid);
					for (FInstallTimeSim& TimeSim : TimeSims) { TimeSim.GetChunk(ChunkInfo); }

					// Write the chunk to file.
					for (FInstallTimeSim& TimeSim : TimeSims) { TimeSim.WriteData(ChunkPart.Size); }
					ChunkReferenceTracker->PopReference(ChunkPart.Guid);
				}
				// If there's an old file to delete, add time for backing up all of the still referenced chunks.
				if (FilesInstalled.Contains(FileToBuild))
				{
					const FFileManifest& OldFileManifest = *CurrentManifest->GetFileManifest(FileToBuild);
					for (FInstallTimeSim& TimeSim : TimeSims) { TimeSim.EvaluateDestruction(OldFileManifest); }
				}
			}

			// Return the simulation results.
			TArray<double> Results;
			for (FInstallTimeSim& TimeSim : TimeSims) { Results.Add(TimeSim.GetTimer()); }
			return Results;
		}
	}

	class FDiffManifests
		: public IDiffManifests
	{
	public:
		FDiffManifests(const FDiffManifestsConfiguration& InConfiguration);
		~FDiffManifests();

		// IChunkDeltaOptimiser interface begin.
		virtual	bool Run() override;
		// IChunkDeltaOptimiser interface end.

	private:
		bool AsyncRun();
		void HandleDownloadComplete(int32 RequestId, const FDownloadRef& Download);

		void ExportPatchDescriptorFiles(const TSet<FString>& InstallTags, const FBuildPatchAppManifestPtr &OldManifest, const FBuildPatchAppManifestPtr &NewManifest);
	private:
		const FDiffManifestsConfiguration Configuration;
		FDownloadCompleteDelegate DownloadCompleteDelegate;
		FDownloadProgressDelegate DownloadProgressDelegate;
		TSharedPtr<IFileSystem> FileSystem;
		TSharedPtr<IHttpManager> HttpManager;
		TSharedPtr<IChunkDataSizeProvider> ChunkDataSizeProvider;
		TSharedPtr<ISpeedRecorder> DownloadSpeedRecorder;
		TSharedPtr<IInstallerAnalytics> InstallerAnalytics;
		TSharedPtr<IDownloadServiceStatistics> DownloadServiceStatistics;
		TUniquePtr<IDownloadService> DownloadService;
		TUniquePtr<FStatsCollector> StatsCollector;
		FThreadSafeBool bShouldRun;

		// Manifest downloading
		int32 RequestIdManifestA;
		int32 RequestIdManifestB;
		TPromise<FBuildPatchAppManifestPtr> PromiseManifestA;
		TPromise<FBuildPatchAppManifestPtr> PromiseManifestB;
		TFuture<FBuildPatchAppManifestPtr> FutureManifestA;
		TFuture<FBuildPatchAppManifestPtr> FutureManifestB;

		TUniquePtr<IMessagePump> MessagePump;
		// Setters will be added in EDS implementation for BPT
		TArray<FMessageHandler*> MessageHandlers;
	};

	FDiffManifests::FDiffManifests(const FDiffManifestsConfiguration& InConfiguration)
		: Configuration(InConfiguration)
		, DownloadCompleteDelegate(FDownloadCompleteDelegate::CreateRaw(this, &FDiffManifests::HandleDownloadComplete))
		, DownloadProgressDelegate()
		, FileSystem(FFileSystemFactory::Create())
		, HttpManager(FHttpManagerFactory::Create())
		, ChunkDataSizeProvider(FChunkDataSizeProviderFactory::Create({}))
		, DownloadSpeedRecorder(FSpeedRecorderFactory::Create())
		, InstallerAnalytics(FInstallerAnalyticsFactory::Create(nullptr))
		, DownloadServiceStatistics(FDownloadServiceStatisticsFactory::Create(DownloadSpeedRecorder, ChunkDataSizeProvider, InstallerAnalytics))
		, DownloadService(FDownloadServiceFactory::Create(HttpManager, FileSystem, DownloadServiceStatistics, InstallerAnalytics))
		, StatsCollector(FStatsCollectorFactory::Create())
		, bShouldRun(true)
		, RequestIdManifestA(INDEX_NONE)
		, RequestIdManifestB(INDEX_NONE)
		, FutureManifestA(PromiseManifestA.GetFuture())
		, FutureManifestB(PromiseManifestB.GetFuture())
		, MessagePump(FMessagePumpFactory::Create())
		, MessageHandlers(Configuration.MessageHandlers)
	{
	}

	FDiffManifests::~FDiffManifests()
	{
		if (!FutureManifestA.IsReady())
		{
			PromiseManifestA.SetValue(FBuildPatchAppManifestPtr());
			FutureManifestA.Wait();
		}
		if (!FutureManifestB.IsReady())
		{
			PromiseManifestB.SetValue(FBuildPatchAppManifestPtr());
			FutureManifestB.Wait();
		}
	}

	bool FDiffManifests::Run()
	{
		// Any code that should be ran after the loop exit.
		ON_SCOPE_EXIT
		{
			GLog->FlushThreadedLogs();
		};

		// Run any core initialisation required.
		FHttpModule::Get();

		// Kick off Manifest downloads.
		if(    !DownloadService->RequestFile(Configuration.ManifestAUri, DownloadCompleteDelegate, DownloadProgressDelegate, &RequestIdManifestA)
		    || !DownloadService->RequestFile(Configuration.ManifestBUri, DownloadCompleteDelegate, DownloadProgressDelegate, &RequestIdManifestB))
		{
			return false;
		}

		// Start the generation thread.
		TFuture<bool> Thread = Async(EAsyncExecution::Thread, [this]() { return AsyncRun(); });

		// Main timers.
		double DeltaTime = 0.0;
		double LastTime = FPlatformTime::Seconds();

		// Setup desired frame times.
		float MainsFramerate = 100.0f;
		const float MainsFrameTime = 1.0f / MainsFramerate;

		// Run the main loop.
		while (bShouldRun)
		{
			// Increment global frame counter once for each app tick.
			GFrameCounter++;

			// Application tick.
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(DeltaTime);

			// Message pump.
			MessagePump->PumpMessages(MessageHandlers);

			GLog->FlushThreadedLogs();

			// Control frame rate.
			FPlatformProcess::Sleep(FMath::Max<float>(0.0f, MainsFrameTime - (FPlatformTime::Seconds() - LastTime)));

			// Calculate deltas.
			const double AppTime = FPlatformTime::Seconds();
			DeltaTime = AppTime - LastTime;
			LastTime = AppTime;
		}

		// Return thread success state.
		return Thread.Get();
	}

	static void WriteStringsToFile(const TUtf8StringBuilder<1024> &Listing, const FString &FileName)
	{
		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*FileName, 0));
		if (!Ar)
		{
			UE_LOGF(LogTemp, Warning, "Unable to open output file: %ls", *FileName);
		}
		else
		{
			FUtf8StringView OutputView = MakeStringView(Listing);
			UTF8CHAR UTF8BOM[] = { (UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF };
			Ar->Serialize(&UTF8BOM, sizeof(UTF8BOM));
			Ar->Serialize((void*)OutputView.GetData(), OutputView.Len() * sizeof(UTF8CHAR));
			Ar->Close();
		}
	}

	void FDiffManifests::ExportPatchDescriptorFiles(const TSet<FString>& InstallTags, const FBuildPatchAppManifestPtr &OldManifest, const FBuildPatchAppManifestPtr &NewManifest)
	{
		UE_LOGF(LogDiffManifests, Display, "Generating PatchDescription Files for %d tags and writing to %ls", InstallTags.Num(), *Configuration.OutputPatchDescriptorPath);

		// We want a separate report for each tag, since tags can't assume files from other tags are installed and it's easy to 
		// accumulate after if we care.
		// We specifically do this for tags not files because the client code installs off of tags not files. If we work off of
		// files we may end up with mulitple install tags for the file, and we can't infer which of those is installed, which
		// means we can't get an accurate representation of what files we can use to patch off of. 
		for (const FString& InstallTag : InstallTags)
		{
			UE_LOGF(LogDiffManifests, Display, "%ls:", *InstallTag);
			TSet<FString> CurrentInstallTag;
			CurrentInstallTag.Add(InstallTag);

			TSet<FString> OldFiles;
			OldManifest->GetTaggedFileList(CurrentInstallTag, OldFiles);

			TArray<FString> NewFiles;
			NewManifest->GetTaggedFileList(CurrentInstallTag, NewFiles);
						
			// Filter to files that need data sent. We don't use GetOutdatedFiles since we don't want files that get deleted.
			// We have to use a Set here for GetChunksRequiredForFiles later.
			TSet<FString> FilesToPatch;
			for (const FString& NewFile : NewFiles)
			{
				if (!OldFiles.Contains(NewFile))
				{
					FilesToPatch.Add(NewFile);
					continue;
				}

				// File exists in both - check if anything changed
				FSHAHash NewHash;
				FSHAHash OldHash;
				bool bNewHashExists = NewManifest->GetFileHash(NewFile, NewHash);
				bool bOldHashExists = OldManifest->GetFileHash(NewFile, OldHash);

				if (!bNewHashExists || !bOldHashExists)
				{
					UE_LOGF(LogDiffManifests, Error, "Hash missing for file! %ls Old: %d New: %d", *NewFile, bOldHashExists, bNewHashExists);
					continue;
				}

				if (NewHash != OldHash)
				{
					FilesToPatch.Add(NewFile);
				}
			}

			// OK got the list of files that need patching for this tag list - we need to generate the chunks we need.
			TSet<FGuid> ChunksForNewFiles;
			NewManifest->GetChunksRequiredForFiles(FilesToPatch, ChunksForNewFiles);

			// Now we need to get the chunks that the old installation can provide *for this tag list*
			TSet<FGuid> ChunksOnDisk;
			TSet<FGuid> ChunksForOldFiles;
			OldManifest->GetChunksRequiredForFiles(OldFiles, ChunksForOldFiles);
			OldManifest->EnumerateProducibleChunks(CurrentInstallTag, ChunksForOldFiles, ChunksOnDisk);

			const TSet<FGuid> ChunksToDownload = ChunksForNewFiles.Difference(ChunksOnDisk);
			uint64 TotalNewChunkBytes = 0;
			uint64 TotalNewChunkDownloadBytes = 0;

			for (const FGuid& Guid : ChunksToDownload)
			{
				const BuildPatchServices::FChunkInfo* ChunkInfo = NewManifest->GetChunkInfo(Guid);
				TotalNewChunkBytes += ChunkInfo->DataSizeUncompressed;
				TotalNewChunkDownloadBytes += ChunkInfo->FileSize;
			}

			// These numbers are different than the actual patch sizes since we might not be using the entire
			// chunk, and we might be using parts more than once.
			UE_LOGF(LogDiffManifests, Display, "    Chunk uncompressed / download sizes: %ls, %ls", 
				*FText::AsNumber(TotalNewChunkBytes).ToString(),
				*FText::AsNumber(TotalNewChunkDownloadBytes).ToString()
			);

			// Now we know which chunks have to be downloaded vs are available on disk, we can iterate the
			// chunk parts and report whether its New or Matched in the format. We try to accumulate runs
			// rather than send a string of adjacent matches.
			uint64 TotalBytesWritten = 0;
			uint64 TotalNewBytes = 0;
			TSet<FGuid> ReferencedChunks;
			for (const FString& PatchFile : FilesToPatch)
			{
				const FFileManifest* FileManifest = NewManifest->GetFileManifest(PatchFile);

				TArray<TPair<uint64, bool>> MatchedOrNot;

				uint64 FileOffset = 0;
				for (const FChunkPart& ChunkPart : FileManifest->ChunkParts)
				{
					if (ChunksOnDisk.Contains(ChunkPart.Guid))
					{
						// Match - add if we don't have any entries, or we're not currently matching (i.e. can't extend)
						if (!MatchedOrNot.Num() || !MatchedOrNot.Top().Value)
						{
							MatchedOrNot.Emplace(FileOffset, true);
						}
					}
					else
					{
						ReferencedChunks.Add(ChunkPart.Guid);
						TotalNewBytes += ChunkPart.Size;

						// Patch - add if we don't have any entries, or we are currently matching (i.e. can't extend)
						if (!MatchedOrNot.Num() || MatchedOrNot.Top().Value)
						{
							MatchedOrNot.Emplace(FileOffset, false);
						}
					}
					FileOffset += ChunkPart.Size;
				}
				TotalBytesWritten += FileOffset;

				// Emit the listing for this file.
				TUtf8StringBuilder<1024> Listing;
				for (int32 i = 0; i < MatchedOrNot.Num(); i++)
				{
					uint64 Offset = MatchedOrNot[i].Key;
					bool bIsMatch = MatchedOrNot[i].Value;

					uint64 End = FileOffset;
					if (i < MatchedOrNot.Num() - 1)
					{
						End = MatchedOrNot[i+1].Key;
					}

					if (bIsMatch)
					{
						// The final 0 is supposed to be the offset in the source file where the data comes from,
						// but we don't have that data (and we don't need it for the purposes of this utility, which is
						// just to evaluate which parts of a file changed).
						Listing.Appendf(UTF8TEXT("M,%llu,%llu,0\r\n"), Offset, End-Offset);
					}
					else
					{
						Listing.Appendf(UTF8TEXT("N,%llu,%llu\r\n"), Offset, End-Offset);
					}
				}

				const FString FileName = (Configuration.OutputPatchDescriptorPath / PatchFile) + ".patch";

				WriteStringsToFile(Listing, FileName);
			} // end each file

			// This can be more than the uncompressed download size if chunks are reused, or smaller if we didn't need
			// an entire chunk.
			UE_LOGF(LogDiffManifests, Display, "    Tag patches %d files, with %ls new uncompressed bytes out of %ls total bytes (%.1f %%).", 
				FilesToPatch.Num(), 
				*FText::AsNumber(TotalNewBytes).ToString(),
				*FText::AsNumber(TotalBytesWritten).ToString(),
				TotalBytesWritten ? (100.0f * TotalNewBytes / TotalBytesWritten) : 0
			);
		}
	}

	bool FDiffManifests::AsyncRun()
	{
		FBuildPatchAppManifestPtr ManifestA = FutureManifestA.Get();
		FBuildPatchAppManifestPtr OriginalManifestB = FutureManifestB.Get();
		bool bSuccess = true;
		if (ManifestA.IsValid() == false)
		{
			UE_LOGF(LogDiffManifests, Error, "Could not download ManifestA from %ls.", *Configuration.ManifestAUri);
			bSuccess = false;
		}
		if (OriginalManifestB.IsValid() == false)
		{
			UE_LOGF(LogDiffManifests, Error, "Could not download ManifestB from %ls.", *Configuration.ManifestBUri);
			bSuccess = false;
		}
		if (bSuccess)
		{
			TArray<FString> CloudDirectories = Configuration.CloudDirs.Num() == 0 ? TArray<FString>{ FPaths::GetPath(Configuration.ManifestBUri) } : Configuration.CloudDirs;

			// Check for delta file, replacing ManifestB if we find one
			FOptimisedDeltaConfiguration OptimisedDeltaConfiguration(OriginalManifestB.ToSharedRef());
			OptimisedDeltaConfiguration.SourceManifest = ManifestA;
			OptimisedDeltaConfiguration.DeltaPolicy = Configuration.bRequireOptimizedDelta ? EDeltaPolicy::Expect : EDeltaPolicy::TryFetchContinueWithout;
			OptimisedDeltaConfiguration.RetriesNumber = CloudDirectories.Num();
			OptimisedDeltaConfiguration.DeltaFilenameTrailer = Configuration.DeltaFilenameTrailer;

			TSharedRef<IUriProvider, ESPMode::ThreadSafe> UriProvider = FUriProviderFactory::Create(MessagePump.Get(), CloudDirectories);
			FOptimisedDeltaDependencies OptimisedDeltaDependencies(MoveTemp(UriProvider));
			OptimisedDeltaDependencies.DownloadService = DownloadService.Get();
			OptimisedDeltaDependencies.UriProvider = MoveTemp(UriProvider);

			TSharedRef<IOptimisedDelta> OptimisedDelta(FOptimisedDeltaFactory::Create(OptimisedDeltaConfiguration, MoveTemp(OptimisedDeltaDependencies)));
			if (OptimisedDelta->GetResult().HasError())
			{
				UE_LOGF(LogDiffManifests, Error, "Optimized delta not found and RequireOptimizedDelta was passed - failing");
				bShouldRun = false;
				return false;
			}
			FBuildPatchAppManifestPtr OptimisedDeltaManifestB = OptimisedDelta->GetResult().GetValue();
			const int32 MetaDownloadBytes = OptimisedDelta->GetMetaDownloadSize();

			TSet<FString> TagsA, TagsB;
			ManifestA->GetFileTagList(TagsA);
			if (Configuration.TagSetA.Num() > 0)
			{
				TagsA = TagsA.Intersect(Configuration.TagSetA);
			}
			OptimisedDeltaManifestB->GetFileTagList(TagsB);
			if (Configuration.TagSetB.Num() > 0)
			{
				TagsB = TagsB.Intersect(Configuration.TagSetB);
			}

			if (Configuration.OutputPatchDescriptorPath.Len())
			{
				// We want to use all NEW tags so that we catch new bundles, not just changed bundles.
				ExportPatchDescriptorFiles(TagsB, ManifestA, OptimisedDeltaManifestB);

				if (Configuration.bOnlyPatchDescriptors)
				{
					// Early out. 
					bShouldRun = false;
					return true;
				}
			}

			int64 NewChunksCount = 0;
			int64 TotalChunkSize = 0;
			TSet<FString> TaggedFileSetA;
			TSet<FString> TaggedFileSetB;
			TSet<FGuid> ChunkSetA;
			TSet<FGuid> ChunkSetB;
			ManifestA->GetTaggedFileList(TagsA, TaggedFileSetA);
			ManifestA->GetChunksRequiredForFiles(TaggedFileSetA, ChunkSetA);
			OptimisedDeltaManifestB->GetTaggedFileList(TagsB, TaggedFileSetB);
			OptimisedDeltaManifestB->GetChunksRequiredForFiles(TaggedFileSetB, ChunkSetB);
			TArray<FString> NewChunkPaths;
			for (FGuid& ChunkB : ChunkSetB)
			{
				if (ChunkSetA.Contains(ChunkB) == false)
				{
					++NewChunksCount;
					int32 ChunkFileSize = OptimisedDeltaManifestB->GetDataSize(ChunkB);
					TotalChunkSize += ChunkFileSize;
					NewChunkPaths.Add(FBuildPatchUtils::GetDataFilename(OptimisedDeltaManifestB.ToSharedRef(), ChunkB));
					UE_LOGF(LogDiffManifests, Verbose, "New chunk discovered: Size: %10d, Path: %ls", ChunkFileSize, *NewChunkPaths.Last());
				}
			}

			UE_LOGF(LogDiffManifests, Log, "New chunks:  %lld", NewChunksCount);
			UE_LOGF(LogDiffManifests, Log, "Total bytes: %lld", TotalChunkSize);

			TSet<FString> NewFilePaths = TaggedFileSetB.Difference(TaggedFileSetA);
			TSet<FString> RemovedFilePaths = TaggedFileSetA.Difference(TaggedFileSetB);
			TSet<FString> ChangedFilePaths;
			TSet<FString> UnchangedFilePaths;

			const TSet<FString>& SetToIterate = TaggedFileSetB.Num() > TaggedFileSetA.Num() ? TaggedFileSetA : TaggedFileSetB;
			for (const FString& TaggedFile : SetToIterate)
			{
				if (!RemovedFilePaths.Contains(TaggedFile) && !NewFilePaths.Contains(TaggedFile))
				{
					FSHAHash FileHashA;
					FSHAHash FileHashB;
					if (ManifestA->GetFileHash(TaggedFile, FileHashA) && OptimisedDeltaManifestB->GetFileHash(TaggedFile, FileHashB))
					{
						if (FileHashA == FileHashB)
						{
							UnchangedFilePaths.Add(TaggedFile);
						}
						else
						{
							ChangedFilePaths.Add(TaggedFile);
						}
					}
				}
			}

			// Log download details.
			const FNumberFormattingOptions SizeFormattingOptions = FNumberFormattingOptions().SetMaximumFractionalDigits(3).SetMinimumFractionalDigits(3);
			const FNumberFormattingOptions PercentFormat = FNumberFormattingOptions().SetMaximumFractionalDigits(1).SetMinimumFractionalDigits(1).SetRoundingMode(ERoundingMode::ToZero);

			const int64 DownloadSizeA = ManifestA->GetDownloadSize(TagsA);
			const int64 BuildSizeA = ManifestA->GetBuildSize(TagsA);
			const int64 DownloadSizeB = OriginalManifestB->GetDownloadSize(TagsB);
			const int64 BuildSizeB = OriginalManifestB->GetBuildSize(TagsB);
			const int64 DeltaDownloadSize = OptimisedDeltaManifestB->GetDeltaDownloadSize(TagsB, ManifestA.ToSharedRef(), TagsA) + MetaDownloadBytes;
			const int64 TempDiskSpaceReq = FileConstructorHelpers::CalculateRequiredDiskSpace(ManifestA, OptimisedDeltaManifestB.ToSharedRef(), EInstallMode::DestructiveInstall, TagsB);

			// Break down the sizes and delta into new chunks per tag.
			TMap<FString, int64> TagDownloadImpactA;
			TMap<FString, int64> TagBuildImpactA;
			TMap<FString, int64> TagDownloadImpactB;
			TMap<FString, int64> TagBuildImpactB;
			TMap<FString, int64> TagDeltaImpact;
			for (const FString& Tag : TagsA)
			{
				TSet<const FFileManifest*> TaggedManifests = ManifestA->GetTaggedFileManifests(TSet<FString>{ Tag });
				TagDownloadImpactA.Add(Tag, ManifestA->GetDownloadSize(TaggedManifests));
				TagBuildImpactA.Add(Tag, ManifestA->GetBuildSize(TaggedManifests));
			}

			TSet<const FFileManifest*> TaggedManifestsA = ManifestA->GetTaggedFileManifests(TagsA);
			TSet<FString> FilesInstalled;
			Algo::Transform(TaggedManifestsA, FilesInstalled, [](const FFileManifest* File) { return File->Filename; });
			TSet<FGuid> ChunksRequired;
			TSet<FGuid> ChunksInstalled;
			ManifestA->GetChunksRequiredForFiles(FilesInstalled, ChunksRequired);
			ManifestA->EnumerateProducibleChunks(TaggedManifestsA, ChunksRequired, ChunksInstalled);

			for (const FString& Tag : TagsB)
			{
				TSet<const FFileManifest*> TaggedManifests = OriginalManifestB->GetTaggedFileManifests(TSet<FString>{ Tag });
				TagDownloadImpactB.Add(Tag, OriginalManifestB->GetDownloadSize(TaggedManifests));
				TagBuildImpactB.Add(Tag, OriginalManifestB->GetBuildSize(TaggedManifests));
				TagDeltaImpact.Add(Tag, OptimisedDeltaManifestB->GetDeltaDownloadSize(TaggedManifests, ManifestA.ToSharedRef(), FilesInstalled, ChunksInstalled));
			}
			if (MetaDownloadBytes > 0)
			{
				TagDeltaImpact.FindOrAdd(TEXT("")) += MetaDownloadBytes;
			}

			// Compare tag sets
			TMap<FString, int64> CompareTagSetDeltaImpact;
			TMap<FString, int64> CompareTagSetBuildImpactA;
			TMap<FString, int64> CompareTagSetDownloadSizeA;
			TMap<FString, int64> CompareTagSetBuildImpactB;
			TMap<FString, int64> CompareTagSetDownloadSizeB;
			TMap<FString, int64> CompareTagSetTempDiskSpaceReqs;
			TSet<FString> CompareTagSetKeys;

			for (int32 TagSetIndex = 0; TagSetIndex < Configuration.CompareTagSetsA.Num(); TagSetIndex++)
			{
				const TSet<FString>& OldTagSet = Configuration.CompareTagSetsA[TagSetIndex];

				// If the set of installed tags changes across the patch, we need to pass both sets to GetDeltaDownloadSize.
				const TSet<FString>* NewTagSet = &OldTagSet;
				if (TagSetIndex < Configuration.CompareTagSetsB.Num())
				{
					NewTagSet = &Configuration.CompareTagSetsB[TagSetIndex];
				}

				TArray<FString> TagArrayCompare = NewTagSet->Array();
				Algo::Sort(TagArrayCompare);
				FString TagSetString = FString::Join(TagArrayCompare, TEXT(", "));
				if (&OldTagSet != NewTagSet)
				{
					TArray<FString> OldTagArray = OldTagSet.Array();
					Algo::Sort(OldTagArray);
					FString OldTagSetString = FString::Join(OldTagArray, TEXT(", "));
					TagSetString = OldTagSetString + TEXT(" -> ") + TagSetString;
				}

				CompareTagSetKeys.Add(TagSetString);

				CompareTagSetDeltaImpact.Add(TagSetString, OptimisedDeltaManifestB->GetDeltaDownloadSize(*NewTagSet, ManifestA.ToSharedRef(), OldTagSet) + MetaDownloadBytes);
				CompareTagSetBuildImpactB.Add(TagSetString, OriginalManifestB->GetBuildSize(*NewTagSet));
				CompareTagSetDownloadSizeB.Add(TagSetString, OriginalManifestB->GetDownloadSize(*NewTagSet));

				CompareTagSetBuildImpactA.Add(TagSetString, ManifestA->GetBuildSize(OldTagSet));
				CompareTagSetDownloadSizeA.Add(TagSetString, ManifestA->GetDownloadSize(OldTagSet));
				CompareTagSetTempDiskSpaceReqs.Add(TagSetString, FileConstructorHelpers::CalculateRequiredDiskSpace(ManifestA, OptimisedDeltaManifestB.ToSharedRef(), EInstallMode::DestructiveInstall, *NewTagSet));
			}

			// Log the information.
			UE_LOGF(LogDiffManifests, Display, "%ls %ls:", *ManifestA->GetAppName(), *ManifestA->GetVersionString());
			UE_LOGF(LogDiffManifests, Display, "    Download Size:   %20ls bytes (%10ls, %11ls)", *FText::AsNumber(DownloadSizeA).ToString(), *FText::AsMemory(DownloadSizeA, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(DownloadSizeA, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
			UE_LOGF(LogDiffManifests, Display, "    Build Size:      %20ls bytes (%10ls, %11ls)", *FText::AsNumber(BuildSizeA).ToString(), *FText::AsMemory(BuildSizeA, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(BuildSizeA, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
			UE_LOGF(LogDiffManifests, Display, "%ls %ls:", *OriginalManifestB->GetAppName(), *OriginalManifestB->GetVersionString());
			UE_LOGF(LogDiffManifests, Display, "    Download Size:   %20ls bytes (%10ls, %11ls)", *FText::AsNumber(DownloadSizeB).ToString(), *FText::AsMemory(DownloadSizeB, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(DownloadSizeB, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
			UE_LOGF(LogDiffManifests, Display, "    Build Size:      %20ls bytes (%10ls, %11ls)", *FText::AsNumber(BuildSizeB).ToString(), *FText::AsMemory(BuildSizeB, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(BuildSizeB, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
			UE_LOGF(LogDiffManifests, Display, "%ls %ls -> %ls %ls:", *ManifestA->GetAppName(), *ManifestA->GetVersionString(), *OptimisedDeltaManifestB->GetAppName(), *OptimisedDeltaManifestB->GetVersionString());
			UE_LOGF(LogDiffManifests, Display, "    Delta Size:      %20ls bytes (%10ls, %11ls)", *FText::AsNumber(DeltaDownloadSize).ToString(), *FText::AsMemory(DeltaDownloadSize, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(DeltaDownloadSize, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
			UE_LOGF(LogDiffManifests, Display, "    Temp Disk Space: %20ls bytes (%10ls, %11ls)", *FText::AsNumber(TempDiskSpaceReq).ToString(), *FText::AsMemory(TempDiskSpaceReq, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(TempDiskSpaceReq, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
			UE_LOGF(LogDiffManifests, Display, "");

			const FString UntaggedLog(TEXT("(untagged)"));
			for (const FString& Tag : TagsB)
			{
				UE_LOGF(LogDiffManifests, Display, "%ls Impact:", *(Tag.IsEmpty() ? UntaggedLog : Tag));
				UE_LOGF(LogDiffManifests, Display, "    Individual Download Size:  %20ls bytes (%10ls, %11ls)", *FText::AsNumber(TagDownloadImpactB[Tag]).ToString(), *FText::AsMemory(TagDownloadImpactB[Tag], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(TagDownloadImpactB[Tag], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
				UE_LOGF(LogDiffManifests, Display, "    Individual Build Size:     %20ls bytes (%10ls, %11ls)", *FText::AsNumber(TagBuildImpactB[Tag]).ToString(), *FText::AsMemory(TagBuildImpactB[Tag], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(TagBuildImpactB[Tag], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
				UE_LOGF(LogDiffManifests, Display, "    Individual Delta Size:     %20ls bytes (%10ls, %11ls)", *FText::AsNumber(TagDeltaImpact[Tag]).ToString(), *FText::AsMemory(TagDeltaImpact[Tag], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(TagDeltaImpact[Tag], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
			}

			for (const FString& TagSet : CompareTagSetKeys)
			{
				const FString& TagSetDisplay = TagSet.IsEmpty() || TagSet.StartsWith(TEXT(",")) ? UntaggedLog + TagSet : TagSet;
				UE_LOGF(LogDiffManifests, Display, "Impact of TagSet: %ls", *TagSetDisplay);
				UE_LOGF(LogDiffManifests, Display, "    Download Size:    %20ls bytes (%10ls, %11ls)", *FText::AsNumber(CompareTagSetDownloadSizeB[TagSet]).ToString(), *FText::AsMemory(CompareTagSetDownloadSizeB[TagSet], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(CompareTagSetDownloadSizeB[TagSet], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
				UE_LOGF(LogDiffManifests, Display, "    Build Size:       %20ls bytes (%10ls, %11ls)", *FText::AsNumber(CompareTagSetBuildImpactB[TagSet]).ToString(), *FText::AsMemory(CompareTagSetBuildImpactB[TagSet], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(CompareTagSetBuildImpactB[TagSet], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
				UE_LOGF(LogDiffManifests, Display, "    Delta Size:       %20ls bytes (%10ls, %11ls)", *FText::AsNumber(CompareTagSetDeltaImpact[TagSet]).ToString(), *FText::AsMemory(CompareTagSetDeltaImpact[TagSet], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(CompareTagSetDeltaImpact[TagSet], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
				UE_LOGF(LogDiffManifests, Display, "    Temp Disk Space:  %20ls bytes (%10ls, %11ls)", *FText::AsNumber(CompareTagSetTempDiskSpaceReqs[TagSet]).ToString(), *FText::AsMemory(CompareTagSetTempDiskSpaceReqs[TagSet], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(CompareTagSetTempDiskSpaceReqs[TagSet], &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
			}

			TArray<double> InstallTimeCoefficients;
			if (Configuration.bSimulateInstallTimes)
			{
				// Hit a destructive and nondestructive simulation for a few different specs.
				TArray<DiffHelpers::FSimConfig> SimConfigs;
				// Add some lower spec values, taken from around 25 percentile of stats at the time of writing [July 2019].
				SimConfigs.AddDefaulted_GetRef().InstallMode = EInstallMode::DestructiveInstall;
				SimConfigs.AddDefaulted_GetRef().InstallMode = EInstallMode::NonDestructiveInstall;
				SimConfigs[0].DownloadSpeed  = SimConfigs[1].DownloadSpeed  =   1200000.0; // 1.2 MB/s
				SimConfigs[0].DiskReadSpeed  = SimConfigs[1].DiskReadSpeed  =  30000000.0; // 30 MB/s
				SimConfigs[0].DiskWriteSpeed = SimConfigs[1].DiskWriteSpeed =  25000000.0; // 25 MB/s
				// We didn't have stats for BackupSerialisationSpeed, but it runs much slower than disk speed and so we tune it to be sure destructive is relatively penalizing.
				SimConfigs[0].BackupSerialisationSpeed = SimConfigs[1].BackupSerialisationSpeed = 10000000.0; // 10 MB/s
				// Add some lower spec values, taken from around 50 percentile of stats at the time of writing [July 2019].
				SimConfigs.AddDefaulted_GetRef().InstallMode = EInstallMode::DestructiveInstall;
				SimConfigs.AddDefaulted_GetRef().InstallMode = EInstallMode::NonDestructiveInstall;
				SimConfigs[2].DownloadSpeed  = SimConfigs[3].DownloadSpeed  =   3500000.0; // 3.5 MB/s
				SimConfigs[2].DiskReadSpeed  = SimConfigs[3].DiskReadSpeed  = 145000000.0; // 145 MB/s
				SimConfigs[2].DiskWriteSpeed = SimConfigs[3].DiskWriteSpeed =  75000000.0; // 75 MB/s
				// We didn't have stats for BackupSerialisationSpeed, but it runs much slower than disk speed and so we tune it to be sure destructive is relatively penalizing.
				SimConfigs[2].BackupSerialisationSpeed = SimConfigs[3].BackupSerialisationSpeed = 20000000.0; // 20 MB/s
				// Add some higher spec values, taken from around 75 percentile of stats at the time of writing [July 2019].
				SimConfigs.AddDefaulted_GetRef().InstallMode = EInstallMode::DestructiveInstall;
				SimConfigs.AddDefaulted_GetRef().InstallMode = EInstallMode::NonDestructiveInstall;
				SimConfigs[4].DownloadSpeed  = SimConfigs[5].DownloadSpeed  =  13000000.0; // 13 MB/s
				SimConfigs[4].DiskReadSpeed  = SimConfigs[5].DiskReadSpeed  = 295000000.0; // 295 MB/s
				SimConfigs[4].DiskWriteSpeed = SimConfigs[5].DiskWriteSpeed = 125000000.0; // 125 MB/s
				// We didn't have stats for BackupSerialisationSpeed, but it runs much slower than disk speed and so we tune it to be sure destructive is relatively penalizing.
				SimConfigs[4].BackupSerialisationSpeed = SimConfigs[5].BackupSerialisationSpeed = 40000000.0; // 40 MB/s

				// Run the calculations and log.
				InstallTimeCoefficients = DiffHelpers::CalculateInstallTimeCoefficient(ManifestA.ToSharedRef(), TagsA, OptimisedDeltaManifestB.ToSharedRef(), TagsB, SimConfigs);
				checkf(6 == InstallTimeCoefficients.Num() && 6 == SimConfigs.Num(), TEXT("Unexpected result size from CalculateInstallTimeCoefficient."));
				UE_LOGF(LogDiffManifests, Display, "");
				UE_LOGF(LogDiffManifests, Display, "Install time coefficients are not accurate timing representations, but are comparable from patch to patch.");
				UE_LOGF(LogDiffManifests, Display, "They can be used to spot out of the ordinary time requirements for installing an update.");
				UE_LOGF(LogDiffManifests, Display, "Install Time Coefficients:");
				UE_LOGF(LogDiffManifests, Display, "    Low-Spec  DestructiveInstall:    %ls", *FPlatformTime::PrettyTime(InstallTimeCoefficients[0]));
				UE_LOGF(LogDiffManifests, Display, "    Low-Spec  NonDestructiveInstall: %ls", *FPlatformTime::PrettyTime(InstallTimeCoefficients[1]));
				UE_LOGF(LogDiffManifests, Display, "    Mid-Spec  DestructiveInstall:    %ls", *FPlatformTime::PrettyTime(InstallTimeCoefficients[2]));
				UE_LOGF(LogDiffManifests, Display, "    Mid-Spec  NonDestructiveInstall: %ls", *FPlatformTime::PrettyTime(InstallTimeCoefficients[3]));
				UE_LOGF(LogDiffManifests, Display, "    High-Spec DestructiveInstall:    %ls", *FPlatformTime::PrettyTime(InstallTimeCoefficients[4]));
				UE_LOGF(LogDiffManifests, Display, "    High-Spec NonDestructiveInstall: %ls", *FPlatformTime::PrettyTime(InstallTimeCoefficients[5]));
			}

			// Output if an optimization was ran, what the overall improvement was.
			UE_LOGF(LogDiffManifests, Display, "");
			if (MetaDownloadBytes > 0)
			{
				const int64 OriginalDeltaDownloadSize = OriginalManifestB->GetDeltaDownloadSize({}, ManifestA.ToSharedRef());
				const int64 OptimisedDeltaDownloadSize = OptimisedDeltaManifestB->GetDeltaDownloadSize({}, ManifestA.ToSharedRef()) + MetaDownloadBytes;
				if (OptimisedDeltaDownloadSize < OriginalDeltaDownloadSize)
				{
					UE_LOGF(LogDiffManifests, Display, "Delta optimisation achieved a download size improvement of %ls.", *FText::AsPercent(1.0 - ((double)OptimisedDeltaDownloadSize / (double)OriginalDeltaDownloadSize), &PercentFormat).ToString());
					UE_LOGF(LogDiffManifests, Display, "    Original:  %20ls bytes (%10ls, %11ls).", *FText::AsNumber(OriginalDeltaDownloadSize).ToString(), *FText::AsMemory(OriginalDeltaDownloadSize, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(OriginalDeltaDownloadSize, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
					UE_LOGF(LogDiffManifests, Display, "    Optimised: %20ls bytes (%10ls, %11ls).", *FText::AsNumber(OptimisedDeltaDownloadSize).ToString(), *FText::AsMemory(OptimisedDeltaDownloadSize, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::SI).ToString(), *FText::AsMemory(OptimisedDeltaDownloadSize, &SizeFormattingOptions, nullptr, EMemoryUnitStandard::IEC).ToString());
				}
				else
				{
					UE_LOGF(LogDiffManifests, Display, "Delta optimisation was completed but did not find an improvement.");
				}
			}
			else
			{
				UE_LOGF(LogDiffManifests, Display, "Delta optimisation has not been completed for patching %ls %ls to %ls %ls.", *ManifestA->GetAppName(), *ManifestA->GetVersionString(), *OriginalManifestB->GetAppName(), *OriginalManifestB->GetVersionString());
			}

			// Save the output.
			if (bSuccess && Configuration.OutputFilePath.IsEmpty() == false)
			{
				FString JsonOutput;
				TSharedRef<FDiffJsonWriter> Writer = FDiffJsonWriterFactory::Create(&JsonOutput);
				Writer->WriteObjectStart();
				{
					Writer->WriteObjectStart(TEXT("ManifestA"));
					{
						Writer->WriteValue(TEXT("AppName"), ManifestA->GetAppName());
						Writer->WriteValue(TEXT("AppId"), static_cast<int32>(ManifestA->GetAppID()));
						Writer->WriteValue(TEXT("VersionString"), ManifestA->GetVersionString());
						Writer->WriteValue(TEXT("DownloadSize"), DownloadSizeA);
						Writer->WriteValue(TEXT("BuildSize"), BuildSizeA);
						Writer->WriteObjectStart(TEXT("IndividualTagDownloadSizes"));
						for (const TPair<FString, int64>& Pair : TagDownloadImpactA)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart("CompareTagSetDownloadSizes");
						for (const TPair<FString, int64>& Pair : CompareTagSetDownloadSizeA)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart(TEXT("IndividualTagBuildSizes"));
						for (const TPair<FString, int64>& Pair : TagBuildImpactA)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart("CompareTagSetBuildSizes");
						for (const TPair<FString, int64>& Pair : CompareTagSetBuildImpactA)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
					}
					Writer->WriteObjectEnd();
					Writer->WriteObjectStart(TEXT("ManifestB"));
					{
						Writer->WriteValue(TEXT("AppName"), OriginalManifestB->GetAppName());
						Writer->WriteValue(TEXT("AppId"), static_cast<int32>(OriginalManifestB->GetAppID()));
						Writer->WriteValue(TEXT("VersionString"), OriginalManifestB->GetVersionString());
						Writer->WriteValue(TEXT("DownloadSize"), DownloadSizeB);
						Writer->WriteValue(TEXT("BuildSize"), BuildSizeB);
						Writer->WriteObjectStart(TEXT("IndividualTagDownloadSizes"));
						for (const TPair<FString, int64>& Pair : TagDownloadImpactB)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart("CompareTagSetDownloadSizes");
						for (const TPair<FString, int64>& Pair : CompareTagSetDownloadSizeB)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart(TEXT("IndividualTagBuildSizes"));
						for (const TPair<FString, int64>& Pair : TagBuildImpactB)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart("CompareTagSetBuildSizes");
						for (const TPair<FString, int64>& Pair : CompareTagSetBuildImpactB)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
					}
					Writer->WriteObjectEnd();
					Writer->WriteObjectStart(TEXT("Differential"));
					{
						Writer->WriteArrayStart(TEXT("NewFilePaths"));
						for (const FString& NewFilePath : NewFilePaths)
						{
							Writer->WriteValue(NewFilePath);
						}
						Writer->WriteArrayEnd();
						Writer->WriteArrayStart(TEXT("RemovedFilePaths"));
						for (const FString& RemovedFilePath : RemovedFilePaths)
						{
							Writer->WriteValue(RemovedFilePath);
						}
						Writer->WriteArrayEnd();
						Writer->WriteArrayStart(TEXT("ChangedFilePaths"));
						for (const FString& ChangedFilePath : ChangedFilePaths)
						{
							Writer->WriteValue(ChangedFilePath);
						}
						Writer->WriteArrayEnd();
						Writer->WriteArrayStart(TEXT("UnchangedFilePaths"));
						for (const FString& UnchangedFilePath : UnchangedFilePaths)
						{
							Writer->WriteValue(UnchangedFilePath);
						}
						Writer->WriteArrayEnd();
						Writer->WriteArrayStart(TEXT("NewChunkPaths"));
						for (const FString& NewChunkPath : NewChunkPaths)
						{
							Writer->WriteValue(NewChunkPath);
						}
						Writer->WriteArrayEnd();
						Writer->WriteValue(TEXT("TotalChunkSize"), TotalChunkSize);
						Writer->WriteValue(TEXT("DeltaDownloadSize"), DeltaDownloadSize);
						Writer->WriteValue(TEXT("TempDiskSpaceReq"), TempDiskSpaceReq);
						Writer->WriteObjectStart(TEXT("IndividualTagDeltaSizes"));
						for (const TPair<FString, int64>& Pair : TagDeltaImpact)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart(TEXT("CompareTagSetDeltaSizes"));
						for (const TPair<FString, int64>& Pair : CompareTagSetDeltaImpact)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						Writer->WriteObjectStart(TEXT("CompareTagSetTempDiskSpaceReqs"));
						for (const TPair<FString, int64>& Pair : CompareTagSetTempDiskSpaceReqs)
						{
							Writer->WriteValue(Pair.Key, Pair.Value);
						}
						Writer->WriteObjectEnd();
						if (Configuration.bSimulateInstallTimes)
						{
							Writer->WriteArrayStart(TEXT("InstallTimeCoefficients"));
							for (const double& InstallTimeCoefficient : InstallTimeCoefficients)
							{
								Writer->WriteValue(InstallTimeCoefficient);
							}
							Writer->WriteArrayEnd();
						}
					}
					Writer->WriteObjectEnd();
				}
				Writer->WriteObjectEnd();
				Writer->Close();
				bSuccess = FFileHelper::SaveStringToFile(JsonOutput, *Configuration.OutputFilePath);
				if (!bSuccess)
				{
					UE_LOGF(LogDiffManifests, Error, "Could not save output to %ls", *Configuration.OutputFilePath);
				}
			}
		}
		bShouldRun = false;
		return bSuccess;
	}

	void FDiffManifests::HandleDownloadComplete(int32 RequestId, const FDownloadRef& Download)
	{
		TPromise<FBuildPatchAppManifestPtr>* RelevantPromisePtr = RequestId == RequestIdManifestA ? &PromiseManifestA : RequestId == RequestIdManifestB ? &PromiseManifestB : nullptr;
		if (RelevantPromisePtr != nullptr)
		{
			if (Download->ResponseSuccessful())
			{
				UE::Tasks::Launch(UE_SOURCE_LOCATION, [Download, RelevantPromisePtr]()
				{
					FBuildPatchAppManifestPtr Manifest = MakeShareable(new FBuildPatchAppManifest());
					if (!Manifest->DeserializeFromData(Download->GetData()))
					{
						Manifest.Reset();
					}
					RelevantPromisePtr->SetValue(Manifest);
				});
			}
			else
			{
				RelevantPromisePtr->SetValue(FBuildPatchAppManifestPtr());
			}
		}
		else
		{
			UE_LOGF(LogDiffManifests, Error, "HandleDownloadComplete RequestId %d not tracked.", RequestId);
		}
	}

	IDiffManifests* FDiffManifestsFactory::Create(const FDiffManifestsConfiguration& Configuration)
	{
		return new FDiffManifests(Configuration);
	}
}