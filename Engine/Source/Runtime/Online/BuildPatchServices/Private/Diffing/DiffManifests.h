// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "BuildPatchSettings.h"

namespace BuildPatchServices
{
	class IDiffManifests
	{
	public:
		virtual ~IDiffManifests()= default;
		virtual bool Run() = 0;
	};

	class FDiffManifestsFactory
	{
	public:
		static IDiffManifests* Create(const FDiffManifestsConfiguration& Configuration);
	};
}
