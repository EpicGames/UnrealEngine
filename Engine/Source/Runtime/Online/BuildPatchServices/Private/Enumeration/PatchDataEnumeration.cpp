// Copyright Epic Games, Inc. All Rights Reserved.

#include "Enumeration/PatchDataEnumeration.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include "Common/FileSystem.h"
#include "Data/ChunkData.h"
#include "BuildPatchManifest.h"
#include "BuildPatchMergeManifests.h"
#include "BuildPatchUtil.h"

DEFINE_LOG_CATEGORY(LogDataEnumeration);

namespace EnumerationHelpers
{
	template< typename DataType >
	FString ToHexString(const DataType& DataVal)
	{
		const void* AsBuffer = &DataVal;
		return BytesToHex(static_cast<const uint8*>(AsBuffer), sizeof(DataType));
	}

	bool IsChunkDbData(FArchive& Archive)
	{
		using namespace BuildPatchServices;
		const int64 ArPos = Archive.Tell();
		FChunkDatabaseHeader ChunkDbHeader;
		Archive << ChunkDbHeader;
		bool bIsChunkDb = ChunkDbHeader.Version > 0;
		Archive.Seek(ArPos);
		return bIsChunkDb;
	}

	TUniquePtr<FBuildPatchAppManifest> LoadManifest(FArchive& Archive)
	{
		TUniquePtr<FBuildPatchAppManifest> Manifest(new FBuildPatchAppManifest());
		TArray<uint8> FileData;
		Archive.Seek(0);
		FileData.AddUninitialized(Archive.TotalSize());
		Archive.Serialize(FileData.GetData(), Archive.TotalSize());
		if (Archive.IsError() || !Manifest->DeserializeFromData(FileData))
		{
			Manifest.Reset();
		}
		return Manifest;
	}

	void AppendManifestDataList(FBuildPatchAppManifest& Manifest, TArray<FString>& OutFiles, TSet<FGuid>& DeDupeSet, bool bIncludeSizes)
	{
		TArray<FGuid> DataList;
		Manifest.GetDataList(DataList);
		UE_LOGF(LogDataEnumeration, Verbose, "Data file list:-");
		for (FGuid& DataGuid : DataList)
		{
			bool bAlreadyInSet = false;
			DeDupeSet.Add(DataGuid, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				FString OutputLine = FBuildPatchUtils::GetDataFilename(Manifest, DataGuid);
				if (bIncludeSizes)
				{
					uint64 FileSize = Manifest.GetDataSize(DataGuid);
					OutputLine += FString::Printf(TEXT("\t%" UINT64_FMT), FileSize);
				}
				UE_LOGF(LogDataEnumeration, Verbose, "%ls", *OutputLine);
				OutFiles.Add(MoveTemp(OutputLine));
			}
		}
	}
}

namespace BuildPatchServices
{
	class FPatchDataEnumeration
		: public IPatchDataEnumeration
	{
	public:

		FPatchDataEnumeration(FPatchDataEnumerationConfiguration InConfiguration);
		~FPatchDataEnumeration();

		// IPatchDataEnumeration interface begin.
		virtual	bool Run() override;
		virtual bool Run(TArray<FString>& OutFiles) override;
		// IPatchDataEnumeration interface end.

	private:
		bool RunInternal(FArchive& Archive, TArray<FString>& OutFiles);
		bool EnumerateManifestData(FArchive& Archive, TArray<FString>& OutFiles);
		bool EnumerateChunkDbData(FArchive& Archive, TArray<FString>& OutFiles);

	private:
		const FPatchDataEnumerationConfiguration Configuration;
		const FString CloudDir;
		TUniquePtr<IFileSystem> FileSystem;
	};

	FPatchDataEnumeration::FPatchDataEnumeration(FPatchDataEnumerationConfiguration InConfiguration)
		: Configuration(MoveTemp(InConfiguration))
		, CloudDir(FPaths::GetPath(Configuration.InputFile))
		, FileSystem(FFileSystemFactory::Create())
	{
	}

	FPatchDataEnumeration::~FPatchDataEnumeration()
	{
	}

	bool FPatchDataEnumeration::Run()
	{
		TArray<FString> FullOutputList;
		bool bSuccess = Run(FullOutputList);
		if (bSuccess)
		{
			FString FullOutput = FString::Join(FullOutputList, TEXT("\r\n"));
			if (FFileHelper::SaveStringToFile(FullOutput, *Configuration.OutputFile))
			{
				UE_LOGF(LogDataEnumeration, Log, "Saved out to %ls", *Configuration.OutputFile);
				bSuccess = true;
			}
			else
			{
				UE_LOGF(LogDataEnumeration, Error, "Failed to save output %ls", *Configuration.OutputFile);
				bSuccess = false;
			}
		}
		return bSuccess;
	}

	bool FPatchDataEnumeration::Run(TArray<FString>& FullOutputList)
	{
		bool bSuccess = false;
		TUniquePtr<FArchive> File = FileSystem->CreateFileReader(*Configuration.InputFile);
		if (File.IsValid())
		{
			bSuccess = RunInternal(*File, FullOutputList);
		}
		else
		{
			UE_LOGF(LogDataEnumeration, Error, "Failed to open the file %ls", *Configuration.InputFile);
		}
		return bSuccess;
	}

	bool FPatchDataEnumeration::RunInternal(FArchive& Archive, TArray<FString>& OutFiles)
	{
		bool bEnumerationOk = false;
		if (EnumerationHelpers::IsChunkDbData(Archive))
		{
			bEnumerationOk = EnumerateChunkDbData(Archive, OutFiles);
		}
		else
		{
			bEnumerationOk = EnumerateManifestData(Archive, OutFiles);
		}
		return bEnumerationOk;
	}

	bool FPatchDataEnumeration::EnumerateManifestData(FArchive& Archive, TArray<FString>& OutFiles)
	{
		using namespace BuildPatchServices;
		bool bSuccess = false;
		TUniquePtr<FBuildPatchAppManifest> Manifest = EnumerationHelpers::LoadManifest(Archive);
		if (Manifest.IsValid())
		{
			bSuccess = true;
			TArray<TUniquePtr<FBuildPatchAppManifest>> DeltaFiles;
			const FString DeltaOptimisationsRoot = CloudDir / FBuildPatchUtils::GetChunkDeltaDirectory(*Manifest.Get());
			TArray<FString> DeltaOptimisationFiles;
			FileSystem->FindFilesRecursively(DeltaOptimisationFiles, *DeltaOptimisationsRoot);
			for (const FString& DeltaOptimisationFile : DeltaOptimisationFiles)
			{
				TUniquePtr<FArchive> File = FileSystem->CreateFileReader(*DeltaOptimisationFile);
				if (File.IsValid())
				{
					TUniquePtr<FBuildPatchAppManifest>& DeltaFile = DeltaFiles.Add_GetRef(EnumerationHelpers::LoadManifest(*File.Get()));
					if (DeltaFile.IsValid())
					{
						FString OutputLine = DeltaOptimisationFile.RightChop(CloudDir.Len() + 1);
						if (Configuration.bIncludeSizes)
						{
							uint64 FileSize = File->TotalSize();
							OutputLine += FString::Printf(TEXT("\t%" UINT64_FMT), FileSize);
						}
						UE_LOGF(LogDataEnumeration, Verbose, "%ls", *OutputLine);
						OutFiles.Add(MoveTemp(OutputLine));
					}
					else
					{
						UE_LOGF(LogDataEnumeration, Error, "Failed to deserialise delta %ls", *DeltaOptimisationFile);
						bSuccess = false;
					}
				}
				else
				{
					UE_LOGF(LogDataEnumeration, Error, "Failed to load delta %ls", *DeltaOptimisationFile);
					bSuccess = false;
				}
			}
			if (bSuccess)
			{
				TSet<FGuid> VisitedDataSet;
				EnumerationHelpers::AppendManifestDataList(*Manifest.Get(), OutFiles, VisitedDataSet, Configuration.bIncludeSizes);
				for (const TUniquePtr<FBuildPatchAppManifest>& DeltaFile : DeltaFiles)
				{
					FBuildPatchAppManifestPtr MergedManifest = FBuildMergeManifests::MergeDeltaManifest(*Manifest.Get(), *DeltaFile.Get());
					if (MergedManifest.IsValid())
					{
						EnumerationHelpers::AppendManifestDataList(*MergedManifest.Get(), OutFiles, VisitedDataSet, Configuration.bIncludeSizes);
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogDataEnumeration, Error, "Failed to extract a manifest data from archive");
		}
		return bSuccess;
	}

	bool FPatchDataEnumeration::EnumerateChunkDbData(FArchive& Archive, TArray<FString>& OutFiles)
	{
		using namespace BuildPatchServices;
		bool bSuccess = false;
		FChunkDatabaseHeader ChunkDbHeader;
		Archive << ChunkDbHeader;
		if (!Archive.IsError())
		{
			bSuccess = true;
			UE_LOGF(LogDataEnumeration, Verbose, "Data file list:-");
			for (const FChunkLocation& Location : ChunkDbHeader.Contents)
			{
				FChunkHeader ChunkHeader;
				Archive.Seek(Location.ByteStart);
				Archive << ChunkHeader;
				FString OutputLine = FString::Printf(TEXT("%s\t%s\t%s"), *Location.ChunkId.ToString(), *EnumerationHelpers::ToHexString(ChunkHeader.RollingHash), *ChunkHeader.SHAHash.ToString());
				if (Configuration.bIncludeSizes)
				{
					OutputLine += FString::Printf(TEXT("\t%u"), Location.ByteSize);
				}
				UE_LOGF(LogDataEnumeration, Verbose, "%ls", *OutputLine);
				OutFiles.Add(MoveTemp(OutputLine));
				// If the header did not give valid info, mark as failed but continue.
				if (!ChunkHeader.Guid.IsValid())
				{
					UE_LOG(LogDataEnumeration, Error, TEXT("Invalid chunk header for %s at %" UINT64_FMT), *Location.ChunkId.ToString(), Location.ByteStart);
					bSuccess = false;
				}
				// We treat a serialization error as critical, and stop reading.
				if (Archive.IsError())
				{
					UE_LOG(LogDataEnumeration, Error, TEXT("Serialization error when reading at byte %" UINT64_FMT ". Aborting."), Location.ByteStart);
					bSuccess = false;
					break;
				}
			}
		}
		else
		{
			UE_LOGF(LogDataEnumeration, Error, "Archive serialization failed because of internal error");
		}
		return bSuccess;
	}

	IPatchDataEnumeration* FPatchDataEnumerationFactory::Create(FPatchDataEnumerationConfiguration Configuration)
	{
		return new FPatchDataEnumeration(MoveTemp(Configuration));
	}
}
