// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Interfaces/IPatchDataEnumeration.h"
#include "BuildPatchSettings.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDataEnumeration, Log, All);

namespace BuildPatchServices
{
	class FPatchDataEnumerationFactory
	{
	public:
		static IPatchDataEnumeration* Create(FPatchDataEnumerationConfiguration Configuration);
	};
}
