// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Core/BlockRange.h"
#include "Core/BlockStructure.h"
#include "Common/FileSystem.h"
#include "Generation/ChunkSearch.h"
#include "Generation/ChunkWriter.h"
#include "Generation/DeltaEnumeration.h"
#include "Generation/DataScanner.h"
#include "BuildPatchUtil.h"

namespace BuildPatchServices
{
	/**
	 * This is a build data verification class that is not used for production code, but can be used to verify scan and other processing results
	 * to help test code while in development.
	 */
	class FBuildDataVerifier
	{
	public:
		FBuildDataVerifier(IFileSystem* InFileSystem, IChunkDataSerialization* InChunkDataSerialization, const FString& InBuildLocation, const FString& InOtherBuildLocation, const FString& InCloudDir, const FBuildPatchAppManifest& InManifest, const FBuildPatchAppManifest& InOtherManifest)
			: FileSystem(InFileSystem)
			, ChunkDataSerialization(InChunkDataSerialization)
			, BuildLocation(InBuildLocation)
			, OtherBuildLocation(InOtherBuildLocation)
			, CloudDir(InCloudDir)
			, Manifest(InManifest)
			, OtherManifest(InOtherManifest)
			, BuildFiles(ListHelpers::GetFileList(Manifest))
			, OtherBuildFiles(ListHelpers::GetFileList(OtherManifest))
		{
		}

		FArchive* LoadFile(const FString& BuildFile, bool bUseOther)
		{
			const FString FullFilename = bUseOther ? OtherBuildLocation / BuildFile : BuildLocation / BuildFile;
			TUniquePtr<FArchive>& LoadedFile = LoadedFiles.FindOrAdd(FullFilename);
			if (LoadedFile.IsValid() == false)
			{
				LoadedFile = FileSystem->CreateFileReader(*FullFilename);
			}
			check(LoadedFile.IsValid());
			return LoadedFile.Get();
		}

		void GetChunkData(const FChunkPart& ChunkPart, TArray<uint8>& OutData, FParallelChunkWriterSummaries* ChunkWriterSummaries = nullptr)
		{
			if (ChunkPart.IsPadding())
			{
				OutData.AddUninitialized(ChunkPart.Size);
				uint8* DataStart = OutData.GetData() + (OutData.Num() - ChunkPart.Size);
				FMemory::Memset(DataStart, ChunkPart.GetPaddingByte(), ChunkPart.Size);
				return;
			}
			if (LoadedChunks.Contains(ChunkPart.Guid) == false)
			{
				EChunkLoadResult ChunkLoadResult;
				FString DataFilename;
				if (ChunkWriterSummaries && ChunkWriterSummaries->ChunkOutputHashes.Contains(ChunkPart.Guid))
				{
					DataFilename = FBuildPatchUtils::GetChunkNewFilename(ChunkWriterSummaries->FeatureLevel, CloudDir, ChunkPart.Guid, ChunkWriterSummaries->ChunkOutputHashes[ChunkPart.Guid]);
				}
				else
				{
					const FBuildPatchAppManifest* ManifestRequired = Manifest.GetChunkInfo(ChunkPart.Guid) == nullptr ? &OtherManifest : &Manifest;
					DataFilename = FBuildPatchUtils::GetDataFilename(*ManifestRequired, CloudDir, ChunkPart.Guid);
				}
				LoadedChunks.Emplace(ChunkPart.Guid, ChunkDataSerialization->LoadFromFile(DataFilename, ChunkLoadResult));
				check(ChunkLoadResult == EChunkLoadResult::Success);
			}
			FScopeLockedChunkData ChunkDataAccess(LoadedChunks[ChunkPart.Guid].Get());
			OutData.Append(ChunkDataAccess.GetData() + ChunkPart.Offset, ChunkPart.Size);
		}

		void GetFileData(const FString& BuildFilename, const FBlockRange& BlockRange, TArray<uint8>& OutData, bool bUseOther = false)
		{
			TestBuffer.SetNumUninitialized(BlockRange.GetSize(), EAllowShrinking::No);
			FArchive* BuildFile = LoadFile(BuildFilename, bUseOther);
			BuildFile->Seek(BlockRange.GetFirst());
			BuildFile->Serialize(TestBuffer.GetData(), BlockRange.GetSize());
			check(BuildFile->IsError() == false);
			OutData.Append(TestBuffer.GetData(), BlockRange.GetSize());
		}

		void GetBuildData(const FBlockRange& BlockRange, TArray<uint8>& OutData, bool bUseOther = false)
		{
			uint64 BuildFileFirst = 0;
			uint64 ChunkPartFirst = 0;
			for (const FString& BuildFilename : (bUseOther ? OtherBuildFiles : BuildFiles))
			{
				const FFileManifest* FileManifest = (bUseOther ? OtherManifest : Manifest).GetFileManifest(BuildFilename);
				check(FileManifest != nullptr);
				const FBlockRange FileRange = FBlockRange::FromFirstAndSize(BuildFileFirst, FileManifest->FileSize);
				if (FileRange.Overlaps(BlockRange))
				{
					ChunkPartFirst = FileRange.GetFirst();
					for (const FChunkPart& ChunkPart : FileManifest->ChunkParts)
					{
						const FBlockRange ChunkPartRange = FBlockRange::FromFirstAndSize(ChunkPartFirst, ChunkPart.Size);
						if (ChunkPartRange.Overlaps(BlockRange))
						{
							const FBlockRange BuildBytesRange = FBlockRange::FromIntersection(BlockRange, ChunkPartRange);
							const int64 FileSeek = BuildBytesRange.GetFirst() - FileRange.GetFirst();
							const int64 DataIndex = BuildBytesRange.GetFirst() - BlockRange.GetFirst();
							TestBuffer.SetNumUninitialized(BuildBytesRange.GetSize(), EAllowShrinking::No);
							FArchive* BuildFile = LoadFile(BuildFilename, bUseOther);
							BuildFile->Seek(FileSeek);
							BuildFile->Serialize(TestBuffer.GetData(), BuildBytesRange.GetSize());
							check(BuildFile->IsError() == false);
							OutData.Append(TestBuffer.GetData(), BuildBytesRange.GetSize());
						}
						ChunkPartFirst += ChunkPartRange.GetSize();
					}
					check(ChunkPartFirst == (BuildFileFirst + FileRange.GetSize()));
				}
				BuildFileFirst += FileRange.GetSize();
			}
		}

		void GetBuildData(const FBlockStructure& BlockStructure, TArray<uint8>& OutData, bool bUseOther = false)
		{
			uint64 ExpectedSize = OutData.Num();
			const FBlockEntry* BlockEntry = BlockStructure.GetHead();
			while (BlockEntry)
			{
				GetBuildData(BlockEntry->AsRange(), OutData, bUseOther);
				ExpectedSize += BlockEntry->GetSize();
				BlockEntry = BlockEntry->GetNext();
			}
			check(OutData.Num() == ExpectedSize);
		}

		void CheckDataAndAssert(const FBlockStructure& BlockStructure, const uint8* Data)
		{
			if (BuildLocation.IsEmpty())
			{
				return;
			}
			TArray<uint8> BuildData;
			GetBuildData(BlockStructure, BuildData);
			check(FMemory::Memcmp(BuildData.GetData(), Data, BuildData.Num()) == 0);
		}

		void CheckDataAndAssert(const FBlockStructure& BlockStructure, const FSHAHash& SHAHash)
		{
			if (BuildLocation.IsEmpty())
			{
				return;
			}
			TArray<uint8> BuildData;
			GetBuildData(BlockStructure, BuildData);
			check(SHAHash == DeltaOptimiseHelpers::GetShaForDataSet(BuildData));
		}

		void CheckDataAndAssert(const IDeltaChunkEnumeration* DeltaChunkEnumeration, const FChunkBuildReference& ChunkBuildReference)
		{
			if (BuildLocation.IsEmpty())
			{
				return;
			}

			const FChunkPart& FirstChunkPart = ChunkBuildReference.Get<0>()[0];
			const FFilenameId& FilenameId = ChunkBuildReference.Get<1>();
			const uint64& FileOffset = ChunkBuildReference.Get<3>();
			const FString& Filename = DeltaChunkEnumeration->GetFilename(FilenameId);

			TArray<uint8> ChunkData;
			GetChunkData(FirstChunkPart, ChunkData);

			TArray<uint8> FileData;
			GetFileData(Filename, FBlockRange::FromFirstAndSize(FileOffset, FirstChunkPart.Size), FileData);

			check(FMemory::Memcmp(ChunkData.GetData(), FileData.GetData(), ChunkData.Num()) == 0);
		}

		void CheckDataAndAssert(const FBlockStructure& BlockStructure, const FBlockStructure& OtherBlockStructure)
		{
			if (BuildLocation.IsEmpty())
			{
				return;
			}
			TArray<uint8> BuildData;
			TArray<uint8> OtherBuildData;
			GetBuildData(BlockStructure, BuildData, false);
			GetBuildData(OtherBlockStructure, OtherBuildData, true);
			check(BuildData.Num() == OtherBuildData.Num());
			check(FMemory::Memcmp(BuildData.GetData(), OtherBuildData.GetData(), BuildData.Num()) == 0);
		}

		void CheckDataAndAssert(const FFileManifestList& FileManifestList, FParallelChunkWriterSummaries* ChunkWriterSummaries = nullptr)
		{
			if (BuildLocation.IsEmpty())
			{
				return;
			}
			TArray<uint8> BuildBuffer;
			TArray<uint8> ChunkBuffer;
			uint64 ChunkPartStart = 0;
			for (const FFileManifest& FileManifest : FileManifestList.FileList)
			{
				for (const FChunkPart& ChunkPart : FileManifest.ChunkParts)
				{
					check(ChunkWriterSummaries == nullptr || ChunkPart.Guid.IsValid());
					if (ChunkPart.Guid.IsValid())
					{
						BuildBuffer.SetNumUninitialized(0, EAllowShrinking::No);
						ChunkBuffer.SetNumUninitialized(0, EAllowShrinking::No);
						FBlockRange ChunkPartRange = FBlockRange::FromFirstAndSize(ChunkPartStart, ChunkPart.Size);
						GetBuildData(ChunkPartRange, BuildBuffer);
						GetChunkData(ChunkPart, ChunkBuffer, ChunkWriterSummaries);
						check(FMemory::Memcmp(BuildBuffer.GetData(), ChunkBuffer.GetData(), ChunkPart.Size) == 0);
					}
					ChunkPartStart += ChunkPart.Size;
				}
			}
		}

		void CheckDataAndAssert(const FBlockStructure& BlockStructure, const FChunkPart& ChunkPart, bool bUseOther = false)
		{
			if (BuildLocation.IsEmpty())
			{
				return;
			}
			TArray<uint8> BuildData;
			TArray<uint8> ChunkData;
			GetBuildData(BlockStructure, BuildData, bUseOther);
			if (ChunkPart.Guid.IsValid())
			{
				GetChunkData(ChunkPart, ChunkData);
				check(BuildData.Num() == ChunkData.Num());
				check(FMemory::Memcmp(BuildData.GetData(), ChunkData.GetData(), BuildData.Num()) == 0);
			}
			else
			{
				check(BuildData.Num() == ChunkPart.Size);
			}
		}

		void CheckDataAndAssert(FChunkSearcher::FChunkDList& ChunkDList, bool bUseOther = false)
		{
			if (BuildLocation.IsEmpty())
			{
				return;
			}
			TArray<uint8> BuildData;
			TArray<uint8> ChunkData;
			FChunkSearcher::FChunkDListNode* ChunkNode = ChunkDList.GetHead();
			while (ChunkNode)
			{
				BuildData.SetNumUninitialized(0, EAllowShrinking::No);
				ChunkData.SetNumUninitialized(0, EAllowShrinking::No);
				GetBuildData(ChunkNode->GetValue().BuildRange, BuildData, bUseOther);
				if (ChunkNode->GetValue().ChunkPart.Guid.IsValid())
				{
					GetChunkData(ChunkNode->GetValue().ChunkPart, ChunkData);
					check(BuildData.Num() == ChunkData.Num());
					check(FMemory::Memcmp(BuildData.GetData(), ChunkData.GetData(), BuildData.Num()) == 0);
				}
				else
				{
					check(BuildData.Num() == ChunkNode->GetValue().ChunkPart.Size);
				}
				ChunkNode = ChunkNode->GetNextNode();
			}
		}

		void CheckDataAndAssert(const FScannerFilesList& ChunkDList, const IDeltaChunkEnumeration* DeltaChunkEnumeration, const TArray<uint8>& ScannerData, bool bUseOther = false)
		{
			if (BuildLocation.IsEmpty())
			{
				return;
			}
			const FScannerFilesListNode* Node = ChunkDList.GetHead();
			while (Node)
			{
				const FScannerFileElement& Element = Node->GetValue();

				const FBlockRange& ScanDataRange = Element.Get<0>();
				const FFilenameId& FilenameId = Element.Get<1>();
				const uint64& FileOffset = Element.Get<3>();
				const FString& Filename = DeltaChunkEnumeration->GetFilename(FilenameId);

				TArray<uint8> FileData;
				GetFileData(Filename, FBlockRange::FromFirstAndSize(FileOffset, ScanDataRange.GetSize()), FileData);

				check(FMemory::Memcmp(ScannerData.GetData() + ScanDataRange.GetFirst(), FileData.GetData(), ScanDataRange.GetSize()) == 0);

				Node = Node->GetNextNode();
			}
		}

		void FindDifferences(const FFileManifestList& FileManifestList, FChunkSearcher::FFileDListNode* FileHead)
		{
			FChunkSearcher::FFileDListNode* FileNode = FileHead;
			uint64 BuildFileFirst = 0;
			uint64 ChunkPartFirst = 0;
			for (const FFileManifest& FileManifest : FileManifestList.FileList)
			{
				const FBlockRange FileRange = FBlockRange::FromFirstAndSize(BuildFileFirst, FileManifest.FileSize);
				check(FileNode->GetValue().Manifest->Filename == FileManifest.Filename);
				check(FileNode->GetValue().BuildRange == FileRange);
				check(FileManifest.ChunkParts.Num() == FileNode->GetValue().ChunkParts.Num());
				FChunkSearcher::FChunkDListNode* ChunkNode = FileNode->GetValue().ChunkParts.GetHead();
				for (const FChunkPart& ChunkPart : FileManifest.ChunkParts)
				{
					const FBlockRange ChunkPartRange = FBlockRange::FromFirstAndSize(ChunkPartFirst, ChunkPart.Size);
					check(ChunkPart.Guid == ChunkNode->GetValue().ChunkPart.Guid);
					check(ChunkPart.Offset == ChunkNode->GetValue().ChunkPart.Offset);
					check(ChunkPart.Size == ChunkNode->GetValue().ChunkPart.Size);
					check(ChunkPartRange == ChunkNode->GetValue().BuildRange);
					ChunkPartFirst += ChunkPart.Size;
					ChunkNode = ChunkNode->GetNextNode();
				}
				BuildFileFirst += FileRange.GetSize();
				FileNode = FileNode->GetNextNode();
			}
		}

		void FindDifferences(const FFileManifestList& FileManifestListA, const FFileManifestList& FileManifestListB)
		{
			uint64 BuildFileFirst = 0;
			uint64 ChunkPartFirst = 0;
			check(FileManifestListA.FileList.Num() == FileManifestListB.FileList.Num());
			for (int32 FileManifestIdx = 0; FileManifestIdx < FileManifestListA.FileList.Num(); ++FileManifestIdx)
			{
				const FFileManifest& FileManifestA = FileManifestListA.FileList[FileManifestIdx];
				const FFileManifest& FileManifestB = FileManifestListB.FileList[FileManifestIdx];
				const FBlockRange FileRangeA = FBlockRange::FromFirstAndSize(BuildFileFirst, FileManifestA.FileSize);
				const FBlockRange FileRangeB = FBlockRange::FromFirstAndSize(BuildFileFirst, FileManifestB.FileSize);

				check(FileManifestA.Filename == FileManifestB.Filename);
				check(FileRangeA == FileRangeB);
				check(FileManifestA.ChunkParts.Num() == FileManifestB.ChunkParts.Num());

				for (int32 ChunkPartIdx = 0; ChunkPartIdx < FileManifestA.ChunkParts.Num(); ++ChunkPartIdx)
				{
					const FChunkPart& ChunkPartA = FileManifestA.ChunkParts[ChunkPartIdx];
					const FChunkPart& ChunkPartB = FileManifestB.ChunkParts[ChunkPartIdx];
					const FBlockRange ChunkPartRangeA = FBlockRange::FromFirstAndSize(ChunkPartFirst, ChunkPartA.Size);
					const FBlockRange ChunkPartRangeB = FBlockRange::FromFirstAndSize(ChunkPartFirst, ChunkPartB.Size);

					check(ChunkPartA.Guid == ChunkPartB.Guid);
					check(ChunkPartA.Offset == ChunkPartB.Offset);
					check(ChunkPartA.Size == ChunkPartB.Size);
					check(ChunkPartRangeA == ChunkPartRangeB);

					ChunkPartFirst += ChunkPartRangeA.GetSize();
				}
				BuildFileFirst += FileRangeA.GetSize();
			}
		}

	private:
		IFileSystem* const FileSystem;
		IChunkDataSerialization* const ChunkDataSerialization;
		const FString BuildLocation;
		const FString OtherBuildLocation;
		const FString CloudDir;
		const FBuildPatchAppManifest& Manifest;
		const FBuildPatchAppManifest& OtherManifest;
		const TArray<FString> BuildFiles;
		const TArray<FString> OtherBuildFiles;
		TMap<FString, TUniquePtr<FArchive>> LoadedFiles;
		TMap<FGuid, TUniquePtr<IChunkDataAccess>> LoadedChunks;
		TArray<uint8> TestBuffer;
	};
}
