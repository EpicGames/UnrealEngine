// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BuildPatchManifest.h"
#include "Common/StatsCollector.h"
#include "Core/BlockStructure.h"
#include "Core/Factory.h"
#include "Core/FileSpan.h"
#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/CloudChunkSource.h"
#include "Misc/SecureHash.h"

namespace BuildPatchServices
{
	class IBuildStreamer
	{
	public:
		virtual ~IBuildStreamer() = default;

		/**
		 * Fetches some data from the buffer, also removing it.
		 * @param   IN  Buffer          Pointer to buffer to receive the data.
		 * @param   IN  ReqSize         The amount of data to attempt to retrieve.
		 * @param   IN  WaitForData     Optional: Default true. Whether to wait until there is enough data in the buffer.
		 * @return the amount of data retrieved.
		 */
		virtual uint32 DequeueData(uint8* Buffer, uint32 ReqSize, bool WaitForData = true) = 0;

		/**
		 * Whether there is any more data available to dequeue from the buffer.
		 * @return true if there is no more data coming in, and the internal buffer is also empty.
		 */
		virtual bool IsEndOfData() const = 0;
	};

	class IDirectoryBuildStreamer
		: public IBuildStreamer
	{
	public:
		/**
		 * Get the total build size that was streamed.
		 * MUST be called only after IsEndOfData returns true.
		 * @return the number of bytes in the streamed build.
		 */
		virtual uint64 GetBuildSize() const = 0; 

		/**
		 * Get the list of file spans for each file in the build, including empty files.
		 * MUST be called only after IsEndOfData returns true.
		 * @return the list of files in the build and their details.
		 */
		virtual TArray<FFileSpan> GetAllFiles() const = 0;

		/**
		 * Gets if the streamer has aborted and is no longer streaming data
		 * @return True if it has aborted
		 */
		virtual bool HasAborted() const = 0;
	};

	class IManifestBuildStreamer
		: public IBuildStreamer
	{
	public:
		// Declares types for expected factory dependencies.
		typedef TArray<FGuid> FCustomChunkReferences;
		typedef TFactory<IChunkReferenceTracker, FCustomChunkReferences> IChunkReferenceTrackerFactory;
		typedef TFactory<ICloudChunkSource, IChunkReferenceTracker*> ICloudChunkSourceFactory;
	public:
		/**
		 * Gets the block structure that this streamer was configured with.
		 * @return the configured DesiredBytes block structure.
		 */
		virtual const FBlockStructure& GetBlockStructure() const = 0;
	};

	// Configuration for constructing a directory build streamer.
	struct FDirectoryBuildStreamerConfig
	{
	public:
		const FString BuildRoot;
		const TArray<FString> BuildFiles;
		const bool bFollowSymlinks = false;
		const bool bDetectMimeType = true;
		const bool bCalculateMD5 = true;
		const bool bCalculateSHA256 = true;
	};

	// Holds all dependencies for constructing a directory build streamer.
	struct FDirectoryBuildStreamerDependencies
	{
	public:
		FStatsCollector* const StatsCollector;
		IFileSystem* const FileSystem;
	};

	// Configuration for constructing a manifest build streamer.
	struct FManifestBuildStreamerConfig
	{
	public:
		TArray<FString> CloudDirectories;
		FBlockStructure DesiredBytes;
	};

	// Holds all dependencies for constructing a manifest build streamer.
	struct FManifestBuildStreamerDependencies
	{
	public:
		IManifestBuildStreamer::IChunkReferenceTrackerFactory* const ChunkReferenceTrackerFactory;
		IManifestBuildStreamer::ICloudChunkSourceFactory* const CloudChunkSourceFactory;
		FStatsCollector* const StatsCollector;
		FBuildPatchAppManifest* const Manifest;
	};

	class FBuildStreamerFactory
	{
	public:
		/**
		 * Factory for constructing a build streamer based on a directory of files.
		 * @param Config        The configuration struct.
		 * @param Dependencies  The dependencies struct.
		 * @return the constructed build streamer.
		 */
		static IDirectoryBuildStreamer* Create(FDirectoryBuildStreamerConfig Config, FDirectoryBuildStreamerDependencies Dependencies);

		/**
		 * Factory for constructing a build streamer based on an existing manifest and block ranges.
		 * @param Config        The configuration struct.
		 * @param Dependencies  The dependencies struct.
		 * @return the constructed build streamer.
		 */
		static IManifestBuildStreamer* Create(FManifestBuildStreamerConfig Config, FManifestBuildStreamerDependencies Dependencies);
	};
}
