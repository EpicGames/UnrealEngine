// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/ChunkDatabaseWriter.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Queue.h"
#include "Async/Async.h"
#include "Algo/Transform.h"
#include "Serialization/MemoryWriter.h"
#include "Core/AsyncHelpers.h"
#include "Data/ChunkData.h"
#include "Common/FileSystem.h"
#include "Common/StatsCollector.h"
#include "Installer/InstallerError.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/ChunkSource.h"
#include "Installer/Statistics/FileConstructorStatistics.h"
#include "Interfaces/IBuildInstaller.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChunkDatabaseWriter, Log, All);
DEFINE_LOG_CATEGORY(LogChunkDatabaseWriter);

namespace BuildPatchServices
{
	// Using initial data buffer of 2MB.
	static const int32 DataMessageBufferSize = 1024 * 1024 * 2;

	uint64 CountTotalChunks(const FChunkDbWriterConfig& Configuration)
	{
		uint64 NumDataMessages = 0;
		for (const FChunkDatabaseFile& ChunkDatabaseFile : Configuration.ChunkDatabaseList)
		{
			NumDataMessages += ChunkDatabaseFile.DataList.Num();
		}
		return NumDataMessages;
	}

	struct FDataMessage
	{
	public:
		static const int64 CreateFileId = -1;
		static const int64 RenameFileId = -2;

		FDataMessage(FString InFilename, int64 MessageId)
			: Filename(InFilename)
			, DataInfo(MessageId)
		{
			if (Filename.IsEmpty() || MessageId >= 0)
			{
				UE_LOGF(LogChunkDatabaseWriter, Error, "Created action message with no filename or position as ID, WILL RESULT IN ERROR.");
			}
		}

		FDataMessage(int64 InPos)
			: Filename()
			, DataInfo(InPos)
		{
			if (InPos < 0)
			{
				UE_LOGF(LogChunkDatabaseWriter, Error, "Created data message with message ID, WILL RESULT IN ERROR.");
			}
			Memory.Reset(DataMessageBufferSize);
		}

	public:
		FString Filename;
		int64 DataInfo;
		TArray<uint8> Memory;

	private:
		FDataMessage() {}
	};

	class FChunkDatabaseWriter
		: public IChunkDatabaseWriter
	{
	public:
		FChunkDatabaseWriter(FChunkDbWriterConfig Configuration, IChunkSource& ChunkSource, IFileSystem& FileSystem, IInstallerError& InstallerError, IChunkReferenceTracker& ChunkReferenceTracker, IChunkDataSerialization& ChunkDataSerialization, IFileConstructorStat& FileConstructorStat, TFunction<void(float)> OnProgress, TFunction<void(bool)> OnComplete);

		~FChunkDatabaseWriter();

		// IControllable interface begin.
		virtual void SetPaused(bool bInIsPaused) override
		{
			bIsPaused = bInIsPaused;
		}

		virtual void Abort() override
		{
			bShouldCancel = true;
		}

		virtual void Reset() override
		{
			bShouldCancel = false;
		}
		// IControllable interface end.

	private:
		void ProcessingWorkerThread();
		void OutputWorkerThread();
		void HandleProgress(float Progress);
		void HandleComplete(bool bSuccess);

	private:
		const FChunkDbWriterConfig Configuration;
		const uint64 TotalNumChunks;
		IChunkSource& ChunkSource;
		IFileSystem& FileSystem;
		IInstallerError& InstallerError;
		IChunkReferenceTracker& ChunkReferenceTracker;
		IChunkDataSerialization& ChunkDataSerialization;
		IFileConstructorStat& FileConstructorStat;
		TFunction<void(float)> OnProgressFn;
		TFunction<void(bool)> OnCompleteFn;
		TFuture<void> ProcessingWorkerFuture;
		TFuture<void> OutputWorkerFuture;
		FThreadSafeBool bIsPaused;
		FThreadSafeBool bShouldCancel;
		FThreadSafeBool bProcessingComplete;
		TQueue<TSharedPtr<FDataMessage, ESPMode::ThreadSafe>, EQueueMode::Spsc> DataPipe;
		FThreadSafeCounter MessageCount;
		FEvent* ThreadTrigger;
	};

	FChunkDatabaseWriter::FChunkDatabaseWriter(FChunkDbWriterConfig InConfiguration, IChunkSource& InChunkSource, IFileSystem& InFileSystem, IInstallerError& InInstallerError, IChunkReferenceTracker& InChunkReferenceTracker, IChunkDataSerialization& InChunkDataSerialization, IFileConstructorStat& InFileConstructorStat, TFunction<void(float)> InOnProgress, TFunction<void(bool)> InOnComplete)
		: Configuration(InConfiguration)
		, TotalNumChunks(CountTotalChunks(Configuration))
		, ChunkSource(InChunkSource)
		, FileSystem(InFileSystem)
		, InstallerError(InInstallerError)
		, ChunkReferenceTracker(InChunkReferenceTracker)
		, ChunkDataSerialization(InChunkDataSerialization)
		, FileConstructorStat(InFileConstructorStat)
		, OnProgressFn(MoveTemp(InOnProgress))
		, OnCompleteFn(MoveTemp(InOnComplete))
		, bShouldCancel(false)
		, bProcessingComplete(false)
	{
		ThreadTrigger = FPlatformProcess::GetSynchEventFromPool(true);
		ProcessingWorkerFuture = Async(EAsyncExecution::Thread, [this]()
		{
			ProcessingWorkerThread();
		});
		OutputWorkerFuture = Async(EAsyncExecution::Thread, [this]()
		{
			OutputWorkerThread();
		});
	}

	FChunkDatabaseWriter::~FChunkDatabaseWriter()
	{
		bShouldCancel = true;
		ProcessingWorkerFuture.Wait();
		OutputWorkerFuture.Wait();
		FPlatformProcess::ReturnSynchEventToPool(ThreadTrigger);
	}

	void FChunkDatabaseWriter::ProcessingWorkerThread()
	{
		bool bSuccess = true;

		// For every entry in the provided ChunkDatabaseList, create the chunkdb, and send serialized data to the output thread for it.
		for (int32 ChunkDatabaseIdx = 0; ChunkDatabaseIdx < Configuration.ChunkDatabaseList.Num() && bSuccess && !bShouldCancel && !InstallerError.HasError(); ++ChunkDatabaseIdx)
		{
			const FChunkDatabaseFile& ChunkDatabaseFile = Configuration.ChunkDatabaseList[ChunkDatabaseIdx];
			UE_LOGF(LogChunkDatabaseWriter, Log, "Start processing chunk database %ls", *ChunkDatabaseFile.DatabaseFilename);

			// Send file create message.
			const FString CreateFilename = ChunkDatabaseFile.DatabaseFilename + (Configuration.bUseTempFile ? TEXT("tmp") : TEXT(""));
			DataPipe.Enqueue(MakeShareable(new FDataMessage(CreateFilename, FDataMessage::CreateFileId)));
			MessageCount.Increment();
			ThreadTrigger->Trigger();

			// Populate header with all required entries.
			FChunkDatabaseHeader ChunkDbHeader;
			Algo::Transform(ChunkDatabaseFile.DataList, ChunkDbHeader.Contents, [](const FGuid& DataId)
			{
				return FChunkLocation{DataId, 0, 0};
			});

			// Write the initial header.
			TUniquePtr<FDataMessage> DataMessage;
			TUniquePtr<FMemoryWriter> MemoryWriter;
			uint64 LastHeaderWrite = 0;
			int64 FileDataPos = 0;
			auto SendHeaderMessage = [&]()
			{
				DataMessage.Reset(new FDataMessage(0));
				MemoryWriter.Reset(new FMemoryWriter(DataMessage->Memory));
				ChunkDbHeader.DataSize = FileDataPos - ChunkDbHeader.HeaderSize;
				*MemoryWriter << ChunkDbHeader;
				int64 NumBytesSent = DataMessage->Memory.Num();
				DataPipe.Enqueue(MakeShareable(DataMessage.Release()));
				MessageCount.Increment();
				ThreadTrigger->Trigger();
				LastHeaderWrite = FStatsCollector::GetCycles();
				return NumBytesSent;
			};
			FileDataPos = SendHeaderMessage();

			// Serialize and write each of the chunk files.
			for (int32 ChunkDataIdx = 0; ChunkDataIdx < ChunkDatabaseFile.DataList.Num() && bSuccess && !bShouldCancel && !InstallerError.HasError(); ChunkDataIdx++)
			{
				const FGuid& ChunkDataId = ChunkDatabaseFile.DataList[ChunkDataIdx];
				IChunkDataAccess* ChunkDataAccess = ChunkSource.Get(ChunkDataId);
				bSuccess = ChunkDataAccess != nullptr;
				if (bSuccess)
				{
					// Prepare new message.
					DataMessage.Reset(new FDataMessage(FileDataPos));
					MemoryWriter.Reset(new FMemoryWriter(DataMessage->Memory));

					// Write to message.
					EChunkSaveResult SaveResult;
					if (Configuration.bReserialise)
					{
						SaveResult = ChunkDataSerialization.SaveToArchive(*MemoryWriter, ChunkDataAccess);
					}
					else
					{
						FScopeLockedChunkData LockedChunkData(ChunkDataAccess);
						*MemoryWriter << *LockedChunkData.GetHeader();
						MemoryWriter->Serialize(LockedChunkData.GetData(), LockedChunkData.GetHeader()->DataSizeCompressed);
						SaveResult = MemoryWriter->IsError() ? EChunkSaveResult::SerializationError : EChunkSaveResult::Success;
					}
					bSuccess = SaveResult == EChunkSaveResult::Success;
					if (!bSuccess)
					{
						const TCHAR* ErrorCode = SaveResult == EChunkSaveResult::FileCreateFail ? ConstructionErrorCodes::FileCreateFail
						                       : SaveResult == EChunkSaveResult::SerializationError ? ConstructionErrorCodes::SerializationError
						                       : ConstructionErrorCodes::UnknownFail;
						InstallerError.SetError(EBuildPatchInstallError::FileConstructionFail, ErrorCode);
					}

					// Set the positional data in the header.
					FChunkLocation& Location = ChunkDbHeader.Contents[ChunkDataIdx];
					Location.ByteStart = FileDataPos;
					Location.ByteSize = DataMessage->Memory.Num();

					// Advance file position.
					FileDataPos += Location.ByteSize;

					// Wait while behind
					while (MessageCount.GetValue() > Configuration.BufferEnqueueMax && !bShouldCancel)
					{
						FPlatformProcess::Sleep(0.1f);
					}

					// Send the data message.
					DataPipe.Enqueue(MakeShareable(DataMessage.Release()));
					MessageCount.Increment();
					ThreadTrigger->Trigger();

					// Pop the chunk we just saved out.
					bSuccess = ChunkReferenceTracker.PopReference(ChunkDataId);
					if (!bSuccess)
					{
						InstallerError.SetError(EBuildPatchInstallError::InitializationError, InitializationErrorCodes::ChunkReferenceTracking);
					}
					else if (Configuration.HeaderUpdateFrequency > 0.0f)
					{
						// Check if we should update the header to disk.
						uint64 CurrentCycles = FStatsCollector::GetCycles();
						if (Configuration.HeaderUpdateFrequency <= FStatsCollector::CyclesToSeconds(CurrentCycles - LastHeaderWrite))
						{
							SendHeaderMessage();
						}
					}
				}
				if (!bSuccess)
				{
					UE_LOGF(LogChunkDatabaseWriter, Error, "    Failed chunk %ls", *ChunkDataId.ToString());
					// We set a catch all error here, which only applies if our own loop above, or the chunk source, did not already set its specific error condition.
					InstallerError.SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::UnknownFail);
				}
				// Wait while paused
				while (bIsPaused && !bShouldCancel)
				{
					FPlatformProcess::Sleep(0.5f);
				}
			}
			if (bSuccess)
			{
				// Write back the header with all chunk positions now filled out accurately.
				SendHeaderMessage();
			}

			// Send file rename message.
			if (Configuration.bUseTempFile)
			{
				DataPipe.Enqueue(MakeShareable(new FDataMessage(ChunkDatabaseFile.DatabaseFilename, FDataMessage::RenameFileId)));
				MessageCount.Increment();
				ThreadTrigger->Trigger();
			}
		}

		// Mark completed.
		bProcessingComplete = true;
		ThreadTrigger->Trigger();
		UE_LOGF(LogChunkDatabaseWriter, Log, "Processer complete! bSuccess:%d", bSuccess);
	}

	void FChunkDatabaseWriter::OutputWorkerThread()
	{
		bool bSuccess = true;
		const float TotalNumChunksFloat = TotalNumChunks;
		float ProcessedNumChunksFloat = 0;
		TArray<FString> FilesCreated;
		TSharedPtr<FDataMessage, ESPMode::ThreadSafe> DataMessage;
		TUniquePtr<FArchive> CurrentFile;
		while (bSuccess && !bShouldCancel && !InstallerError.HasError())
		{
			// See if we have a message.
			if (DataPipe.Dequeue(DataMessage))
			{
				MessageCount.Decrement();
				// Process a file create message.
				if (DataMessage->DataInfo == FDataMessage::CreateFileId)
				{
					if (CurrentFile.IsValid())
					{
						FileConstructorStat.OnFileCompleted(CurrentFile->GetArchiveName(), !CurrentFile->IsError());
					}
					UE_LOGF(LogChunkDatabaseWriter, Log, "Writing chunk database %ls", *DataMessage->Filename);
					ISpeedRecorder::FRecord ActivityRecord;
					FileConstructorStat.OnFileStarted(DataMessage->Filename, 0);
					FileConstructorStat.OnBeforeAdminister();
					ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
					CurrentFile = FileSystem.CreateFileWriter(*DataMessage->Filename);
					ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
					ActivityRecord.Size = 0;
					FileConstructorStat.OnAfterAdminister(ActivityRecord);
					FilesCreated.Add(DataMessage->Filename);
					if (CurrentFile.IsValid() == false)
					{
						bSuccess = false;
						UE_LOGF(LogChunkDatabaseWriter, Error, "Failed to create file with name %ls", *DataMessage->Filename);
						InstallerError.SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::FileCreateFail);
					}
				}
				// Process a data serialize.
				else if (CurrentFile.IsValid() && DataMessage->DataInfo >= 0)
				{
					if (CurrentFile->Tell() != DataMessage->DataInfo)
					{
						ISpeedRecorder::FRecord ActivityRecord;
						FileConstructorStat.OnBeforeAdminister();
						ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
						CurrentFile->Seek(DataMessage->DataInfo);
						ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
						ActivityRecord.Size = 0;
						FileConstructorStat.OnAfterAdminister(ActivityRecord);
					}
					ISpeedRecorder::FRecord ActivityRecord;
					FileConstructorStat.OnBeforeWrite();
					ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
					CurrentFile->Serialize(DataMessage->Memory.GetData(), DataMessage->Memory.Num());
					ActivityRecord.Size = DataMessage->Memory.Num();
					ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
					FileConstructorStat.OnAfterWrite(ActivityRecord);
					FileConstructorStat.OnFileProgress(CurrentFile->GetArchiveName(), CurrentFile->TotalSize());
					ProcessedNumChunksFloat += 1.0f;
					HandleProgress(ProcessedNumChunksFloat / TotalNumChunksFloat);
				}
				// Process a file rename message.
				else if (CurrentFile.IsValid() && DataMessage->DataInfo == FDataMessage::RenameFileId)
				{
					const FString OldFilename = CurrentFile->GetArchiveName();
					bSuccess = CurrentFile->Close();
					CurrentFile.Reset();
					FileConstructorStat.OnFileCompleted(OldFilename, bSuccess);
					if (bSuccess)
					{
						ISpeedRecorder::FRecord ActivityRecord;
						FileConstructorStat.OnBeforeAdminister();
						ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
						FileSystem.MoveFile(*DataMessage->Filename, *OldFilename);
						ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
						ActivityRecord.Size = 0;
						FileConstructorStat.OnAfterAdminister(ActivityRecord);
						UE_LOGF(LogChunkDatabaseWriter, Log, "Chunk database complete, renamed %ls", *DataMessage->Filename);
					}
					else
					{
						UE_LOGF(LogChunkDatabaseWriter, Error, "Serialisation error reported on file close %ls", *OldFilename);
						InstallerError.SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::SerializationError);
					}
				}
				// An error if we do not have a file open and we were sent any message other than create.
				else
				{
					bSuccess = false;
					UE_LOGF(LogChunkDatabaseWriter, Error, "Output fail, message without a file");
					InstallerError.SetError(EBuildPatchInstallError::FileConstructionFail, ConstructionErrorCodes::MissingFileInfo);
				}
			}
			// Quit if no more messages
			else if (bProcessingComplete)
			{
				break;
			}
			// Wait up to 1 second for an enqueue trigger.
			else
			{
				ThreadTrigger->Wait(1000);
				ThreadTrigger->Reset();
			}
		}

		// Close the last open file.
		if (CurrentFile.IsValid())
		{
			FileConstructorStat.OnFileCompleted(CurrentFile->GetArchiveName(), !CurrentFile->IsError());
		}
		CurrentFile.Reset();

		// Check whether the process was canceled or an error occurred.
		bSuccess = bSuccess && !bShouldCancel && !InstallerError.HasError();
		UE_LOGF(LogChunkDatabaseWriter, Log, "Writer complete! bSuccess:%d", bSuccess);

		// Delete any created files if we failed.
		if (!bSuccess && Configuration.bDeleteFilesOnFailure)
		{
			UE_LOGF(LogChunkDatabaseWriter, Error, "Chunkdb generation failed. All created files will be deleted.");
			for (const FString& FileToDelete : FilesCreated)
			{
				FileSystem.DeleteFile(*FileToDelete);
				UE_LOGF(LogChunkDatabaseWriter, Log, "Deleted %ls", *FileToDelete);
			}
		}

		// We're done so call the complete callback.
		HandleComplete(bSuccess);
	}

	void FChunkDatabaseWriter::HandleProgress(float Progress)
	{
		if (Configuration.bCallbackOnMainThread)
		{
			AsyncHelpers::ExecuteOnGameThread(OnProgressFn, Progress).Wait();
		}
		else
		{
			OnProgressFn(Progress);
		}
	}

	void FChunkDatabaseWriter::HandleComplete(bool bSuccess)
	{
		if (Configuration.bCallbackOnMainThread)
		{
			AsyncHelpers::ExecuteOnGameThread(OnCompleteFn, bSuccess).Wait();
		}
		else
		{
			OnCompleteFn(bSuccess);
		}
	}

	IChunkDatabaseWriter* FChunkDatabaseWriterFactory::Create(FChunkDbWriterConfig Configuration, IChunkSource* ChunkSource, IFileSystem* FileSystem, IInstallerError* InstallerError, IChunkReferenceTracker* ChunkReferenceTracker, IChunkDataSerialization* ChunkDataSerialization, IFileConstructorStat* FileConstructorStat, TFunction<void(float)> OnProgress, TFunction<void(bool)> OnComplete)
	{
		check(ChunkSource != nullptr);
		check(FileSystem != nullptr);
		check(InstallerError != nullptr);
		check(ChunkReferenceTracker != nullptr);
		check(FileConstructorStat != nullptr);
		return new FChunkDatabaseWriter(MoveTemp(Configuration), *ChunkSource, *FileSystem, *InstallerError, *ChunkReferenceTracker, *ChunkDataSerialization, *FileConstructorStat, MoveTemp(OnProgress), MoveTemp(OnComplete));
	}
}
