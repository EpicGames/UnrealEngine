// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Installer/Controllable.h"

namespace BuildPatchServices
{
	class IChunkSource;
	class IFileSystem;
	class IInstallerError;
	class IChunkReferenceTracker;
	class IChunkDataSerialization;
	class IFileConstructorStat;

	struct FChunkDatabaseFile
	{
		FString DatabaseFilename;
		TArray<FGuid> DataList;
	};

	/**
	 * An interface providing providing access to a system which writes chunk database files, given a chunk source and details of the chunks to put
	 * into databases.
	 */
	class IChunkDatabaseWriter
		: public IControllable
	{
	public:
		virtual ~IChunkDatabaseWriter() = default;
	};

	/**
	 * A struct containing the configuration values for a chunkdb writer.
	 */
	struct FChunkDbWriterConfig
	{
		// The array of chunk database files to create and the chunks to place in them.
		TArray<FChunkDatabaseFile> ChunkDatabaseList;
		// The max threaded messages to wait on in case of HDD write is slower that chunk fetch.
		int32 BufferEnqueueMax = 100;
		// Whether to use temp file when building, this is safer against failures, but if false, can support resuming.
		// It is advised to use true when generating for cloud upload, and false if producing for local use only on a client for later consumption.
		bool bUseTempFile = true;
		// If more than 0, the header will be updated every X seconds to minimise loss of data if the process is cancelled.
		float HeaderUpdateFrequency = 0.0f;
		// If true, will potentially re-encrypt and/or re-compress on save to chumdb.
		// If false, the chunk will be stored as deserialised, and thus decrypted/decompressed where it was possible.
		bool bReserialise = true;
		// If true, will delete created files upon a failure, otherwise will leave potentially unfinished files on disk.
		bool bDeleteFilesOnFailure = true;
		// If set to false, the progress and complete callbacks will be executed on the worker threads.
		bool bCallbackOnMainThread = true;

		/**
		 * Constructor which sets usual defaults, and takes params for values that cannot use a default.
		 * @param InChunkDatabaseList    The chunkdb filename array.
		 */
		FChunkDbWriterConfig(TArray<FChunkDatabaseFile> InChunkDatabaseList)
			: ChunkDatabaseList(MoveTemp(InChunkDatabaseList))
		{
		}
	};

	/**
	 * A factory for creating an IChunkDatabaseWriter instance.
	 */
	class FChunkDatabaseWriterFactory
	{
	public:
		/**
		 * This implementation returns a chunk database writer that immediately kicks off the work and calls a provided callback when complete.
		 * @param Configuration             The configuration struct for this source.
		 * @param ChunkSource               A chunk source for pulling required chunks from.
		 * @param FileSystem                A files system interface for writing out the chunkdb files.
		 * @param InstallerError            The error interface for aborting on other errors or registering our own.
		 * @param ChunkReferenceTracker     Chunk reference tracker to keep up to date.
		 * @param ChunkDataSerialization    Chunk data serialization implementation.
		 * @param InFileConstructorStat     A file constructor stats class to track HDD activity.
		 * @param OnProgress                Function to call with a progress, 0->1. Called on main thread.
		 * @param OnComplete                Function to call when the database files have been created. Called on main thread.
		 * @return the new IChunkDatabaseWriter.
		 */
		static IChunkDatabaseWriter* Create(
			FChunkDbWriterConfig Configuration,
			IChunkSource* ChunkSource,
			IFileSystem* FileSystem,
			IInstallerError* InstallerError,
			IChunkReferenceTracker* ChunkReferenceTracker,
			IChunkDataSerialization* ChunkDataSerialization,
			IFileConstructorStat* InFileConstructorStat,
			TFunction<void(float)> OnProgress,
			TFunction<void(bool)> OnComplete);
	};
}
