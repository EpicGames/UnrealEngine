// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/ChunkDeltaOptimiser.h"

#include "Async/Async.h"
#include "BuildPatchHash.h"
#include "BuildPatchServicesModule.h"
#include "BuildPatchUtil.h"
#include "Common/ChunkDataSizeProvider.h"
#include "Common/Crypto.h"
#include "Common/FileSystem.h"
#include "Common/HttpManager.h"
#include "Common/SpeedRecorder.h"
#include "Common/StatsCollector.h"
#include "Containers/List.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "Core/AsyncHelpers.h"
#include "Core/MeanValue.h"
#include "Core/Platform.h"
#include "Core/ProcessTimer.h"
#include "Generation/BuildStreamer.h"
#include "Generation/ChunkMatchProcessor.h"
#include "Generation/ChunkSearch.h"
#include "Generation/ChunkWriter.h"
#include "Generation/DataScanner.h"
#include "Generation/DeltaEnumeration.h"
#include "HAL/ThreadSafeBool.h"
#include "HttpModule.h"
#include "IBuildManifestSet.h"
#include "Installer/ChunkEvictionPolicy.h"
#include "Installer/InstallerAnalytics.h"
#include "Installer/InstallerError.h"
#include "Installer/InstallerSharedContext.h"
#include "Installer/MemoryChunkStore.h"
#include "Installer/MessagePump.h"
#include "Installer/OptimisedDelta.h"
#include "Installer/UriProvider.h"
#include "Installer/Statistics/CloudChunkSourceStatistics.h"
#include "Installer/Statistics/CloudChunkSourceStat.h"
#include "Installer/Statistics/DownloadServiceStatistics.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChunkDeltaOptimiser, Log, All);
DEFINE_LOG_CATEGORY(LogChunkDeltaOptimiser);

// For the output file we'll use pretty json in debug, otherwise condensed.
#if UE_BUILD_DEBUG
typedef  TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FDeltaJsonWriter;
typedef  TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FDeltaJsonWriterFactory;
#else
typedef  TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FDeltaJsonWriter;
typedef  TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FDeltaJsonWriterFactory;
#endif //UE_BUILD_DEBUG

namespace DeltaOptimiseHelpers
{
	using namespace BuildPatchServices;

	FSHAHash GetShaForDataSet(const uint8* Data, uint32 Size)
	{
		FSHAHash SHAHash;
		FSHA1::HashBuffer(Data, Size, SHAHash.Hash);
		return SHAHash;
	}

	FSHAHash GetShaForDataSet(const TArray<uint8>& Data)
	{
		return GetShaForDataSet(Data.GetData(), Data.Num());
	}

	int32 GetMaxScannerBacklogCount()
	{
		int32 MaxScannerBacklogCount = 75;
		GConfig->GetInt(TEXT("BuildPatchServices"), TEXT("MaxScannerBacklog"), MaxScannerBacklogCount, GEngineIni);
		MaxScannerBacklogCount = FMath::Clamp<int32>(MaxScannerBacklogCount, 5, 500);
		return MaxScannerBacklogCount;
	}

	bool HasUnusedCpu()
	{
		static const int32 NumThreadsAvailable = GThreadPool->GetNumThreads();
		const bool bHasUnusedCpu = NumThreadsAvailable > FDataScannerCounter::GetNumRunningScanners();
#if UE_BUILD_DEBUG
		static const bool bSingleScannerThread = FParse::Param(FCommandLine::Get(), TEXT("singlescanneronly"));
		return bSingleScannerThread ? false : bHasUnusedCpu;
#else
		return bHasUnusedCpu;
#endif
	}

	template <typename T>
	bool BacklogIsFull(const TArray<T>& Scanners)
	{
		static int32 MaxScannerBacklogCount = GetMaxScannerBacklogCount();
		return Scanners.Num() >= MaxScannerBacklogCount;
	}

	template <typename T>
	bool ScannerArrayFull(const TArray<T>& Scanners)
	{
		const bool bScannerArrayFull = (FDataScannerCounter::GetNumIncompleteScanners() > FDataScannerCounter::GetNumRunningScanners()) || BacklogIsFull(Scanners);
#if UE_BUILD_DEBUG
		static const bool bSingleScannerThread = FParse::Param(FCommandLine::Get(), TEXT("singlescanneronly"));
		return bSingleScannerThread ? (FDataScannerCounter::GetNumIncompleteScanners() + FDataScannerCounter::GetNumRunningScanners()) > 0 : bScannerArrayFull;
#else
		return bScannerArrayFull;
#endif
	}

	FChunkPart SelectBytes(const FChunkPart& FullPart, uint32 LeftChop, uint32 Size)
	{
		FChunkPart Selected = FullPart;
		Selected.Offset += LeftChop;
		Selected.Size = Size;
		return Selected;
	}

	void StompChunkPart(const FChunkPart& NewMatchPart, const FBlockStructure& NewMatchBlocks, FChunkSearcher& ChunkSearcher, TSet<FChunkSearcher::FFileNode*>& UpdatedFiles)
	{
		uint64 NewMatchPartStart = 0;
		ChunkSearcher.ForEachOverlap(NewMatchBlocks, [&](const FBlockRange& OverlapRange, FChunkSearcher::FFileDListNode* File, FChunkSearcher::FChunkDListNode* Chunk)
		{
			FChunkSearcher::FChunkNode& ChunkNode = Chunk->GetValue();
			UpdatedFiles.Add(&File->GetValue());
			const FChunkPart NewMatchPartBlock = DeltaOptimiseHelpers::SelectBytes(NewMatchPart, NewMatchPartStart, OverlapRange.GetSize());

			NewMatchPartStart += OverlapRange.GetSize();
			// If we fully replace this part.
			if (OverlapRange == ChunkNode.BuildRange)
			{
				ChunkNode.ChunkPart = NewMatchPartBlock;
			}
			// If we insert before this part, left chopping it.
			else if (OverlapRange.GetFirst() == ChunkNode.BuildRange.GetFirst())
			{
				// Make the new node.
				const FChunkSearcher::FChunkNode NewMatchChunkNode(NewMatchPartBlock, FBlockRange::FromFirstAndSize(ChunkNode.BuildRange.GetFirst(), OverlapRange.GetSize()));
				// Left chop current node.
				ChunkNode.ChunkPart.Offset += OverlapRange.GetSize();
				ChunkNode.ChunkPart.Size -= OverlapRange.GetSize();
				ChunkNode.BuildRange = FBlockRange::FromFirstAndLast(NewMatchChunkNode.BuildRange.GetLast() + 1, ChunkNode.BuildRange.GetLast());
				// Insert new node before current node.
				ListHelpers::InsertBefore(NewMatchChunkNode, File->GetValue().ChunkParts, Chunk);
			}
			// If we insert after this part, right chopping it.
			else if (OverlapRange.GetLast() == ChunkNode.BuildRange.GetLast())
			{
				// Right chop current node.
				ChunkNode.ChunkPart.Size -= OverlapRange.GetSize();
				ChunkNode.BuildRange = FBlockRange::FromFirstAndSize(ChunkNode.BuildRange.GetFirst(), ChunkNode.ChunkPart.Size);
				// Make the new node.
				const FChunkSearcher::FChunkNode NewMatchChunkNode(NewMatchPartBlock, FBlockRange::FromFirstAndSize(ChunkNode.BuildRange.GetLast() + 1, OverlapRange.GetSize()));
				// Insert chunk part after.
				ListHelpers::InsertAfter(NewMatchChunkNode, File->GetValue().ChunkParts, Chunk);
			}
			// If we insert inside this part.
			else
			{
				// Make the right side.
				const uint32 LeftChopSize = (OverlapRange.GetLast() - ChunkNode.BuildRange.GetFirst()) + 1;
				const uint32 RightSideSize = ChunkNode.BuildRange.GetSize() - LeftChopSize;
				const FChunkPart RightSide = DeltaOptimiseHelpers::SelectBytes(ChunkNode.ChunkPart, LeftChopSize, RightSideSize);
				const FChunkSearcher::FChunkNode RightSideChunkNode(RightSide, FBlockRange::FromFirstAndSize(OverlapRange.GetLast() + 1, RightSideSize));
				// Make the middle piece.
				const FChunkSearcher::FChunkNode MiddleChunkNode(NewMatchPartBlock, OverlapRange);
				// Right chop current node.
				ChunkNode.ChunkPart.Size = OverlapRange.GetFirst() - ChunkNode.BuildRange.GetFirst();
				ChunkNode.BuildRange = FBlockRange::FromFirstAndSize(ChunkNode.BuildRange.GetFirst(), ChunkNode.ChunkPart.Size);
				check(OverlapRange == FBlockRange::FromFirstAndLast(ChunkNode.BuildRange.GetLast() + 1, RightSideChunkNode.BuildRange.GetFirst() - 1));
				// Insert right side part after current.
				ListHelpers::InsertAfter(RightSideChunkNode, File->GetValue().ChunkParts, Chunk);
				// Insert middle part after current (thus before right side).
				ListHelpers::InsertAfter(MiddleChunkNode, File->GetValue().ChunkParts, Chunk);
			}
		});
	}

	void MakeScannerLocalList(const TMap<FString, FString>& AliasedTags, const TSet<FString>& IgnoredTags, FChunkSearcher& ChunkSearcher, IDeltaChunkEnumeration* Enumeration, const FBlockStructure& BuildStructure, FScannerFilesList& Result)
	{
		uint64 FirstByte = 0;
		ChunkSearcher.ForEachOverlap(BuildStructure, [&](const FBlockRange& OverlapRange, FChunkSearcher::FFileDListNode* File, FChunkSearcher::FChunkDListNode* Chunk)
		{
			const FFilenameId FilenameId = Enumeration->MakeFilenameId(File->GetValue().Manifest->Filename);
			const FBlockRange& FileRange = File->GetValue().BuildRange;
			const uint64 FileOffset = OverlapRange.GetFirst() - FileRange.GetFirst();

			// Replace any aliased tags.
			TSet<FString> FileTagset;
			FileTagset.Reserve(File->GetValue().Manifest->InstallTags.Num());
			for (const FString& InstallTag : File->GetValue().Manifest->InstallTags)
			{
				if (IgnoredTags.Contains(InstallTag))
				{
					continue;
				}

				const FString* AliasedTag = AliasedTags.Find(InstallTag);
				FileTagset.Add(AliasedTag ? *AliasedTag : InstallTag);
			}

			Result.AddTail(FScannerFileElement{FBlockRange::FromFirstAndSize(FirstByte, OverlapRange.GetSize()), FilenameId, MoveTemp(FileTagset), FileOffset});
			FirstByte += OverlapRange.GetSize();
		});
		check(BlockStructureHelpers::CountSize(BuildStructure) == FirstByte);
	}

	uint64 AsPercentageOf(const BuildPatchServices::FDiffAbortThreshold& DiffAbortThreshold, uint64 TotalBinarySize)
	{
		check(DiffAbortThreshold.Unit == BuildPatchServices::EDiffAbortThresholdUnits::Absolute);
		return TotalBinarySize == 0 ? 100 : DiffAbortThreshold.Value * 100 / TotalBinarySize;
	}

	uint64 AsAbsoluteOf(const BuildPatchServices::FDiffAbortThreshold& DiffAbortThreshold, uint64 TotalBinarySize)
	{
		check(DiffAbortThreshold.Unit == EDiffAbortThresholdUnits::Percentage);
		return TotalBinarySize * DiffAbortThreshold.Value / 100;
	}

	BuildPatchServices::FDiffAbortThreshold ClampToSaneMinBasedOnActualBinarySize(TArray<FString>& FinalStatLogs, const FDiffAbortThreshold& DiffAbortThreshold, uint64 TotalBinarySize)
	{
		if (DiffAbortThreshold.Unit == EDiffAbortThresholdUnits::Absolute)
		{
			check(DiffAbortThreshold.Value >= DiffAbortThresholdLimits::MinAbsolute); // it must be clamped previously
			return DiffAbortThreshold;
		}

		check(DiffAbortThreshold.Unit == EDiffAbortThresholdUnits::Percentage);

		uint64 MaxPercentageOfTotalSize = AsPercentageOf({ DiffAbortThresholdLimits::MinAbsolute, EDiffAbortThresholdUnits::Absolute }, TotalBinarySize);
		MaxPercentageOfTotalSize = FMath::Max(MaxPercentageOfTotalSize, DiffAbortThresholdLimits::MinPercentage);
		if (DiffAbortThreshold.Value < MaxPercentageOfTotalSize)
		{
			// Clamp DiffAbortThreshold to sane min.
			FinalStatLogs.Add(FString::Printf(TEXT("Requested -DiffAbortThreshold=%llu%% is below allowed minimum n >= 1GB. Please update your arg to be above limit. Continuing with DiffAbortThreshold=%llu%%."),
				DiffAbortThreshold.Value, MaxPercentageOfTotalSize));
			return { MaxPercentageOfTotalSize, EDiffAbortThresholdUnits::Percentage };
		}
		return DiffAbortThreshold;
	}

	struct FAbsoluteAndPercentage
	{
		uint64 Absolute = 0;
		uint64 Percentage = 0;
	};

	FAbsoluteAndPercentage RecalculateAsAbsoluteAndPercentage(const FDiffAbortThreshold& DiffAbortThreshold, uint64 TotalBinarySize)
	{
		switch (DiffAbortThreshold.Unit)
		{
		case EDiffAbortThresholdUnits::Absolute:
			return { DiffAbortThreshold.Value, AsPercentageOf(DiffAbortThreshold, TotalBinarySize) };
		case EDiffAbortThresholdUnits::Percentage:
			return { AsAbsoluteOf(DiffAbortThreshold, TotalBinarySize), DiffAbortThreshold.Value};
		default:
			unimplemented();
			break;
		}
		return {};
	}


}

namespace DeltaStats
{
	using namespace BuildPatchServices;

	class FNoMemoryChunkStoreStat
		: public IMemoryChunkStoreStat
	{
	public:
		FNoMemoryChunkStoreStat() { }
		~FNoMemoryChunkStoreStat() { }

		// IMemoryChunkStoreStat interface begin.
		virtual void OnChunkStored(const FGuid& ChunkId) override { }
		virtual void OnChunkReleased(const FGuid& ChunkId) override { }
		virtual void OnChunkBooted(const FGuid& ChunkId) override { }
		virtual void OnStoreUseUpdated(int32 ChunkCount) override { }
		virtual void OnStoreSizeUpdated(int32 Size) override { }
		// IMemoryChunkStoreStat interface end.
	};
}

namespace DeltaFactories
{
	using namespace BuildPatchServices;

	class FChunkReferenceTrackerFactory : public IManifestBuildStreamer::IChunkReferenceTrackerFactory
	{
	public:
		FChunkReferenceTrackerFactory() { }
		virtual ~FChunkReferenceTrackerFactory() { }

		// IManifestBuildStreamer::IChunkReferenceTrackerFactory interface begin.
		virtual IChunkReferenceTracker* Create(IManifestBuildStreamer::FCustomChunkReferences CustomChunkReferences) override
		{
			return BuildPatchServices::FChunkReferenceTrackerFactory::Create(MoveTemp(CustomChunkReferences));
		}
		// IManifestBuildStreamer::IChunkReferenceTrackerFactory interface end.
	};

	struct FCloudChunkSourceFactoryShared
	{
	public:
		IFileSystem* FileSystem;
		IDownloadService* DownloadService;
		IChunkDataSerialization* ChunkDataSerialization;
		IMessagePump* MessagePump;
		IBuildManifestSet* ManifestSet;
		ICloudChunkSourceStat* CloudChunkSourceStat;
	};

	class FCloudChunkSourceFactory : public IManifestBuildStreamer::ICloudChunkSourceFactory
	{
	private:
		struct FInstanceDependencies
		{
			TUniquePtr<IChunkEvictionPolicy> MemoryEvictionPolicy;
			TUniquePtr<IMemoryChunkStore> CloudChunkStore;
			TUniquePtr<IDownloadConnectionCount> ConnectionCount;
		};

	public:
		FCloudChunkSourceFactory(TArray<FString> CloudDirs, FCloudChunkSourceFactoryShared InShared)
			: Shared(MoveTemp(InShared))
			, CloudSourceSharedContext(FBuildInstallerSharedContextFactory::Create(TEXT("CloudChunkSourceFactory")))
			, CloudSourceConfig(MoveTemp(CloudDirs))
			, Platform(FPlatformFactory::Create())
			, MemoryChunkStoreStat(new DeltaStats::FNoMemoryChunkStoreStat())
			, InstallerError(FInstallerErrorFactory::Create())
		{
			CloudSourceSharedContext->PreallocateThreads(1);

			CloudSourceConfig.bBeginDownloadsOnFirstGet = false;
			CloudSourceConfig.MaxRetryCount = 30;
			GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkRetries"), CloudSourceConfig.MaxRetryCount, GEngineIni);
			CloudSourceConfig.RetriesToEnableVerbose = CloudSourceConfig.MaxRetryCount - 1;
			// Lower the prefetch max as the default is a bit high for our use case here
			CloudSourceConfig.PreFetchMaximum = 100;
			CloudSourceConfig.SharedContext = &CloudSourceSharedContext.Get();
		}

		virtual ~FCloudChunkSourceFactory()
		{
		}

		// IManifestBuildStreamer::ICloudChunkSourceFactory interface begin.
		virtual ICloudChunkSource* Create(IChunkReferenceTracker* ChunkReferenceTracker) override
		{
			InstanceDependencies.AddDefaulted();
			FInstanceDependencies& Dependencies = InstanceDependencies.Last();
			Dependencies.MemoryEvictionPolicy.Reset(FChunkEvictionPolicyFactory::Create(ChunkReferenceTracker));
			Dependencies.CloudChunkStore.Reset(FMemoryChunkStoreFactory::Create(
				// The chunk store should be bigger than the max prefetch to allow for multi-use chunks to stay in memory.
				CloudSourceConfig.PreFetchMaximum + 50,
				Dependencies.MemoryEvictionPolicy.Get(),
				nullptr,
				MemoryChunkStoreStat.Get(),
				nullptr));
			Dependencies.ConnectionCount.Reset(BuildPatchServices::FDownloadConnectionCountFactory::Create(ConnectionCountConfig, nullptr));
			ICloudChunkSource* CloudChunkSource = BuildPatchServices::FCloudChunkSourceFactory::Create(
				CloudSourceConfig,
				Platform.Get(),
				Dependencies.CloudChunkStore.Get(),
				Shared.DownloadService,
				ChunkReferenceTracker,
				Shared.ChunkDataSerialization,
				Shared.MessagePump,
				InstallerError.Get(),
				Dependencies.ConnectionCount.Get(),
				Shared.CloudChunkSourceStat,
				Shared.ManifestSet,
				ChunkReferenceTracker->GetReferencedChunks());

			TFunction<void(const FGuid&)> LostChunkCallback = [CloudChunkSource](const FGuid& LostChunk)
			{
				CloudChunkSource->AddRepeatRequirement(LostChunk);
			};
			Dependencies.CloudChunkStore->SetLostChunkCallback(LostChunkCallback);

			return CloudChunkSource;
		}
		// IManifestBuildStreamer::ICloudChunkSourceFactory interface end.

	private:
		FCloudChunkSourceFactoryShared Shared;
		IBuildInstallerSharedContextRef CloudSourceSharedContext;
		FCloudSourceConfig CloudSourceConfig;
		FDownloadConnectionCountConfig ConnectionCountConfig;
		TUniquePtr<IPlatform> Platform;
		TUniquePtr<IMemoryChunkStoreStat> MemoryChunkStoreStat;
		TUniquePtr<IInstallerError> InstallerError;
		TArray<FInstanceDependencies> InstanceDependencies;
	};
}

namespace BuildPatchServices
{
	class FChunkMatchStomper
	{
	public:
		typedef TTuple<TArray<FChunkPart>, FBlockStructure> FNewMatch;
		typedef TQueue<FNewMatch, EQueueMode::Spsc> FNewMatchQueue;

		FChunkMatchStomper(const FBuildPatchAppManifest& InManifestA, const FBuildPatchAppManifest& InManifestB)
			: ManifestA(InManifestA)
			, ManifestB(InManifestB)
			, BuildAFiles(ListHelpers::GetFileList(ManifestA))
			, BuildBFiles(ListHelpers::GetFileList(ManifestB))
			, bExpectsMoreData(true)
			, ThreadTrigger(FPlatformProcess::GetSynchEventFromPool(true))
		{
			FileManifestListFuture = Async(EAsyncExecution::Thread, [this]() { return AsyncRun(); });
		}

		~FChunkMatchStomper()
		{
			// Ensures the thread work completes.
			bExpectsMoreData = false;
			ThreadTrigger->Trigger();
			FileManifestListFuture.Wait();
			FPlatformProcess::ReturnSynchEventToPool(ThreadTrigger);
		}

		FFileManifestList AsyncRun()
		{
			FChunkSearcher SearcherB(ManifestB);
			TSet<FChunkSearcher::FFileNode*> UpdatedFiles;

			// Start with searcher B invalidating unknown chunks.
			FChunkSearcher::FFileDListNode* FileBNode = SearcherB.GetHead();
			while (FileBNode)
			{
				FChunkSearcher::FChunkDListNode* ChunkBNode = FileBNode->GetValue().ChunkParts.GetHead();
				while (ChunkBNode)
				{
					if (ManifestA.GetChunkInfo(ChunkBNode->GetValue().ChunkPart.Guid) == nullptr)
					{
						ChunkBNode->GetValue().ChunkPart.Guid.Invalidate();
					}
					ChunkBNode = ChunkBNode->GetNextNode();
				}
				FileBNode = FileBNode->GetNextNode();
			}

			bool bHasNewMatch = false;
			FNewMatch NewMatch;
			while ((bHasNewMatch = NewMatchQueue.Dequeue(NewMatch), bHasNewMatch) || bExpectsMoreData)
			{
				if (bHasNewMatch)
				{
					const TArray<FChunkPart>& NewChunkParts = NewMatch.Get<0>();
					const FBlockStructure& BuildBStructure = NewMatch.Get<1>();
					uint64 ByteCount = 0;
					for (const FChunkPart& NewChunkPart : NewChunkParts)
					{
						FBlockStructure PartStructure;
						BuildBStructure.SelectSerialBytes(ByteCount, NewChunkPart.Size, PartStructure);
						DeltaOptimiseHelpers::StompChunkPart(NewChunkPart, PartStructure, SearcherB, UpdatedFiles);
						ByteCount += NewChunkPart.Size;
					}
				}
				else
				{
					ThreadTrigger->Wait(1000);
					ThreadTrigger->Reset();
				}
			}

			// Ensure priority to original matches?
			ClobberAllKnownChunks(SearcherB, UpdatedFiles);

			// Collapse all adjacent chunkparts.
			FileBNode = SearcherB.GetHead();
			while (FileBNode)
			{
				MergeAdjacentChunkParts(FileBNode->GetValue().ChunkParts);
				FileBNode = FileBNode->GetNextNode();
			}

			return SearcherB.BuildNewFileManifestList();
		}

		void ReplaceChunkReferences(const TArray<FChunkPart>& NewChunkReferences, const FBlockStructure& BuildBStructure)
		{
			checkf(bExpectsMoreData, TEXT("You can't provide more data after collecting the result."));
			NewMatchQueue.Enqueue(FNewMatch{NewChunkReferences, BuildBStructure});
			ThreadTrigger->Trigger();
		}

		FFileManifestList GetNewFileManifests()
		{
			bExpectsMoreData = false;
			ThreadTrigger->Trigger();
			return FileManifestListFuture.Get();
		}

	private:
		void ClobberAllKnownChunks(FChunkSearcher& ChunkSearcher, TSet<FChunkSearcher::FFileNode*>& UpdatedFiles)
		{
			uint64 BuildFileFirst = 0;
			uint64 ChunkPartFirst = 0;
			for (const FString& BuildFilename : BuildBFiles)
			{
				const FFileManifest* FileManifest = ManifestB.GetFileManifest(BuildFilename);
				check(FileManifest != nullptr);
				const FBlockRange FileRange = FBlockRange::FromFirstAndSize(BuildFileFirst, FileManifest->FileSize);
				if (FileRange.GetSize() > 0)
				{
					ChunkPartFirst = FileRange.GetFirst();
					for (const FChunkPart& ChunkPart : FileManifest->ChunkParts)
					{
						const FBlockRange ChunkPartRange = FBlockRange::FromFirstAndSize(ChunkPartFirst, ChunkPart.Size);
						if (ManifestA.GetChunkInfo(ChunkPart.Guid) != nullptr)
						{
							DeltaOptimiseHelpers::StompChunkPart(ChunkPart, FBlockStructure(ChunkPartRange.GetFirst(), ChunkPartRange.GetSize()), ChunkSearcher, UpdatedFiles);
						}
						ChunkPartFirst += ChunkPartRange.GetSize();
					}
				}
				check(ChunkPartFirst == (BuildFileFirst + FileRange.GetSize()));
				BuildFileFirst += FileRange.GetSize();
			}
		}

		void MergeAdjacentChunkParts(FChunkSearcher::FChunkDList& ChunkParts)
		{
			FChunkSearcher::FChunkDListNode* ChunkNode = ChunkParts.GetHead();
			while (ChunkNode)
			{
				FChunkSearcher::FChunkDListNode* NextChunkNode = ChunkNode->GetNextNode();
				while (NextChunkNode)
				{
					// Assert if we skipped build data
					check((ChunkNode->GetValue().BuildRange.GetLast() + 1) == NextChunkNode->GetValue().BuildRange.GetFirst());
					FChunkPart& ThisChunkPart = ChunkNode->GetValue().ChunkPart;
					FChunkPart& NextChunkPart = NextChunkNode->GetValue().ChunkPart;
					const FBlockRange LastMatchPartRange = FBlockRange::FromFirstAndSize(ThisChunkPart.Offset, ThisChunkPart.Size);
					const FBlockRange ThisMatchPartRange = FBlockRange::FromFirstAndSize(NextChunkPart.Offset, NextChunkPart.Size);
					const bool bBothInvalid = !NextChunkPart.Guid.IsValid() && !ThisChunkPart.Guid.IsValid();
					const bool bBothSamePadding = ThisChunkPart.IsPadding() && NextChunkPart.IsPadding() && ThisChunkPart.GetPaddingByte() == NextChunkPart.GetPaddingByte();
					const bool bSameChunk = NextChunkPart.Guid == ThisChunkPart.Guid;
					const bool bAdjacentData = (LastMatchPartRange.GetLast() + 1) == ThisMatchPartRange.GetFirst();
					bool bMerged = false;
					FChunkSearcher::FChunkDListNode* NextNextChunkNode = NextChunkNode->GetNextNode();
					if (bBothInvalid)
					{
						const uint64 TotalSize = NextChunkNode->GetValue().BuildRange.GetSize() + ChunkNode->GetValue().BuildRange.GetSize();
						if (TotalSize < TNumericLimits<uint32>::Max())
						{
							ThisChunkPart.Size = TotalSize;
							ChunkNode->GetValue().BuildRange = FBlockRange::FromFirstAndSize(ChunkNode->GetValue().BuildRange.GetFirst(), TotalSize);
							ChunkParts.RemoveNode(NextChunkNode);
							bMerged = true;
						}
					}
					else if (bBothSamePadding)
					{
						const uint64 TotalSize = NextChunkNode->GetValue().BuildRange.GetSize() + ChunkNode->GetValue().BuildRange.GetSize();
						if (TotalSize < PaddingChunk::ChunkSize)
						{
							ThisChunkPart.Offset = 0;
							ThisChunkPart.Size = TotalSize;
							ChunkNode->GetValue().BuildRange = FBlockRange::FromFirstAndSize(ChunkNode->GetValue().BuildRange.GetFirst(), TotalSize);
							ChunkParts.RemoveNode(NextChunkNode);
							bMerged = true;
						}
					}
					else if (bSameChunk && bAdjacentData)
					{
						const FBlockRange MergedPartRange = FBlockRange::FromMerge(ThisMatchPartRange, LastMatchPartRange);
						ThisChunkPart.Offset = MergedPartRange.GetFirst();
						ThisChunkPart.Size = MergedPartRange.GetSize();
						ChunkNode->GetValue().BuildRange = FBlockRange::FromMerge(ChunkNode->GetValue().BuildRange, NextChunkNode->GetValue().BuildRange);
						ChunkParts.RemoveNode(NextChunkNode);
						bMerged = true;
					}
					if (!bMerged)
					{
						ChunkNode = NextChunkNode;
					}
					NextChunkNode = NextNextChunkNode;
				}
				ChunkNode = NextChunkNode;
			}
		}

	private:
		const FBuildPatchAppManifest& ManifestA;
		const FBuildPatchAppManifest& ManifestB;
		const TArray<FString> BuildAFiles;
		const TArray<FString> BuildBFiles;
		FThreadSafeBool bExpectsMoreData;
		FEvent* ThreadTrigger;
		TFuture<FFileManifestList> FileManifestListFuture;
		FNewMatchQueue NewMatchQueue;
	};
}

namespace BuildPatchServices
{
	struct FDeltaScannerEntry
	{
	public:
		FDeltaScannerEntry()
			: bIsFinalScanner(false)
			, bWasFork(false)
			, Offset(0)
		{ }

	public:
		TArray<uint8> Data;
		FScannerFilesList FilesList;
		TUniquePtr<IDataScanner> Scanner;
		bool bIsFinalScanner;
		bool bWasFork;
		uint64 Offset;
	};

	class FChunkDeltaOptimiser
		: public IChunkDeltaOptimiser
	{
	public:
		FChunkDeltaOptimiser(FChunkDeltaOptimiserConfiguration InConfiguration);
		~FChunkDeltaOptimiser();

		// IChunkDeltaOptimiser interface begin.
		virtual bool Run() override;
		virtual FOnChunkFileWritten& OnChunkFileWritten() override;
		virtual FOnDeltaFileWritten& OnDeltaFileWritten() override;
		// IChunkDeltaOptimiser interface end.

	private:
		enum class EFetchResult
		{
			Success = 0,
			DoesNotExist,
			CouldNotDeserialize
		};
		typedef TTuple<FBuildPatchAppManifestPtr, int32, EFetchResult> FFetchResult;

	private:
		TArray<FString> AsyncRun();
		void BroadcastChunkFile(const FString& FullFilePath, const FMD5Hash& MD5Hash);
		void BroadcastDelta(const FString& FullFilePath, const FMD5Hash& MD5Hash, const FSHAHash& SHA1Hash);
		TFuture<FFetchResult> FetchManifest(const FString& ManifestUri);
		void HandleDownloadComplete(int32 RequestId, const FDownloadRef& Download, TSharedRef<TPromise<FFetchResult>, ESPMode::ThreadSafe> Promise);
		FBlockStructure GetDesiredBytes(const FBuildPatchAppManifestPtr& Manifest, const TSet<FGuid>& Chunks);
		FOptimisedDeltaConfiguration BuildOptimisedDeltaConfig(FBuildPatchAppManifestPtr SourceManifest, FBuildPatchAppManifestRef DestinationManifest, const TArray<FString>& DownloadCloudDirectories, const FString& FilenameTrailer);
		FOptimisedDeltaDependencies BuildOptimisedDeltaDependencies(TSharedRef<IUriProvider, ESPMode::ThreadSafe> UriProvider);

	private:
		const FChunkDeltaOptimiserConfiguration Configuration;
		FTSTicker& CoreTicker;
		FDownloadCompleteDelegate DownloadCompleteDelegate;
		FDownloadProgressDelegate DownloadProgressDelegate;
		TSharedPtr<IFileSystem> FileSystem;
		TUniquePtr<ICrypto> Crypto;
		TSharedPtr<IHttpManager> HttpManager;
		TSharedPtr<IChunkDataSizeProvider> ChunkDataSizeProvider;
		TSharedPtr<ISpeedRecorder> DownloadSpeedRecorder;
		TSharedPtr<IInstallerAnalytics> InstallerAnalytics;
		TSharedPtr<IDownloadServiceStatistics> DownloadServiceStatistics;
		TUniquePtr<IDownloadService> DownloadService;
		TUniquePtr<IMessagePump> MessagePumpA;
		TUniquePtr<IMessagePump> MessagePumpB;
		TUniquePtr<FStatsCollector> StatsCollector;
		TArray<FMessageHandler*> MessageHandlersA;
		TArray<FMessageHandler*> MessageHandlersB;
		FThreadSafeBool bShouldRun;
		FThreadSafeBool bSuccess;
		IChunkDeltaOptimiser::FOnChunkFileWritten ChunkFileWrittenEvent;
		IChunkDeltaOptimiser::FOnDeltaFileWritten DeltaFileWrittenEvent;
	};

	FChunkDeltaOptimiser::FChunkDeltaOptimiser(FChunkDeltaOptimiserConfiguration InConfiguration)
		: Configuration(MoveTemp(InConfiguration))
		, CoreTicker(FTSTicker::GetCoreTicker())
		, DownloadProgressDelegate()
		, FileSystem(FFileSystemFactory::Create())
		, Crypto(FCryptoFactory::Create())
		, HttpManager(FHttpManagerFactory::Create())
		, ChunkDataSizeProvider(FChunkDataSizeProviderFactory::Create({}))
		, DownloadSpeedRecorder(FSpeedRecorderFactory::Create())
		, InstallerAnalytics(FInstallerAnalyticsFactory::Create(nullptr))
		, DownloadServiceStatistics(FDownloadServiceStatisticsFactory::Create(DownloadSpeedRecorder, ChunkDataSizeProvider, InstallerAnalytics))
		, DownloadService(FDownloadServiceFactory::Create(HttpManager, FileSystem, DownloadServiceStatistics, InstallerAnalytics))
		, MessagePumpA(FMessagePumpFactory::Create())
		, MessagePumpB(FMessagePumpFactory::Create())
		, StatsCollector(FStatsCollectorFactory::Create())
		, MessageHandlersA(Configuration.MessageHandlersA)
		, MessageHandlersB(Configuration.MessageHandlersB)
		, bShouldRun(true)
	{
	}

	FChunkDeltaOptimiser::~FChunkDeltaOptimiser()
	{
	}

	bool FChunkDeltaOptimiser::Run()
	{
		// Run any core initialisation required.
		FHttpModule::Get();

		// Setup Generation stats.
		volatile FStatsCollector::FAtomicValue* StatTotalTime = StatsCollector->CreateStat(TEXT("Generation: Total Time"), EStatFormat::Timer);
		volatile FStatsCollector::FAtomicValue* StatAverageStreamSpeed = StatsCollector->CreateStat(TEXT("Generation: Average stream speed"), EStatFormat::DataSpeed);
		volatile FStatsCollector::FAtomicValue* StatCurrentStreamSpeed = StatsCollector->CreateStat(TEXT("Generation: Current stream speed"), EStatFormat::DataSpeed);
		const uint64 StartTime = FStatsCollector::GetCycles();
		const float SpeedStatOverTimeAverage = TNumericLimits<float>::Max();
		const float SpeedStatOverTimeCurrent = 10.f;

		// Start the generation thread.
		TFuture<TArray<FString>> Thread = Async(EAsyncExecution::Thread, [this](){ return AsyncRun(); });

		// Main timers.
		double DeltaTime = 0.0;
		double LastTime = FPlatformTime::Seconds();

		// Setup desired frame times.
		float MainsFramerate = 100.0f;
		const float MainsFrameTime = 1.0f / MainsFramerate;

		// Load settings from config.
		float StatsLoggerTimeSeconds = 10.0f;
		GConfig->GetFloat(TEXT("BuildPatchServices"), TEXT("StatsLoggerTimeSeconds"), StatsLoggerTimeSeconds, GEngineIni);
		StatsLoggerTimeSeconds = FMath::Clamp<float>(StatsLoggerTimeSeconds, 1.0f, 60.0f);

		// Run the main loop.
		while (bShouldRun)
		{
			// Increment global frame counter once for each app tick.
			GFrameCounter++;

			// Application tick.
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(DeltaTime);

			// Message pump.
			MessagePumpA->PumpMessages(MessageHandlersA);
			MessagePumpB->PumpMessages(MessageHandlersB);

			// Log collected stats.
			GLog->FlushThreadedLogs();
			FStatsCollector::Set(StatTotalTime, FStatsCollector::GetCycles() - StartTime);
			FStatsCollector::Set(StatAverageStreamSpeed, DownloadSpeedRecorder->GetAverageSpeed(SpeedStatOverTimeAverage));
			FStatsCollector::Set(StatCurrentStreamSpeed, DownloadSpeedRecorder->GetAverageSpeed(SpeedStatOverTimeCurrent));
			StatsCollector->LogStats(StatsLoggerTimeSeconds);

			// Control frame rate.
			FPlatformProcess::Sleep(FMath::Max<float>(0.0f, MainsFrameTime - (FPlatformTime::Seconds() - LastTime)));

			// Calculate deltas.
			const double AppTime = FPlatformTime::Seconds();
			DeltaTime = AppTime - LastTime;
			LastTime = AppTime;
		}

		// Log collected stats.
		TArray<FString> FinalStatLogs = Thread.Get();
		GLog->FlushThreadedLogs();
		FStatsCollector::Set(StatTotalTime, FStatsCollector::GetCycles() - StartTime);
		FStatsCollector::Set(StatAverageStreamSpeed, DownloadSpeedRecorder->GetAverageSpeed(SpeedStatOverTimeAverage));
		FStatsCollector::Set(StatCurrentStreamSpeed, DownloadSpeedRecorder->GetAverageSpeed(SpeedStatOverTimeCurrent));
		StatsCollector->LogStats();
		for (const FString& LogLine : FinalStatLogs)
		{
			UE_LOGF(LogChunkDeltaOptimiser, Display, "%ls", *LogLine);
		}

		// Return thread success.
		return bSuccess;
	}

	IChunkDeltaOptimiser::FOnChunkFileWritten& FChunkDeltaOptimiser::OnChunkFileWritten()
	{
		return ChunkFileWrittenEvent;
	}

	IChunkDeltaOptimiser::FOnDeltaFileWritten& FChunkDeltaOptimiser::OnDeltaFileWritten()
	{
		return DeltaFileWrittenEvent;
	}

	FOptimisedDeltaConfiguration FChunkDeltaOptimiser::BuildOptimisedDeltaConfig(FBuildPatchAppManifestPtr SourceManifest, FBuildPatchAppManifestRef DestinationManifest, const TArray<FString>& DownloadCloudDirectories, const FString& FilenameTrailer)
	{
		// The optimised delta can deal with getting dupe manifests so lets just allow that for easy config.
		FOptimisedDeltaConfiguration OptimisedDeltaConfiguration(DestinationManifest);
		OptimisedDeltaConfiguration.SourceManifest = SourceManifest;
		OptimisedDeltaConfiguration.RetriesNumber = DownloadCloudDirectories.Num();
		
		// When the delta file can't be deserialized we don't want to proceed with re-uploading it.
		// The file should be deleted on backend first, and then we can re-upload it.
		OptimisedDeltaConfiguration.DeltaPolicy = EDeltaPolicy::Expect;
		OptimisedDeltaConfiguration.DeltaFilenameTrailer = FilenameTrailer;
		return OptimisedDeltaConfiguration;
	}

	FOptimisedDeltaDependencies FChunkDeltaOptimiser::BuildOptimisedDeltaDependencies(TSharedRef<IUriProvider, ESPMode::ThreadSafe> UriProvider)
	{
		FOptimisedDeltaDependencies OptimisedDeltaDependencies(MoveTemp(UriProvider));
		OptimisedDeltaDependencies.DownloadService = DownloadService.Get();
		return OptimisedDeltaDependencies;
	}

	TArray<FString> FChunkDeltaOptimiser::AsyncRun()
	{
		const FNumberFormattingOptions PercentFormat = FNumberFormattingOptions().SetMaximumFractionalDigits(1).SetMinimumFractionalDigits(1).SetRoundingMode(ERoundingMode::ToZero);
		TFuture<FFetchResult> FutureManifestResultA = FetchManifest(Configuration.ManifestAUri);
		TFuture<FFetchResult> FutureManifestResultB = FetchManifest(Configuration.ManifestBUri);
		FFetchResult ManifestResultA = FutureManifestResultA.Get();
		FFetchResult ManifestResultB = FutureManifestResultB.Get();
		TArray<FString> FinalStatLogs;
		bSuccess = true;
		if (ManifestResultA.Get<2>() != EFetchResult::Success)
		{
			UE_LOGF(LogChunkDeltaOptimiser, Error, "Could not download ManifestA from %ls.", *Configuration.ManifestAUri);
			bSuccess = false;
		}
		if (ManifestResultB.Get<2>() != EFetchResult::Success)
		{
			UE_LOGF(LogChunkDeltaOptimiser, Error, "Could not download ManifestB from %ls.", *Configuration.ManifestBUri);
			bSuccess = false;
		}
		const FBuildPatchAppManifestPtr& ManifestA = ManifestResultA.Get<0>();
		const FBuildPatchAppManifestPtr& ManifestB = ManifestResultB.Get<0>();
		if (bSuccess)
		{
			TSet<FGuid> AvailableSecrets;
			Configuration.EncryptionSecrets.GetKeys(AvailableSecrets);
			TSet<FGuid> NecessaryEncryptionSecrets;
			NecessaryEncryptionSecrets.Append(ManifestA->GetNecessaryEncryptionSecretIds());
			NecessaryEncryptionSecrets.Append(ManifestB->GetNecessaryEncryptionSecretIds());
			TSet<FGuid> MissingSecrets = NecessaryEncryptionSecrets.Difference(AvailableSecrets);
			if (MissingSecrets.Num() > 0)
			{
				UE_LOGF(LogChunkDeltaOptimiser, Error, "Cannot continue without all necessary secret keys.");
				for (const FGuid& SecretId : MissingSecrets)
				{
					UE_LOGF(LogChunkDeltaOptimiser, Error, "    Missing secret with ID: %ls", *SecretId.ToString());
				}
				bSuccess = false;
			}
		}
		if (bSuccess)
		{
			UE_LOGF(LogChunkDeltaOptimiser, Display, "Running optimisation for patching %ls -> %ls", *ManifestA->GetVersionString(), *ManifestB->GetVersionString());
			FProcessTimer ProcessTimer;
			FProcessTimer ChunkingTimer;
			FProcessTimer ScanningTimer;
			ProcessTimer.Start();

			TSet<FGuid> ChunksA;
			TSet<FGuid> ChunksB;
			ManifestA->GetDataList(ChunksA);
			ManifestB->GetDataList(ChunksB);

			// Check for ManifestA -> ManifestB compatibility. We don't yet support downgrading chunk version, only upgrading.
			const TCHAR* ManifestAChunkSubdir = ManifestVersionHelpers::GetChunkSubdir(ManifestA->ManifestMeta.FeatureLevel);
			const TCHAR* ManifestBChunkSubdir = ManifestVersionHelpers::GetChunkSubdir(ManifestB->ManifestMeta.FeatureLevel);
			const bool bUsingDifferentChunkSubdir = ManifestAChunkSubdir != ManifestBChunkSubdir;
			const bool bIsDowngrade = ManifestB->ManifestMeta.FeatureLevel < ManifestA->ManifestMeta.FeatureLevel;
			if (bUsingDifferentChunkSubdir && bIsDowngrade)
			{
				UE_LOGF(LogChunkDeltaOptimiser, Error, "Destination manifest does not support source manifest's FeatureLevel (%ls [%d] -> %ls [%d]).", LexToString(ManifestA->ManifestMeta.FeatureLevel), (int32)ManifestA->ManifestMeta.FeatureLevel, LexToString(ManifestB->ManifestMeta.FeatureLevel), (int32)ManifestB->ManifestMeta.FeatureLevel);
				bSuccess = false;
			}

			// Check for output chunk size compatibility changes.
			const uint32 OutputChunkSize = ManifestB->ManifestMeta.FeatureLevel >= EFeatureLevel::VariableSizeChunks ? Configuration.OutputChunkSize : 1024*1024;
			if (Configuration.OutputChunkSize != OutputChunkSize)
			{
				UE_LOGF(LogChunkDeltaOptimiser, Log, "Destination manifest does not support EFeatureLevel::VariableSizeChunks, reverting OutputChunkSize to %u.", OutputChunkSize);
			}

			// Set cloud directories
			TArray<FString> DownloadCloudDirectories(Configuration.DownloadCloudDirectories);
			DownloadCloudDirectories.Add(Configuration.OutputCloudDirectory);

			// Check that an optimisation does not already exist in any potential locations, and skip long process if so.
			TSharedRef<IUriProvider, ESPMode::ThreadSafe> UriProvider = FUriProviderFactory::Create(MessagePumpB.Get(), Configuration.DownloadCloudDirectories);
			FOptimisedDeltaDependencies OptimisedDeltaDependencies = BuildOptimisedDeltaDependencies(MoveTemp(UriProvider));
			TSharedRef<IOptimisedDelta> OptimisedDelta = FOptimisedDeltaFactory::Create(BuildOptimisedDeltaConfig(ManifestA, ManifestB.ToSharedRef(), DownloadCloudDirectories, Configuration.DeltaFilenameTrailer), MoveTemp(OptimisedDeltaDependencies));

			const FString RelativeDeltaFilename = FBuildPatchUtils::GetChunkDeltaFilename(*ManifestA.Get(), *ManifestB.Get(), Configuration.DeltaFilenameTrailer);
			const FString OutputDeltaFilename = Configuration.OutputCloudDirectory / RelativeDeltaFilename;

			const IOptimisedDelta::FResultValueOrError& OptimisedDeltaResult = OptimisedDelta->GetResult();

			// Check whether the delta file was retrieved and can't be serialized
			if (OptimisedDeltaResult.HasError() && OptimisedDeltaResult.GetError() == DownloadErrorCodes::UnserialisableDeltaFile)
			{
				UE_LOGF(LogChunkDeltaOptimiser, Error, "Optimised delta completed previously but could not be loaded %ls.", *RelativeDeltaFilename);
				UE_LOGF(LogChunkDeltaOptimiser, Error, "This file should be deleted as it will cause patching errors.");
				bSuccess = false;
			}

			const bool bDeltaPreviouslyCompleted = OptimisedDeltaResult.IsValid() && OptimisedDeltaResult.GetValue() != ManifestB;
			FBuildPatchAppManifest DeltaManifest;
			int32 LoadedDeltaSize = INDEX_NONE;
			if (bSuccess && bDeltaPreviouslyCompleted)
			{
				FinalStatLogs.Add(FString::Printf(TEXT("** Chunk delta optimisation already completed for provided manifests. **")));
				FinalStatLogs.Add(FString::Printf(TEXT("Loaded optimised delta file %s"), *RelativeDeltaFilename));
				DeltaManifest = *OptimisedDelta->GetResult().GetValue();
				LoadedDeltaSize = OptimisedDelta->GetMetaDownloadSize();
			}

			// Check for aborting if original delta is over provided threshold.
			TSet<FString> TagsA, TagsB;
			ManifestA->GetFileTagList(TagsA);
			ManifestB->GetFileTagList(TagsB);
			const uint64 OriginalUnknownCompressedBytes = (uint64)ManifestB->GetDeltaDownloadSize(TagsB, ManifestA.ToSharedRef(), TagsA);
			bool bOverAbortThreshold = false;
			if (Configuration.DiffAbortThreshold)
			{
				const FDiffAbortThreshold DiffAbortThreshold = DeltaOptimiseHelpers::ClampToSaneMinBasedOnActualBinarySize(FinalStatLogs, *Configuration.DiffAbortThreshold, ManifestB->GetBuildSize());
				const DeltaOptimiseHelpers::FAbsoluteAndPercentage AbsoluteAndPercentage = DeltaOptimiseHelpers::RecalculateAsAbsoluteAndPercentage(DiffAbortThreshold, ManifestB->GetBuildSize());
				bOverAbortThreshold = OriginalUnknownCompressedBytes >= AbsoluteAndPercentage.Absolute;
				if (!bDeltaPreviouslyCompleted && bOverAbortThreshold)
				{
					const uint64 OriginalUnknownCompressedBytesInPercentage = DeltaOptimiseHelpers::AsPercentageOf({ OriginalUnknownCompressedBytes, EDiffAbortThresholdUnits::Absolute }, ManifestB->GetBuildSize());
					FinalStatLogs.Add(FString::Printf(TEXT("** Aborting delta optimisation due to original delta over the %s threshold. **"), *LexToString(DiffAbortThreshold.Unit)));
					FinalStatLogs.Add(FString::Printf(TEXT("%llu >= %llu Bytes (or in percentage %llu%% >= %llu%%)"), 
						OriginalUnknownCompressedBytes, AbsoluteAndPercentage.Absolute, OriginalUnknownCompressedBytesInPercentage,  AbsoluteAndPercentage.Percentage));
				}
			}

			// Check for aborting if original delta is less than window size.
			const bool bIsLessThanWindowSize = OriginalUnknownCompressedBytes < (int32)Configuration.ScanWindowSize;
			if (!bDeltaPreviouslyCompleted && bIsLessThanWindowSize)
			{
				FinalStatLogs.Add(FString::Printf(TEXT("** Aborting delta optimisation due to original delta less than scan window size. **")));
				FinalStatLogs.Add(FString::Printf(TEXT("%llu < %u"), OriginalUnknownCompressedBytes, Configuration.ScanWindowSize));
			}

			// Calculate the desired bytes for manifest streams.
			FBlockStructure ManifestADesiredBytes = GetDesiredBytes(ManifestA, ChunksA.Difference(ChunksB));
			FBlockStructure ManifestBDesiredBytes = GetDesiredBytes(ManifestB, ChunksB.Difference(ChunksA));
			const uint64 ManifestAStreamSize = BlockStructureHelpers::CountSize(ManifestADesiredBytes);
			const uint64 ManifestBStreamSize = BlockStructureHelpers::CountSize(ManifestBDesiredBytes);

			const bool bIsManifestASizeLessThanWindowsSize = ManifestAStreamSize < (int32)Configuration.ScanWindowSize;
			if (!bDeltaPreviouslyCompleted && bIsManifestASizeLessThanWindowsSize)
			{
				FinalStatLogs.Add(FString::Printf(TEXT("** Aborting delta optimisation due to manifest A size less than scan window size. **")));
				FinalStatLogs.Add(FString::Printf(TEXT("%llu < %u"), ManifestAStreamSize, Configuration.ScanWindowSize));
			}

			const bool bIsManifestBSizeLessThanWindowsSize = ManifestBStreamSize < (int32)Configuration.ScanWindowSize;
			if (!bDeltaPreviouslyCompleted && bIsManifestBSizeLessThanWindowsSize)
			{
				FinalStatLogs.Add(FString::Printf(TEXT("** Aborting delta optimisation due to manifest B size less than scan window size. **")));
				FinalStatLogs.Add(FString::Printf(TEXT("%llu < %u"), ManifestBStreamSize, Configuration.ScanWindowSize));
			}

			const bool bRunProcess = bSuccess && !bDeltaPreviouslyCompleted && !bOverAbortThreshold && !bIsLessThanWindowSize && !bIsManifestBSizeLessThanWindowsSize && !bIsManifestASizeLessThanWindowsSize;
			if (bRunProcess)
			{
				// Runtime composition.
				TUniquePtr<ICloudChunkSourceStat> CloudChunkSourceStat(FCloudChunkSourceStatFactory::Create(StatsCollector.Get()));
				TUniquePtr<IChunkDataSerialization> ChunkDataSerializationReader(FChunkDataSerializationFactory::Create(FileSystem.Get(), Crypto.Get(), { ManifestB->ManifestMeta.FeatureLevel, Configuration.EncryptionSecrets, ManifestB->GetEncryptionSecretId() /* UE5 MERGE TODO : , CloudChunkSourceStat.Get()*/ }));
				TUniquePtr<DeltaFactories::FChunkReferenceTrackerFactory> ChunkReferenceTrackerFactory(new DeltaFactories::FChunkReferenceTrackerFactory());
				TUniquePtr<IBuildManifestSet> SetA(FBuildManifestSetFactory::Create({ FInstallerAction::MakeInstall(ManifestA.ToSharedRef()) }));
				TUniquePtr<IBuildManifestSet> SetB(FBuildManifestSetFactory::Create({ FInstallerAction::MakeInstall(ManifestB.ToSharedRef()) }));
				DeltaFactories::FCloudChunkSourceFactoryShared CloudChunkSourceFactorySharedA;
				CloudChunkSourceFactorySharedA.FileSystem = FileSystem.Get();
				CloudChunkSourceFactorySharedA.DownloadService = DownloadService.Get();
				CloudChunkSourceFactorySharedA.ChunkDataSerialization = ChunkDataSerializationReader.Get();
				CloudChunkSourceFactorySharedA.MessagePump = MessagePumpA.Get();
				CloudChunkSourceFactorySharedA.ManifestSet = SetA.Get();
				CloudChunkSourceFactorySharedA.CloudChunkSourceStat = CloudChunkSourceStat.Get();
				DeltaFactories::FCloudChunkSourceFactoryShared CloudChunkSourceFactorySharedB;
				CloudChunkSourceFactorySharedB.FileSystem = FileSystem.Get();
				CloudChunkSourceFactorySharedB.DownloadService = DownloadService.Get();
				CloudChunkSourceFactorySharedB.ChunkDataSerialization = ChunkDataSerializationReader.Get();
				CloudChunkSourceFactorySharedB.MessagePump = MessagePumpB.Get();
				CloudChunkSourceFactorySharedB.ManifestSet = SetB.Get();
				CloudChunkSourceFactorySharedB.CloudChunkSourceStat = CloudChunkSourceStat.Get();
				TUniquePtr<DeltaFactories::FCloudChunkSourceFactory> CloudChunkSourceFactoryA(new DeltaFactories::FCloudChunkSourceFactory(Configuration.DownloadCloudDirectories, CloudChunkSourceFactorySharedA));
				TUniquePtr<DeltaFactories::FCloudChunkSourceFactory> CloudChunkSourceFactoryB(new DeltaFactories::FCloudChunkSourceFactory(Configuration.DownloadCloudDirectories, CloudChunkSourceFactorySharedB));

				// Buffer for data streaming.
				const EAllowShrinking AllowShrinking = EAllowShrinking::No;
				const uint32 StreamBufferReadSize = Configuration.ScanWindowSize * 32;
				const uint32 ScannerDataSize = StreamBufferReadSize;
				TArray<uint8> StreamBuffer;
				StreamBuffer.Reserve(StreamBufferReadSize + Configuration.ScanWindowSize);

				// Start the ManifestA stream and chunk enumeration.
				FManifestBuildStreamerConfig ManifestAStreamConfig({Configuration.DownloadCloudDirectories, ManifestADesiredBytes});
				FManifestBuildStreamerDependencies ManifestAStreamDependencies({ChunkReferenceTrackerFactory.Get(), CloudChunkSourceFactoryA.Get(), StatsCollector.Get(), ManifestA.Get()});
				TUniquePtr<IManifestBuildStreamer> ManifestAStream(FBuildStreamerFactory::Create(MoveTemp(ManifestAStreamConfig), MoveTemp(ManifestAStreamDependencies)));

				// First we re-chunk prev unknown build parts into the scanner window size.
				TUniquePtr<IDeltaChunkEnumeration> DeltaChunkEnumeration(FDeltaChunkEnumerationFactory::Create(ManifestAStream.Get(), StatsCollector.Get(), *ManifestA.Get(), Configuration.ScanWindowSize, Configuration.ManifestATagAliases, Configuration.ManifestAIgnoreTags));
				ChunkingTimer.Start();
				DeltaChunkEnumeration->Run();
				ChunkingTimer.Stop();

				if (!DeltaChunkEnumeration->IsSuccess())
				{
					bShouldRun = false;
					bSuccess = false;
					UE_LOGF(LogChunkDeltaOptimiser, Error, "Failed to enumerate chunk data.");
					return FinalStatLogs;
				}

				// Setup scanning stats.
				volatile int64* StatScannerBacklog = StatsCollector->CreateStat(TEXT("BuildB: Scanner backlog"), EStatFormat::Value);
				volatile int64* StatScannerForks = StatsCollector->CreateStat(TEXT("BuildB: Scanner forks"), EStatFormat::Value);
				volatile int64* StatScanningTime = StatsCollector->CreateStat(TEXT("BuildB: Scanning time"), EStatFormat::Timer);
				volatile int64* StatScanningCompleted = StatsCollector->CreateStat(TEXT("BuildB: Progress"), EStatFormat::Percentage);

				// Place to dump details logging until we write it to a file at the end.
				TArray<FString> SourceDetailsOutputLines;
				
				// Start the ManifestB stream.
				FManifestBuildStreamerConfig ManifestBStreamConfig({Configuration.DownloadCloudDirectories, ManifestBDesiredBytes});
				FManifestBuildStreamerDependencies ManifestBStreamDependencies({ChunkReferenceTrackerFactory.Get(), CloudChunkSourceFactoryB.Get(), StatsCollector.Get(), ManifestB.Get()});
				TUniquePtr<IManifestBuildStreamer> ManifestBStream(FBuildStreamerFactory::Create(MoveTemp(ManifestBStreamConfig), MoveTemp(ManifestBStreamDependencies)));

				// Our second loop which finds matching chunks in the new build.
				ScanningTimer.Start();
				FChunkSearcher FileListSearcher(*ManifestB.Get());
				TUniquePtr<FChunkMatchStomper> ChunkMatchStomper(new FChunkMatchStomper(*ManifestA.Get(), *ManifestB.Get()));
				const uint32 ScannerOverlapSize = Configuration.ScanWindowSize - 1;
				TUniquePtr<IChunkMatchProcessor> ChunkMatchProcessor(FChunkMatchProcessorFactory::Create());
				TArray<TUniquePtr<FDeltaScannerEntry>> DataScanners;
				int32 NumScannersCreated = 0;
				int32 NumScannersRequired = ManifestBStreamSize / (ScannerDataSize - ScannerOverlapSize);
				FMeanValue MeanScannerTime(5);
				int32 ConsumedBufferData = 0;
				uint64 StreamStartPosition = 0;
				StreamBuffer.SetNumUninitialized(0, AllowShrinking);
				const TMap<FDeltaChunkId, FChunkBuildReference>& ChunkBuildReferences = DeltaChunkEnumeration->GetChunkBuildReferences();
				uint64 BuildBScanTimer;
				FStatsCollector::AccumulateTimeBegin(BuildBScanTimer);
				while (ManifestBStream->IsEndOfData() == false || DataScanners.Num() > 0)
				{
					// Grab new stream data.
					check(StreamBuffer.Num() >= ConsumedBufferData);
					uint32 BufferDataSize = StreamBuffer.Num() - ConsumedBufferData;
					if (!ManifestBStream->IsEndOfData() && (BufferDataSize < ScannerDataSize))
					{
						// Move unconsumed data to the beginning.
						if (BufferDataSize > 0)
						{
							uint8* const CopyTo = StreamBuffer.GetData();
							const uint8* const CopyFrom = &StreamBuffer[ConsumedBufferData];
							FMemory::Memcpy(CopyTo, CopyFrom, BufferDataSize);
						}
						StreamStartPosition += ConsumedBufferData;
						ConsumedBufferData = 0;

						// Fill the rest of the buffer.
						StreamBuffer.SetNumUninitialized(BufferDataSize + StreamBufferReadSize, AllowShrinking);
						const uint32 SizeRead = ManifestBStream->DequeueData(StreamBuffer.GetData() + BufferDataSize, StreamBufferReadSize);
						StreamBuffer.SetNumUninitialized(BufferDataSize + SizeRead, AllowShrinking);
						BufferDataSize = StreamBuffer.Num();
					}

					// Grab a scanner result.
					if (DataScanners.Num() > 0 && DataScanners[0]->Scanner->IsComplete())
					{
						FDeltaScannerEntry& ScannerDetails = *DataScanners[0];
						if (!ScannerDetails.bWasFork)
						{
							MeanScannerTime.AddSample(ScannerDetails.Scanner->GetTimeRunning());
						}
						TArray<FChunkMatch> ChunkMatches = ScannerDetails.Scanner->GetResultWhenComplete();
						for (FChunkMatch& ChunkMatch : ChunkMatches)
						{
							FBlockStructure ChunkCBuildBStructure;
							ChunkMatch.DataOffset += ScannerDetails.Offset;
							ManifestBDesiredBytes.SelectSerialBytes(ChunkMatch.DataOffset, ChunkMatch.WindowSize, ChunkCBuildBStructure);
							ChunkMatchProcessor->ProcessMatch(0, ChunkMatch, MoveTemp(ChunkCBuildBStructure));
						}
						const FBlockRange ScannerRange = FBlockRange::FromFirstAndSize(ScannerDetails.Offset, ScannerDetails.Data.Num());
						const uint64 SafeFlushSize = ScannerDetails.bIsFinalScanner ? ScannerRange.GetLast() + 1 : ScannerRange.GetFirst();
						if (SafeFlushSize > 0)
						{
							ChunkMatchProcessor->FlushLayer(0, SafeFlushSize);
						}
						DataScanners.RemoveAt(0);
					}

					// Handle extra matches accepted.
					TArray<FMatchEntry> AcceptedChunkMatches;
					const FBlockRange CollectionRange = ChunkMatchProcessor->CollectLayer(0, AcceptedChunkMatches);
					if (CollectionRange.GetSize() > 0)
					{
						for (FMatchEntry& AcceptedChunkMatch : AcceptedChunkMatches)
						{
							const FChunkMatch& ChunkCMatch = AcceptedChunkMatch.ChunkMatch;
							const TArray<FChunkPart>& NewChunkReferences = ChunkBuildReferences[ChunkCMatch.ChunkGuid].ChunkParts;
							const FBlockStructure& ChunkCBuildBStructure = AcceptedChunkMatch.BlockStructure;
							ChunkMatchStomper->ReplaceChunkReferences(NewChunkReferences, ChunkCBuildBStructure);
						}
					}

					// Create new scanner.
					uint32 SizeToScan = FMath::Min(ScannerDataSize, BufferDataSize);
					const bool bHasData = SizeToScan == ScannerDataSize || (ManifestBStream->IsEndOfData() && BufferDataSize > 0);
					if (bHasData && !DeltaOptimiseHelpers::ScannerArrayFull(DataScanners))
					{
						FDeltaScannerEntry* NewScanner = new FDeltaScannerEntry();
						NewScanner->Data.Append(StreamBuffer.GetData() + ConsumedBufferData, SizeToScan);
						NewScanner->Offset = StreamStartPosition + ConsumedBufferData;

						FBlockStructure ScannerBuildStructure;
						ManifestBDesiredBytes.SelectSerialBytes(NewScanner->Offset, SizeToScan, ScannerBuildStructure);
						DeltaOptimiseHelpers::MakeScannerLocalList(Configuration.ManifestBTagAliases, Configuration.ManifestBIgnoreTags, FileListSearcher, DeltaChunkEnumeration.Get(), ScannerBuildStructure, NewScanner->FilesList);

						NewScanner->Scanner.Reset(FDeltaScannerFactory::Create(Configuration.ScanWindowSize, NewScanner->Data, NewScanner->FilesList, DeltaChunkEnumeration.Get(), StatsCollector.Get()));
						ConsumedBufferData += SizeToScan;
						NewScanner->bIsFinalScanner = ManifestBStream->IsEndOfData() && ConsumedBufferData >= StreamBuffer.Num();
						if (!NewScanner->bIsFinalScanner)
						{
							ConsumedBufferData -= ScannerOverlapSize;
						}
						DataScanners.Emplace(NewScanner);
						++NumScannersCreated;
					}

					// Fork a scanner with too much work?
					if (DataScanners.Num() > 0 && MeanScannerTime.IsReliable() && DeltaOptimiseHelpers::HasUnusedCpu())
					{
						FDeltaScannerEntry& DataScannerEntry = *DataScanners[0];
						const double TopScannerTime = DataScannerEntry.Scanner->GetTimeRunning();
						double DownloadTimeMean;
						double DownloadTimeStd;
						MeanScannerTime.GetValues(DownloadTimeMean, DownloadTimeStd);
						const double BreakingPoint = FMath::Max<double>(0.25, DownloadTimeMean + DownloadTimeStd);
						if (TopScannerTime > BreakingPoint && DataScannerEntry.Scanner->SupportsFork())
						{
							DataScannerEntry.bWasFork = true;
							FStatsCollector::Accumulate(StatScannerForks, 1);
							const FBlockRange UnscannedRange = DataScannerEntry.Scanner->Fork();
							const uint64 ForkSize = (UnscannedRange.GetSize() / 2) + 1;
							if (ForkSize < UnscannedRange.GetSize())
							{
								// Insert the right fork first.
								const FBlockRange RightFork = FBlockRange::FromFirstAndLast(UnscannedRange.GetLast() - ForkSize, UnscannedRange.GetLast());
								FDeltaScannerEntry* NewScanner = new FDeltaScannerEntry();
								NewScanner->Data.Append(DataScannerEntry.Data.GetData() + RightFork.GetFirst(), RightFork.GetSize());
								NewScanner->Offset = DataScannerEntry.Offset + RightFork.GetFirst();

								FBlockStructure ScannerBuildStructure;
								ManifestBDesiredBytes.SelectSerialBytes(NewScanner->Offset, RightFork.GetSize(), ScannerBuildStructure);
								DeltaOptimiseHelpers::MakeScannerLocalList(Configuration.ManifestBTagAliases, Configuration.ManifestBIgnoreTags, FileListSearcher, DeltaChunkEnumeration.Get(), ScannerBuildStructure, NewScanner->FilesList);

								NewScanner->Scanner.Reset(FDeltaScannerFactory::Create(Configuration.ScanWindowSize, NewScanner->Data, NewScanner->FilesList, DeltaChunkEnumeration.Get(), StatsCollector.Get()));
								NewScanner->bIsFinalScanner = DataScannerEntry.bIsFinalScanner;
								NewScanner->bWasFork = true;
								DataScanners.EmplaceAt(1, NewScanner);

								// Insert the left fork.
								const FBlockRange LeftFork = FBlockRange::FromFirstAndLast(UnscannedRange.GetFirst(), UnscannedRange.GetFirst() + ForkSize);
								NewScanner = new FDeltaScannerEntry();
								NewScanner->Data.Append(DataScannerEntry.Data.GetData() + LeftFork.GetFirst(), LeftFork.GetSize());
								NewScanner->Offset = DataScannerEntry.Offset + LeftFork.GetFirst();

								ScannerBuildStructure.Empty();
								ManifestBDesiredBytes.SelectSerialBytes(NewScanner->Offset, LeftFork.GetSize(), ScannerBuildStructure);
								DeltaOptimiseHelpers::MakeScannerLocalList(Configuration.ManifestBTagAliases, Configuration.ManifestBIgnoreTags, FileListSearcher, DeltaChunkEnumeration.Get(), ScannerBuildStructure, NewScanner->FilesList);

								NewScanner->Scanner.Reset(FDeltaScannerFactory::Create(Configuration.ScanWindowSize, NewScanner->Data, NewScanner->FilesList, DeltaChunkEnumeration.Get(), StatsCollector.Get()));
								NewScanner->bIsFinalScanner = false;
								NewScanner->bWasFork = true;
								DataScanners.EmplaceAt(1, NewScanner);

								// Adjust original meta.
								DataScannerEntry.bIsFinalScanner = false;
								DataScannerEntry.Data.SetNumUninitialized(UnscannedRange.GetFirst(), EAllowShrinking::No);
							}
							else
							{
								// Something has gone wrong with the size calculation, this is fatal.
								check(ForkSize < UnscannedRange.GetSize());
							}
						}
					}

					const double PercentScanned = (double)(NumScannersCreated - DataScanners.Num()) / (double)NumScannersRequired;
					FStatsCollector::SetAsPercentage(StatScanningCompleted, PercentScanned);
					FStatsCollector::Set(StatScannerBacklog, DataScanners.Num());
					FStatsCollector::AccumulateTimeEnd(StatScanningTime, BuildBScanTimer);
					FStatsCollector::AccumulateTimeBegin(BuildBScanTimer);
				}
				FStatsCollector::AccumulateTimeEnd(StatScanningTime, BuildBScanTimer);
				FStatsCollector::SetAsPercentage(StatScanningCompleted, 1.0);
				ScanningTimer.Stop();

				// Grab the new manifest data.
				FFileManifestList FileManifestList = ChunkMatchStomper->GetNewFileManifests();

				// For all unknown data we need to re-chunk it out and fill in the gaps we have.
				FBlockStructure NewStreamBlocks;
				TArray<TTuple<FBlockStructure, FChunkPart>> NewChunks;
				uint64 ByteLocation = 0;
				for (const FFileManifest& FileManifest : FileManifestList.FileList)
				{
					for (const FChunkPart& ChunkPart : FileManifest.ChunkParts)
					{
						if (ChunkPart.Guid.IsValid() == false)
						{
							uint64 PartByteLocation = ByteLocation;
							uint64 PartSizeRemaining = ChunkPart.Size;
							while (PartSizeRemaining > 0)
							{
								// Start new chunk?
								if (NewChunks.Num() == 0 || NewChunks.Last().Get<1>().Size >= OutputChunkSize)
								{
									NewChunks.AddDefaulted_GetRef().Get<1>().Guid = FGuid::NewGuid();
								}

								TTuple<FBlockStructure, FChunkPart>* LastChunkDetail = &NewChunks.Last();
								const uint64 NewTotalSize = LastChunkDetail->Get<1>().Size + PartSizeRemaining;
								const uint64 ChunkPartConsume = NewTotalSize > OutputChunkSize ? PartSizeRemaining - (NewTotalSize - OutputChunkSize) : PartSizeRemaining;
								check(PartSizeRemaining >= ChunkPartConsume);

								NewStreamBlocks.Add(PartByteLocation, ChunkPartConsume, ESearchDir::FromEnd);
								LastChunkDetail->Get<0>().Add(PartByteLocation, ChunkPartConsume, ESearchDir::FromEnd);
								LastChunkDetail->Get<1>().Size += ChunkPartConsume;
								PartByteLocation += ChunkPartConsume;
								PartSizeRemaining -= ChunkPartConsume;
							}
						}
						ByteLocation += ChunkPart.Size;
					}
				}

				// Save out all new chunk data.
				TMap<FGuid, uint32> NewChunkWindowSizes;
				TSet<FChunkSearcher::FFileNode*> UpdatedFiles;
				FChunkSearcher ManifestSearcher(FileManifestList);
				FManifestBuildStreamerConfig UnknownDataStreamConfig({Configuration.DownloadCloudDirectories, NewStreamBlocks});
				FManifestBuildStreamerDependencies UnknownDataStreamDependencies({ChunkReferenceTrackerFactory.Get(), CloudChunkSourceFactoryB.Get(), StatsCollector.Get(), ManifestB.Get()});
				TUniquePtr<IManifestBuildStreamer> UnknownDataStream(FBuildStreamerFactory::Create(MoveTemp(UnknownDataStreamConfig), MoveTemp(UnknownDataStreamDependencies)));
				TUniquePtr<IChunkDataSerialization> ChunkDataSerializationWriter(FChunkDataSerializationFactory::Create(FileSystem.Get(), Crypto.Get(), { ManifestB->ManifestMeta.FeatureLevel, Configuration.EncryptionSecrets, ManifestB->GetEncryptionSecretId() /* UE5 MERGE TODO : , CloudChunkSourceStat.Get()*/ }));
				FParallelChunkWriterConfig ChunkWriterConfig = FParallelChunkWriterConfig({5, 5, 50, 8, Configuration.bResaveExistingChunks, Configuration.OutputCloudDirectory, ManifestB->ManifestMeta.FeatureLevel});
				TUniquePtr<IParallelChunkWriter> ChunkWriter(FParallelChunkWriterFactory::Create(ChunkWriterConfig, FileSystem.Get(), ChunkDataSerializationWriter.Get(), StatsCollector.Get()));
				// Register chunk writer events and pump from main thread.
				FTSTicker::FDelegateHandle ChunkWriterPumpHandle;
				AsyncHelpers::ExecuteOnGameThread<void>([&]() { ChunkWriterPumpHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([&](float) { ChunkWriter->PumpEvents(); return true; })); }).Wait();
				if (OnChunkFileWritten().IsBound())
				{
					ChunkWriter->OnChunkFileWritten().AddLambda([this](const FString& FullFilePath, const FMD5Hash& MD5Hash, const FSHAHash& SHA1Hash)
					{
						OnChunkFileWritten().Broadcast(FullFilePath, MD5Hash);
					});
				}
				ChunkWriter->OnChunkFileWriteFailed().AddLambda([this, ChunkWriterPtr = ChunkWriter.Get()](const FString& FullFilePath) {
					ChunkWriterPtr->Abort();
				});
				auto WhenChunkWriterFailed = [&]() {
					bShouldRun = false;
					bSuccess = false;
					UE_LOGF(LogChunkDeltaOptimiser, Error, "Chunk writer aborted");
					return FinalStatLogs;
				};

				StreamBuffer.SetNumUninitialized(0, AllowShrinking);
				for (const TTuple<FBlockStructure, FChunkPart>& NewChunk : NewChunks)
				{
					const FBlockStructure& NewChunkStructure = NewChunk.Get<0>();
					const FChunkPart& NewChunkPart = NewChunk.Get<1>();
					NewChunkWindowSizes.Add(NewChunkPart.Guid, NewChunkPart.Size);

					DeltaOptimiseHelpers::StompChunkPart(NewChunkPart, NewChunkStructure, ManifestSearcher, UpdatedFiles);

					// Collect all the chunk data.
					const FBlockEntry* NewChunkBlock = NewChunkStructure.GetHead();
					StreamBuffer.SetNumUninitialized(NewChunkPart.Size, AllowShrinking);
					uint32 ChunkLocationOffset = 0;
					while (NewChunkBlock)
					{
						const uint32 SizeRead = UnknownDataStream->DequeueData(StreamBuffer.GetData() + ChunkLocationOffset, NewChunkBlock->GetSize());
						check(SizeRead == NewChunkBlock->GetSize());
						ChunkLocationOffset += NewChunkBlock->GetSize();
						NewChunkBlock = NewChunkBlock->GetNext();
					}
					check(ChunkLocationOffset == StreamBuffer.Num());

					// Ensure padding if necessary.
					StreamBuffer.SetNumZeroed(OutputChunkSize, AllowShrinking);

					// Save out new chunk.
					const uint64 NewChunkHash = FRollingHash::GetHashForDataSet(StreamBuffer.GetData(), StreamBuffer.Num());
					const FSHAHash NewChunkSha = DeltaOptimiseHelpers::GetShaForDataSet(StreamBuffer.GetData(), StreamBuffer.Num());

					// Save it out.
					if (!ChunkWriter->AddChunkData(StreamBuffer, NewChunkPart.Guid, NewChunkHash, NewChunkSha))
					{
						return WhenChunkWriterFailed();
					}
				}

				// Iterate the output file manifest and find how much each file changed, and what the sources were.
				// Track where each chunk came from so we can report source information for matches.
				struct FGuidSourceInfo
				{
					FString File;
					uint64 FileOffset, Size;
					uint32 ReadOffset;
				};
				TMap<FGuid, TArray<FGuidSourceInfo>> SourceInfos;

				if (Configuration.SourceDetailsLogFilename.Len())
				{
					TArray<FString> SourceFiles;
					ManifestA->GetFileList(SourceFiles);
					
					TMap<uint64, uint32> ChunkSizes;
					for (FString& SourceFile : SourceFiles)
					{
						const FFileManifest* SourceManifest = ManifestA->GetFileManifest(SourceFile);
						uint64 SourceFileOffset = 0;
						TSet<FGuid> ChunksInFile;

						for (const FChunkPart& ChunkPart : SourceManifest->ChunkParts)
						{
							ChunksInFile.Add(ChunkPart.Guid);

							FGuidSourceInfo NewInfo;
							NewInfo.File = SourceFile;
							NewInfo.FileOffset = SourceFileOffset;
							NewInfo.Size = ChunkPart.Size;
							SourceFileOffset += ChunkPart.Size;
							NewInfo.ReadOffset = ChunkPart.Offset;

							SourceInfos.FindOrAdd(ChunkPart.Guid).Add(MoveTemp(NewInfo));

							const BuildPatchServices::FChunkInfo* ChunkInfo = ManifestA->GetChunkInfo(ChunkPart.Guid);
							ChunkSizes.FindOrAdd(ChunkInfo->DataSizeUncompressed, 0)++;
						}

						SourceDetailsOutputLines.Add(FString::Printf(TEXT("OldFileChunks: %s"), *SourceFile));
						for (FGuid& G : ChunksInFile)
						{
							SourceDetailsOutputLines.Add(FString::Printf(TEXT("    OFC: %s"), *LexToString(G)));
						}
					}

					SourceDetailsOutputLines.Add(TEXT("Old Build Chunk Size Counts"));
					for (TPair<uint64, uint32> P : ChunkSizes)
					{
						SourceDetailsOutputLines.Add(FString::Printf(TEXT("    %s: %s"), *FText::AsNumber(P.Key).ToString(), *FText::AsNumber(P.Value).ToString()));
					}
				}


				// We also need to potentially upgrade chunks from ManifestA into ManifestB feature level.
				if (bUsingDifferentChunkSubdir)
				{
					// Enumerate all chunks referenced from ManifestA.
					TSet<FGuid> SourceChunkReferences;
					FChunkSearcher::FFileDListNode* FileNode = ManifestSearcher.GetHead();
					while (FileNode)
					{
						FChunkSearcher::FChunkDListNode* ChunkNode = FileNode->GetValue().ChunkParts.GetHead();
						while (ChunkNode)
						{
							if (ManifestA->GetChunkInfo(ChunkNode->GetValue().ChunkPart.Guid) != nullptr)
							{
								SourceChunkReferences.Add(ChunkNode->GetValue().ChunkPart.Guid);
							}
							ChunkNode = ChunkNode->GetNextNode();
						}
						FileNode = FileNode->GetNextNode();
					}

					// Load these chunks and save them out in the new format.
					TUniquePtr<IChunkReferenceTracker> UpgradeChunkReferenceTracker(ChunkReferenceTrackerFactory->Create(SourceChunkReferences.Array()));
					TUniquePtr<ICloudChunkSource> UpgradeCloudChunkSource(CloudChunkSourceFactoryA->Create(UpgradeChunkReferenceTracker.Get()));
					for (const FGuid& UpgradeChunk : SourceChunkReferences)
					{
						const FChunkInfo* UpgradeChunkInfo = ManifestA->GetChunkInfo(UpgradeChunk);
						IChunkDataAccess* UpgradeChunkDataAccess = UpgradeCloudChunkSource->Get(UpgradeChunk);
						checkf(UpgradeChunkDataAccess != nullptr, TEXT("Failed to download chunk from source %s."), *FBuildPatchUtils::GetDataFilename(*ManifestA.Get(), UpgradeChunk));
						FScopeLockedChunkData LockedChunkData(UpgradeChunkDataAccess);
						TArray<uint8> ChunkDataArray(LockedChunkData.GetData(), LockedChunkData.GetHeader()->DataSizeUncompressed);
						if (!ChunkWriter->AddChunkData(MoveTemp(ChunkDataArray), UpgradeChunk, UpgradeChunkInfo->Hash, UpgradeChunkInfo->ShaHash))
						{
							return WhenChunkWriterFailed();
						}
						UpgradeChunkReferenceTracker->PopReference(UpgradeChunk);
					}
				}

				// We always make sure padding chunks are saved out, so a legacy client could actually grab it.
				FSHAHash PaddingChunkSha;
				FGuid PaddingChunkId = PaddingChunk::MakePaddingGuid(0);
				NewChunkWindowSizes.Add(PaddingChunkId, PaddingChunk::ChunkSize);
				StreamBuffer.SetNumUninitialized(PaddingChunk::ChunkSize);
				FMemory::Memset(StreamBuffer.GetData(), PaddingChunkId.D, PaddingChunk::ChunkSize);
				FSHA1::HashBuffer(StreamBuffer.GetData(), PaddingChunk::ChunkSize, PaddingChunkSha.Hash);
				if (!ChunkWriter->AddChunkData(StreamBuffer, PaddingChunkId, FRollingHash::GetHashForDataSet(StreamBuffer.GetData(), PaddingChunk::ChunkSize), PaddingChunkSha))
				{
					return WhenChunkWriterFailed();
				}
				for (uint32 LoopIdx = 1; LoopIdx <= 255; ++LoopIdx)
				{
					const uint8 Byte = LoopIdx & 0xFF;
					PaddingChunkId.D = Byte;
					NewChunkWindowSizes.Add(PaddingChunkId, PaddingChunk::ChunkSize);
					FMemory::Memset(StreamBuffer.GetData(), PaddingChunkId.D, PaddingChunk::ChunkSize);
					FSHA1::HashBuffer(StreamBuffer.GetData(), PaddingChunk::ChunkSize, PaddingChunkSha.Hash);
					if (!ChunkWriter->AddChunkData(StreamBuffer, PaddingChunkId, FRollingHash::GetHashForDataSet(StreamBuffer.GetData(), PaddingChunk::ChunkSize), PaddingChunkSha))
					{
						return WhenChunkWriterFailed();
					}
				}

				// Wait for chunk writer to be completed.
				while (ChunkWriter->GetInFlightChunkCount() > 0 && !ChunkWriter->HasAborted())
				{
					FPlatformProcess::Sleep(0.01f);
				}

				if (ChunkWriter->HasAborted())
				{
					return WhenChunkWriterFailed();
				}

				// Use main thread to remove the pump.
				if (ChunkWriterPumpHandle.IsValid())
				{
					AsyncHelpers::ExecuteOnGameThread<void>([&]() { FTSTicker::GetCoreTicker().RemoveTicker(ChunkWriterPumpHandle); }).Wait();
				}

				// Complete chunk writer.
				FParallelChunkWriterSummaries ChunkWriterSummaries = ChunkWriter->OnProcessComplete();

				// Produce the new stomped file manifests, but remove any that we no longer need if they don't actually change with the delta.
				FileManifestList = ManifestSearcher.BuildNewFileManifestList();
				for (auto FileListIterator = FileManifestList.FileList.CreateIterator(); FileListIterator; ++FileListIterator)
				{
					if (ManifestB->IsFileOutdated(ManifestA.ToSharedRef(), (*FileListIterator).Filename) == false)
					{
						FileListIterator.RemoveCurrent();
					}
				}

				// Save out the delta file.
				DeltaManifest.ManifestMeta = ManifestB->ManifestMeta;
				DeltaManifest.CustomFields = ManifestB->CustomFields;
				DeltaManifest.FileManifestList = MoveTemp(FileManifestList);
				TSet<FGuid> AddedChunkInfos;
				for (const FFileManifest& FileManifest : DeltaManifest.FileManifestList.FileList)
				{
					TMap<FString, uint64> MatchedBytesFromFile;
					TSet<FGuid> GuidsRequiredFromInstall;

					SourceDetailsOutputLines.Add(FileManifest.Filename);

					uint64 CurrentFileOffset = 0;
					for (const FChunkPart& ChunkPart : FileManifest.ChunkParts)
					{
						if (Configuration.SourceDetailsLogFilename.Len())
						{
							// Here we find out if this chunk part is from the old build. If it is, we try and find out 
							const TArray<FGuidSourceInfo>* SourceInfoPtr = SourceInfos.Find(ChunkPart.Guid);
						
							if (!SourceInfoPtr)
							{
								MatchedBytesFromFile.FindOrAdd(TEXT("new"), 0) += ChunkPart.Size;
							}
							else
							{
								// Here on, it's matched data off the install.

								GuidsRequiredFromInstall.Add(ChunkPart.Guid);
								const TArray<FGuidSourceInfo>& SourceInfo = *SourceInfoPtr;

								if (SourceInfo.Num() == 1)
								{
									// The match pulls from a single source file.
									MatchedBytesFromFile.FindOrAdd(SourceInfo[0].File, 0) += ChunkPart.Size;
									
									FString LogLine = FString::Printf(TEXT("    %s: [%s size=%d] came from %s at %s+%d read offset %s"), *LexToString(ChunkPart.Guid), *FText::AsNumber(CurrentFileOffset).ToString(),
										ChunkPart.Size, *SourceInfo[0].File, *FText::AsNumber(SourceInfo[0].FileOffset).ToString(), ChunkPart.Offset, *FText::AsNumber(SourceInfo[0].ReadOffset).ToString());
									SourceDetailsOutputLines.Add(MoveTemp(LogLine));

									// <good one> UE_LOGF(LogTemp, Warning, "%ls: [%llu size=%d] came from %ls at %llu+%d, read offset %d", *LexToString(ChunkPart.Guid), CurrentOffset, ChunkPart.Size, *SourceInfo[0].File, SourceInfo[0].FileOffset, ChunkPart.Offset, SourceInfo[0].ReadOffset);
								}
								else
								{
									// The match pulls from multiple files. This is rare enough on large files we don't emit detailed debug info.
									MatchedBytesFromFile.FindOrAdd(TEXT("mul"), 0) += ChunkPart.Size;

									// We know we need to read ChunkPart.Offset,ChunkPart.Size from the chunk, so see which source provides that range.
									// This is commented out if we need it.

									//FBlockRange Range = FBlockRange::FromFirstAndSize(ChunkPart.Offset, ChunkPart.Size);
									//for (const FGuidSourceInfo& Source : SourceInfo)
									//{
									//	FBlockRange SourceRange = FBlockRange::FromFirstAndSize(Source.ReadOffset, Source.Size);
									//	if (SourceRange.Overlaps(Range))
									//	{
									//		// This source provides some of the data.
									//		FBlockRange Intersection = SourceRange.FromIntersection(SourceRange, Range);
									//
									//		//UE_LOGF(LogTemp, Warning, "%ls MUL: [%llu size=%d] came from %ls at %llu+%d (%llu) read offset %d", *LexToString(ChunkPart.Guid), CurrentOffset, ChunkPart.Size, *Source.File, Source.FileOffset, Intersection.GetFirst(), Intersection.GetSize(), Source.ReadOffset);
									//	}
									//}
									//for (const FGuidSourceInfo& Source : SourceInfo)
									//{
									//	UE_LOGF(LogTemp, Warning, "%ls MUL: [%llu size=%d] came from %ls at %llu+%d (%llu) read offset %d", *LexToString(ChunkPart.Guid), CurrentOffset, ChunkPart.Size, *Source.File, Source.FileOffset, ChunkPart.Offset, Source.Size, Source.ReadOffset);
									//}
								}
							}
						} // end if emitting detailed source info

						CurrentFileOffset += ChunkPart.Size;

						bool bWasAlreadyInSet = false;
						AddedChunkInfos.Add(ChunkPart.Guid, &bWasAlreadyInSet);
						if (!bWasAlreadyInSet)
						{
							const FChunkInfo* OldChunkInfo = ManifestB->GetChunkInfo(ChunkPart.Guid);
							if (OldChunkInfo != nullptr)
							{
								DeltaManifest.ChunkDataList.ChunkList.Add(*OldChunkInfo);
							}
							else
							{
								OldChunkInfo = ManifestA->GetChunkInfo(ChunkPart.Guid);
								if (OldChunkInfo != nullptr)
								{
									DeltaManifest.ChunkDataList.ChunkList.Add(*OldChunkInfo);
								}
								else
								{
									FChunkInfo& NewChunkInfo = DeltaManifest.ChunkDataList.ChunkList.AddDefaulted_GetRef();
									NewChunkInfo.Guid = ChunkPart.Guid;
									NewChunkInfo.Hash = ChunkWriterSummaries.ChunkOutputHashes[ChunkPart.Guid];
									NewChunkInfo.ShaHash = ChunkWriterSummaries.ChunkOutputShas[ChunkPart.Guid];
									NewChunkInfo.GroupNumber = FCrc::MemCrc32(&ChunkPart.Guid, sizeof(FGuid)) % 100;
									NewChunkInfo.DataSizeUncompressed = NewChunkWindowSizes[ChunkPart.Guid];
									NewChunkInfo.FileSize = ChunkWriterSummaries.ChunkOutputFileSizes[ChunkPart.Guid];
									if (const int64* DataSizeCompressed = ChunkWriterSummaries.ChunkOutputCompressedSizes.Find(ChunkPart.Guid))
									{
										NewChunkInfo.DataSizeCompressed = *DataSizeCompressed;
									}
									if (const FAESAuthTag* AESAuthTag = ChunkWriterSummaries.ChunkOutputAuthTags.Find(ChunkPart.Guid))
									{
										NewChunkInfo.AESAuthTag = *AESAuthTag;
										NewChunkInfo.EncryptionSecretId = ManifestB->GetEncryptionSecretId();
									}
								}
							}
						}
					} // end for each chunk park in the file.

					if (Configuration.SourceDetailsLogFilename.Len())
					{
						SourceDetailsOutputLines.Add(TEXT("    Matches Bytes From File:"));
						for (TPair<FString, uint64>& Sizes : MatchedBytesFromFile)
						{
							// Note: This is a little bit weird because if the chunk that matches is a full container chunk (i.e. the size of the base build chunk,
							// not the small optimized scan window), it'll match the entire chunk and end up claiming that it's matching from a random file when it's actually
							// a patch. There's some sort of accumulation phase I don't have full understanding of. The easiest way to see this in the log is the file source
							// information a) won't make sense and b) will be exactly a multiple of the base chunk size.
							SourceDetailsOutputLines.Add(FString::Printf(TEXT("        %s: %s (%.2f)"), *Sizes.Key, *FText::AsNumber(Sizes.Value).ToString(), Sizes.Value * 100.0 / CurrentFileOffset));
							
							//UE_LOGF(LogTemp, Warning, "    %ls: %llu (%.2f)", *Sizes.Key, Sizes.Value, Sizes.Value * 100.0 / CurrentFileOffset);
						}
						SourceDetailsOutputLines.Add(TEXT("        Guids Requires From Install Source:"));
						for (FGuid& Guid : GuidsRequiredFromInstall)
						{
							SourceDetailsOutputLines.Add(FString::Printf(TEXT("        IC: %s"), *LexToString(Guid)));
							//UE_LOGF(LogTemp, Warning, "    <%ls>", *LexToString(Guid));
						}
					}
				} // end each file in the new manifest

				DeltaManifest.InitLookups();
				if (ManifestB->GetEncryptionSecretId().IsValid())
				{
					DeltaManifest.EncryptData(ManifestB->GetEncryptionSecretId(), Configuration.EncryptionSecrets[ManifestB->GetEncryptionSecretId()]);
				}
				// For save format, we'll pick the newest of two input manifests, or EFeatureLevel::FirstOptimisedDelta if manifests are older.
				EFeatureLevel DeltaOutputFormat = FMath::Max3(EFeatureLevel::FirstOptimisedDelta, ManifestA->GetFeatureLevel(), ManifestB->GetFeatureLevel());
				FMD5Hash MD5Hash;
				FSHAHash SHAHash;
				const FString TmpOutputDeltaFilename = OutputDeltaFilename + TEXT("tmp");
				if (DeltaManifest.SaveToFile(TmpOutputDeltaFilename, DeltaOutputFormat, &SHAHash, &MD5Hash) && FileSystem->MoveFile(*OutputDeltaFilename, *TmpOutputDeltaFilename))
				{
					FinalStatLogs.Add(FString::Printf(TEXT("Saved new optimised delta file %s"), *OutputDeltaFilename));
					// Run the notification for saving delta file.
					BroadcastDelta(OutputDeltaFilename, MD5Hash, SHAHash);
				}
				else
				{
					UE_LOGF(LogChunkDeltaOptimiser, Error, "Optimised delta completed successfully but could not be saved %ls.", *OutputDeltaFilename);
					bSuccess = false;
				}

				if (Configuration.SourceDetailsLogFilename.Len())
				{
					if (FFileHelper::SaveStringArrayToFile(SourceDetailsOutputLines, *Configuration.SourceDetailsLogFilename))
					{
						FinalStatLogs.Add(FString::Printf(TEXT("Saved source details log %s"), *Configuration.SourceDetailsLogFilename));
					}
					else
					{
						FinalStatLogs.Add(FString::Printf(TEXT("Failed to save source details log %s"), *Configuration.SourceDetailsLogFilename));
					}
				}
			}

			if (bSuccess)
			{
				// Count stats?
				TSet<FGuid> ChunksUnknown = ChunksB.Difference(ChunksA);
				uint64 OriginalUnknownBytes = 0;
				for (const FString& ManifestBFile : ListHelpers::GetFileList(*ManifestB))
				{
					for (const FChunkPart& ChunkPart : ManifestB->GetFileManifest(ManifestBFile)->ChunkParts)
					{
						if (ChunksUnknown.Contains(ChunkPart.Guid))
						{
							OriginalUnknownBytes += ChunkPart.Size;
						}
					}
				}
				uint64 FinalUnknownBytes = 0;
				for (const FFileManifest& FileManifest : DeltaManifest.FileManifestList.FileList)
				{
					for (const FChunkPart& ChunkPart : FileManifest.ChunkParts)
					{
						const bool bDeltaUniqueChunk = ManifestA->GetChunkInfo(ChunkPart.Guid) == nullptr && ManifestB->GetChunkInfo(ChunkPart.Guid) == nullptr;
						if (bDeltaUniqueChunk)
						{
							FinalUnknownBytes += ChunkPart.Size;
						}
					}
				}
				uint64 FinalUnknownCompressedBytes = 0;
				TSet<FGuid> TempTest;
				for (const FChunkInfo& DeltaChunkInfo : DeltaManifest.ChunkDataList.ChunkList)
				{
					const bool bDeltaUniqueChunk = ManifestA->GetChunkInfo(DeltaChunkInfo.Guid) == nullptr && ManifestB->GetChunkInfo(DeltaChunkInfo.Guid) == nullptr;
					if (bDeltaUniqueChunk)
					{
						FinalUnknownCompressedBytes += DeltaChunkInfo.FileSize;
						check(TempTest.Contains(DeltaChunkInfo.Guid) == false);
						TempTest.Add(DeltaChunkInfo.Guid);
					}
				}
				int64 DeltaFileSize = 0;
				if (!bDeltaPreviouslyCompleted)
				{
					if (bRunProcess && !FileSystem->GetFileSize(*OutputDeltaFilename, DeltaFileSize))
					{
						UE_LOGF(LogChunkDeltaOptimiser, Error, "Could not save output to %ls", *OutputDeltaFilename);
						bSuccess = false;
						DeltaFileSize = 0;
					}
				}
				else
				{
					DeltaFileSize = LoadedDeltaSize;
				}
				ProcessTimer.Stop();

				// Final improvement stat logs.
				if (bRunProcess || bDeltaPreviouslyCompleted)
				{
					FinalUnknownCompressedBytes += DeltaFileSize;
					FinalStatLogs.Add(FString::Printf(TEXT("Final unknown compressed bytes, plus meta %llu"), FinalUnknownCompressedBytes));
					FinalStatLogs.Add(FString::Printf(TEXT("Original unknown compressed bytes         %llu"), OriginalUnknownCompressedBytes));
					if (OriginalUnknownCompressedBytes > FinalUnknownCompressedBytes)
					{
						FinalStatLogs.Add(FString::Printf(TEXT("Improvement: %s"), *FText::AsPercent(1.0 - ((double)FinalUnknownCompressedBytes / (double)OriginalUnknownCompressedBytes), &PercentFormat).ToString()));
					}
				}

				if (bRunProcess)
				{
					const FString TempMetaFilename = OutputDeltaFilename.Replace(TEXT("Deltas/"), TEXT("DeltaMetas/")).Replace(TEXT(".delta"), TEXT(".json"));
					FString JsonOutput;
					TSharedRef<FDeltaJsonWriter> Writer = FDeltaJsonWriterFactory::Create(&JsonOutput);
					Writer->WriteObjectStart();
					{
						Writer->WriteValue(TEXT("SourceBuildVersion"), ManifestA->GetVersionString());
						Writer->WriteValue(TEXT("DestinationBuildVersion"), ManifestB->GetVersionString());
						Writer->WriteValue(TEXT("OriginalUnknownBuildBytes"), (int64)OriginalUnknownBytes);
						Writer->WriteValue(TEXT("FinalUnknownBuildBytes"), (int64)FinalUnknownBytes);
						Writer->WriteValue(TEXT("OriginalUnknownCompressedBytes"), (int64)OriginalUnknownCompressedBytes);
						Writer->WriteValue(TEXT("FinalUnknownCompressedBytes"), (int64)FinalUnknownCompressedBytes);
						Writer->WriteValue(TEXT("ChunkBuildATime"), ChunkingTimer.GetSeconds());
						Writer->WriteValue(TEXT("ScanBuildBTime"), ScanningTimer.GetSeconds());
						Writer->WriteValue(TEXT("TotalProcessTime"), ProcessTimer.GetSeconds());
					}
					Writer->WriteObjectEnd();
					Writer->Close();
					if (!FFileHelper::SaveStringToFile(JsonOutput, *TempMetaFilename))
					{
						UE_LOGF(LogChunkDeltaOptimiser, Error, "Could not save output to %ls", *TempMetaFilename);
						bSuccess = false;
					}
				}
			}
		}

		bShouldRun = false;
		return FinalStatLogs;
	}

	void FChunkDeltaOptimiser::BroadcastDelta(const FString& FullFilePath, const FMD5Hash& MD5Hash, const FSHAHash& SHA1Hash)
	{
		AsyncHelpers::ExecuteOnGameThread<void>([&]() { OnDeltaFileWritten().Broadcast(FullFilePath, MD5Hash, SHA1Hash); }).Wait();
	}

	TFuture<FChunkDeltaOptimiser::FFetchResult> FChunkDeltaOptimiser::FetchManifest(const FString& ManifestUri)
	{
		TSharedRef<TPromise<FFetchResult>, ESPMode::ThreadSafe> Promise(new TPromise<FFetchResult>());
		FDownloadCompleteDelegate CompleteDelegate = FDownloadCompleteDelegate::CreateRaw(this, &FChunkDeltaOptimiser::HandleDownloadComplete, Promise);
		DownloadService->RequestFile(ManifestUri, CompleteDelegate, DownloadProgressDelegate);
		return Promise->GetFuture();
	}

	void FChunkDeltaOptimiser::HandleDownloadComplete(int32 RequestId, const FDownloadRef& Download, TSharedRef<TPromise<FFetchResult>, ESPMode::ThreadSafe> Promise)
	{
		if (Download->ResponseSuccessful())
		{
			Async(EAsyncExecution::ThreadPool, [Download, Promise]()
			{
				FBuildPatchAppManifestPtr Manifest = MakeShareable(new FBuildPatchAppManifest());
				if (Manifest->DeserializeFromData(Download->GetData()))
				{
					Promise->SetValue(FFetchResult{ Manifest, Download->GetData().Num(), EFetchResult::Success });
				}
				else
				{
					Promise->SetValue(FFetchResult{ nullptr, Download->GetData().Num(), EFetchResult::CouldNotDeserialize });
				}
			});
		}
		else
		{
			Promise->SetValue(FFetchResult{ nullptr, INDEX_NONE, EFetchResult::DoesNotExist });
		}
	}

	FBlockStructure FChunkDeltaOptimiser::GetDesiredBytes(const FBuildPatchAppManifestPtr& Manifest, const TSet<FGuid>& UnknownChunks)
	{
		uint64 UnknownCount = 0;
		FBlockStructure DesiredBytes;
		uint64 ChunkPartCount = 0;
		for (const FString& BuildFile : ListHelpers::GetFileList(*Manifest.Get()))
		{
			const FFileManifest* FileManifest = Manifest->GetFileManifest(BuildFile);
			for (const FChunkPart& ChunkPart : FileManifest->ChunkParts)
			{
				if (UnknownChunks.Contains(ChunkPart.Guid))
				{
					DesiredBytes.Add(ChunkPartCount, ChunkPart.Size, ESearchDir::FromEnd);
					UnknownCount += ChunkPart.Size;
				}
				ChunkPartCount += ChunkPart.Size;
			}
		}
		check(UnknownCount == BlockStructureHelpers::CountSize(DesiredBytes));
		return DesiredBytes;
	}

	IChunkDeltaOptimiser* FChunkDeltaOptimiserFactory::Create(FChunkDeltaOptimiserConfiguration Configuration)
	{
		return new FChunkDeltaOptimiser(MoveTemp(Configuration));
	}
}
