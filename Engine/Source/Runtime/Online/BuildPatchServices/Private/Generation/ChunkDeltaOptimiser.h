// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Interfaces/IChunkDeltaOptimiser.h"
#include "BuildPatchSettings.h"

namespace BuildPatchServices
{
	class FChunkDeltaOptimiserFactory
	{
	public:
		static IChunkDeltaOptimiser* Create(FChunkDeltaOptimiserConfiguration Configuration);
	};
}
