// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/ChunkHarvester.h"

#include "Async/Async.h"
#include "Async/Future.h"
#include "BuildPatchHash.h"
#include "BuildPatchUtil.h"
#include "Common/Crypto.h"
#include "Common/FileSystem.h"
#include "Common/StatsCollector.h"
#include "Containers/Ticker.h"
#include "Core/AsyncHelpers.h"
#include "Generation/ChunkWriter.h"
#include "HAL/ThreadSafeBool.h"
#include "Installer/ChunkEvictionPolicy.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/InstallChunkSource.h"
#include "Installer/InstallerError.h"
#include "Installer/MemoryChunkStore.h"
#include "Installer/Statistics/CloudChunkSourceStat.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceRedirector.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChunkHarvester, Log, All);
DEFINE_LOG_CATEGORY(LogChunkHarvester);

namespace BuildPatchServices
{
	class FNoMemoryChunkStoreStat
		: public IMemoryChunkStoreStat
	{
	public:
		FNoMemoryChunkStoreStat() { }
		~FNoMemoryChunkStoreStat() { }

		// IMemoryChunkStoreStat interface begin.
		virtual void OnChunkStored(const FGuid& ChunkId) override { }
		virtual void OnChunkReleased(const FGuid& ChunkId) override { }
		virtual void OnChunkBooted(const FGuid& ChunkId) override { }
		virtual void OnStoreUseUpdated(int32 ChunkCount) override { }
		virtual void OnStoreSizeUpdated(int32 Size) override { }
		// IMemoryChunkStoreStat interface end.
	};

	class FNoInstallChunkSourceStat
		: public IInstallChunkSourceStat
	{
	public:
		FNoInstallChunkSourceStat() { }
		~FNoInstallChunkSourceStat() { }

		// IInstallChunkSourceStat interface begin.
		virtual void OnBatchStarted(const TArray<FGuid>& ChunkIds) override { }
		virtual void OnLoadStarted(const FGuid& ChunkId) override { }
		virtual void OnLoadComplete(const FGuid& ChunkId, const ELoadResult& Result, const ISpeedRecorder::FRecord& Record) override { }
		virtual void OnAcceptedNewRequirements(const TSet<FGuid>& ChunkIds) override { }
	};

	class FChunkWriterEventsHandler
	{
	public:
		FChunkWriterEventsHandler(IParallelChunkWriter& InChunkWriter)
			: ChunkWriter(InChunkWriter)
		{
			RegisterTicker();
		}

		~FChunkWriterEventsHandler()
		{
			RemoveTicker();
		}

		void RegisterTicker()
		{
			if (!ChunkWriterPumpHandle.IsValid())
			{
				AsyncHelpers::ExecuteOnGameThread<void>([this]()
					{
						FTickerDelegate Delegate = FTickerDelegate::CreateLambda([&](float) { ChunkWriter.PumpEvents(); return true; });
						ChunkWriterPumpHandle = FTSTicker::GetCoreTicker().AddTicker(MoveTemp(Delegate));
					}).Wait();
			}
		}

		void RemoveTicker()
		{
			if (ChunkWriterPumpHandle.IsValid())
			{
				AsyncHelpers::ExecuteOnGameThread<void>([this]()
					{
						FTSTicker::GetCoreTicker().RemoveTicker(ChunkWriterPumpHandle);
						ChunkWriter.PumpEvents();
						ChunkWriterPumpHandle.Reset();
					}).Wait();
			}
		}

	private:
		FTSTicker::FDelegateHandle ChunkWriterPumpHandle;
		IParallelChunkWriter& ChunkWriter;
	};

	class FChunkHarvester
		: public IChunkHarvester
	{
	public:

		FChunkHarvester(FChunkHarvesterConfiguration InConfiguration);
		~FChunkHarvester();

		// IChunkHarvester interface begin.
		virtual FOnChunkFileWritten& OnChunkFileWritten() override;
		virtual FOnManifestLoaded& OnManifestLoaded() override;
		virtual bool Run() override;
		virtual void Abort() override;
		// IChunkHarvester interface end.

	private:
		bool RunAsync();

	private:
		const FChunkHarvesterConfiguration Configuration;
		TUniquePtr<IFileSystem> FileSystem;
		TUniquePtr<ICrypto> Crypto;
		IChunkHarvester::FOnChunkFileWritten ChunkFileWrittenEvent;
		IChunkHarvester::FOnManifestLoaded ManifestLoadedEvent;
		FThreadSafeBool bShouldAbort;
	};

	FChunkHarvester::FChunkHarvester(FChunkHarvesterConfiguration InConfiguration)
		: Configuration(MoveTemp(InConfiguration))
		, FileSystem(FFileSystemFactory::Create())
		, Crypto(FCryptoFactory::Create())
		, bShouldAbort(false)
	{
	}

	FChunkHarvester::~FChunkHarvester()
	{
	}

	IChunkHarvester::FOnChunkFileWritten& FChunkHarvester::OnChunkFileWritten()
	{
		return ChunkFileWrittenEvent;
	}

	IChunkHarvester::FOnManifestLoaded& FChunkHarvester::OnManifestLoaded()
	{
		return ManifestLoadedEvent;
	}

	bool FChunkHarvester::Run()
	{
		check(IsInGameThread());
		// Kick off async task.
		TFuture<bool> Future = Async(EAsyncExecution::Thread, [this]() { return RunAsync(); });

		// Kick of the main thread tick.
		float BatteryFramerate = 30.0f;
		const float FrameTime = 1.0f / 60.0f;
		double DeltaTime = 0.0;
		double LastTime = FPlatformTime::Seconds();
		while (!Future.IsReady())
		{
			// Increment global frame counter once for each app tick.
			GFrameCounter++;

			// Tick our sub-systems
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(DeltaTime);

			GLog->FlushThreadedLogs();

			// Throttle frame rate
			FPlatformProcess::Sleep(FMath::Max<float>(0.0f, FrameTime - (FPlatformTime::Seconds() - LastTime)));

			// Calculate deltas
			const double AppTime = FPlatformTime::Seconds();
			DeltaTime = AppTime - LastTime;
			LastTime = AppTime;
		}

		return Future.Get();
	}

	void FChunkHarvester::Abort()
	{
		bShouldAbort = true;
	}

	bool FChunkHarvester::RunAsync()
	{
		// Load the manifest
		TUniquePtr<FArchive> File = FileSystem->CreateFileReader(*Configuration.ManifestFilePath);
		if (File.IsValid())
		{
			FBuildPatchAppManifestRef Manifest = MakeShareable(new FBuildPatchAppManifest());
			TArray<uint8> FileData;
			File->Seek(0);
			FileData.AddUninitialized(File->TotalSize());
			File->Serialize(FileData.GetData(), File->TotalSize());
			if (File->IsError() || !Manifest->DeserializeFromData(FileData))
			{
				UE_LOGF(LogChunkHarvester, Error, "Failed to load manifest file %ls.", *Configuration.ManifestFilePath);
				return false;
			}
			AsyncHelpers::ExecuteOnGameThread<void>([&]()
				{
					OnManifestLoaded().Broadcast(Manifest.Get());
				}).Wait();

			// Build necessary configuration.
			EFeatureLevel FeatureLevel = Configuration.FeatureLevelOverride == EFeatureLevel::Invalid ? Manifest->GetFeatureLevel() : Configuration.FeatureLevelOverride;
			FParallelChunkWriterConfig ParallelChunkWriterConfig = { 5, 5, 50, 8, Configuration.bResaveExistingChunks, Configuration.CloudDir, FeatureLevel };

			TMultiMap<FString, FBuildPatchAppManifestRef> InstallationSources;
			InstallationSources.Add(Configuration.BuildRoot, Manifest);

			// Build the systems for pulling chunks from an installation.
			TUniquePtr<FStatsCollector> StatsCollector(FStatsCollectorFactory::Create());
			TUniquePtr<IInstallChunkSourceStat> InstallChunkSourceStat(new FNoInstallChunkSourceStat());
			FChunkDataSerializationConfig ChunkDataSerializationConfig;
			// UE5 MERGE TODO: ChunkDataSerializationConfig.CloudChunkSourceStat = CloudChunkSourceStat.Get();
			TUniquePtr<IChunkDataSerialization> ChunkDataSerialization(FChunkDataSerializationFactory::Create(FileSystem.Get(), Crypto.Get(), ChunkDataSerializationConfig));

			TUniquePtr<IParallelChunkWriter> ChunkWriter(FParallelChunkWriterFactory::Create(ParallelChunkWriterConfig, FileSystem.Get(), ChunkDataSerialization.Get(), StatsCollector.Get()));
			FChunkWriterEventsHandler ChunkWriterEventsHandler(*ChunkWriter.Get());

			TUniquePtr<IChunkReferenceTracker> ChunkReferenceTracker(FChunkReferenceTrackerFactory::Create(
				CustomChunkReferencesHelpers::OrderedUniqueReferences(Manifest)));

			TArray<FGuid> OrderedUseList;
			ChunkReferenceTracker->CopyOutOrderedUseList(OrderedUseList);

			TUniquePtr<IConstructorInstallChunkSource> InstallChunkSource(IConstructorInstallChunkSource::CreateInstallSource(
				FileSystem.Get(),
				InstallChunkSourceStat.Get(),
				InstallationSources,
				TSet<FGuid>(OrderedUseList),
				true,
				{}));


			ChunkWriter->OnChunkFileWritten().AddLambda([this](const FString& FullFilePath, const FMD5Hash& MD5Hash, const FSHAHash&)
				{
					OnChunkFileWritten().Broadcast(FullFilePath, MD5Hash);
				});

			UE_LOGF(LogChunkHarvester, Display, "Harvesting %ls chunks:", *FText::AsNumber(OrderedUseList.Num()).ToString());

			// Loop through and save all chunk files.
			bool bSuccess = true;
			uint64 DataProcessed = 0;
			uint64 DataSkipped = 0;
			constexpr int32 SecondsBetweenLogs = 10;
			double StartTime = FPlatformTime::Seconds();
			double LastLog = StartTime - (SecondsBetweenLogs - 1); // Make it so the first log shows up after 1 second so we have some data, but the rest are normal interval
			int32 ChunkIndex = 0;
			for (const FGuid& ReferencedChunk : OrderedUseList)
			{
				double CurrentTime = FPlatformTime::Seconds();
				if (CurrentTime - LastLog > SecondsBetweenLogs)
				{
					LastLog = CurrentTime;
					int32 MiBPerSecond = (int32)((DataProcessed) / ((1024 * 1024) * (CurrentTime - StartTime)));
					
					UE_LOGF(LogChunkHarvester, Display, "%.1f%%: DataProcessed: %ls DataSkipped: %ls Processing Rate: %ls MiB/s", \
						100.0f * ChunkIndex / (float)OrderedUseList.Num(),
						*FText::AsNumber(DataProcessed).ToString(), 
						*FText::AsNumber(DataSkipped).ToString(),
						*FText::AsNumber(MiBPerSecond).ToString()
						);
				}
				ChunkIndex++;

				// Check skipping this chunk.
				if (!Configuration.bResaveExistingChunks)
				{
					const FChunkInfo* ChunkInfo = Manifest->GetChunkInfo(ReferencedChunk);
					if (ChunkInfo != nullptr)
					{
						const FString NewChunkFilename = Configuration.CloudDir / FBuildPatchUtils::GetChunkNewFilename(FeatureLevel, *ChunkInfo);
						if (FileSystem->FileExists(*NewChunkFilename))
						{
							if (!ChunkReferenceTracker->PopReference(ReferencedChunk))
							{
								// Tracking issue is fatal, so break.
								UE_LOGF(LogChunkHarvester, Error, "There was an unexpected system error with chunk data tracking.");
								bSuccess = false;
								break;
							}
							DataSkipped += Manifest->GetChunkInfo(ReferencedChunk)->DataSizeUncompressed;
							continue;
						}
					}
				}

				TArray<uint8> ChunkBuffer;
				ChunkBuffer.SetNumUninitialized(Manifest->GetChunkInfo(ReferencedChunk)->DataSizeUncompressed);
				
				UE::FMutex DoneMutex;
				DoneMutex.Lock();

				bool bChunkSuccess = false;

				IConstructorChunkSource::FRequestProcessFn FetchChunkFn = InstallChunkSource->CreateRequest(ReferencedChunk, FMutableMemoryView((void*)ChunkBuffer.GetData(), ChunkBuffer.Num()), nullptr, 
					IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateLambda([&DoneMutex, &bChunkSuccess](const FGuid& DataId, bool bAborted, bool bFailedToRead, void* UserPtr)
					{
						bChunkSuccess = !bAborted && !bFailedToRead;
						DoneMutex.Unlock();
					}));

				// Kick the async request
				FetchChunkFn(false);

				// Here we wait for the read to complete.
				DoneMutex.Lock();

				if (!bChunkSuccess)
				{
					bSuccess = false;
					UE_LOGF(LogChunkHarvester, Error, "There was an error loading data.");
				}
				else
				{
					// The install source verifies the chunk, but doesn't give us the hashes we need to make
					// the filename so unfortunately we need to do that work again.
					FSHAHash ShaHash;
					uint64 RollingHash;
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(ChunkHarvest_Rehash);
						
						FSHA1::HashBuffer(ChunkBuffer.GetData(), ChunkBuffer.Num(), ShaHash.Hash);
						RollingHash = FRollingHash::GetHashForDataSet(ChunkBuffer.GetData(), ChunkBuffer.Num());
					}

					DataProcessed += ChunkBuffer.Num();
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(ChunkHarvest_QueueChunkData);
						ChunkWriter->AddChunkData(MoveTemp(ChunkBuffer), ReferencedChunk, RollingHash, ShaHash);
					}

					if (!ChunkReferenceTracker->PopReference(ReferencedChunk))
					{
						// Tracking issue is fatal, so break.
						UE_LOGF(LogChunkHarvester, Error, "There was an unexpected system error with chunk data tracking.");
						bSuccess = false;
						break;
					}
				}
			}
			ChunkWriter->OnProcessComplete();
			return bSuccess;
		}
		return false;
	}

	IChunkHarvester* FChunkHarvesterFactory::Create(FChunkHarvesterConfiguration Configuration)
	{
		return new FChunkHarvester(MoveTemp(Configuration));
	}
}
