// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Interfaces/IChunkHarvester.h"
#include "BuildPatchSettings.h"

namespace BuildPatchServices
{
	class FChunkHarvesterFactory
	{
	public:
		static IChunkHarvester* Create(FChunkHarvesterConfiguration Configuration);
	};
}
