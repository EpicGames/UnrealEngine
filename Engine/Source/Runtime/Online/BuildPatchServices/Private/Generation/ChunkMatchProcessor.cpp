// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/ChunkMatchProcessor.h"
#include "Templates/Tuple.h"
#include "Containers/Array.h"
#include "Algo/Sort.h"
#include "Core/BlockStructure.h"
#include "Generation/DataScanner.h"

namespace BuildPatchServices
{
	struct FChunkMatchProcessor
		: public IChunkMatchProcessor
	{
		typedef TArray<FMatchEntry> FMatchQueue;
	public:
		FChunkMatchProcessor()
		{
		}

		~FChunkMatchProcessor()
		{
		}

		// IChunkMatchProcessor interface begin.
		virtual void ProcessMatch(const int32 Layer, const FChunkMatch& InNewMatch, FBlockStructure InNewBuildSpace) override
		{
			InitLayer(Layer);
			FMatchQueue& Matches = LayerMatches[Layer];
			FMatchQueue& Rejects = LayerRejects[Layer];

			// When processing a piece that is added earlier, we have to re-evaluate existing rejected pieces.
			TArray<FMatchEntry> PiecesToProcess;
			PiecesToProcess.Add(FMatchEntry{InNewMatch, MoveTemp(InNewBuildSpace)});
			while (PiecesToProcess.Num() > 0)
			{
				// Some variables to help make code easy to read.
				const FChunkMatch& NewMatch = PiecesToProcess[0].ChunkMatch;
				const uint64 NewMatchFirst = NewMatch.DataOffset;
				const uint64 NewMatchSize = NewMatch.WindowSize;
				const uint64 NewMatchLast = (NewMatchFirst + NewMatchSize) - 1;

				if (Matches.Num() > 0)
				{
					// Find search location
					const int32 SearchIdx = FindSearchStartIdx(Matches, NewMatch);

					// Search idx is now the first index of a match that we could clobber.
					bool bAccepted = false;
					bool bRejected = false;
					for (int32 MatchesIdx = SearchIdx; MatchesIdx < Matches.Num(); ++MatchesIdx)
					{
						const FChunkMatch& Match = Matches[MatchesIdx].ChunkMatch;

						// Some variables to help make code easy to read.
						const uint64 ThisMatchFirst = Match.DataOffset;
						const uint64 ThisMatchSize = Match.WindowSize;
						const uint64 ThisMatchLast = (ThisMatchFirst + ThisMatchSize) - 1;

						// If NewMatch comes entirely after ThisMatch, we continue.
						if (NewMatchFirst > ThisMatchLast)
						{
							continue;
						}
						// If we overlap
						if (NewMatchFirst <= ThisMatchLast && NewMatchLast >= ThisMatchFirst)
						{
							// We always accept NewMatch if it is larger, or if the same size and earlier.
							// We never accept if smaller.
							bAccepted = NewMatchSize > ThisMatchSize || (NewMatchSize == ThisMatchSize && NewMatchFirst < ThisMatchFirst);
							if (bAccepted)
							{
								// We pop this match and all the rest on to re-evaluate them.
								FMatchEntry NewMatchEntry = MoveTemp(PiecesToProcess[0]);
								PiecesToProcess[0] = MoveTemp(Matches[MatchesIdx]);
								Matches[MatchesIdx] = MoveTemp(NewMatchEntry);
								const int32 NewMatchesArraySize = MatchesIdx + 1;
								for (int32 MatchIdxToPop = NewMatchesArraySize; MatchIdxToPop < Matches.Num(); ++MatchIdxToPop)
								{
									PiecesToProcess.Add(MoveTemp(Matches[MatchIdxToPop]));
								}
								checkf(NewMatchesArraySize <= Matches.Num(), TEXT("Matches array should never grow uninitialised."));
								Matches.SetNumUninitialized(NewMatchesArraySize, EAllowShrinking::No);
								// Since we adjusted the history, we need to replay the rejected matches too.
								PiecesToProcess.Append(MoveTemp(Rejects));
							}
							else
							{
								bRejected = true;
								Rejects.Add(MoveTemp(PiecesToProcess[0]));
								PiecesToProcess.RemoveAt(0);
							}
							// Break from match loop.
							break;
						}
						else if (NewMatchLast < ThisMatchFirst)
						{
							// We just slipped into an empty gap.
							Matches.Insert(MoveTemp(PiecesToProcess[0]), MatchesIdx);
							PiecesToProcess.RemoveAt(0);
							bAccepted = true;
							// Break from match loop.
							break;
						}
						// Unreachable code.
						checkf(false, TEXT("NewMatchFirst:%llu NewMatchSize:%llu NewMatchLast:%llu ThisMatchFirst:%llu ThisMatchSize:%llu ThisMatchLast:%llu"), NewMatchFirst, NewMatchSize, NewMatchLast, ThisMatchFirst, ThisMatchSize, ThisMatchLast);
					}
					// If we made no decision, it means NewMatch comes after all current matches.
					if (!bAccepted && !bRejected)
					{
						Matches.Add(MoveTemp(PiecesToProcess[0]));
						PiecesToProcess.RemoveAt(0);
					}
				}
				else
				{
					Matches.Add(MoveTemp(PiecesToProcess[0]));
					PiecesToProcess.RemoveAt(0);
				}
				Algo::SortBy(PiecesToProcess, [](const FMatchEntry& MatchEntry) { return MatchEntry.ChunkMatch.DataOffset; });
			}
		}

		virtual void FlushLayer(const int32 Layer, const uint64 SafeByteSize)
		{
			check(SafeByteSize > 0);
			InitLayer(Layer);
			const FMatchQueue& Matches = LayerMatches[Layer];
			FMatchQueue& Rejects = LayerRejects[Layer];
			int32& FlushedMatch = LayerFlushedMatch[Layer];
			uint64& FlushedSize = LayerFlushedSize[Layer];
			const uint64 PrevFlushedSize = FlushedSize;
			// Mark flushed size for when there are no matches within the range.
			FlushedSize = SafeByteSize;
			check(FlushedSize >= PrevFlushedSize);

			// Count flushed matches.
			for (int32 MatchesIdx = FlushedMatch + 1; MatchesIdx < Matches.Num(); ++MatchesIdx)
			{
				// Some variables to help make code easy to read.
				const FChunkMatch& Match = Matches[MatchesIdx].ChunkMatch;
				const FBlockRange ThisRange = FBlockRange::FromFirstAndSize(Match.DataOffset, Match.WindowSize);

				// Stop when we skipped over the safe byte. (Last safe byte is SafeByteSize-1, which we can include).
				if (ThisRange.GetFirst() >= SafeByteSize)
				{
					// Then we skipped right over the safe byte.
					break;
				}

				// Stop if this match is passing over the last safe byte.
				if (ThisRange.GetLast() >= SafeByteSize)
				{
					// Mark flushed size as this first byte which we cannot flush past.
					FlushedSize = ThisRange.GetFirst();
					check(FlushedSize >= PrevFlushedSize);
					break;
				}

				// We can take this match!
				FlushedMatch = MatchesIdx;
			}

			// Discard rejects no longer needed.
			Algo::SortBy(Rejects, [](const FMatchEntry& MatchEntry) { return MatchEntry.ChunkMatch.DataOffset; });
			int32 RemoveCount = INDEX_NONE;
			for (int32 RejectsIdx = 0; RejectsIdx < Rejects.Num(); ++RejectsIdx)
			{
				const FChunkMatch& Reject = Rejects[RejectsIdx].ChunkMatch;
				const FBlockRange ThisRange = FBlockRange::FromFirstAndSize(Reject.DataOffset, Reject.WindowSize);

				// We can keep rejects that are still entirely past the flushed size.
				if (ThisRange.GetFirst() >= FlushedSize)
				{
					break;
				}

				// We can discard this reject!
				RemoveCount = RejectsIdx;
			}
			Rejects.RemoveAt(0, RemoveCount + 1, EAllowShrinking::No);
		}

		virtual FBlockRange CollectLayer(const int32 Layer, TArray<FMatchEntry>& OutData)
		{
			InitLayer(Layer);
			FMatchQueue& Matches = LayerMatches[Layer];
			int32& FlushedMatch = LayerFlushedMatch[Layer];
			uint64& CollectedSize = LayerCollectedSize[Layer];
			const uint64& FlushedSize = LayerFlushedSize[Layer];
			check(FlushedSize >= CollectedSize);
			const FBlockRange ReturnedRange = FBlockRange::FromFirstAndSize(CollectedSize, FlushedSize - CollectedSize);

			// Copy out flushed matches.
			OutData.Reserve(FlushedMatch + 1);
			for (int32 MatchesIdx = 0; MatchesIdx <= FlushedMatch; ++MatchesIdx)
			{
				FMatchEntry& MatchToMove = Matches[MatchesIdx];
				OutData.Add(FMatchEntry{MatchToMove.ChunkMatch, MoveTemp(MatchToMove.BlockStructure)});
			}

			// Remove copied out matches.
			Matches.RemoveAt(0, FlushedMatch + 1);
			FlushedMatch = INDEX_NONE;

			// Update tracking.
			CollectedSize = FlushedSize;

			return ReturnedRange;
		}
		// IChunkMatchProcessor interface end.

	private:
		void InitLayer(const int32 Layer)
		{
			if (!LayerMatches.Contains(Layer))
			{
				LayerMatches.Add(Layer, FMatchQueue());
			}
			if (!LayerRejects.Contains(Layer))
			{
				LayerRejects.Add(Layer, FMatchQueue());
			}
			if (!LayerFlushedMatch.Contains(Layer))
			{
				LayerFlushedMatch.Add(Layer, -1);
			}
			if (!LayerFlushedSize.Contains(Layer))
			{
				LayerFlushedSize.Add(Layer, 0);
			}
			if (!LayerCollectedSize.Contains(Layer))
			{
				LayerCollectedSize.Add(Layer, 0);
			}
		}

		int32 FindSearchStartIdx(const FMatchQueue& MatchQueue, const FChunkMatch& ChunkMatch)
		{
			const int32 LastMatchIdx = MatchQueue.Num() - 1;
			int32 SearchIdx = LastMatchIdx;
			if (LastMatchIdx >= 0)
			{
				// Find search location
				while (MatchQueue[SearchIdx].ChunkMatch.DataOffset > ChunkMatch.DataOffset)
				{
					--SearchIdx;
					if (SearchIdx < 0)
					{
						SearchIdx = 0;
						break;
					}
				}
			}
			return SearchIdx;
		}

	private:
		TMap<int32, FMatchQueue> LayerMatches;
		TMap<int32, FMatchQueue> LayerRejects;
		TMap<int32, int32> LayerFlushedMatch;
		TMap<int32, uint64> LayerFlushedSize;
		TMap<int32, uint64> LayerCollectedSize;
	};

	IChunkMatchProcessor* FChunkMatchProcessorFactory::Create()
	{
		return new FChunkMatchProcessor();
	}
}
