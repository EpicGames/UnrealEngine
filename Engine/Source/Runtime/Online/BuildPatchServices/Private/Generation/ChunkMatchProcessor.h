// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/Tuple.h"
#include "Core/BlockRange.h"
#include "Core/BlockStructure.h"
#include "Generation/DataScanner.h"

namespace BuildPatchServices
{
	struct FMatchEntry
	{
		FChunkMatch ChunkMatch;
		FBlockStructure BlockStructure;
	};

	class IChunkMatchProcessor
	{
	public:
		virtual ~IChunkMatchProcessor() = default;
		virtual void ProcessMatch(const int32 Layer, const FChunkMatch& Match, FBlockStructure BuildSpace) = 0;
		virtual void FlushLayer(const int32 Layer, const uint64 UpToByteOffset) = 0;
		virtual FBlockRange CollectLayer(const int32 Layer, TArray<FMatchEntry>& OutData) = 0;
	};

	class FChunkMatchProcessorFactory
	{
	public:
		static IChunkMatchProcessor* Create();
	};
}
