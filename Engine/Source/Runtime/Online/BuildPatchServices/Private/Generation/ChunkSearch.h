// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Core/BlockRange.h"
#include "Core/BlockStructure.h"
#include "CoreMinimal.h"
#include "Containers/List.h"

#include "BuildPatchManifest.h"
#include "BuildPatchSettings.h"

namespace BuildPatchServices
{
	namespace ListHelpers
	{
		template <typename ElementType>
		FORCEINLINE void Copy(const TDoubleLinkedList<ElementType>& CopyFrom, TDoubleLinkedList<ElementType>& CopyTo)
		{
			for (const ElementType& Elem : CopyFrom)
			{
				CopyTo.AddTail(Elem);
			}
		}

		template <typename ListType, typename NodeType, typename ElementType>
		FORCEINLINE void InsertBefore(const ElementType& NewNode, ListType& List, NodeType* Node)
		{
			List.InsertNode(NewNode, Node);
		}

		template <typename ListType, typename NodeType, typename ElementType>
		FORCEINLINE void InsertAfter(const ElementType& NewNode, ListType& List, NodeType* Node)
		{
			NodeType* NextNode = Node->GetNextNode();
			if (NextNode)
			{
				List.InsertNode(NewNode, NextNode);
			}
			else
			{
				List.AddTail(NewNode);
			}
		}

		FORCEINLINE TArray<FString> GetFileList(const FBuildPatchAppManifest& Manifest)
		{
			TArray<FString> BuildFiles;
			Manifest.GetFileList(BuildFiles);
			return BuildFiles;
		}

		FORCEINLINE void ForEach(const FBuildPatchAppManifest& Manifest, const TFunction<void(const FFileManifest&)>& Func)
		{
			for (const FString& ManifestFile : GetFileList(Manifest))
			{
				Func(*Manifest.GetFileManifest(ManifestFile));
			}
		}

		FORCEINLINE void ForEach(const FFileManifestList& FileManifestList, const TFunction<void(const FFileManifest&)>& Func)
		{
			for (const FFileManifest& FileManifest : FileManifestList.FileList)
			{
				Func(FileManifest);
			}
		}
	}

	class FChunkSearcher
	{
	public:
		struct FChunkNode
		{
		public:
			FChunkNode(const FChunkPart& InChunkPart, const FBlockRange& InBuildRange)
				: BuildRange(InBuildRange)
				, ChunkPart(InChunkPart)
			{
			}
			FChunkNode(const FChunkNode& CopyFrom)
				: BuildRange(CopyFrom.BuildRange)
				, ChunkPart(CopyFrom.ChunkPart)
			{
			}

		public:
			FBlockRange BuildRange;
			FChunkPart ChunkPart;
		};
		typedef TDoubleLinkedList<FChunkNode> FChunkDList;
		typedef FChunkDList::TDoubleLinkedListNode FChunkDListNode;

		struct FFileNode
		{
		public:
			FFileNode(const FFileManifest *const InManifest, const FBlockRange& InBuildRange)
				: Manifest(InManifest)
				, BuildRange(InBuildRange)
			{
			}
			FFileNode(const FFileNode& CopyFrom)
				: Manifest(CopyFrom.Manifest)
				, BuildRange(CopyFrom.BuildRange)
			{
				ListHelpers::Copy(CopyFrom.ChunkParts, ChunkParts);
			}

		public:
			const FFileManifest *const Manifest;
			const FBlockRange BuildRange;
			FChunkDList ChunkParts;
		};
		typedef TDoubleLinkedList<FFileNode> FFileDList;
		typedef FFileDList::TDoubleLinkedListNode FFileDListNode;

		template<typename InitType>
		FChunkSearcher(const InitType& InitClass)
		{
			uint64 LocationCount = 0;
			ListHelpers::ForEach(InitClass, [&](const FFileManifest& FileManifest)
			{
				FFileNode FileNode(&FileManifest, FBlockRange::FromFirstAndSize(LocationCount, FileManifest.FileSize));
				for (const FChunkPart& ChunkPart : FileNode.Manifest->ChunkParts)
				{
					FChunkNode ChunkNode(ChunkPart, FBlockRange::FromFirstAndSize(LocationCount, ChunkPart.Size));
					FileNode.ChunkParts.AddTail(ChunkNode);
					LocationCount += ChunkPart.Size;
				}
				FileLinkedList.AddTail(FileNode);
			});
			SetStart();
		}

		void ForEachOverlap(const FBlockStructure& BlockStructure, const TFunction<void(const FBlockRange&, FFileDListNode*, FChunkDListNode*)>& Handler)
		{
			const FBlockEntry* Block = BlockStructure.GetHead();
			while (Block)
			{
				ForEachOverlap(Block->AsRange(), Handler);
				Block = Block->GetNext();
			}
		}

		void ForEachOverlap(const FBlockRange& BlockRange, const TFunction<void(const FBlockRange&, FFileDListNode*, FChunkDListNode*)>& Handler)
		{
			if (!CurrFile || !CurrChunk)
			{
				SetStart();
			}

			// Find file to start on.
			for (FFileDListNode* StartFile = FindFirst(&FileLinkedList, CurrFile, BlockRange); CurrFile != StartFile;)
			{
				CurrFile = StartFile;
				CurrChunk = CurrFile->GetValue().ChunkParts.GetHead();
			}
			check(CurrFile->GetValue().BuildRange.Overlaps(BlockRange));

			// Find chunk to start on.
			CurrChunk = FindFirst(&CurrFile->GetValue().ChunkParts, CurrChunk, BlockRange);
			check(CurrChunk->GetValue().BuildRange.Overlaps(BlockRange));

			// Will be searching forwards only.
			while (CurrFile && CurrChunk)
			{
				if (CurrFile->GetValue().BuildRange.Overlaps(BlockRange))
				{
					if (CurrChunk->GetValue().BuildRange.Overlaps(BlockRange))
					{
						FChunkDListNode* NextChunk = CurrChunk->GetNextNode();
						const FBlockRange OverlapRange = FBlockRange::FromIntersection(CurrChunk->GetValue().BuildRange, BlockRange);
						Handler(OverlapRange, CurrFile, CurrChunk);
						// Find next chunk ptr.
						CurrChunk = NextChunk;
						while (!CurrChunk && CurrFile)
						{
							CurrFile = CurrFile->GetNextNode();
							if (CurrFile)
							{
								CurrChunk = CurrFile->GetValue().ChunkParts.GetHead();
							}
						}
					}
					// First non-overlap we hit we are finished.
					else
					{
						return;
					}
				}
				// First non-overlap we hit we are finished.
				else
				{
					return;
				}
			}
		}

		FFileDListNode* GetHead()
		{
			return FileLinkedList.GetHead();
		}

		FFileManifestList BuildNewFileManifestList()
		{
			FFileManifestList NewFileManifestList;
			NewFileManifestList.FileList.Reserve(FileLinkedList.Num());
			for (const FFileNode& FileNode : FileLinkedList)
			{
				FFileManifest& FileManifest = NewFileManifestList.FileList.Add_GetRef(*FileNode.Manifest);
				FileManifest.ChunkParts.Empty(FileNode.ChunkParts.Num());
				for (const FChunkNode& ChunkNode : FileNode.ChunkParts)
				{
					FileManifest.ChunkParts.Add(ChunkNode.ChunkPart);
				}
			}
			return NewFileManifestList;
		}

	private:
		void SetStart()
		{
			CurrFile = FileLinkedList.GetHead();
			CurrChunk = CurrFile->GetValue().ChunkParts.GetHead();
		}

		template<typename ListType, typename ListNodeType>
		ListNodeType* FindFirst(ListType* List, ListNodeType* Current, const FBlockRange& BlockRange)
		{
			// If we need to search backward.
			if (BlockRange.GetLast() < Current->GetValue().BuildRange.GetFirst())
			{
				while (Current && !Current->GetValue().BuildRange.Overlaps(BlockRange))
				{
					Current = Current->GetPrevNode();
				}
			}
			// Else we need to search forward.
			else
			{
				while (Current && !Current->GetValue().BuildRange.Overlaps(BlockRange))
				{
					Current = Current->GetNextNode();
				}
			}
			// Now we are overlapping, we continue reverse until we find one that doesn't, ignoring empty files too.
			while (Current && (Current->GetValue().BuildRange.Overlaps(BlockRange) || Current->GetValue().BuildRange.GetSize() == 0))
			{
				Current = Current->GetPrevNode();
			}
			// If we ended with null, then head is the first.
			if (!Current)
			{
				Current = List->GetHead();
			}
			// Otherwise, we then go back to the first that overlaps, which also skips empties.
			else
			{
				while (Current && !Current->GetValue().BuildRange.Overlaps(BlockRange))
				{
					Current = Current->GetNextNode();
				}
			}
			return Current;
		}

	private:
		FFileDList FileLinkedList;
		FFileDListNode* CurrFile;
		FChunkDListNode* CurrChunk;
	};
}
