// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/ChunkWriter.h"

#include "Async/Async.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "HAL/ThreadSafeBool.h"
#include "Logging/LogMacros.h"
#include "Misc/Base64.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/SecureHash.h"

#include "Common/Crypto.h"
#include "Common/FileSystem.h"
#include "Common/StatsCollector.h"
#include "Core/AsyncHelpers.h"
#include "Data/ChunkData.h"
#include "BuildPatchUtil.h"
#include "BuildPatchServicesModule.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChunkWriter, Log, All);
DEFINE_LOG_CATEGORY(LogChunkWriter);

namespace BuildPatchServices
{
	typedef TTuple<TArray<uint8>, FGuid, uint64, FSHAHash> FChunkDataJob;
	typedef TTuple <FGuid, int64> FChunkOutputSize;
	typedef TTuple <FGuid, uint64> FChunkOutputHash;
	typedef TTuple <FGuid, FSHAHash> FChunkOutputSha;
	typedef TTuple <FGuid, FGuid> FChunkOutputSecretId;
	typedef TTuple <FGuid, FAESAuthTag> FChunkOutputAuthTag;
	typedef TTuple <FString, FMD5Hash, FSHAHash> FChunkFileCompleteInfo;

	class FWriterChunkDataAccess
		: public IChunkDataAccess
	{
	public:
		FWriterChunkDataAccess(TArray<uint8>& InDataRef, FChunkHeader& InHeaderRef)
			: DataRef(InDataRef)
			, HeaderRef(InHeaderRef)
		{
		}
		// IChunkDataAccess interface begin.
		virtual void GetDataLock(const uint8** OutChunkData, const FChunkHeader** OutChunkHeader) const override
		{
			(*OutChunkData) = DataRef.GetData();
			(*OutChunkHeader) = &HeaderRef;
		}
		virtual void GetDataLock(uint8** OutChunkData, FChunkHeader** OutChunkHeader) override
		{
			(*OutChunkData) = DataRef.GetData();
			(*OutChunkHeader) = &HeaderRef;
		}
		virtual void ReleaseDataLock() const
		{
		}
		// IChunkDataAccess interface end.

	private:
		TArray<uint8>& DataRef;
		FChunkHeader& HeaderRef;
	};

	class FParallelChunkWriter
		: public IParallelChunkWriter
	{
	public:
		FParallelChunkWriter(FParallelChunkWriterConfig InConfig, IFileSystem* InFileSystem, IChunkDataSerialization* InChunkDataSerialization, FStatsCollector* InStatsCollector)
			: Config(MoveTemp(InConfig))
			, FileSystem(InFileSystem)
			, ChunkDataSerialization(InChunkDataSerialization)
			, StatsCollector(InStatsCollector)
			, bMoreDataIsExpected(true)
			, bShouldAbort(false)
			, InFlightChunks(0)
		{
			StatUncompressesData = 0;
			StatSerlialiseTime = StatsCollector->CreateStat(TEXT("Chunk Writer: Serialize Time"), EStatFormat::Timer);
			StatChunksSaved = StatsCollector->CreateStat(TEXT("Chunk Writer: Num Saved"), EStatFormat::Value);
			StatDataWritten = StatsCollector->CreateStat(TEXT("Chunk Writer: Data Size Written"), EStatFormat::DataSize);
			StatCompressionRatio = StatsCollector->CreateStat(TEXT("Chunk Writer: Compression Ratio"), EStatFormat::Percentage);
			StatDataWriteSpeed = StatsCollector->CreateStat(TEXT("Chunk Writer: Data Write Speed"), EStatFormat::DataSpeed);

			// In case of network interruption it is needed to recheck again the cloud directory existance
			bool bMakeDirSuccess = false;
			int32 RetryCount = Config.OperationRetryCount;
			while (!bMakeDirSuccess && RetryCount-- >= 0)
			{
				if (FileSystem->MakeDirectory(*Config.ChunkDirectory))
				{
					bMakeDirSuccess = true;
					continue;
				}

				if (RetryCount >= 0)
				{
					UE_LOGF(LogChunkWriter, Verbose, "Retry to create cloud directory (%ls), attempts left (%d).", *Config.ChunkDirectory, RetryCount + 1);
					FPlatformProcess::Sleep(Config.OperationRetryTime);
				}
			}

			if (!bMakeDirSuccess)
			{
				UE_LOGF(LogChunkWriter, Error, "Could not create cloud directory (%ls).", *Config.ChunkDirectory);
				check(bMakeDirSuccess); // Abort the program because cloud directory was not created
			}

			for (int32 ThreadIdx = 0; ThreadIdx < Config.NumberOfThreads; ++ThreadIdx)
			{
				WriterThreads.Add(Async(EAsyncExecution::Thread, [this]()
				{
					WriterThread();
				}));
			}
		}

		~FParallelChunkWriter()
		{
			Abort();
			for (const TFuture<void>& Thread : WriterThreads)
			{
				Thread.Wait();
			}
			WriterThreads.Empty();
		}

		// IParallelChunkWriter interface begin.
		virtual bool AddChunkData(TArray<uint8> ChunkData, const FGuid& ChunkGuid, const uint64& ChunkHash, const FSHAHash& ChunkSha) override
		{
			if (!bShouldAbort)
			{
				DebugCheckSingleProducer();
				// Check for violations of feature level.
				check(Config.FeatureLevel >= EFeatureLevel::VariableSizeChunksWithoutWindowSizeChunkInfo || ChunkData.Num() == LegacyFixedChunkWindow);
				InFlightChunks.Increment();
				// Wait for queue space.
				while (ChunkDataJobQueueCount.GetValue() >= Config.MaxQueueSize)
				{
					if (bShouldAbort)
					{
						InFlightChunks.Decrement();
						return false;
					}
					FPlatformProcess::Sleep(0);
				}
				ChunkDataJobQueueCount.Increment();
				ChunkDataJobQueue.Enqueue(FChunkDataJob(MoveTemp(ChunkData), ChunkGuid, ChunkHash, ChunkSha));
				return true;
			}
			return false;
		}

		virtual int32 GetInFlightChunkCount() override
		{
			return InFlightChunks.GetValue();
		}

		virtual void Abort() override
		{
			bShouldAbort = true;
			bMoreDataIsExpected = false;
		}

		virtual bool HasAborted() const override
		{
			return bShouldAbort;
		}

		virtual FParallelChunkWriterSummaries OnProcessComplete() override
		{
			bMoreDataIsExpected = false;
			for (const TFuture<void>& Thread : WriterThreads)
			{
				Thread.Wait();
			}
			WriterThreads.Empty();
			ParallelChunkWriterSummaries.FeatureLevel = Config.FeatureLevel;
			FChunkOutputSize ChunkOutputSize;
			while (ChunkOutputCompressedSizeQueue.Dequeue(ChunkOutputSize))
			{
				ParallelChunkWriterSummaries.ChunkOutputCompressedSizes.Add(ChunkOutputSize.Get<0>(), ChunkOutputSize.Get<1>());
			}
			while (ChunkOutputFileSizeQueue.Dequeue(ChunkOutputSize))
			{
				ParallelChunkWriterSummaries.ChunkOutputFileSizes.Add(ChunkOutputSize.Get<0>(), ChunkOutputSize.Get<1>());
			}
			FChunkOutputHash ChunkOutputHash;
			while (ChunkOutputHashQueue.Dequeue(ChunkOutputHash))
			{
				ParallelChunkWriterSummaries.ChunkOutputHashes.Add(ChunkOutputHash.Get<0>(), ChunkOutputHash.Get<1>());
			}
			FChunkOutputSha ChunkOutputSha;
			while (ChunkOutputShaQueue.Dequeue(ChunkOutputSha))
			{
				ParallelChunkWriterSummaries.ChunkOutputShas.Add(ChunkOutputSha.Get<0>(), ChunkOutputSha.Get<1>());
			}
			FChunkOutputSecretId ChunkOutputSecretId;
			while (ChunkOutputSecretIdQueue.Dequeue(ChunkOutputSecretId))
			{
				ParallelChunkWriterSummaries.ChunkOutputSecretIds.Add(ChunkOutputSecretId.Get<0>(), ChunkOutputSecretId.Get<1>());
			}
			FChunkOutputAuthTag ChunkOutputAuthTag;
			while (ChunkOutputAuthTagQueue.Dequeue(ChunkOutputAuthTag))
			{
				ParallelChunkWriterSummaries.ChunkOutputAuthTags.Add(ChunkOutputAuthTag.Get<0>(), ChunkOutputAuthTag.Get<1>());
			}
			return ParallelChunkWriterSummaries;
		}

		virtual FOnChunkFileWritten& OnChunkFileWritten() override
		{
			return ChunkFileWrittenEvent;
		}

		virtual FOnChunkFileWriteFailed& OnChunkFileWriteFailed() override
		{
			return ChunkFileWriteFailedEvent;
		}

		virtual void PumpEvents() override
		{
			FChunkFileCompleteInfo CompletionInfo;
			while (ChunkCompleteQueue.Dequeue(CompletionInfo))
			{
				OnChunkFileWritten().Broadcast(CompletionInfo.Get<0>(), CompletionInfo.Get<1>(), CompletionInfo.Get<2>());
				InFlightChunks.Decrement();
			}
			FString FailedFilename;
			while (ChunkFailedQueue.Dequeue(FailedFilename))
			{
				OnChunkFileWriteFailed().Broadcast(FailedFilename);
			}
		}
		// IParallelChunkWriter interface end.

	private:
		void WriterThread()
		{
			TArray<uint8> ChunkFileBytes;
			uint64 ChunkWriteTime;
			bool bReceivedJob;
			FChunkDataJob ChunkDataJob;
			while (!bShouldAbort && (bMoreDataIsExpected || ChunkDataJobQueueCount.GetValue() > 0))
			{
				ChunkDataJobQueueConsumerCS.Lock();
				bReceivedJob = ChunkDataJobQueue.Dequeue(ChunkDataJob);
				ChunkDataJobQueueConsumerCS.Unlock();
				if (bReceivedJob)
				{
					ChunkDataJobQueueCount.Decrement();
					TArray<uint8>& ChunkData = ChunkDataJob.Get<0>();
					FGuid& ChunkGuid = ChunkDataJob.Get<1>();
					uint64& ChunkHash = ChunkDataJob.Get<2>();
					FSHAHash& ChunkSha = ChunkDataJob.Get<3>();
					FChunkHeader ChunkHeader;
					ChunkHeader.Guid = ChunkGuid;
					ChunkHeader.DataSizeCompressed = ChunkData.Num();
					ChunkHeader.DataSizeUncompressed = ChunkData.Num();
					ChunkHeader.StoredAs = EChunkStorageFlags::RawData;
					ChunkHeader.HashType = EChunkHashFlags::RollingPoly64 | EChunkHashFlags::Sha1;
					ChunkHeader.RollingHash = ChunkHash;
					ChunkHeader.SHAHash = ChunkSha;
					const TUniquePtr<FWriterChunkDataAccess> ChunkDataAccess(new FWriterChunkDataAccess(ChunkData, ChunkHeader));
					// We start with a memory writer, the only reason this would fail is fatal, so we don't keep retrying the serialisation.
					// We also need information from the serialisation class. This get put into ChunkHeader, and is then used in chunk filename generation.
					ChunkFileBytes.Reset();
					FMemoryWriter ChunkBytesWriter(ChunkFileBytes);
					FStatsCollector::AccumulateTimeBegin(ChunkWriteTime);
					const EChunkSaveResult ChunkSaveResult = ChunkDataSerialization->SaveToArchive(ChunkBytesWriter, ChunkDataAccess.Get());
					FStatsCollector::AccumulateTimeEnd(StatSerlialiseTime, ChunkWriteTime);
					const int64 ChunkFileSize = ChunkFileBytes.Num();
					bool bChunkSuccess = ChunkBytesWriter.Close() && ChunkSaveResult == EChunkSaveResult::Success;
					if (!bChunkSuccess)
					{
						UE_LOGF(LogChunkWriter, Log, "Could not serialise chunk to memory [%d %ls].", ChunkBytesWriter.IsError(), ToString(ChunkSaveResult));
					}
					const FString NewChunkFilename = Config.ChunkDirectory / FBuildPatchUtils::GetChunkNewFilename(Config.FeatureLevel, ChunkHeader);
					const bool bSaveToFile = bChunkSuccess && (Config.bResaveExistingChunks || !FileSystem->FileExists(*NewChunkFilename));
					if (bSaveToFile)
					{
						bChunkSuccess = false;
						int32 RetryCount = Config.OperationRetryCount;
						while (!bShouldAbort && !bChunkSuccess && RetryCount-- >= 0)
						{
							FStatsCollector::AccumulateTimeBegin(ChunkWriteTime);
							FileSystem->MakeDirectory(*FPaths::GetPath(NewChunkFilename));
							bChunkSuccess = FileSystem->SaveArrayToFile(*NewChunkFilename, ChunkFileBytes) && FileSystem->FileExists(*NewChunkFilename);
							FStatsCollector::AccumulateTimeEnd(StatSerlialiseTime, ChunkWriteTime);
							if (!bChunkSuccess)
							{
								UE_LOGF(LogChunkWriter, Log, "Could not create chunk (%ls). %ls", *NewChunkFilename, RetryCount > 0 ? TEXT("Retrying...") : TEXT("Fatal!"));
								FPlatformProcess::Sleep(Config.OperationRetryTime);
								continue;
							}
							ChunkOutputCompressedSizeQueue.Enqueue(FChunkOutputSize(ChunkGuid, ChunkHeader.DataSizeCompressed));
							ChunkOutputFileSizeQueue.Enqueue(FChunkOutputSize(ChunkGuid, ChunkFileSize));
							if (EnumHasAnyFlags(ChunkHeader.StoredAs, EChunkStorageFlags::Encrypted))
							{
								ChunkOutputSecretIdQueue.Enqueue(FChunkOutputSecretId(ChunkGuid, ChunkHeader.EncryptionSecretId));
								ChunkOutputAuthTagQueue.Enqueue(FChunkOutputAuthTag(ChunkGuid, ChunkHeader.AESAuthTag));
							}
							ChunkOutputHashQueue.Enqueue(FChunkOutputHash(ChunkGuid, ChunkHash));
							ChunkOutputShaQueue.Enqueue(FChunkOutputSha(ChunkGuid, ChunkSha));
							if (OnChunkFileWritten().IsBound())
							{
								FMD5 Md5Gen;
								FMD5Hash MD5Hash;
								FSHAHash SHA1Hash;
								Md5Gen.Update(ChunkFileBytes.GetData(), ChunkFileBytes.Num());
								MD5Hash.Set(Md5Gen);
								FSHA1::HashBuffer(ChunkFileBytes.GetData(), ChunkFileBytes.Num(), SHA1Hash.Hash);
								ChunkCompleteQueue.Enqueue(FChunkFileCompleteInfo(NewChunkFilename, MD5Hash, SHA1Hash));
							}
							else
							{
								InFlightChunks.Decrement();
							}
							FStatsCollector::Accumulate(StatChunksSaved, 1);
							FStatsCollector::Accumulate(StatDataWritten, ChunkFileSize);
							FStatsCollector::Accumulate(&StatUncompressesData, ChunkHeader.HeaderSize + ChunkHeader.DataSizeUncompressed);
							const double DataWrittenD = *StatDataWritten;
							const double UncompressesDataD = StatUncompressesData;
							if (UncompressesDataD > 0)
							{
								FStatsCollector::SetAsPercentage(StatCompressionRatio, DataWrittenD / UncompressesDataD);
							}
							const double SerlialiseTime = FStatsCollector::CyclesToSeconds(*StatSerlialiseTime);
							if (SerlialiseTime > 0)
							{
								FStatsCollector::Set(StatDataWriteSpeed, *StatDataWritten / SerlialiseTime);
							}
						}
					}
					else
					{
						TUniquePtr<FArchive> ChunkFile = FileSystem->CreateFileReader(*NewChunkFilename);
						bChunkSuccess = ChunkFile.IsValid();
						if (bChunkSuccess)
						{
							*ChunkFile << ChunkHeader;
							ChunkOutputCompressedSizeQueue.Enqueue(FChunkOutputSize(ChunkGuid, ChunkHeader.DataSizeCompressed));
							ChunkOutputFileSizeQueue.Enqueue(FChunkOutputSize(ChunkGuid, ChunkFile->TotalSize()));
							ChunkOutputHashQueue.Enqueue(FChunkOutputHash(ChunkGuid, ChunkHeader.RollingHash));
							ChunkOutputShaQueue.Enqueue(FChunkOutputSha(ChunkGuid, ChunkHeader.SHAHash));
							if (EnumHasAnyFlags(ChunkHeader.StoredAs, EChunkStorageFlags::Encrypted))
							{
								ChunkOutputSecretIdQueue.Enqueue(FChunkOutputSecretId(ChunkGuid, ChunkHeader.EncryptionSecretId));
								ChunkOutputAuthTagQueue.Enqueue(FChunkOutputAuthTag(ChunkGuid, ChunkHeader.AESAuthTag));
							}
							InFlightChunks.Decrement();
							UE_LOGF(LogChunkWriter, Log, "Skipped save of existing chunk (%ls).", *NewChunkFilename);
						}
						else
						{
							UE_LOGF(LogChunkWriter, Log, "Could not read existing chunk (%ls).", *NewChunkFilename);
						}
					}
					if (!bChunkSuccess)
					{
						InFlightChunks.Decrement();
						if (OnChunkFileWriteFailed().IsBound())
						{
							ChunkFailedQueue.Enqueue(NewChunkFilename);
						}
						else
						{
							UE_LOGF(LogChunkWriter, Fatal, "Failed to serialise chunk (%ls).", *NewChunkFilename);
						}
					}
				}
				else
				{
					FPlatformProcess::Sleep(1.0f/10.0f);
				}
			}
		}

		void DebugCheckSingleProducer()
		{
#if !UE_BUILD_SHIPPING
			const uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
			if (ProducerThreadId.IsSet())
			{
				check(CurrentThreadId == ProducerThreadId.GetValue());
			}
			else
			{
				ProducerThreadId = CurrentThreadId;
			}
#endif
		}

	private:
		const FParallelChunkWriterConfig Config;
		IFileSystem* const FileSystem;
		IChunkDataSerialization* const ChunkDataSerialization;
		FStatsCollector* const StatsCollector;

		volatile int64 StatUncompressesData;
		volatile int64* StatSerlialiseTime;
		volatile int64* StatChunksSaved;
		volatile int64* StatDataWritten;
		volatile int64* StatDataWriteSpeed;
		volatile int64* StatCompressionRatio;

		TArray<TFuture<void>> WriterThreads;
		FThreadSafeBool bMoreDataIsExpected;
		FThreadSafeBool bShouldAbort;
		FThreadSafeCounter InFlightChunks;

		FCriticalSection ChunkDataJobQueueConsumerCS;
		TQueue<FChunkDataJob, EQueueMode::Spsc> ChunkDataJobQueue;
		FThreadSafeInt32 ChunkDataJobQueueCount;

		TQueue<FChunkOutputSize, EQueueMode::Mpsc> ChunkOutputCompressedSizeQueue;
		TQueue<FChunkOutputSize, EQueueMode::Mpsc> ChunkOutputFileSizeQueue;
		TQueue<FChunkOutputHash, EQueueMode::Mpsc> ChunkOutputHashQueue;
		TQueue<FChunkOutputSha, EQueueMode::Mpsc> ChunkOutputShaQueue;
		TQueue<FChunkOutputSecretId, EQueueMode::Mpsc> ChunkOutputSecretIdQueue;
		TQueue<FChunkOutputAuthTag, EQueueMode::Mpsc> ChunkOutputAuthTagQueue;
		TQueue<FChunkFileCompleteInfo, EQueueMode::Mpsc> ChunkCompleteQueue;
		TQueue<FString, EQueueMode::Mpsc> ChunkFailedQueue;
		FParallelChunkWriterSummaries ParallelChunkWriterSummaries;

		IParallelChunkWriter::FOnChunkFileWritten ChunkFileWrittenEvent;
		IParallelChunkWriter::FOnChunkFileWriteFailed ChunkFileWriteFailedEvent;

#if !UE_BUILD_SHIPPING
		TOptional<uint32> ProducerThreadId;
#endif
	};

	IParallelChunkWriter* FParallelChunkWriterFactory::Create(FParallelChunkWriterConfig Config, IFileSystem* FileSystem, IChunkDataSerialization* ChunkDataSerialization, FStatsCollector* StatsCollector)
	{
		return new FParallelChunkWriter(MoveTemp(Config), FileSystem, ChunkDataSerialization, StatsCollector);
	}
}
