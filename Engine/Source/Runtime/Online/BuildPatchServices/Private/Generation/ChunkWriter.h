// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/SecureHash.h"

#include "BuildPatchManifest.h"
#include "Common/Crypto.h"

namespace BuildPatchServices
{
	class IFileSystem;
	class IChunkDataSerialization;
	class FStatsCollector;

	struct FParallelChunkWriterConfig
	{
		int32 OperationRetryCount;
		float OperationRetryTime;
		int32 MaxQueueSize;
		int32 NumberOfThreads;
		bool bResaveExistingChunks;
		FString ChunkDirectory;
		EFeatureLevel FeatureLevel;
	};

	struct FParallelChunkWriterSummaries
	{
		EFeatureLevel FeatureLevel;
		TMap<FGuid, int64> ChunkOutputCompressedSizes;
		TMap<FGuid, int64> ChunkOutputFileSizes;
		TMap<FGuid, uint64> ChunkOutputHashes;
		TMap<FGuid, FSHAHash> ChunkOutputShas;
		TMap<FGuid, FGuid> ChunkOutputSecretIds;
		TMap<FGuid, FAESAuthTag> ChunkOutputAuthTags;
	};

	class IParallelChunkWriter
	{
	public:
		virtual ~IParallelChunkWriter() = default;

		/**
		 * Declares an event type exposed on this class for when chunks have been saved to the cloud directory.
		 * The event offers two types of hash of the filedata saved out.
		 * @param FullFilePath   The full file path on disk for the file saved.
		 * @param MD5Hash        The MD5 checksum for the file data.
		 * @param SHA1Hash       The SHA1 checksum for the file data. This is not the ChunkSha, it is the serialized file SHA1.
		 */
		DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnChunkFileWritten, const FString& /*FullFilePath*/, const FMD5Hash& /*MD5Hash*/, const FSHAHash& /*SHA1Hash*/);

		/**
		 * Declares an event type exposed on this class for when chunks have failed to be saved to the cloud directory.
		 * The event offers an error message of happened error.
		 * @param FullFilePath   The full file path on disk for for the file that failed.		 
		 */
		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnChunkFileWriteFailed, const FString& /*FullFilePath*/);

		/**
		 * @return a delegate that will broadcast when a chunk was written to the cloud directory.
		 */
		virtual FOnChunkFileWritten& OnChunkFileWritten() = 0;

		/**
		 * @return a delegate that will broadcast when a chunk failed to be written to the cloud directory.
		 */
		virtual FOnChunkFileWriteFailed& OnChunkFileWriteFailed() = 0;

		/**
		 * Must be called periodically to flush events, from the desired receiving thread.
		 * If this is not called the OnChunkFileWritten() will never broadcast, and GetInFlightChunkCount() can remain high.
		 */
		virtual void PumpEvents() = 0;

		/**
		 * Queues up serialization for a chunk file.
		 * @param ChunkData      The raw chunk data from the build.
		 * @param ChunkGuid      The identifier for the chunk.
		 * @param ChunkHash      The rolling hash for the chunk.
		 * @param ChunkSha       The SHA1 hash for the chunk.
		 * @return true if queued up successfully.
		 */
		virtual bool AddChunkData(TArray<uint8> ChunkData, const FGuid& ChunkGuid, const uint64& ChunkHash, const FSHAHash& ChunkSha) = 0;

		/**
		 * @return the number of chunk writes still outstanding.
		 */
		virtual int32 GetInFlightChunkCount() = 0;

		/**
		 * Gets the summary data for the process. The function will not return until all threads have completed.
		 * @return the summary data upon completion of the process.
		 */
		virtual FParallelChunkWriterSummaries OnProcessComplete() = 0;

		/**
		 * Aborts current operations and no longer accepts new chunks.
		 */
		virtual void Abort() = 0;

		/**
		 * Gets if the writer has aborted
		 * @return True if it has aborted
		 */
		virtual bool HasAborted() const = 0;

	};

	class FParallelChunkWriterFactory
	{
	public:
		static IParallelChunkWriter* Create(FParallelChunkWriterConfig Config, IFileSystem* FileSystem, IChunkDataSerialization* ChunkDataSerialization, FStatsCollector* StatsCollector);
	};
}
