// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/CloudEnumeration.h"

#include "Async/Async.h"
#include "Async/Future.h"
#include "BuildPatchManifest.h"
#include "BuildPatchUtil.h"
#include "Core/BlockStructure.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeLock.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCloudEnumeration, Log, All);
DEFINE_LOG_CATEGORY(LogCloudEnumeration);

namespace BuildPatchServices
{
	struct FTriggerExitScope
	{
		FTriggerExitScope(FEvent* InEvent)
			: Event(InEvent)
		{}
		~FTriggerExitScope()
		{
			Event->Trigger();
		}
		FEvent* Event;
	};

	class FCloudEnumeration
		: public ICloudEnumeration
	{
	public:
		/**
		 * Creates an instance of FCloudEnumeration for managing cloud uploaded manifests and chunks.
		 *
		 * @param CloudDirectory The cloud root directory.
		 * @param ManifestAgeThreshold The threshold age for manifest files; files older than this will be ignored.
		 * @param Predicate A callable that takes a const reference to an IBuildManifest and returns true to include the manifest, or false to ignore it.
		 * @param OutputFeatureLevel The desired feature level for enumerated output.
		 * @param StatsCollector A pointer to the statistics collector for recording related data.
		 */
		FCloudEnumeration(const FString& CloudDirectory, const FDateTime& ManifestAgeThreshold, const TFunction<bool(const IBuildManifest&)>& InPredicate, const EFeatureLevel& InOutputFeatureLevel, FStatsCollector* StatsCollector);
		virtual ~FCloudEnumeration();

		virtual bool IsComplete() const override;
		virtual const TSet<uint32>& GetUniqueWindowSizes() const override;
		virtual const TMap<uint64, TSet<FGuid>>& GetChunkInventory() const override;
		virtual const TMap<FGuid, int64>& GetChunkFileSizes() const override;
		virtual const TMap<FGuid, int64>& GetChunkCompressedSizes() const override;
		virtual const TMap<FGuid, FSHAHash>& GetChunkShaHashes() const override;
		virtual const TMap<FGuid, FAESAuthTag>& GetChunkAESAuthTags() const override;
		virtual const TMap<FGuid, uint32>& GetChunkWindowSizes() const override;
		virtual const TMap<FGuid, FGuid>& GetChunkSecretIds() const override;
		virtual bool IsChunkFeatureLevelMatch(const FGuid& ChunkId) const override;
		virtual const uint64& GetChunkHash(const FGuid& ChunkId) const override;
		virtual const FSHAHash& GetChunkShaHash(const FGuid& ChunkId) const override;
		virtual const TMap<FSHAHash, TSet<FGuid>>& GetIdenticalChunks() const override;
		virtual const TArray<FString>& GetUnsupportedFeatureLevelBuilds() const override;

	private:
		void EnumerateCloud();
		TFuture<FBuildPatchAppManifestPtr> AsyncLoadManifest(const FString& ManifestFilename);
		void EnumerateManifestData(const FBuildPatchAppManifestRef& Manifest);
		TTuple<TSet<uint32>, TMap<FGuid, uint32>> CalculateChunkWindowSizes(const FBuildPatchAppManifestRef& Manifest);

	private:
		const FString CloudDirectory;
		const FDateTime ManifestAgeThreshold;
		const TFunction<bool(const IBuildManifest&)> Predicate;
		const TCHAR* FeatureLevelChunkSubdir;
		FEvent* ManifestLoadedEvent;
		TMap<uint64, TSet<FGuid>> ChunkInventory;
		TMap<FGuid, int64> ChunkFileSizes;
		TMap<FGuid, int64> ChunkCompressedSizes;
		TMap<FGuid, uint64> ChunkHashes;
		TMap<FGuid, FSHAHash> ChunkShaHashes;
		TMap<FGuid, FAESAuthTag> ChunkAESAuthTags;
		TSet<uint32> UniqueWindowSizes;
		TMap<FGuid, uint32> ChunkWindowSizes;
		TMap<uint32, TSet<FGuid>> WindowsSizeChunks;
		TSet<FGuid> FeatureLevelMatchedChunks;
		TMap<FSHAHash, TSet<FGuid>> IdenticalChunks;
		TMap<FGuid, FGuid> ChunkSecretIds;
		TArray<FString> UnsupportedBuildVersions;
		FStatsCollector* StatsCollector;
		TFuture<void> Future;
		volatile FStatsCollector::FAtomicValue* StatManifestsFound;
		volatile FStatsCollector::FAtomicValue* StatManifestsLoading;
		volatile FStatsCollector::FAtomicValue* StatManifestsLoaded;
		volatile FStatsCollector::FAtomicValue* StatManifestsRejected;
		volatile FStatsCollector::FAtomicValue* StatChunksEnumerated;
		volatile FStatsCollector::FAtomicValue* StatChunksRejected;
		volatile FStatsCollector::FAtomicValue* StatTotalTime;
		volatile FStatsCollector::FAtomicValue* StatUniqueWindowSizes;
	};

	FCloudEnumeration::FCloudEnumeration(const FString& InCloudDirectory, const FDateTime& InManifestAgeThreshold, const TFunction<bool(const IBuildManifest&)>& InPredicate, const EFeatureLevel& InOutputFeatureLevel, FStatsCollector* InStatsCollector)
		: CloudDirectory(InCloudDirectory)
		, ManifestAgeThreshold(InManifestAgeThreshold)
		, Predicate(InPredicate)
		, FeatureLevelChunkSubdir(ManifestVersionHelpers::GetChunkSubdir(InOutputFeatureLevel))
		, ManifestLoadedEvent(FPlatformProcess::GetSynchEventFromPool())
		, StatsCollector(InStatsCollector)
	{
		// Create statistics.
		StatManifestsFound = StatsCollector->CreateStat(TEXT("Cloud Enumeration: Manifests Found"), EStatFormat::Value);
		StatManifestsLoading = StatsCollector->CreateStat(TEXT("Cloud Enumeration: Manifests Loading"), EStatFormat::Value);
		StatManifestsLoaded = StatsCollector->CreateStat(TEXT("Cloud Enumeration: Manifests Loaded"), EStatFormat::Value);
		StatManifestsRejected = StatsCollector->CreateStat(TEXT("Cloud Enumeration: Manifests Rejected"), EStatFormat::Value);
		StatChunksEnumerated = StatsCollector->CreateStat(TEXT("Cloud Enumeration: Chunks Enumerated"), EStatFormat::Value);
		StatChunksRejected = StatsCollector->CreateStat(TEXT("Cloud Enumeration: Chunks Rejected"), EStatFormat::Value);
		StatTotalTime = StatsCollector->CreateStat(TEXT("Cloud Enumeration: Enumeration Time"), EStatFormat::Timer);
		StatUniqueWindowSizes = StatsCollector->CreateStat(TEXT("Cloud Enumeration: Unique Window Sizes"), EStatFormat::Value);

		// Queue thread.
		TFunction<void()> Task = [this]() { EnumerateCloud(); };
		Future = Async(EAsyncExecution::Thread, MoveTemp(Task));
	}

	FCloudEnumeration::~FCloudEnumeration()
	{
		Future.Wait();
		FPlatformProcess::ReturnSynchEventToPool(ManifestLoadedEvent);
	}

	bool FCloudEnumeration::IsComplete() const
	{
		return Future.IsReady();
	}

	const TSet<uint32>& FCloudEnumeration::GetUniqueWindowSizes() const
	{
		Future.Wait();
		return UniqueWindowSizes;
	}

	const TMap<uint64, TSet<FGuid>>& FCloudEnumeration::GetChunkInventory() const
	{
		Future.Wait();
		return ChunkInventory;
	}

	const TMap<FGuid, int64>& FCloudEnumeration::GetChunkFileSizes() const
	{
		Future.Wait();
		return ChunkFileSizes;
	}

	const TMap<FGuid, int64>& FCloudEnumeration::GetChunkCompressedSizes() const
	{
		Future.Wait();
		return ChunkCompressedSizes;
	}

	const TMap<FGuid, FSHAHash>& FCloudEnumeration::GetChunkShaHashes() const
	{
		Future.Wait();
		return ChunkShaHashes;
	}
	
	const TMap<FGuid, FAESAuthTag>& FCloudEnumeration::GetChunkAESAuthTags() const
	{
		Future.Wait();
		return ChunkAESAuthTags;
	}

	const TMap<FGuid, uint32>& FCloudEnumeration::GetChunkWindowSizes() const
	{
		Future.Wait();
		return ChunkWindowSizes;
	}

	const TMap<FGuid, FGuid>& FCloudEnumeration::GetChunkSecretIds() const
	{
		Future.Wait();
		return ChunkSecretIds;
	}

	bool FCloudEnumeration::IsChunkFeatureLevelMatch(const FGuid& ChunkId) const
	{
		Future.Wait();
		return FeatureLevelMatchedChunks.Contains(ChunkId);
	}

	const uint64& FCloudEnumeration::GetChunkHash(const FGuid& ChunkId) const
	{
		Future.Wait();
		return ChunkHashes[ChunkId];
	}

	const FSHAHash& FCloudEnumeration::GetChunkShaHash(const FGuid& ChunkId) const
	{
		Future.Wait();
		return ChunkShaHashes[ChunkId];
	}

	const TMap<FSHAHash, TSet<FGuid>>& FCloudEnumeration::GetIdenticalChunks() const
	{
		Future.Wait();
		return IdenticalChunks;
	}

	const TArray<FString>& FCloudEnumeration::GetUnsupportedFeatureLevelBuilds() const
	{
		Future.Wait();
		return UnsupportedBuildVersions;
	}

	void FCloudEnumeration::EnumerateCloud()
	{
		// Grab a config for parallel load count.
		int32 NumParallelManifestLoads = 10;
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("CloudParallelManifestLoads"), NumParallelManifestLoads, GEngineIni);
		NumParallelManifestLoads = FMath::Clamp(NumParallelManifestLoads, 1, 100);
		UE_LOGF(LogCloudEnumeration, Log, "Loading manifests %d at a time", NumParallelManifestLoads);

		uint64 EnumerationTimer;
		IFileManager& FileManager = IFileManager::Get();

		// Find all manifest files
		FStatsCollector::AccumulateTimeBegin(EnumerationTimer);
		if (FileManager.DirectoryExists(*CloudDirectory))
		{
			TArray<FString> AllManifestFilenames;
			FileManager.FindFiles(AllManifestFilenames, *(CloudDirectory / TEXT("*.manifest")), true, false);
			FStatsCollector::Set(StatManifestsFound, AllManifestFilenames.Num());
			FStatsCollector::AccumulateTimeEnd(StatTotalTime, EnumerationTimer);
			FStatsCollector::AccumulateTimeBegin(EnumerationTimer);

			// Load and process all manifest files
			TArray<TFuture<FBuildPatchAppManifestPtr>> ManifestFutures;
			while (AllManifestFilenames.Num() > 0 || ManifestFutures.Num() > 0)
			{
				// Process all completed manifests
				TArray<TFuture<FBuildPatchAppManifestPtr>> TempManifestFutures = MoveTemp(ManifestFutures);
				for (TFuture<FBuildPatchAppManifestPtr>& ManifestFuture : TempManifestFutures)
				{
					if (ManifestFuture.IsReady())
					{
						// Determine chunks from manifest file
						FBuildPatchAppManifestPtr BuildManifest = ManifestFuture.Get();
						if (BuildManifest.IsValid())
						{
							EnumerateManifestData(BuildManifest.ToSharedRef());
						}
						FStatsCollector::AccumulateTimeEnd(StatTotalTime, EnumerationTimer);
						FStatsCollector::AccumulateTimeBegin(EnumerationTimer);
					}
					else
					{
						ManifestFutures.Add(MoveTemp(ManifestFuture));
					}
				}
				// Kick off new manifests
				while (ManifestFutures.Num() < NumParallelManifestLoads && AllManifestFilenames.Num() > 0)
				{
					ManifestFutures.Add(AsyncLoadManifest(CloudDirectory / AllManifestFilenames.Pop()));
				}
				// Wait for any manifest to complete (with defensive max 20s).
				ManifestLoadedEvent->Wait(2000);
			}
		}
		else
		{
			UE_LOGF(LogCloudEnumeration, Log, "Cloud directory does not exist: %ls", *CloudDirectory);
		}
		FStatsCollector::AccumulateTimeEnd(StatTotalTime, EnumerationTimer);
	}

	TFuture<FBuildPatchAppManifestPtr> FCloudEnumeration::AsyncLoadManifest(const FString& ManifestFilename)
	{
		return Async(EAsyncExecution::ThreadPool, [this, ManifestFilename]() -> FBuildPatchAppManifestPtr
		{
			FTriggerExitScope TriggerExitScope(ManifestLoadedEvent);
			FStatsScopedCounter StatsScopedCounter(StatManifestsLoading);
			const uint64 TimeStampStart = FStatsCollector::GetCycles();
			const FDateTime TimeStamp = IFileManager::Get().GetTimeStamp(*ManifestFilename);
			const uint64 TimeStampEnd = FStatsCollector::GetCycles();
			UE_LOGF(LogCloudEnumeration, Log, "Timestamp %ls for manifest %ls (in %.2f sec)", *TimeStamp.ToString(), *ManifestFilename, FStatsCollector::CyclesToSeconds(TimeStampEnd - TimeStampStart));
			if (TimeStamp < ManifestAgeThreshold)
			{
				FStatsCollector::Accumulate(StatManifestsRejected, 1);
				return nullptr;
			}
			TArray<uint8> FileData;
			const uint64 LoadStart = FStatsCollector::GetCycles();
			const bool bLoadSuccess = FFileHelper::LoadFileToArray(FileData, *ManifestFilename);
			const uint64 LoadEnd = FStatsCollector::GetCycles();
			UE_LOGF(LogCloudEnumeration, Log, "Loaded manifest %ls (%ls in %.2f sec)", *ManifestFilename, *FText::AsMemory(FileData.Num(), EMemoryUnitStandard::SI).ToString(), FStatsCollector::CyclesToSeconds(LoadEnd - LoadStart));
			FBuildPatchAppManifestRef BuildManifest = MakeShareable(new FBuildPatchAppManifest());
			const uint64 DeserializeStart = FStatsCollector::GetCycles();
			if (bLoadSuccess && BuildManifest->DeserializeFromData(FileData))
			{
				const uint64 DeserializeEnd = FStatsCollector::GetCycles();
				UE_LOGF(LogCloudEnumeration, Log, "Deserialized manifest %ls (%ls in %.2f sec)", *ManifestFilename, *FText::AsMemory(FileData.Num(), EMemoryUnitStandard::SI).ToString(), FStatsCollector::CyclesToSeconds(DeserializeEnd - DeserializeStart));
				UE_LOGF(LogCloudEnumeration, Log, "Manifest %ls details: BuildVersion: %ls, FeatureLevel: %ls", *ManifestFilename, *BuildManifest->GetVersionString(), LexToString(BuildManifest->GetFeatureLevel()));
				// if predicate specified and returns false
				if (Predicate && !Predicate(*BuildManifest))
				{
					FStatsCollector::Accumulate(StatManifestsRejected, 1);
					UE_LOGF(LogCloudEnumeration, Warning, "Manifest '%ls' rejected by custom predicate.", *ManifestFilename);
					return nullptr;
				}

				FStatsCollector::Accumulate(StatManifestsLoaded, 1);
				return BuildManifest;
			}
			else
			{
				FStatsCollector::Accumulate(StatManifestsRejected, 1);
				UE_LOGF(LogCloudEnumeration, Warning, "Could not read Manifest file. Data recognition will suffer (%ls)", *ManifestFilename);
			}
			return nullptr;
		});
	}

	void FCloudEnumeration::EnumerateManifestData(const FBuildPatchAppManifestRef& Manifest)
	{
		typedef TTuple<TSet<uint32>, TMap<FGuid, uint32>> FWindowSizes;
		// Check if this chunk will already beling to our matching feature level sub dir - ptr==ptr test is fine, no need to string compare.
		const bool bMatchingChunkSubdir = FeatureLevelChunkSubdir == ManifestVersionHelpers::GetChunkSubdir(Manifest->GetFeatureLevel());
		if (Manifest->GetFeatureLevel() > EFeatureLevel::Latest)
		{
			UnsupportedBuildVersions.Add(Manifest->GetVersionString());
		}
		TFuture<FWindowSizes> CalculateChunkWindowSizesFuture = Async(EAsyncExecution::TaskGraph, [&](){ return CalculateChunkWindowSizes(Manifest); });
		TArray<FGuid> DataList;
		Manifest->GetDataList(DataList);
		if (!Manifest->IsFileDataManifest())
		{
			for (const FGuid& DataGuid : DataList)
			{
				const FChunkInfo* ChunkInfo = Manifest->GetChunkInfo(DataGuid);
				const bool bChunkAccepted = ChunkInfo != nullptr && ChunkInfo->Hash != 0;
				if (bChunkAccepted)
				{
					bool bIsAlreadyInSet = false;
					ChunkInventory.FindOrAdd(ChunkInfo->Hash).Add(DataGuid, &bIsAlreadyInSet);
					if (!bIsAlreadyInSet)
					{
						// Added new chunk, fill out all the various tracking data.
						ChunkFileSizes.Add(DataGuid, Manifest->GetDataSize(DataGuid));
						if (ChunkInfo->DataSizeCompressed > 0)
						{
							ChunkCompressedSizes.Add(DataGuid, ChunkInfo->DataSizeCompressed);
						}
						ChunkHashes.Add(DataGuid, ChunkInfo->Hash);
						FMemory::Memcpy(ChunkShaHashes.FindOrAdd(DataGuid).Hash, ChunkInfo->ShaHash.Hash, FSHA1::DigestSize);
						if (ChunkInfo->EncryptionSecretId.IsValid())
						{
							FMemory::Memcpy(ChunkAESAuthTags.FindOrAdd(DataGuid).AuthTag, ChunkInfo->AESAuthTag.AuthTag, AES256_GCM_AuthTagSizeInBytes);
							ChunkSecretIds.Add(DataGuid, ChunkInfo->EncryptionSecretId);
						}
						UniqueWindowSizes.Add(ChunkInfo->DataSizeUncompressed);
						ChunkWindowSizes.Add(DataGuid, ChunkInfo->DataSizeUncompressed);
						IdenticalChunks.FindOrAdd(ChunkInfo->ShaHash).Add(DataGuid);
						FStatsCollector::Accumulate(StatChunksEnumerated, 1);
					}
					if (bMatchingChunkSubdir)
					{
						FeatureLevelMatchedChunks.Add(DataGuid);
					}
				}
				else
				{
					FStatsCollector::Accumulate(StatChunksRejected, 1);
				}
			}
		}
		else
		{
			FStatsCollector::Accumulate(StatManifestsRejected, 1);
		}
		FWindowSizes CalculatedChunkWindowSizes = CalculateChunkWindowSizesFuture.Get();
		UniqueWindowSizes.Append(CalculatedChunkWindowSizes.Get<0>());
		ChunkWindowSizes.Append(CalculatedChunkWindowSizes.Get<1>());
		for (const TPair<FGuid, uint32>& CalculatedChunkWindowSizePair : CalculatedChunkWindowSizes.Get<1>())
		{
			WindowsSizeChunks.FindOrAdd(CalculatedChunkWindowSizePair.Value).Add(CalculatedChunkWindowSizePair.Key);
		}
		FStatsCollector::Set(StatUniqueWindowSizes, UniqueWindowSizes.Num());
	}

	TTuple<TSet<uint32>, TMap<FGuid, uint32>> FCloudEnumeration::CalculateChunkWindowSizes(const FBuildPatchAppManifestRef& Manifest)
	{
		using namespace BuildPatchServices;
		TTuple<TSet<uint32>, TMap<FGuid, uint32>> DiscoveredDetails;
		// We only need to perform calculations if the manifest is a specific version, otherwise the default, or the serialised value, is the correct one to use.
		if (Manifest->GetFeatureLevel() == EFeatureLevel::VariableSizeChunksWithoutWindowSizeChunkInfo)
		{
			TMap<FGuid, FBlockStructure> ChunkBlockStructures;
			TMap<uint32, TSet<FGuid>> WindowSizeChunks;
			TArray<FString> Files;
			Manifest->GetFileList(Files);
			for (const FString& File : Files)
			{
				const FFileManifest* FileManifest = Manifest->GetFileManifest(File);
				if (FileManifest != nullptr)
				{
					for (const FChunkPart& ChunkPart : FileManifest->ChunkParts)
					{
						FBlockStructure& ChunkBlockStructure = ChunkBlockStructures.FindOrAdd(ChunkPart.Guid);
						ChunkBlockStructure.Add(ChunkPart.Offset, ChunkPart.Size);
					}
				}
			}
			for (const TPair<FGuid, FBlockStructure>& ChunkBlockStructurePair : ChunkBlockStructures)
			{
				const FBlockStructure& ChunkBlockStructure = ChunkBlockStructurePair.Value;
				// If we got a single block, from 0 to n, that's going to be the window size.
				if (ChunkBlockStructure.GetHead() != nullptr && ChunkBlockStructure.GetHead() == ChunkBlockStructure.GetTail() && ChunkBlockStructure.GetHead()->GetOffset() == 0)
				{
					const uint32 ChunkWindowSize = static_cast<uint32>(ChunkBlockStructure.GetHead()->GetSize());
					DiscoveredDetails.Get<0>().Add(ChunkWindowSize);
					DiscoveredDetails.Get<1>().Add(ChunkBlockStructurePair.Key, ChunkWindowSize);
					WindowSizeChunks.FindOrAdd(ChunkWindowSize).Add(ChunkBlockStructurePair.Key);
				}
			}
			// We check any chunks that have their own unique size as these were probably padded.
			TSet<FGuid> ChunksToCheck;
			for (const TPair<uint32, TSet<FGuid>>& WindowSizeChunksPair : WindowSizeChunks)
			{
				if (WindowSizeChunksPair.Value.Num() <= 1)
				{
					DiscoveredDetails.Get<0>().Remove(WindowSizeChunksPair.Key);
					for (const FGuid& TheChunk : WindowSizeChunksPair.Value)
					{
						DiscoveredDetails.Get<1>().Remove(TheChunk);
						ChunksToCheck.Add(TheChunk);
					}
				}
			}
			for (const FGuid& ChunkToCheck : ChunksToCheck)
			{
				FString FinalChunkPartFilename = CloudDirectory / FBuildPatchUtils::GetDataFilename(Manifest, ChunkToCheck);
				IFileManager& FileManager = IFileManager::Get();
				TUniquePtr<FArchive> FinalChunkPartFile(FileManager.CreateFileReader(*FinalChunkPartFilename));
				if (FinalChunkPartFile.IsValid())
				{
					FChunkHeader ChunkHeader;
					*FinalChunkPartFile << ChunkHeader;
					DiscoveredDetails.Get<0>().Add(ChunkHeader.DataSizeUncompressed);
					DiscoveredDetails.Get<1>().Add(ChunkToCheck, ChunkHeader.DataSizeUncompressed);
				}
			}
		}
		return DiscoveredDetails;
	}

	ICloudEnumeration* FCloudEnumerationFactory::Create(const FString& CloudDirectory, const FDateTime& ManifestAgeThreshold, const TFunction<bool(const IBuildManifest&)>& Predicate, const EFeatureLevel& OutputFeatureLevel, FStatsCollector* StatsCollector)
	{
		return new FCloudEnumeration(CloudDirectory, ManifestAgeThreshold, Predicate ? Predicate : [](const IBuildManifest&) {return true; }, OutputFeatureLevel, StatsCollector);
	}
}
