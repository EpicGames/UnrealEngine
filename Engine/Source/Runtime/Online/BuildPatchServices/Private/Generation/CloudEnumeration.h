// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/SecureHash.h"
#include "Common/Crypto.h"
#include "Common/StatsCollector.h"
#include "BuildPatchFeatureLevel.h"

class IBuildManifest;

namespace BuildPatchServices
{
	class ICloudEnumeration
	{
	public:
		virtual ~ICloudEnumeration()= default;
		virtual bool IsComplete() const = 0;
		virtual const TSet<uint32>& GetUniqueWindowSizes() const = 0;
		virtual const TMap<uint64, TSet<FGuid>>& GetChunkInventory() const = 0;
		virtual const TMap<FGuid, int64>& GetChunkFileSizes() const = 0;
		virtual const TMap<FGuid, int64>& GetChunkCompressedSizes() const = 0;
		virtual const TMap<FGuid, FSHAHash>& GetChunkShaHashes() const = 0;
		virtual const TMap<FGuid, FAESAuthTag>& GetChunkAESAuthTags() const = 0;
		virtual const TMap<FGuid, uint32>& GetChunkWindowSizes() const = 0;
		virtual const TMap<FGuid, FGuid>& GetChunkSecretIds() const = 0;
		virtual bool IsChunkFeatureLevelMatch(const FGuid& ChunkId) const = 0;
		virtual const uint64& GetChunkHash(const FGuid& ChunkId) const = 0;
		virtual const FSHAHash& GetChunkShaHash(const FGuid& ChunkId) const = 0;
		virtual const TMap<FSHAHash, TSet<FGuid>>& GetIdenticalChunks() const = 0;
		/* Returns the versions of build which generated with Feature Level greater than EFeatureLevel::Last. */
		virtual const TArray<FString>& GetUnsupportedFeatureLevelBuilds() const = 0;
	};

	class FCloudEnumerationFactory
	{
	public:
		/**
		 * Creates an instance of ICloudEnumeration for managing cloud uploaded chunks.
		 * The predicate allows to filter manifests based on custom logic.
		 * @param CloudDirectory The cloud root directory.
		 * @param ManifestAgeThreshold The threshold age for manifest files; files older than this will be ignored.
		 * @param Predicate A callable that takes a const reference to an IBuildManifest and returns true to include the manifest, or false to ignore it.
		 * @param OutputFeatureLevel The desired feature level for enumerated output.
		 * @param StatsCollector A pointer to the statistics collector for recording related data.
		 *
		 * @return A pointer to the created ICloudEnumeration instance.
		 */
		static ICloudEnumeration* Create(const FString& CloudDirectory, const FDateTime& ManifestAgeThreshold, const TFunction<bool(const IBuildManifest&)>& Predicate, const EFeatureLevel& OutputFeatureLevel, FStatsCollector* StatsCollector);
	};
}
