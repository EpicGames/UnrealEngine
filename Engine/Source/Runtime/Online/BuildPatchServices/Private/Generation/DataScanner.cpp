// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/DataScanner.h"

#include "HAL/ThreadSafeBool.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"

#include "Core/ProcessTimer.h"
#include "Data/ChunkData.h"
#include "Generation/CloudEnumeration.h"
#include "BuildPatchHash.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDataScanner, Log, All);
DEFINE_LOG_CATEGORY(LogDataScanner);

namespace BuildPatchServices
{
	bool ReadForceSkipMatches()
	{
		FString ForceSkipMatches;
		if (!FParse::Value(FCommandLine::Get(), TEXT("ForceSkipMatches="), ForceSkipMatches))
		{
			GConfig->GetString(TEXT("Portal.BuildPatch"), TEXT("ForceSkipMatches"), ForceSkipMatches, GEngineIni);
		}
		return ForceSkipMatches == TEXT("true");
	}

	bool GetForceSkipMatches()
	{
		static bool bForceSkipMatches = ReadForceSkipMatches();
		return bForceSkipMatches;
	}

	uint32 ReadMaxCollisionCountPerMB()
	{
		uint32 CommandlineValue = TNumericLimits<uint32>::Max();
		if (!FParse::Value(FCommandLine::Get(), TEXT("MaxCollisionCountPerMB="), CommandlineValue))
		{
			int32 ConfigValue = 0;
			if (GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("MaxCollisionCountPerMB"), ConfigValue, GEngineIni))
			{
				return static_cast<uint32>(ConfigValue);
			}
		}
		return CommandlineValue;
	}

	uint32 GetCollisionCountLimit(uint64 DataSize)
	{
		static uint64 MaxCollisionCountPerMB = ReadMaxCollisionCountPerMB();
		const uint64 NumMBsInData = DataSize / 1000000;
		const uint64 CollisionCountLimit = NumMBsInData * MaxCollisionCountPerMB;
		return FMath::Min<uint64>(TNumericLimits<uint32>::Max(), CollisionCountLimit);
	}

	class FDataScanner
		: public IDataScanner
	{
	public:
		FDataScanner(const TArray<uint32>& ChunkWindowSizes, const TArray<uint8>& Data, const ICloudEnumeration* CloudEnumeration, FStatsCollector* StatsCollector, uint64 DataOffset);
		virtual ~FDataScanner();

		virtual bool IsComplete() override;
		virtual TArray<FChunkMatch> GetResultWhenComplete() override;

		virtual double GetTimeRunning() override;
		virtual bool SupportsFork() override;
		virtual FBlockRange Fork() override;
	private:
		uint32 ConsumeData(FRollingHash& RollingHash, const uint8* Data, uint32 DataLen);
		bool FindChunkDataMatch(const FRollingHash& RollingHash, const uint32& DataStart, FGuid& OutChunkMatch, FSHAHash& OutChunkSha);
		int32 InsertMatch(TArray<FChunkMatch>& CurrentMatches, int32 SearchIdx, const uint64& InDataOffset, const FGuid& InChunkGuid, const uint32& InWindowSize);
		TArray<FChunkMatch> ScanData();

	private:
		const TArray<uint32>& ChunkWindowSizes;
		const TArray<uint8>& Data;
		const ICloudEnumeration* CloudEnumeration;
		FStatsCollector* StatsCollector;
		const TMap<uint64, TSet<FGuid>>& ChunkInventory;
		const TMap<FGuid, FSHAHash>& ChunkShaHashes;
		const TMap<FSHAHash, TSet<FGuid>>& IdenticalChunks;
		const uint64 DataOffset;
		const uint32 CollisionCountLimit;
		const bool bForceSkipMatches;
		FThreadSafeBool bIsComplete;
		FThreadSafeBool bShouldAbort;
		TFuture<TArray<FChunkMatch>> FutureResult;
		FProcessTimer ScanTimer;
		uint32 CollisionCount;

		// Fast forward tech variables.
		const int32 MatchHistorySize;
		TArray<FGuid> MatchHistory;
		int32 MatchHistoryNextIdx;
		uint32 MatchHistoryNextOffset;
		uint32 CurrentRepeatRunLength;
		uint32 CurrentRepeatIdx;

		volatile FStatsCollector::FAtomicValue* StatCreatedScanners;
		volatile FStatsCollector::FAtomicValue* StatRunningScanners;
		volatile FStatsCollector::FAtomicValue* StatCompleteScanners;
		volatile FStatsCollector::FAtomicValue* StatCpuTime;
		volatile FStatsCollector::FAtomicValue* StatRealTime;
		volatile FStatsCollector::FAtomicValue* StatHashCollisions;
		volatile FStatsCollector::FAtomicValue* StatTotalData;
		volatile FStatsCollector::FAtomicValue* StatSkippedData;
		volatile FStatsCollector::FAtomicValue* StatProcessingSpeed;

	public:
		static FThreadSafeCounter NumIncompleteScanners;
		static FThreadSafeCounter NumRunningScanners;
	};

	FDataScanner::FDataScanner(const TArray<uint32>& InChunkWindowSizes, const TArray<uint8>& InData, const ICloudEnumeration* InCloudEnumeration, FStatsCollector* InStatsCollector, uint64 InDataOffset)
		: ChunkWindowSizes(InChunkWindowSizes)
		, Data(InData)
		, CloudEnumeration(InCloudEnumeration)
		, StatsCollector(InStatsCollector)
		, ChunkInventory(CloudEnumeration->GetChunkInventory())
		, ChunkShaHashes(CloudEnumeration->GetChunkShaHashes())
		, IdenticalChunks(CloudEnumeration->GetIdenticalChunks())
		, DataOffset(InDataOffset)
		, CollisionCountLimit(GetCollisionCountLimit(Data.Num()))
		, bForceSkipMatches(GetForceSkipMatches())
		, bIsComplete(false)
		, bShouldAbort(false)
		, CollisionCount(0)
		, MatchHistorySize(100)
		, MatchHistoryNextIdx(0)
		, MatchHistoryNextOffset(0)
		, CurrentRepeatRunLength(0)
		, CurrentRepeatIdx(0)
	{
		MatchHistory.Reserve(MatchHistorySize);
		MatchHistory.AddDefaulted(MatchHistorySize);

		// Create statistics.
		StatCreatedScanners = StatsCollector->CreateStat(TEXT("Scanner: Created Scanners"), EStatFormat::Value);
		StatRunningScanners = StatsCollector->CreateStat(TEXT("Scanner: Running Scanners"), EStatFormat::Value);
		StatCompleteScanners = StatsCollector->CreateStat(TEXT("Scanner: Complete Scanners"), EStatFormat::Value);
		StatCpuTime = StatsCollector->CreateStat(TEXT("Scanner: CPU Time"), EStatFormat::Timer);
		StatRealTime = StatsCollector->CreateStat(TEXT("Scanner: Real Time"), EStatFormat::Timer);
		StatHashCollisions = StatsCollector->CreateStat(TEXT("Scanner: Hash Collisions"), EStatFormat::Value);
		StatTotalData = StatsCollector->CreateStat(TEXT("Scanner: Total Data"), EStatFormat::DataSize);
		StatSkippedData = StatsCollector->CreateStat(TEXT("Scanner: Skipped Data"), EStatFormat::DataSize);
		StatProcessingSpeed = StatsCollector->CreateStat(TEXT("Scanner: Processing Speed"), EStatFormat::DataSpeed);
		FStatsCollector::Accumulate(StatCreatedScanners, 1);

		// Queue thread.
		NumIncompleteScanners.Increment();
		TFunction<TArray<FChunkMatch>()> Task = [this]()
		{
			TArray<FChunkMatch> Result = ScanData();
			NumIncompleteScanners.Decrement();
			FStatsCollector::Accumulate(StatCompleteScanners, 1);
			return MoveTemp(Result);
		};
		FutureResult = Async(EAsyncExecution::ThreadPool, MoveTemp(Task));
	}

	FDataScanner::~FDataScanner()
	{
		// Make sure the task is complete.
		bShouldAbort = true;
		FutureResult.Wait();
	}

	bool FDataScanner::IsComplete()
	{
		return bIsComplete;
	}

	TArray<FChunkMatch> FDataScanner::GetResultWhenComplete()
	{
		return FutureResult.Get();
	}

	double FDataScanner::GetTimeRunning()
	{
		return ScanTimer.GetSeconds();
	}

	bool FDataScanner::SupportsFork()
	{
		// Standard chunking fork is not yet implemented.
		// Currently it's not simple to fork from multiple window size scenario.
		// May need to reconsider having a scanner per window size instead of scanners doing all window sizes.
		return false;
	}

	FBlockRange FDataScanner::Fork()
	{
		check(false);
		return FBlockRange::FromFirstAndSize(0, 0);
	}

	uint32 FDataScanner::ConsumeData(FRollingHash& RollingHash, const uint8* DataPtr, uint32 DataLen)
	{
		uint32 NumDataNeeded = RollingHash.GetNumDataNeeded();
		if (NumDataNeeded > 0 && NumDataNeeded <= DataLen)
		{
			RollingHash.ConsumeBytes(DataPtr, NumDataNeeded);
			checkSlow(RollingHash.GetNumDataNeeded() == 0);
			return NumDataNeeded;
		}
		return 0;
	}

	bool FDataScanner::FindChunkDataMatch(const FRollingHash& RollingHash, const uint32& DataStart, FGuid& OutChunkMatch, FSHAHash& OutChunkSha)
	{
		const uint32 LastMatchIdx = MatchHistoryNextIdx > 0 ? MatchHistoryNextIdx - 1 : 0;
		// Able to start tracking a new cyclic pattern?
		if (DataStart == MatchHistoryNextOffset && MatchHistory[0].IsValid() && LastMatchIdx > 0 && MatchHistory[0] == MatchHistory[LastMatchIdx])
		{
			// We got a repeated match so we can fast forward, avoiding SHA1 calculations.
			if (CurrentRepeatRunLength == 0)
			{
				CurrentRepeatRunLength = LastMatchIdx;
				CurrentRepeatIdx = 1;
			}
			const uint32 LastByte = DataStart + (RollingHash.GetWindowSize() - 1);
			const uint8* RepeatStartByte = &Data[LastByte - CurrentRepeatRunLength];
			const uint8* RepeatEndByte = &Data[LastByte];
			const bool bCanFastForward = *RepeatStartByte == *RepeatEndByte;
			if (bCanFastForward)
			{
				// Use repeat match.
				OutChunkMatch = MatchHistory[CurrentRepeatIdx];
				const bool bRepeatIsChunk = OutChunkMatch.IsValid();
				if (bRepeatIsChunk)
				{
					OutChunkSha = ChunkShaHashes[OutChunkMatch];
				}
				// Advance the repeat, we always count with index base 1, rather than 0, i.e. 1 up to and including CurrentRepeatRunLength;
				++MatchHistoryNextOffset;
				++CurrentRepeatIdx;
				if (CurrentRepeatIdx > CurrentRepeatRunLength)
				{
					CurrentRepeatIdx = 1;
				}
				return bRepeatIsChunk;
			}
			else
			{
				// Cause reset.
				MatchHistoryNextOffset = 0;
			}
		}
		// Reset fast forward?
		if (DataStart != MatchHistoryNextOffset || MatchHistoryNextIdx >= MatchHistorySize)
		{
			MatchHistory[0].Invalidate();
			MatchHistoryNextIdx = 0;
			MatchHistoryNextOffset = 0;
			CurrentRepeatRunLength = 0;
			CurrentRepeatIdx = 0;
		}

		// Check for rolling hash match.
		const bool bMightHaveMatch = ChunkInventory.Contains(RollingHash.GetWindowHash());
		if (bMightHaveMatch)
		{
			// Check for SHA match.
			RollingHash.GetWindowData().GetShaHash(OutChunkSha);
			if (IdenticalChunks.Contains(OutChunkSha))
			{
				// Grab first match.
				OutChunkMatch = IdenticalChunks[OutChunkSha][FSetElementId::FromInteger(0)];
				// Set it in the history.
				MatchHistory[MatchHistoryNextIdx++] = OutChunkMatch;
				MatchHistoryNextOffset = DataStart + 1;
				return true;
			}
			else
			{
				UE_LOGF(LogDataScanner, VeryVerbose, "@%llu: Collision at %d with chunkhash %llX SHA %ls", DataOffset, DataStart, RollingHash.GetWindowHash(), *OutChunkSha.ToString());
				++CollisionCount;
				FStatsCollector::Accumulate(StatHashCollisions, 1);
				// Check for aborting based on high collisions. This is an emergency feature for rare problems
				// with unusually high processing times.
				if (CollisionCount > CollisionCountLimit)
				{
					UE_LOGF(LogDataScanner, Log, "@%llu: Aborting scanner with %d collisions!", DataOffset, CollisionCount);
					bShouldAbort = true;
				}
			}
		}
		// Set blank in the history.
		if (MatchHistoryNextIdx > 0)
		{
			MatchHistory[MatchHistoryNextIdx++].Invalidate();
			MatchHistoryNextOffset++;
		}
		return false;
	}

	int32 FDataScanner::InsertMatch(TArray<FChunkMatch>& CurrentMatches, int32 SearchIdx, const uint64& DataFirst, const FGuid& ChunkGuid, const uint32& DataSize)
	{
		const uint64 DataLast = (DataFirst + DataSize) - 1;

		// The rule is it can overlap anything before it, but the next item in the list must not be overlapped if it is bigger
		// This is assuming a lot about the code calling it, but that is ok for now
		// There are several places that need behavior like this, FBlockStructure should be extended to support merge-able meta, or no-merge, ignore/replace type behavior.

		// Find where start sits between
		for(int32 Idx = SearchIdx; Idx < CurrentMatches.Num(); ++Idx)
		{
			const uint64 ThisMatchFirst = CurrentMatches[Idx].DataOffset;
			const uint64 ThisMatchLast = (ThisMatchFirst + CurrentMatches[Idx].WindowSize) - 1;
			const uint64 ThisMatchSize = CurrentMatches[Idx].WindowSize;

			// Can be inserted before?
			if(DataFirst < ThisMatchFirst)
			{
				// We insert if we fit entirely before ThisMatch.
				const bool bFitsInGap = DataLast < ThisMatchFirst;
				if (bFitsInGap)
				{
					check(DataSize < ThisMatchSize);
					CurrentMatches.EmplaceAt(Idx, DataFirst, ChunkGuid, DataSize);
					return Idx;
				}
				return SearchIdx;
			}
			// We don't accept perfect overlaps.
			else if (DataFirst == ThisMatchFirst)
			{
				return Idx;
			}
			// If last is less or equal here the chunk is smaller.
			else if (DataLast <= ThisMatchLast)
			{
				return Idx;
			}
			// Otherwise continue search..
		}

		// If we did nothing in the loop, we add to end!
		return CurrentMatches.Emplace(DataFirst, ChunkGuid, DataSize);
	}

	TArray<FChunkMatch> FDataScanner::ScanData()
	{
		static volatile FStatsCollector::FAtomicValue TempTimerValue;

		// Count running scanners.
		NumRunningScanners.Increment();
		
		// The return data.
		TArray<FChunkMatch> DataScanResult;

		ScanTimer.Start();
		for (const uint32 WindowSize : ChunkWindowSizes)
		{
			FRollingHash RollingHash(WindowSize);

			// Temp values.
			int32 TempMatchIdx = 0;
			FGuid ChunkMatch;
			FSHAHash ChunkSha;
			uint64 CpuTimer;

			// Track last match so we know if we can start skipping data. This will also cover us for the overlap with previous scanner.
			uint64 LastMatch = 0;

			// Loop over and process all data.
			uint32 NextByte = ConsumeData(RollingHash, &Data[0], Data.Num());
			bool bScanningData = true;
			{
				FStatsCollector::AccumulateTimeBegin(CpuTimer);
				FStatsParallelScopeTimer ParallelScopeTimer(&TempTimerValue, StatRealTime, StatRunningScanners);
				while (bScanningData && !bShouldAbort)
				{
					const uint32 DataStart = NextByte - WindowSize;
					const bool bChunkOverlap = DataStart < (LastMatch + WindowSize);
					// Check for a chunk match at this offset.
					const bool bFoundChunkMatch = FindChunkDataMatch(RollingHash, DataStart, ChunkMatch, ChunkSha);
					if (bFoundChunkMatch)
					{
						LastMatch = DataStart;
						TempMatchIdx = InsertMatch(DataScanResult, TempMatchIdx, DataStart, ChunkMatch, WindowSize);
					}
					// We can start skipping over the chunk that we matched if we have no overlap potential, i.e. we know this match will not be rejected.
					const bool bCanSkipData = bFoundChunkMatch && (!bChunkOverlap || bForceSkipMatches);
					if (bCanSkipData)
					{
						RollingHash.Clear();
						const bool bHasEnoughData = (NextByte + WindowSize - 1) < static_cast<uint32>(Data.Num());
						if (bHasEnoughData)
						{
							const uint32 Consumed = ConsumeData(RollingHash, &Data[NextByte], Data.Num() - NextByte);
							FStatsCollector::Accumulate(StatSkippedData, Consumed);
							NextByte += Consumed;
						}
						else
						{
							bScanningData = false;
						}
					}
					// Otherwise we only move forwards by one byte.
					else
					{
						const bool bHasMoreData = NextByte < static_cast<uint32>(Data.Num());
						if (bHasMoreData)
						{
							// Roll over next byte.
							RollingHash.RollForward(Data[NextByte++]);
						}
						else
						{
							bScanningData = false;
						}
					}
				}
				FStatsCollector::AccumulateTimeEnd(StatCpuTime, CpuTimer);
				FStatsCollector::Accumulate(StatTotalData, Data.Num());
				FStatsCollector::Set(StatProcessingSpeed, *StatTotalData / FStatsCollector::CyclesToSeconds(ParallelScopeTimer.GetCurrentTime()));
			}
		}
		ScanTimer.Stop();

		// Count running scanners.
		NumRunningScanners.Decrement();

		UE_LOGF(LogDataScanner, Log, "@%llu: Scanner completed in %ls with %d collisions.", DataOffset, *FPlatformTime::PrettyTime(ScanTimer.GetSeconds()), CollisionCount);

		bIsComplete = true;
		return DataScanResult;
	}

	FThreadSafeCounter FDataScanner::NumIncompleteScanners;
	FThreadSafeCounter FDataScanner::NumRunningScanners;

	int32 FDataScannerCounter::GetNumIncompleteScanners()
	{
		return FDataScanner::NumIncompleteScanners.GetValue();
	}

	int32 FDataScannerCounter::GetNumRunningScanners()
	{
		return FDataScanner::NumRunningScanners.GetValue();
	}

	void FDataScannerCounter::IncrementIncomplete()
	{
		FDataScanner::NumIncompleteScanners.Increment();
	}

	void FDataScannerCounter::DecrementIncomplete()
	{
		FDataScanner::NumIncompleteScanners.Decrement();
	}

	void FDataScannerCounter::IncrementRunning()
	{
		FDataScanner::NumRunningScanners.Increment();
	}

	void FDataScannerCounter::DecrementRunning()
	{
		FDataScanner::NumRunningScanners.Decrement();
	}

	IDataScanner* FDataScannerFactory::Create(const TArray<uint32>& ChunkWindowSizes, const TArray<uint8>& Data, const ICloudEnumeration* CloudEnumeration, FStatsCollector* StatsCollector, uint64 DataOffset)
	{
		return new FDataScanner(ChunkWindowSizes, Data, CloudEnumeration, StatsCollector, DataOffset);
	}
}
