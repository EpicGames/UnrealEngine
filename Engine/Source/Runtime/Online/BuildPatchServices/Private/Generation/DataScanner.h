// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/List.h"
#include "Misc/Guid.h"

#include "Core/BlockRange.h"
#include "Generation/CloudEnumeration.h"
#include "Generation/DeltaEnumeration.h"

namespace BuildPatchServices
{
	class FStatsCollector;

	struct FChunkMatch
	{
		FChunkMatch(const uint64& InDataOffset, const FGuid& InChunkGuid, const uint32& InWindowSize)
			: DataOffset(InDataOffset)
			, ChunkGuid(InChunkGuid)
			, WindowSize(InWindowSize)
		{}

		// Offset into provided data.
		uint64 DataOffset;
		// The chunk matched.
		FGuid ChunkGuid;
		// The window size.
		uint32 WindowSize;
	};

	struct FScannerFileElement
	{
		FBlockRange FileScannerRange;
		FFilenameId ScanFilename;
		TSet<FString> ScanTagSet;
		uint64 ScannerFileLocation;
	};
	
	typedef TDoubleLinkedList<FScannerFileElement> FScannerFilesList;
	typedef FScannerFilesList::TDoubleLinkedListNode FScannerFilesListNode;

	class IDataScanner
	{
	public:
		virtual ~IDataScanner() = default;
		virtual bool IsComplete() = 0;
		virtual TArray<FChunkMatch> GetResultWhenComplete() = 0;
		virtual double GetTimeRunning() = 0;
		virtual bool SupportsFork() = 0;
		virtual FBlockRange Fork() = 0;
	};

	class FDataScannerCounter
	{
	public:
		static int32 GetNumIncompleteScanners();
		static int32 GetNumRunningScanners();
		static void IncrementIncomplete();
		static void DecrementIncomplete();
		static void IncrementRunning();
		static void DecrementRunning();
	};

	class FDataScannerFactory
	{
	public:
		static IDataScanner* Create(const TArray<uint32>& ChunkWindowSizes, const TArray<uint8>& Data, const ICloudEnumeration* CloudEnumeration, FStatsCollector* StatsCollector, uint64 DataOffset);
	};

	class FDeltaScannerFactory
	{
	public:
		static IDataScanner* Create(const uint32 WindowSize, const TArray<uint8>& Data, const FScannerFilesList& FilesList, const IDeltaChunkEnumeration* CloudEnumeration, FStatsCollector* StatsCollector);
	};
}
