// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/DeltaEnumeration.h"

#include "Core/BlockStructure.h"
#include "Core/ProcessTimer.h"
#include "Generation/BuildStreamer.h"
#include "Generation/ChunkSearch.h"
#include "BuildPatchHash.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDeltaEnumeration, Log, All);
DEFINE_LOG_CATEGORY(LogDeltaEnumeration);

namespace DeltaChunkEnumerationHelpers
{
	uint64 GetPaddingChunkHash(uint8 Byte, uint32 WindowSize)
	{
		TArray <uint8> TempData;
		TempData.SetNumUninitialized(WindowSize);
		FMemory::Memset(TempData.GetData(), Byte, WindowSize);
		return FRollingHash::GetHashForDataSet(TempData.GetData(), WindowSize);
	}

	FSHAHash GetPaddingChunkSha(uint8 Byte, uint32 WindowSize)
	{
		FSHAHash SHAHash;
		TArray <uint8> TempData;
		TempData.SetNumUninitialized(WindowSize);
		FMemory::Memset(TempData.GetData(), Byte, WindowSize);
		FSHA1::HashBuffer(TempData.GetData(), WindowSize, SHAHash.Hash);
		return SHAHash;
	}
}

namespace BuildPatchServices
{
	class FDeltaChunkEnumeration
		: public IDeltaChunkEnumeration
	{
	public:
		FDeltaChunkEnumeration(IManifestBuildStreamer* Streamer, FStatsCollector* StatsCollector, const FBuildPatchAppManifest& Manifest, const uint32 WindowSize, const TMap<FString, FString>& InAliasedTags, const TSet<FString>& InIgnoredTags);
		~FDeltaChunkEnumeration();

		// FDeltaChunkEnumeration interface begin.
		virtual void Run() override;
		virtual bool IsComplete() const override;
		virtual bool IsSuccess() const override;
		virtual const TMap<uint64, TSet<FDeltaChunkId>>& GetChunkInventory() const override;
		virtual const TMap<FDeltaChunkId, FShaId>& GetChunkShaHashes() const override;
		virtual const TMap<FDeltaChunkId, FChunkBuildReference>& GetChunkBuildReferences() const override;
		virtual const TMap<FShaId, TSet<FDeltaChunkId>>& GetIdenticalChunks() const override;
		virtual const uint64& GetChunkHash(const FDeltaChunkId& ChunkId) const override;
		virtual const FSHAHash& GetChunkShaHash(const FDeltaChunkId& ChunkId) const override;
		virtual FFilenameId MakeFilenameId(const FString& Filename) override;
		virtual FShaId MakeShaId(const FSHAHash& SHAHash) override;
		virtual FFilenameId GetFilenameId(const FString& Filename) const override;
		virtual FShaId GetShaId(const FSHAHash& SHAHash) const override;
		virtual const FString& GetFilename(const FFilenameId& Filename) const override;
		virtual const FSHAHash& GetSha(const FShaId& SHAHash) const override;
		// FDeltaChunkEnumeration interface end.

	private:
		void MakeChunk(const TArray<uint8>& StreamBuffer, const int32& StreamBufferPosition, const FBlockStructure& StreamBuildStructure);
		FSHAHash GetShaForDataSet(const uint8* Data, uint32 Size);

	private:
		IManifestBuildStreamer* Streamer;
		FStatsCollector* StatsCollector;
		FChunkSearcher ChunkSearcher;
		const FNumberFormattingOptions PercentFormat;
		const uint32 WindowSize;
		const TSet<uint32> UniqueWindowSizes;
		TSet<FString> FilenameTable;
		TSet<FSHAHash> ShaTable;
		bool bHasRan;
		bool bIsSuccess;
		TMap<uint64, TSet<FDeltaChunkId>> ChunkInventory;
		TMap<FDeltaChunkId, uint64> ChunkHashes;
		TMap<FDeltaChunkId, FShaId> ChunkShaHashes;
		TMap<FDeltaChunkId, FChunkBuildReference> ChunkBuildReferences;
		TMap<FShaId, TSet<FDeltaChunkId>> IdenticalChunks;
		TMap<FString, FString> AliasedTags;
		TSet<FString> IgnoredTags;
	};

	FDeltaChunkEnumeration::FDeltaChunkEnumeration(IManifestBuildStreamer* InStreamer, FStatsCollector* InStatsCollector, const FBuildPatchAppManifest& InManifest, const uint32 InWindowSize, const TMap<FString, FString>& InAliasedTags, const TSet<FString>& InIgnoredTags)
		: Streamer(InStreamer)
		, StatsCollector(InStatsCollector)
		, ChunkSearcher(InManifest)
		, PercentFormat(FNumberFormattingOptions().SetMaximumFractionalDigits(1).SetMinimumFractionalDigits(1).SetRoundingMode(ERoundingMode::ToZero))
		, WindowSize(InWindowSize)
		, UniqueWindowSizes({WindowSize})
		, bHasRan(false)
		, bIsSuccess(true)
		, AliasedTags(InAliasedTags)
		, IgnoredTags(InIgnoredTags)
	{
		// Make sure we enumerate special chunk SHAs.
		FDeltaChunkId PaddingChunkId = PaddingChunk::MakePaddingGuid(0);
		FShaId PaddingShaId = MakeShaId(DeltaChunkEnumerationHelpers::GetPaddingChunkSha(0, WindowSize));
		uint64 PaddingHash = DeltaChunkEnumerationHelpers::GetPaddingChunkHash(0, WindowSize);
		FChunkBuildReference PaddingReference;
		PaddingReference.ChunkParts.AddDefaulted_GetRef().Guid = PaddingChunkId;
		PaddingReference.ChunkParts[0].Offset = 0;
		PaddingReference.ChunkParts[0].Size = WindowSize;
		PaddingReference.FileReferences.AddDefaulted_GetRef().Location = 0;

		// Add already setup variables for byte 0.
		ChunkInventory.Add(PaddingHash);
		ChunkHashes.Add(PaddingChunkId, PaddingHash);
		ChunkShaHashes.Add(PaddingChunkId, PaddingShaId);
		ChunkBuildReferences.Add(PaddingChunkId, PaddingReference);
		// Add byte 1 through 255.
		for (uint32 LoopIdx = 1; LoopIdx <= 255; ++LoopIdx)
		{
			const uint8 Byte = LoopIdx & 0xFF;
			PaddingChunkId.D = Byte;
			PaddingShaId = MakeShaId(DeltaChunkEnumerationHelpers::GetPaddingChunkSha(Byte, WindowSize));
			PaddingHash = DeltaChunkEnumerationHelpers::GetPaddingChunkHash(Byte, WindowSize);
			PaddingReference.ChunkParts[0].Guid = PaddingChunkId;
			ChunkInventory.Add(PaddingHash);
			ChunkHashes.Add(PaddingChunkId, PaddingHash);
			ChunkShaHashes.Add(PaddingChunkId, PaddingShaId);
			ChunkBuildReferences.Add(PaddingChunkId, PaddingReference);
		}
	}

	FDeltaChunkEnumeration::~FDeltaChunkEnumeration()
	{
	}

	bool FDeltaChunkEnumeration::IsComplete() const
	{
		return bHasRan;
	}

	bool FDeltaChunkEnumeration::IsSuccess() const
	{
		return bHasRan && bIsSuccess;
	}

	const TMap<uint64, TSet<FDeltaChunkId>>& FDeltaChunkEnumeration::GetChunkInventory() const
	{
		return ChunkInventory;
	}

	const TMap<FDeltaChunkId, FShaId>& FDeltaChunkEnumeration::GetChunkShaHashes() const
	{
		return ChunkShaHashes;
	}

	void FDeltaChunkEnumeration::Run()
	{
		// Setup scanning stats.
		volatile int64* StatChunkingSize = StatsCollector->CreateStat(TEXT("BuildA: Chunked data size"), EStatFormat::DataSize);
		volatile int64* StatChunkingTime = StatsCollector->CreateStat(TEXT("BuildA: Chunking time"), EStatFormat::Timer);
		volatile int64* StatChunkingCompleted = StatsCollector->CreateStat(TEXT("BuildA: Progress"), EStatFormat::Percentage);

		const EAllowShrinking AllowShrinking = EAllowShrinking::No;
		const FBlockStructure& StreamerBlocks = Streamer->GetBlockStructure();
		const uint64 ManifestAStreamSize = BlockStructureHelpers::CountSize(StreamerBlocks);
		const uint32 StreamBufferReadSize = WindowSize * 32;
		uint64 StreamStartPosition = 0;
		TArray<uint8> StreamBuffer;
		StreamBuffer.Reserve(StreamBufferReadSize + WindowSize);
		uint64 BuildAChunkTimer;
		FStatsCollector::AccumulateTimeBegin(BuildAChunkTimer);
		while (Streamer->IsEndOfData() == false)
		{
			// Grab some data, maintaining accurate buffer size.
			const uint32 StreamBufferPadding = StreamBuffer.Num();
			StreamBuffer.SetNumUninitialized(StreamBufferPadding + StreamBufferReadSize, AllowShrinking);
			const uint32 SizeRead = Streamer->DequeueData(StreamBuffer.GetData() + StreamBufferPadding, StreamBufferReadSize);
			StreamBuffer.SetNumUninitialized(StreamBufferPadding + SizeRead, AllowShrinking);

			// Calculate the buffer's build structure.
			FBlockStructure StreamBuildStructure;
			const uint64 StreamStructureSize = StreamerBlocks.SelectSerialBytes(StreamStartPosition, StreamBuffer.Num(), StreamBuildStructure);

			if (!(SizeRead == StreamBufferReadSize || Streamer->IsEndOfData()))
			{
				UE_LOGF(LogDeltaEnumeration, Error, "Failed to read from chunk stream.");
				bIsSuccess = false;
				break;
			}

			check(StreamStructureSize == StreamBuffer.Num());

			// Chunk up data.
			int32 StreamBufferPosition = 0;
			const int32 LastStreamBufferPosition = StreamBuffer.Num() - WindowSize;
			for (; StreamBufferPosition <= LastStreamBufferPosition; StreamBufferPosition += WindowSize)
			{
				MakeChunk(StreamBuffer, StreamBufferPosition, StreamBuildStructure);
			}

			// Is the remainder the final bytes we'll get?
			if (Streamer->IsEndOfData())
			{
				StreamBufferPosition = StreamBuffer.Num();
				// We lose out on matching the last bytes if there is not enough for one chunk in the buffer.
				// This is something that we could fix with padding, but it is really not necessary yet.
				checkSlow(WindowSize < (uint32)TNumericLimits<int32>::Max());
				if (StreamBuffer.Num() >= (int32)WindowSize)
				{
					const int32 StreamBufferFinalChunkIdx = StreamBuffer.Num() - WindowSize;
					MakeChunk(StreamBuffer, StreamBufferFinalChunkIdx, StreamBuildStructure);
				}
			}
			check(StreamBufferPosition <= StreamBuffer.Num());

			// Copy remainder bytes.
			StreamStartPosition += StreamBuffer.Num();
			const int32 CopySize = StreamBuffer.Num() - StreamBufferPosition;
			if (CopySize > 0)
			{
				uint8* const CopyTo = StreamBuffer.GetData();
				const uint8* const CopyFrom = &StreamBuffer[StreamBufferPosition];
				FMemory::Memcpy(CopyTo, CopyFrom, CopySize);
			}
			StreamBuffer.SetNum(CopySize, AllowShrinking);
			StreamStartPosition -= StreamBuffer.Num();

			const double PercentChunked = (double)StreamStartPosition / (double)ManifestAStreamSize;
			FStatsCollector::SetAsPercentage(StatChunkingCompleted, PercentChunked);
			FStatsCollector::Set(StatChunkingSize, StreamStartPosition);
			FStatsCollector::AccumulateTimeEnd(StatChunkingTime, BuildAChunkTimer);
			FStatsCollector::AccumulateTimeBegin(BuildAChunkTimer);
		}
		FStatsCollector::SetAsPercentage(StatChunkingCompleted, 1.0);
		FStatsCollector::Set(StatChunkingSize, ManifestAStreamSize);

		// Collect up all chunks that were identical.
		for (const TPair<FDeltaChunkId, FShaId>& ChunkShaHashPair : ChunkShaHashes)
		{
			IdenticalChunks.FindOrAdd(ChunkShaHashPair.Value).Add(ChunkShaHashPair.Key);
		}

		FStatsCollector::AccumulateTimeEnd(StatChunkingTime, BuildAChunkTimer);
		bHasRan = true;
	}

	const TMap<FDeltaChunkId, FChunkBuildReference>& FDeltaChunkEnumeration::GetChunkBuildReferences() const
	{
		return ChunkBuildReferences;
	}

	const TMap<FShaId, TSet<FDeltaChunkId>>& FDeltaChunkEnumeration::GetIdenticalChunks() const
	{
		return IdenticalChunks;
	}

	const uint64& FDeltaChunkEnumeration::GetChunkHash(const FDeltaChunkId& ChunkId) const
	{
		return ChunkHashes[ChunkId];
	}

	const FSHAHash& FDeltaChunkEnumeration::GetChunkShaHash(const FDeltaChunkId& ChunkId) const
	{
		return GetSha(ChunkShaHashes[ChunkId]);
	}

	FFilenameId FDeltaChunkEnumeration::MakeFilenameId(const FString& Filename)
	{
		return FilenameTable.Add(Filename);
	}

	FShaId FDeltaChunkEnumeration::MakeShaId(const FSHAHash& Filename)
	{
		return ShaTable.Add(Filename);
	}

	FFilenameId FDeltaChunkEnumeration::GetFilenameId(const FString& Filename) const
	{
		return FilenameTable.FindId(Filename);
	}

	FShaId FDeltaChunkEnumeration::GetShaId(const FSHAHash& Filename) const
	{
		return ShaTable.FindId(Filename);
	}

	const FString& FDeltaChunkEnumeration::GetFilename(const FFilenameId& FilenameId) const
	{
		return FilenameTable[FilenameId];
	}

	const FSHAHash& FDeltaChunkEnumeration::GetSha(const FShaId& ShaId) const
	{
		return ShaTable[ShaId];
	}

	void FDeltaChunkEnumeration::MakeChunk(const TArray<uint8>& StreamBuffer, const int32& StreamBufferPosition, const FBlockStructure& StreamBuildStructure)
	{
		const uint64 ChunkHash = FRollingHash::GetHashForDataSet(&StreamBuffer[StreamBufferPosition], WindowSize);
		// We get major high collision issues with trying to match chunk hash 0.
		// This is a weakness in the rolling hash, and is usually generated by either 0 byte padding, or short run length cyclic data.
		// Both of these types of data compress very well and so downloading it is not a huge loss.
		if (ChunkHash != 0)
		{
			FDeltaChunkId ChunkId = FGuid::NewGuid();
			ChunkInventory.Add(ChunkHash);
			ChunkHashes.Add(ChunkId, ChunkHash);
			ChunkShaHashes.Add(ChunkId, MakeShaId(GetShaForDataSet(&StreamBuffer[StreamBufferPosition], WindowSize)));
			FBlockStructure ChunkBuildStructure;
			const uint64 ChunkStructureSize = StreamBuildStructure.SelectSerialBytes(StreamBufferPosition, WindowSize, ChunkBuildStructure);
			check(ChunkStructureSize == WindowSize);
			ChunkSearcher.ForEachOverlap(ChunkBuildStructure, [&](const FBlockRange& OverlapRange, FChunkSearcher::FFileDListNode* File, FChunkSearcher::FChunkDListNode* Chunk)
			{
				const FChunkSearcher::FChunkNode& ChunkNode = Chunk->GetValue();
				FChunkBuildReference& ChunkBuildReference = ChunkBuildReferences.FindOrAdd(ChunkId);
				const int64 ChunkLeftChop = OverlapRange.GetFirst() - ChunkNode.BuildRange.GetFirst();
				const int64 ChunkRightChop = ChunkNode.BuildRange.GetLast() - OverlapRange.GetLast();
				const int64 SizeChop = ChunkLeftChop + ChunkRightChop;

				// Track where we came from.
				const FChunkSearcher::FFileNode& FileNode = File->GetValue();
				FChunkBuildReferenceSourceFile& SourceFile = ChunkBuildReference.FileReferences.AddDefaulted_GetRef();
				SourceFile.Filename = MakeFilenameId(FileNode.Manifest->Filename);
				SourceFile.Location = (ChunkNode.BuildRange.GetFirst() - FileNode.BuildRange.GetFirst()) + ChunkLeftChop;
				// If we have any tag aliases, swap out the tags so that they all match the same, and remove any tags that shouldn't be matched against in the matcher.
				SourceFile.Tagset.Reserve(FileNode.Manifest->InstallTags.Num());
				for (const FString& InstallTag : FileNode.Manifest->InstallTags)
				{
					if (IgnoredTags.Contains(InstallTag))
					{
						continue;
					}
					const FString* AliasedTag = AliasedTags.Find(InstallTag);
					SourceFile.Tagset.Add(AliasedTag ? *AliasedTag : InstallTag);
				}
				
				ChunkBuildReference.ChunkParts.Emplace(ChunkNode.ChunkPart.Guid, ChunkNode.ChunkPart.Offset + ChunkLeftChop, ChunkNode.ChunkPart.Size - SizeChop);
			});
		}
	}

	FSHAHash FDeltaChunkEnumeration::GetShaForDataSet(const uint8* Data, uint32 Size)
	{
		FSHAHash SHAHash;
		FSHA1::HashBuffer(Data, Size, SHAHash.Hash);
		return SHAHash;
	}

	IDeltaChunkEnumeration* FDeltaChunkEnumerationFactory::Create(IManifestBuildStreamer* Streamer, FStatsCollector* StatsCollector, const FBuildPatchAppManifest& Manifest, const uint32 WindowSize, const TMap<FString, FString>& AliasedTags, const TSet<FString>& IgnoredTags)
	{
		return new FDeltaChunkEnumeration(Streamer, StatsCollector, Manifest, WindowSize, AliasedTags, IgnoredTags);
	}
}
