// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Data/ChunkData.h"
#include "Generation/CloudEnumeration.h"

class FBuildPatchAppManifest;

inline uint32 GetTypeHash(const FSetElementId A)
{
	return A.AsInteger();
}

namespace BuildPatchServices
{
	class FBlockStructure;
	class IManifestBuildStreamer;

	typedef FGuid FDeltaChunkId;
	typedef FSetElementId FFilenameId;
	typedef FSetElementId FShaId;

	struct FChunkBuildReferenceSourceFile
	{
		FFilenameId Filename;
		TSet<FString> Tagset;
		uint64 Location;
	};

	struct FChunkBuildReference
	{
		TArray<FChunkPart> ChunkParts;
		TArray<FChunkBuildReferenceSourceFile> FileReferences;
	};

	class IDeltaChunkEnumeration
	{
	public:
		virtual ~IDeltaChunkEnumeration() = default;
		virtual void Run() = 0;
		virtual bool IsComplete() const = 0;
		virtual bool IsSuccess() const = 0;
		virtual const TMap<uint64, TSet<FDeltaChunkId>>& GetChunkInventory() const = 0;
		virtual const TMap<FDeltaChunkId, FShaId>& GetChunkShaHashes() const = 0;
		virtual const TMap<FDeltaChunkId, FChunkBuildReference>& GetChunkBuildReferences() const = 0;
		virtual const TMap<FShaId, TSet<FDeltaChunkId>>& GetIdenticalChunks() const = 0;
		virtual const uint64& GetChunkHash(const FDeltaChunkId& ChunkId) const = 0;
		virtual const FSHAHash& GetChunkShaHash(const FDeltaChunkId& ChunkId) const = 0;
		virtual FFilenameId MakeFilenameId(const FString& Filename) = 0;
		virtual FShaId MakeShaId(const FSHAHash& SHAHash) = 0;
		virtual FFilenameId GetFilenameId(const FString& Filename) const = 0;
		virtual FShaId GetShaId(const FSHAHash& SHAHash) const = 0;
		virtual const FString& GetFilename(const FFilenameId& Filename) const = 0;
		virtual const FSHAHash& GetSha(const FShaId& SHAHash) const = 0;
	};

	class FDeltaChunkEnumerationFactory
	{
	public:
		static IDeltaChunkEnumeration* Create(IManifestBuildStreamer* Streamer, FStatsCollector* StatsCollector, const FBuildPatchAppManifest& Manifest, const uint32 WindowSize, const TMap<FString, FString>& InAliasedTags, const TSet<FString>& InIgnoredTags);
	};
}
