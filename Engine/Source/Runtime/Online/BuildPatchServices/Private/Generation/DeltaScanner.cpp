// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/DataScanner.h"

#include "Algo/Transform.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "HAL/ThreadSafeBool.h"

#include "Core/ProcessTimer.h"
#include "Data/ChunkData.h"
#include "Generation/DeltaEnumeration.h"
#include "BuildPatchHash.h"

namespace DeltaScannerHelpers
{
	FSHAHash GetZeroChunkSha(uint32 WindowSize)
	{
		FSHAHash SHAHash;
		TArray <uint8> TempData;
		TempData.SetNumZeroed(WindowSize);
		FSHA1::HashBuffer(TempData.GetData(), WindowSize, SHAHash.Hash);
		return SHAHash;
	}
}

namespace BuildPatchServices
{
	// The chunk match, the data range the match is in, whether it was same file match.
	struct FChunkMatchMeta
	{
	public:
		FChunkMatchMeta(const FChunkMatch& InMatch, const FBlockRange& InRange, const bool& bInSameFile, const uint64& InFileLocOffset)
			: Match(InMatch)
			, Range(InRange)
			, bSameFile(bInSameFile)
			, FileLocOffset(InFileLocOffset)
		{ }

		FChunkMatchMeta()
			: Match(0, FDeltaChunkId(), 0)
			, Range(FBlockRange::FromFirstAndSize(0,0))
			, bSameFile(false)
			, FileLocOffset(0)
		{ }

		// This is the match as reported by the hash comparisions. It's offset and size should always exactly match the Range below.
		const FChunkMatch Match;

		// This is the range in the new version data where we have the match. 
		const FBlockRange Range;

		// Whether the match came from the same source file by name.
		const bool bSameFile;

		// This is the distance to the match in the source stream. It's used for selecting closer matches in the event of multiple valid matches.
		const uint64 FileLocOffset;
	};

	class FDeltaMatchProcessor
	{
	public:
		FDeltaMatchProcessor()
			: LastAcceptedIdx(INDEX_NONE)
		{
		}
		
		bool AddMatch(const FChunkMatchMeta& NewMatchMeta, bool& bForceSkip)
		{
			static FChunkMatchMeta Invalidate;
			bForceSkip = false;
			bool bAddedMatch = false;
			if (LastAcceptedIdx == INDEX_NONE)
			{
				Matches.Add(NewMatchMeta);
				bAddedMatch = true;
				// Also update last accepted to this new one since no previous overlapping.
				LastAcceptedIdx = Matches.Num() - 1;
			}
			else
			{
				// First advance idx if no overlap in the source data. If the new match is distinct, then
				// we end up at Matches.Num(), which falls through to accepting the match.
				while (LastAcceptedIdx < Matches.Num() && !Matches[LastAcceptedIdx].Range.Overlaps(NewMatchMeta.Range))
				{
					++LastAcceptedIdx;
				}

				// Handle padding chunk.
				if (PaddingChunk::IsPadding(NewMatchMeta.Match.ChunkGuid))
				{
					if (Matches.IsValidIndex(LastAcceptedIdx))
					{
						Matches.RemoveAt(LastAcceptedIdx, Matches.Num() - LastAcceptedIdx, EAllowShrinking::No);
					}
					Matches.Add(NewMatchMeta);
					bAddedMatch = true;
					bForceSkip = true;
					return true;
				}

				// If there were no overlaps, just take this match.
				if (!Matches.IsValidIndex(LastAcceptedIdx))
				{
					Matches.Add(NewMatchMeta);
					bAddedMatch = true;
					// Also update last accepted to this new one since no previous overlapping.
					LastAcceptedIdx = Matches.Num() - 1;
					if (NewMatchMeta.bSameFile)
					{
						const FChunkMatchMeta& PrevMatchMeta = Matches[LastAcceptedIdx - 1];
						const bool bAdjacentRange = (PrevMatchMeta.Range.GetLast() + 1) == NewMatchMeta.Range.GetFirst();
						const bool bFileContigeous = PrevMatchMeta.FileLocOffset == NewMatchMeta.FileLocOffset;
						bForceSkip = PrevMatchMeta.bSameFile && bAdjacentRange && bFileContigeous;
					}
				}
				// If this new match is not same file, we can only add it if we found no same file overlaps.
				else if (!NewMatchMeta.bSameFile)
				{
					// We know that LastAcceptedIdx refers to an overlapping entry, and also that all remaining entries overlap too.
					bool bCanAddMatch = true;
					for (int32 SearchIdx = LastAcceptedIdx; bCanAddMatch && SearchIdx < Matches.Num(); ++SearchIdx)
					{
						if (Matches[SearchIdx].bSameFile)
						{
							bCanAddMatch = false;
							break;
						}
					}
					if (bCanAddMatch)
					{
						Matches.Add(NewMatchMeta);
						bAddedMatch = true;
					}
				}
				// If this new match is for the same file, we always take it, removing all overlapping non same files.
				else if (NewMatchMeta.bSameFile)
				{
					// First off, if this chunk is an identical overlap, we simply clobber based on improved location offset.
					const FChunkMatchMeta& LastMatch = Matches.Last();
					if (LastMatch.Range == NewMatchMeta.Range)
					{
						if (NewMatchMeta.FileLocOffset < LastMatch.FileLocOffset)
						{
							Move(Matches.Last(), NewMatchMeta);
							bAddedMatch = true;
							LastAcceptedIdx = Matches.Num() - 1;
						}
					}
					else
					{
						// If it is not an identical overlap, but does overlap other matches, we remove overlapped matches that have less preferable offset.
						// We know that LastAcceptedIdx refers to an overlapping entry, and also that all remaining entries overlap too.
						bAddedMatch = false;
						for (int32 SearchIdx = LastAcceptedIdx; SearchIdx < Matches.Num(); ++SearchIdx)
						{
							FChunkMatchMeta& MetaElement = Matches[SearchIdx];
							const bool bNewPreferred = !MetaElement.bSameFile || NewMatchMeta.FileLocOffset < MetaElement.FileLocOffset;
							if (bNewPreferred)
							{
								Move(MetaElement, Invalidate);
								bAddedMatch = true;
							}
						}
						Matches.RemoveAll([&](const FChunkMatchMeta& MetaElement) { return !MetaElement.Match.ChunkGuid.IsValid(); });
						if (bAddedMatch)
						{
							Matches.Add(NewMatchMeta);
							LastAcceptedIdx = Matches.Num() - 1;
							if (NewMatchMeta.bSameFile && LastAcceptedIdx > 0)
							{
								const FChunkMatchMeta& PrevMatchMeta = Matches[LastAcceptedIdx - 1];
								const bool bAdjacentRange = (PrevMatchMeta.Range.GetLast() + 1) == NewMatchMeta.Range.GetFirst();
								const bool bFileContigeous = PrevMatchMeta.FileLocOffset == NewMatchMeta.FileLocOffset;
								bForceSkip = PrevMatchMeta.bSameFile && bAdjacentRange && bFileContigeous;
							}
						}
					}
				}
			}
			return bAddedMatch;
		}

		TArray<FChunkMatchMeta> Matches;
		int32 LastAcceptedIdx;
	};

	class FDeltaScanner
		: public IDataScanner
	{
	public:
		FDeltaScanner(const uint32 WindowSize, const TArray<uint8>& Data, const FScannerFilesList& FilesList, const IDeltaChunkEnumeration* CloudEnumeration, FStatsCollector* StatsCollector);
		virtual ~FDeltaScanner();

		virtual bool IsComplete() override;
		virtual TArray<FChunkMatch> GetResultWhenComplete() override;
		virtual double GetTimeRunning() override;
		virtual bool SupportsFork() override;
		virtual FBlockRange Fork() override;
	private:
		uint32 ConsumeData(FRollingHash& RollingHash, const uint8* Data, uint32 DataLen);
		void FindChunkDataMatches(const FRollingHash& RollingHash, const TSet<FDeltaChunkId>*& ChunkMatches, FShaId& ChunkSha);
		bool ProcessMatch(const FDeltaChunkId& ChunkMatch, const FBlockRange& ScanLocation, bool& bSameFile, uint64& LocationOffset);
		TArray<FChunkMatch> ScanData();

	private:
		const uint32 WindowSize;
		const TArray<uint8>& Data;
		const FScannerFilesList& FilesList;
		const IDeltaChunkEnumeration* CloudEnumeration;
		const TMap<uint64, TSet<FDeltaChunkId>>& ChunkInventory;
		const TMap<FDeltaChunkId, FShaId>& ChunkShaHashes;
		const TMap<FShaId, TSet<FDeltaChunkId>>& IdenticalChunks;
		const TMap<FDeltaChunkId, FChunkBuildReference>& ChunkBuildReference;
		FStatsCollector* StatsCollector;
		const FScannerFilesListNode* FilesListNode;
		FThreadSafeBool bIsComplete;
		FThreadSafeBool bShouldAbort;
		FThreadSafeCounter BytesProcessed;
		TFuture<TArray<FChunkMatch>> FutureResult;
		FProcessTimer ScanTimer;
		volatile FStatsCollector::FAtomicValue* StatCreatedScanners;
		volatile FStatsCollector::FAtomicValue* StatRunningScanners;
		volatile FStatsCollector::FAtomicValue* StatCompleteScanners;
		volatile FStatsCollector::FAtomicValue* StatCpuTime;
		volatile FStatsCollector::FAtomicValue* StatRealTime;
		volatile FStatsCollector::FAtomicValue* StatHashCollisions;
		volatile FStatsCollector::FAtomicValue* StatTotalData;
		volatile FStatsCollector::FAtomicValue* StatSkippedData;
		volatile FStatsCollector::FAtomicValue* StatProcessingSpeed;
	};

	FDeltaScanner::FDeltaScanner(const uint32 InWindowSize, const TArray<uint8>& InData, const FScannerFilesList& InFilesList, const IDeltaChunkEnumeration* InCloudEnumeration, FStatsCollector* InStatsCollector)
		: WindowSize(InWindowSize)
		, Data(InData)
		, FilesList(InFilesList)
		, CloudEnumeration(InCloudEnumeration)
		, ChunkInventory(InCloudEnumeration->GetChunkInventory())
		, ChunkShaHashes(InCloudEnumeration->GetChunkShaHashes())
		, IdenticalChunks(InCloudEnumeration->GetIdenticalChunks())
		, ChunkBuildReference(InCloudEnumeration->GetChunkBuildReferences())
		, StatsCollector(InStatsCollector)
		, FilesListNode(FilesList.GetHead())
		, bIsComplete(false)
		, bShouldAbort(false)
		, BytesProcessed(0)
	{
		// Create statistics.
		StatCreatedScanners = StatsCollector->CreateStat(TEXT("Scanner: Created Scanners"), EStatFormat::Value);
		StatRunningScanners = StatsCollector->CreateStat(TEXT("Scanner: Running Scanners"), EStatFormat::Value);
		StatCompleteScanners = StatsCollector->CreateStat(TEXT("Scanner: Complete Scanners"), EStatFormat::Value);
		StatCpuTime = StatsCollector->CreateStat(TEXT("Scanner: CPU Time"), EStatFormat::Timer);
		StatRealTime = StatsCollector->CreateStat(TEXT("Scanner: Real Time"), EStatFormat::Timer);
		StatHashCollisions = StatsCollector->CreateStat(TEXT("Scanner: Hash Collisions"), EStatFormat::Value);
		StatTotalData = StatsCollector->CreateStat(TEXT("Scanner: Total Data"), EStatFormat::DataSize);
		StatSkippedData = StatsCollector->CreateStat(TEXT("Scanner: Skipped Data"), EStatFormat::DataSize);
		StatProcessingSpeed = StatsCollector->CreateStat(TEXT("Scanner: Processing Speed"), EStatFormat::DataSpeed);
		FStatsCollector::Accumulate(StatCreatedScanners, 1);

		// Queue thread.
		FDataScannerCounter::IncrementIncomplete();
		TFunction<TArray<FChunkMatch>()> Task = [this]()
		{
			TArray<FChunkMatch> Result = ScanData();
			FDataScannerCounter::DecrementIncomplete();
			FStatsCollector::Accumulate(StatCompleteScanners, 1);
			return MoveTemp(Result);
		};
		FutureResult = Async(EAsyncExecution::ThreadPool, MoveTemp(Task));
	}

	FDeltaScanner::~FDeltaScanner()
	{
		// Make sure the task is complete.
		bShouldAbort = true;
		FutureResult.Wait();
	}

	bool FDeltaScanner::IsComplete()
	{
		return bIsComplete;
	}

	TArray<FChunkMatch> FDeltaScanner::GetResultWhenComplete()
	{
		return FutureResult.Get();
	}

	double FDeltaScanner::GetTimeRunning()
	{
		return ScanTimer.GetSeconds();
	}

	bool FDeltaScanner::SupportsFork()
	{
		const uint32 BytesComplete = BytesProcessed.GetValue();
		const uint32 BytesLeft = Data.Num() - BytesComplete;
		return !bIsComplete && BytesComplete > 0 && BytesLeft > (WindowSize * 3);
	}

	FBlockRange FDeltaScanner::Fork()
	{
		// Stop processing.
		bShouldAbort = true;
		FutureResult.Wait();
		// Return range that still needs scanning.
		return FBlockRange::FromFirstAndLast(BytesProcessed.GetValue(), Data.Num() - 1);
	}

	uint32 FDeltaScanner::ConsumeData(FRollingHash& RollingHash, const uint8* DataPtr, uint32 DataLen)
	{
		uint32 NumDataNeeded = RollingHash.GetNumDataNeeded();
		if (NumDataNeeded > 0 && NumDataNeeded <= DataLen)
		{
			RollingHash.ConsumeBytes(DataPtr, NumDataNeeded);
			checkSlow(RollingHash.GetNumDataNeeded() == 0);
			return NumDataNeeded;
		}
		return 0;
	}

	void FDeltaScanner::FindChunkDataMatches(const FRollingHash& RollingHash, const TSet<FDeltaChunkId>*& ChunkMatches, FShaId& ChunkShaId)
	{
		ChunkMatches = nullptr;
		const bool bMightHaveMatch = ChunkInventory.Contains(RollingHash.GetWindowHash());
		if (bMightHaveMatch)
		{
			FSHAHash ChunkSha;
			RollingHash.GetWindowData().GetShaHash(ChunkSha);
			ChunkShaId = CloudEnumeration->GetShaId(ChunkSha);
			if (ChunkShaId.IsValidId())
			{
				// These are the actual chunks we got from the base build. We might have duplicates. We don't get to know where they came from.
				ChunkMatches = &IdenticalChunks[ChunkShaId];
				return;
			}
			else
			{
				FStatsCollector::Accumulate(StatHashCollisions, 1);
			}
		}
	}

	bool FDeltaScanner::ProcessMatch(const FDeltaChunkId& ChunkMatch, const FBlockRange& ScanLocation, bool& bSameFile, uint64& LocationOffset)
	{
		// We find out here if same tagset, culling all matches that are not usable, where not usable means from a tagset that doesn't match, and not empty tagset.
		// E.g. if matched a chunk with non-empty tagset, then the current file's tagset must be equal or subset of matched chunk tagset.

		// Early out padding chunks.
		if (PaddingChunk::IsPadding(ChunkMatch))
		{
			bSameFile = true;
			LocationOffset = 0;
			return true;
		}

		// Setup current file node.
		// We safely assume this data has been setup correctly to never fail to find.
		while (!FilesListNode->GetValue().FileScannerRange.Overlaps(ScanLocation))
		{
			FilesListNode = FilesListNode->GetNextNode();
		}

		const FChunkBuildReference& MatchBuildReference = ChunkBuildReference[ChunkMatch];
		const FScannerFileElement& ScanFileInfo = FilesListNode->GetValue();

		check(ScanLocation.GetFirst() >= ScanFileInfo.FileScannerRange.GetFirst());
		const uint64 ScanFileLocation = ScanFileInfo.ScannerFileLocation + (ScanLocation.GetFirst() - ScanFileInfo.FileScannerRange.GetFirst());
		bSameFile = false;

		// We only accept matches from the same tagset. Note that it matches _any_ tag in the set, so having multiple tags
		// can run into issues where if the tag sets are not installed, this might think they are and expect an on-disk match
		// that doesn't exist.
		if (MatchBuildReference.FileReferences[0].Tagset.Num() == 0 || MatchBuildReference.FileReferences[0].Tagset.Includes(ScanFileInfo.ScanTagSet))
		{
			bSameFile = MatchBuildReference.FileReferences[0].Filename == ScanFileInfo.ScanFilename;
			if (bSameFile)
			{
				LocationOffset = FMath::Max<uint64>(MatchBuildReference.FileReferences[0].Location, ScanFileLocation) - FMath::Min<uint64>(MatchBuildReference.FileReferences[0].Location, ScanFileLocation);
			}
			else
			{
				// So that we wouldn't confuse a great match with a different file one, set offset to max.
				LocationOffset = TNumericLimits<uint64>::Max();
			}
			return true;
		}

		return false;
	}

	TArray<FChunkMatch> FDeltaScanner::ScanData()
	{
		static volatile FStatsCollector::FAtomicValue TempTimerValue;

		// Count running scanners.
		FDataScannerCounter::IncrementRunning();
		
		// Temp values.
		const FShaId ZeroChunkSha = CloudEnumeration->GetShaId(DeltaScannerHelpers::GetZeroChunkSha(WindowSize));
		const TSet<FDeltaChunkId>* ChunkMatches = nullptr;
		FShaId ChunkSha;
		bool bSameFile = false;
		bool bForceSkip = false;
		uint64 FileLocOffset = TNumericLimits<uint64>::Max();
		uint64 CpuTimer = 0;

		// Fastforward tech variables.
		const int32 MatchHistorySize = 100;
		TArray<const TSet<FDeltaChunkId>*> MatchHistory;
		MatchHistory.Reserve(MatchHistorySize);
		MatchHistory.AddDefaulted(MatchHistorySize);
		int32 MatchHistoryNextIdx = 0;
		uint32 MatchHistoryNextOffset = 0;

		ScanTimer.Start();
		// Loop over and process all data.
		FDeltaMatchProcessor MatchProcessor;
		FRollingHash RollingHash(WindowSize);
		uint32 NextByte = ConsumeData(RollingHash, &Data[0], Data.Num());
		bool bScanningData = true;
		{
			FStatsCollector::AccumulateTimeBegin(CpuTimer);
			FStatsParallelScopeTimer ParallelScopeTimer(&TempTimerValue, StatRealTime, StatRunningScanners);
			while (bScanningData && !bShouldAbort)
			{
				uint32 FastForwardCount = 0;
				const uint32 DataStart = NextByte - WindowSize;
				const FBlockRange CurrentRange(FBlockRange::FromFirstAndSize(DataStart, WindowSize));
				// Check for a chunk match at this offset. All matches are identical copies - we could have it in many locations
				FindChunkDataMatches(RollingHash, ChunkMatches, ChunkSha);
				const bool bFoundChunkMatch = ChunkMatches != nullptr;
				bSameFile = false;
				if (bFoundChunkMatch)
				{
					// Process each match for contextual info and add to the selector.
					bool bAccepted = false;

					// FindChunkDataMatches returns a bunch of matches that are the same data, and we need to pick one. Any of them will provide
					// the data we want.
					for (const FDeltaChunkId& ChunkMatch : *ChunkMatches)
					{
						if (ProcessMatch(ChunkMatch, CurrentRange, bSameFile, FileLocOffset))
						{
							bAccepted = MatchProcessor.AddMatch(FChunkMatchMeta(FChunkMatch(DataStart, ChunkMatch, WindowSize), CurrentRange, bSameFile, FileLocOffset), bForceSkip) || bAccepted;
						}
						if (bForceSkip)
						{
							break;
						}
					}
				}
				// Skip if the match processor told us to.
				if (bForceSkip)
				{
					bForceSkip = false;
					RollingHash.Clear();
					MatchHistoryNextIdx = 0;
					const bool bHasEnoughData = (NextByte + WindowSize - 1) < static_cast<uint32>(Data.Num());
					if (bHasEnoughData)
					{
						const uint32 Consumed = ConsumeData(RollingHash, &Data[NextByte], Data.Num() - NextByte);
						FStatsCollector::Accumulate(StatSkippedData, Consumed);
						NextByte += Consumed;
					}
					else
					{
						bScanningData = false;
					}
				}
				// Otherwise we only move forwards by one byte.
				else
				{
					const bool bHasMoreData = NextByte < static_cast<uint32>(Data.Num());
					if (bHasMoreData)
					{
						// Look for ability to fast forward over repeated data patterns, to avoid additional SHA calculations.
						bool bDidFastForward = false;
						if (bFoundChunkMatch)
						{
							if (MatchHistoryNextIdx == 0)
							{
								MatchHistory[0] = ChunkMatches;
								MatchHistoryNextIdx = 1;
								MatchHistoryNextOffset = DataStart + 1;
							}
							else
							{
								// Check for resetting first.
								if (MatchHistoryNextOffset != DataStart || MatchHistoryNextIdx >= MatchHistorySize)
								{
									MatchHistory[0] = ChunkMatches;
									MatchHistoryNextIdx = 1;
									MatchHistoryNextOffset = DataStart + 1;
								}
								else
								{
									// Check for repeated match.
									if (MatchHistory[0] != nullptr && MatchHistory[0] == ChunkMatches)
									{
										// We got a repeated match so we can fast forward through all repeated matches
										// adding them to the processor as we go, avoiding SHA1 calculations.
										const uint32 RepeatRunLength = MatchHistoryNextIdx;
										const uint8* RepeatStartByte = &Data[NextByte - RepeatRunLength];
										const uint8* RepeatEndByte = &Data[NextByte];
										const uint8* DataLast = &Data.Last();
										uint32 ForceSkipCount = 0;
										bDidFastForward = *RepeatStartByte == *RepeatEndByte;
										// We loop through every offset where the byte after the end of the first match in the history is the same
										// as the byte after the repeated match, we know we would get the same matches results for all bytes between.
										while (!bShouldAbort && RepeatEndByte <= DataLast && *(RepeatStartByte++) == *(RepeatEndByte++))
										{
											const int32 RepeatMatchIdx = ++MatchHistoryNextIdx % RepeatRunLength;
											const uint32 RepeatMatchOffset = ++MatchHistoryNextOffset;
											const TSet<FDeltaChunkId>* RepeatChunkMatches = MatchHistory[RepeatMatchIdx];
											if (ForceSkipCount == 0)
											{
												if (RepeatChunkMatches != nullptr)
												{
													const FBlockRange RepeatBlockRange = FBlockRange::FromFirstAndSize(RepeatMatchOffset, WindowSize);
													// Process each match for contextual info to find the preferred one at this offset.
													FDeltaChunkId BestMatch;
													bool bBestIsSameFile = false;
													uint64 BestFileLocOffset = TNumericLimits<uint64>::Max();
													for (const FDeltaChunkId& RepeatChunkMatch : *RepeatChunkMatches)
													{
														if (ProcessMatch(RepeatChunkMatch, RepeatBlockRange, bSameFile, FileLocOffset))
														{
															const bool bAcceptableFirstMatch = !BestMatch.IsValid();
															const bool bSameFileImprovement = !bBestIsSameFile && bSameFile;
															const bool bFileOffsetImprovement = FileLocOffset < BestFileLocOffset;
															if (bAcceptableFirstMatch || bSameFileImprovement || bFileOffsetImprovement)
															{
																BestMatch = RepeatChunkMatch;
																bBestIsSameFile = bSameFile;
																BestFileLocOffset = FileLocOffset;
															}
														}
													}
													// Add the match
													bool bAccepted = false;
													if (BestMatch.IsValid())
													{
														bAccepted = MatchProcessor.AddMatch(FChunkMatchMeta(FChunkMatch(RepeatMatchOffset, BestMatch, WindowSize), RepeatBlockRange, bBestIsSameFile, BestFileLocOffset), bForceSkip);
														// Being to to skip here means we can continue through repeated bytes without testing results for the length of this accepted match.
														if (bForceSkip)
														{
															ForceSkipCount = WindowSize;
															bForceSkip = false;
														}
													}
												}
											}
											else
											{
												--ForceSkipCount;
											}
											RollingHash.RollForward(Data[NextByte++]);
											++FastForwardCount;
											BytesProcessed.Set(DataStart + FastForwardCount);
										}
										while (ForceSkipCount > 0 && NextByte < static_cast<uint32>(Data.Num()))
										{
											--ForceSkipCount;
											RollingHash.RollForward(Data[NextByte++]);
											++FastForwardCount;
											BytesProcessed.Set(DataStart + FastForwardCount);
										}
										// Now reset again.
										MatchHistoryNextIdx = 0;
									}
									else
									{
										MatchHistory[MatchHistoryNextIdx++] = ChunkMatches;
										++MatchHistoryNextOffset;
									}
								}
							}
						}
						else if (MatchHistoryNextIdx > 0 && MatchHistoryNextIdx < MatchHistorySize && MatchHistory[0] != nullptr)
						{
							MatchHistory[MatchHistoryNextIdx++] = nullptr;
							++MatchHistoryNextOffset;
						}
						else
						{
							MatchHistoryNextIdx = 0;
						}

						// If there was no fast forward, we move to the next offset.
						if (!bDidFastForward)
						{
							RollingHash.RollForward(Data[NextByte++]);
						}
					}
					else
					{
						bScanningData = false;
					}
				}
				BytesProcessed.Set(DataStart + FastForwardCount);
			}
			FStatsCollector::AccumulateTimeEnd(StatCpuTime, CpuTimer);
			FStatsCollector::Accumulate(StatTotalData, Data.Num());
			FStatsCollector::Set(StatProcessingSpeed, *StatTotalData / FStatsCollector::CyclesToSeconds(ParallelScopeTimer.GetCurrentTime()));
		}

		// Copy matches to expected return type.
		TArray<FChunkMatch> DataScanResult;
		Algo::Transform(MatchProcessor.Matches, DataScanResult, &FChunkMatchMeta::Match);

		ScanTimer.Stop();

		// Count running scanners.
		FDataScannerCounter::DecrementRunning();

		bIsComplete = true;
		return DataScanResult;
	}

	IDataScanner* FDeltaScannerFactory::Create(const uint32 WindowSize, const TArray<uint8>& Data, const FScannerFilesList& FilesList, const IDeltaChunkEnumeration* CloudEnumeration, FStatsCollector* StatsCollector)
	{
		check(Data.Num() >= (int32)WindowSize);
		check(CloudEnumeration != nullptr);
		check(StatsCollector != nullptr);
		return new FDeltaScanner(WindowSize, Data, FilesList, CloudEnumeration, StatsCollector);
	}
}
