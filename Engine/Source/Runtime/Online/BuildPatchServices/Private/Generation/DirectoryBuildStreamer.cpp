// Copyright Epic Games, Inc. All Rights Reserved.
#include "Generation/BuildStreamer.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/ScopeLock.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "Core/RingBuffer.h"
#include "Common/FileSystem.h"

#if PLATFORM_MAC || PLATFORM_LINUX
#include <sys/stat.h>
#include <unistd.h>
#endif

#if BPS_WITH_OPENSSL
#include <openssl/sha.h>
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogBuildStreamer, Log, All);
DEFINE_LOG_CATEGORY(LogBuildStreamer);

namespace BuildPatchServices
{
	// Buffer sizes
	enum
	{
		FileBufferSize = 1024 * 1024 * 10,      // 10 MiB
		StreamBufferSize = 1024 * 1024 * 100    // 100 MiB
	};
	static_assert(StreamBufferSize > (FileBufferSize * 2), "Stream buffers should be able to take at least two enqueues without a dequeue.");

	static bool IsUnixExecutable(const TCHAR* Filename)
	{
#if PLATFORM_MAC || PLATFORM_LINUX
		struct stat FileInfo;
		if (stat(TCHAR_TO_UTF8(Filename), &FileInfo) == 0)
		{
			return (FileInfo.st_mode & S_IXUSR) != 0;
		}
#endif
		return false;
	}

	static FString GetSymlinkTarget(const TCHAR* Filename)
	{
#if PLATFORM_MAC || PLATFORM_LINUX
		ANSICHAR SymlinkTarget[PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED] = { 0 };
		if (readlink(TCHAR_TO_UTF8(Filename), SymlinkTarget, PLATFORM_MAX_FILEPATH_LENGTH_DEPRECATED) != -1)
		{
			return UTF8_TO_TCHAR(SymlinkTarget);
		}
#endif
		return TEXT("");
	}

	class FSHA256
	{
	public:
		enum { DigestSize = 32 };
		FSHA256()
		{
			SHA256_Init(&ShaContext);
		}

		void Update(const uint8* Data, uint64 ByteSize)
		{
			SHA256_Update(&ShaContext, Data, ByteSize);
		}

		void Final()
		{
			SHA256_Final(Digest.GetData(), &ShaContext);
		}

		void GetHash(uint8* OutHash)
		{
			FMemory::Memcpy(OutHash, Digest.GetData(), DigestSize);
		}

	private:
		// Stub noops for platforms without OPENSSL.
#if !BPS_WITH_OPENSSL
		struct SHA256_CTX {};
		void SHA256_Init(SHA256_CTX*) {}
		void SHA256_Update(SHA256_CTX*, const uint8*, uint64) {}
		void SHA256_Final(const uint8*, SHA256_CTX*) {}
#endif // !BPS_WITH_OPENSSL

	private:
		SHA256_CTX ShaContext;
		TArray<uint8, TFixedAllocator<DigestSize>> Digest;
	};

	struct FMIMETypeLookup
	{
	public:
		enum { NumTypes = 11 };
		FMIMETypeLookup()
		{
			int32 i = 0;
			const char* NoWild = "nnnnnnnnnnnn";
			// gif detection
			const TCHAR* GifName = TEXT("image/gif");
			const uint8* GifSig1 = (const uint8*)"\x47\x49\x46\x38\x37\x61";
			const uint8* GifSig2 = (const uint8*)"\x47\x49\x46\x38\x39\x61";
			Names[i] = GifName; Signatures[i] = GifSig1; Wildcards[i] = NoWild; SignatureSizes[i] = 6; SignatureOffsets[i++] = 0;
			Names[i] = GifName; Signatures[i] = GifSig2; Wildcards[i] = NoWild; SignatureSizes[i] = 6; SignatureOffsets[i++] = 0;
			// png detection
			const TCHAR* PngName = TEXT("image/png");
			const uint8* PngSig = (const uint8*)"\x89\x50\x4E\x47\x0D\x0A\x1A\x0A";
			Names[i] = PngName; Signatures[i] = PngSig; Wildcards[i] = NoWild; SignatureSizes[i] = 8; SignatureOffsets[i++] = 0;
			// tif detection
			const TCHAR* TifName = TEXT("image/tiff");
			const uint8* TifSig1 = (const uint8*)"\x49\x49\x2A\x00";
			const uint8* TifSig2 = (const uint8*)"\x4D\x4D\x00\x2A";
			Names[i] = TifName; Signatures[i] = TifSig1; Wildcards[i] = NoWild; SignatureSizes[i] = 4; SignatureOffsets[i++] = 0;
			Names[i] = TifName; Signatures[i] = TifSig2; Wildcards[i] = NoWild; SignatureSizes[i] = 4; SignatureOffsets[i++] = 0;
			// bmp detection
			const TCHAR* BmpName = TEXT("image/bmp");
			const uint8* BmpSig = (const uint8*)"\x42\x4D";
			Names[i] = BmpName; Signatures[i] = BmpSig; Wildcards[i] = NoWild; SignatureSizes[i] = 2; SignatureOffsets[i++] = 0;
			// jpg detection
			const TCHAR* JpgName = TEXT("image/jpeg");
			const uint8* JpgSig1 = (const uint8*)"\xFF\xD8\xFF\xDB";
			const uint8* JpgSig2 = (const uint8*)"\xFF\xD8\xFF\xE0\x00\x10\x4A\x46\x49\x46\x00\x01";
			const uint8* JpgSig3 = (const uint8*)"\xFF\xD8\xFF\xEE";
			const uint8* JpgSig4 = (const uint8*)"\xFF\xD8\xFF\xE1??\x45\x78\x69\x66\x00\x00";
			const char* JpgWild4 = "nnnnyynnnnnn";
			Names[i] = JpgName; Signatures[i] = JpgSig1; Wildcards[i] = NoWild;   SignatureSizes[i] = 4;  SignatureOffsets[i++] = 0;
			Names[i] = JpgName; Signatures[i] = JpgSig2; Wildcards[i] = NoWild;   SignatureSizes[i] = 12; SignatureOffsets[i++] = 0;
			Names[i] = JpgName; Signatures[i] = JpgSig3; Wildcards[i] = NoWild;   SignatureSizes[i] = 4;  SignatureOffsets[i++] = 0;
			Names[i] = JpgName; Signatures[i] = JpgSig4; Wildcards[i] = JpgWild4; SignatureSizes[i] = 12; SignatureOffsets[i++] = 0;
			// exe/dll detection
			const TCHAR* ExeName = TEXT("application/vnd.microsoft.portable-executable");
			const uint8* ExeSig = (const uint8*)"\x4D\x5A";
			Names[i] = ExeName; Signatures[i] = ExeSig; Wildcards[i] = NoWild;   SignatureSizes[i] = 2;  SignatureOffsets[i++] = 0;
		}

		const TCHAR* Names[NumTypes];
		const uint8* Signatures[NumTypes];
		const char* Wildcards[NumTypes];
		uint8  SignatureSizes[NumTypes];
		uint8  SignatureOffsets[NumTypes];
	};

	class FMIMEType
	{
	public:
		FMIMEType()
			: bCalculated(false)
		{
		}

		void Update(const uint8* Data, uint64 ByteSize)
		{
			if (!bCalculated)
			{
				bCalculated = true;
				for (int32 MIMEIdx = 0; MIMEIdx < FMIMETypeLookup::NumTypes; ++MIMEIdx)
				{
					const int32 SigSize = MIMETypeLookup.SignatureSizes[MIMEIdx];
					const int32 SigOffset = MIMETypeLookup.SignatureOffsets[MIMEIdx];
					const int32 SizeNeeded = SigOffset + SigSize;
					check(SigSize >= 0);
					if (SizeNeeded <= ByteSize)
					{
						bool bMatched = true;
						for (int32 ByteIdx = 0; ByteIdx < SigSize; ByteIdx++)
						{
							if (MIMETypeLookup.Wildcards[MIMEIdx][ByteIdx] == 'n' && MIMETypeLookup.Signatures[MIMEIdx][ByteIdx] != Data[SigOffset + ByteIdx])
							{
								bMatched = false;
								break;
							}
						}
						if (bMatched)
						{
							MIMEType = MIMETypeLookup.Names[MIMEIdx];
							return;
						}
					}
				}
			}
		}

		const FString& GetMIMEType()
		{
			return MIMEType;
		}

	private:
		static const FMIMETypeLookup MIMETypeLookup;
		FString MIMEType;
		bool bCalculated;
	};
	const FMIMETypeLookup FMIMEType::MIMETypeLookup;

	class FAsyncFileHahser
	{
	public:
		FAsyncFileHahser(const FDirectoryBuildStreamerConfig& InConfig)
			: Config(InConfig)
		{
			DataCopy.SetNumUninitialized(BuildPatchServices::FileBufferSize);
		}

		void AsyncUpdate(const uint8* Data, uint64 ByteSize)
		{
			check(ByteSize <= BuildPatchServices::FileBufferSize);
			Wait();
			FMemory::Memcpy(DataCopy.GetData(), Data, ByteSize);
			HashFutures[0] = Async(EAsyncExecution::ThreadPool, [this, ByteSize]() { FileSHA1Hasher.Update(DataCopy.GetData(), ByteSize); });
			if (Config.bDetectMimeType)
			{
				HashFutures[1] = Async(EAsyncExecution::ThreadPool, [this, ByteSize]() { MIMETypeCalc.Update(DataCopy.GetData(), ByteSize); });
			}
			if (Config.bCalculateMD5)
			{
				HashFutures[2] = Async(EAsyncExecution::ThreadPool, [this, ByteSize]() { FileMD5Hasher.Update(DataCopy.GetData(), ByteSize); });
			}
			if (Config.bCalculateSHA256)
			{
				HashFutures[3] = Async(EAsyncExecution::ThreadPool, [this, ByteSize]() { FileSHA256Hasher.Update(DataCopy.GetData(), ByteSize); });
			}
		}

		void Final()
		{
			Wait();
			FileSHA1Hasher.Final();
			if (Config.bCalculateSHA256)
			{
				FileSHA256Hasher.Final();
			}
			// We do not call final on MD5 here, as the setting of the value will call final internally.
		}

		void GetHash(FMD5Hash& MD5Hash)
		{
			if (Config.bCalculateMD5)
			{
				MD5Hash.Set(FileMD5Hasher);
			}
			else
			{
				FMemory::Memzero(MD5Hash);
			}
		}

		void GetHash(FSHAHash& SHA1Hash)
		{
			FileSHA1Hasher.GetHash(SHA1Hash.Hash);
		}

		void GetHash(FSHA256Signature& SHA256Hash)
		{
			if (Config.bCalculateSHA256)
			{
				FileSHA256Hasher.GetHash(SHA256Hash.Signature);
			}
			else
			{
				FMemory::Memzero(SHA256Hash);
			}
		}

		void GetMime(FString& MIMEType)
		{
			if (Config.bDetectMimeType)
			{
				MIMEType = MIMETypeCalc.GetMIMEType();
			}
			else
			{
				MIMEType.Empty();
			}
		}

	private:
		void Wait()
		{
			for (TFuture<void>& HashFuture : HashFutures)
			{
				HashFuture.Wait();
			}
		}

	private:
		const FDirectoryBuildStreamerConfig& Config;
		FSHA1 FileSHA1Hasher;
		FMIMEType MIMETypeCalc;
		FMD5 FileMD5Hasher;
		FSHA256 FileSHA256Hasher;
		TFuture<void> HashFutures[4];
		TArray<uint8> DataCopy;
	};

	class FDataStream
	{
	public:
		FDataStream();
		~FDataStream();
		void Clear();
		uint32 FreeSpace() const;
		uint32 UsedSpace() const;
		uint64 TotalDataPushed() const;
		void EnqueueData(const uint8* Buffer, const uint32& Len);
		uint32 DequeueData(uint8* Buffer, const uint32& ReqSize, bool WaitForData);
		bool IsEndOfStream() const;
		void SetEndOfStream();

	private:
		mutable FCriticalSection BuildDataStreamCS;
		TRingBuffer<uint8> BuildDataStream;
		FThreadSafeBool EndOfStream;
	};

	class FDirectoryBuildStreamer
		: public IDirectoryBuildStreamer
	{

	public:
		FDirectoryBuildStreamer(FDirectoryBuildStreamerConfig Config, FDirectoryBuildStreamerDependencies Dependencies);
		virtual ~FDirectoryBuildStreamer();

		// IBuildStreamer interface begin.
		virtual uint32 DequeueData(uint8* Buffer, uint32 ReqSize, bool WaitForData = true) override;
		virtual bool IsEndOfData() const override;
		// IBuildStreamer interface end.

		// IDirectoryBuildStreamer interface begin.
		virtual uint64 GetBuildSize() const override;
		virtual TArray<FFileSpan> GetAllFiles() const override;
		virtual bool HasAborted() const override;
		// IDirectoryBuildStreamer interface end.

	private:
		void ReadData();

	private:
		const FDirectoryBuildStreamerConfig Config;
		const FDirectoryBuildStreamerDependencies Dependencies;
		FDataStream DataStream;
		mutable FCriticalSection FilesCS;
		TArray<FFileSpan> Files;
		TFuture<void> Future;
		FThreadSafeBool bShouldAbort;
	};

	FDataStream::FDataStream()
		: BuildDataStream(BuildPatchServices::StreamBufferSize)
		, EndOfStream(false)
	{
	}

	FDataStream::~FDataStream()
	{
	}

	void FDataStream::Clear()
	{
		BuildDataStreamCS.Lock();
		BuildDataStream.Empty();
		BuildDataStreamCS.Unlock();
	}

	uint32 FDataStream::FreeSpace() const
	{
		uint32 rtn;
		BuildDataStreamCS.Lock();
		rtn = BuildDataStream.RingDataSize() - BuildDataStream.RingDataUsage();
		BuildDataStreamCS.Unlock();
		return rtn;
	}

	uint32 FDataStream::UsedSpace() const
	{
		uint32 rtn;
		BuildDataStreamCS.Lock();
		rtn = BuildDataStream.RingDataUsage();
		BuildDataStreamCS.Unlock();
		return rtn;
	}

	uint64 FDataStream::TotalDataPushed() const
	{
		uint64 rtn;
		BuildDataStreamCS.Lock();
		rtn = BuildDataStream.TotalDataPushed();
		BuildDataStreamCS.Unlock();
		return rtn;
	}

	void FDataStream::EnqueueData(const uint8* Buffer, const uint32& Len)
	{
		checkf(!EndOfStream, TEXT("More data was added after specifying the end of stream"));
		BuildDataStreamCS.Lock();
		while (FreeSpace() < Len)
		{
			BuildDataStreamCS.Unlock();
			FPlatformProcess::Sleep(0.01f);
			BuildDataStreamCS.Lock();
		}
		BuildDataStream.Enqueue(Buffer, Len);
		BuildDataStreamCS.Unlock();
	}

	uint32 FDataStream::DequeueData(uint8* Buffer, const uint32& ReqSize, bool WaitForData)
	{
		BuildDataStreamCS.Lock();
		uint32 ReadLen = BuildDataStream.Dequeue(Buffer, ReqSize);
		if (WaitForData && ReadLen < ReqSize)
		{
			while (ReadLen < ReqSize && !EndOfStream)
			{
				BuildDataStreamCS.Unlock();
				FPlatformProcess::Sleep(0.01f);
				BuildDataStreamCS.Lock();
				ReadLen += BuildDataStream.Dequeue(&Buffer[ReadLen], ReqSize - ReadLen);
			}
		}
		BuildDataStreamCS.Unlock();
		return ReadLen;
	}

	bool FDataStream::IsEndOfStream() const
	{
		return EndOfStream;
	}

	void FDataStream::SetEndOfStream()
	{
		EndOfStream = true;
	}

	FDirectoryBuildStreamer::FDirectoryBuildStreamer(FDirectoryBuildStreamerConfig InConfig, FDirectoryBuildStreamerDependencies InDependencies)
		: Config(MoveTemp(InConfig))
		, Dependencies(MoveTemp(InDependencies))
		, bShouldAbort(false)
	{
		TFunction<void()> Task = [this]() { ReadData(); };
		Future = Async(EAsyncExecution::Thread, MoveTemp(Task));
	}

	FDirectoryBuildStreamer::~FDirectoryBuildStreamer()
	{
		bShouldAbort = true;
		DataStream.Clear();
		Future.Wait();
	}

	uint32 FDirectoryBuildStreamer::DequeueData(uint8* Buffer, uint32 ReqSize, bool WaitForData /*= true*/)
	{
		return DataStream.DequeueData(Buffer, ReqSize, WaitForData);
	}

	bool FDirectoryBuildStreamer::IsEndOfData() const
	{
		return DataStream.IsEndOfStream() && DataStream.UsedSpace() == 0;
	}

	uint64 FDirectoryBuildStreamer::GetBuildSize() const
	{
		check(DataStream.IsEndOfStream());
		return DataStream.TotalDataPushed();
	}

	TArray<FFileSpan> FDirectoryBuildStreamer::GetAllFiles() const
	{
		check(DataStream.IsEndOfStream());
		FScopeLock ScopeLock(&FilesCS);
		return Files;
	}

	bool FDirectoryBuildStreamer::HasAborted() const
	{
		return bShouldAbort;
	}

	void FDirectoryBuildStreamer::ReadData()
	{
		// Stats
		volatile int64* StatFileOpenTime = Dependencies.StatsCollector->CreateStat(TEXT("Build Stream: Open Time"), EStatFormat::Timer);
		volatile int64* StatFileReadTime = Dependencies.StatsCollector->CreateStat(TEXT("Build Stream: Read Time"), EStatFormat::Timer);
		volatile int64* StatFileHashTime = Dependencies.StatsCollector->CreateStat(TEXT("Build Stream: Hash Time"), EStatFormat::Timer);
		volatile int64* StatDataEnqueueTime = Dependencies.StatsCollector->CreateStat(TEXT("Build Stream: Enqueue Time"), EStatFormat::Timer);
		volatile int64* StatDataAccessSpeed = Dependencies.StatsCollector->CreateStat(TEXT("Build Stream: Data Access Speed"), EStatFormat::DataSpeed);
		volatile int64* StatPotentialThroughput = Dependencies.StatsCollector->CreateStat(TEXT("Build Stream: Potential Throughput"), EStatFormat::DataSpeed);
		volatile int64* StatTotalDataRead = Dependencies.StatsCollector->CreateStat(TEXT("Build Stream: Total Data Read"), EStatFormat::DataSize);
		uint64 TempValue;

		// Clear the build stream
		DataStream.Clear();

		// Allocate our file read buffer
		uint8* FileReadBuffer = new uint8[BuildPatchServices::FileBufferSize];

		for (const FString& BuildFile : Config.BuildFiles)
		{
			FAsyncFileHahser Hasher(Config);

			if (bShouldAbort)
			{
				break;
			}

			// Read the file
			const FString SourceFile = Config.BuildRoot / BuildFile;
			FStatsCollector::AccumulateTimeBegin(TempValue);
			TUniquePtr<FArchive> FileReader = Dependencies.FileSystem->CreateFileReader(*SourceFile);
			uint32 LastError = FileReader.IsValid() ? 0 : FPlatformMisc::GetLastError();
			bool bIsUnixExe = IsUnixExecutable(*SourceFile);
			FString SymlinkTarget = Config.bFollowSymlinks ? TEXT("") : GetSymlinkTarget(*SourceFile);
			FStatsCollector::AccumulateTimeEnd(StatFileOpenTime, TempValue);
			// Not being able to load a required file from the build is a hard error.
			if (SymlinkTarget.IsEmpty() && !FileReader.IsValid())
			{
				UE_LOGF(LogBuildStreamer, Error, "Failed to open file reader for %ls (%u)", *SourceFile, LastError);
				bShouldAbort = true;
				break;
			}

			// Process files that have bytes.
			uint64 FileStartIdx = DataStream.TotalDataPushed();
			int64 FileSize = SymlinkTarget.IsEmpty() ? FileReader->TotalSize() : 0;
			if (FileSize > 0)
			{
				while (!FileReader->AtEnd() && !bShouldAbort)
				{
					// Read data file
					const int64 SizeLeft = FileSize - FileReader->Tell();
					const uint32 ReadLen = FMath::Min< int64 >(BuildPatchServices::FileBufferSize, SizeLeft);
					FStatsCollector::AccumulateTimeBegin(TempValue);
					FileReader->Serialize(FileReadBuffer, ReadLen);
					FStatsCollector::AccumulateTimeEnd(StatFileReadTime, TempValue);
					FStatsCollector::Accumulate(StatTotalDataRead, ReadLen);

					// Kick off hashing.
					FStatsCollector::AccumulateTimeBegin(TempValue);
					Hasher.AsyncUpdate(FileReadBuffer, ReadLen);
					FStatsCollector::AccumulateTimeEnd(StatFileHashTime, TempValue);

					// Copy into data stream
					FStatsCollector::AccumulateTimeBegin(TempValue);
					DataStream.EnqueueData(FileReadBuffer, ReadLen);
					FStatsCollector::AccumulateTimeEnd(StatDataEnqueueTime, TempValue);

					// Calculate other stats
					FStatsCollector::Set(StatDataAccessSpeed, *StatTotalDataRead / FStatsCollector::CyclesToSeconds(*StatFileOpenTime + *StatFileReadTime));
					FStatsCollector::Set(StatPotentialThroughput, *StatTotalDataRead / FStatsCollector::CyclesToSeconds(*StatFileOpenTime + *StatFileReadTime + *StatFileHashTime));
				}
			}
			// Finalise hashing.
			FStatsCollector::AccumulateTimeBegin(TempValue);
			Hasher.Final();
			FStatsCollector::AccumulateTimeEnd(StatFileHashTime, TempValue);
			// Save the file span.
			FFileSpan FileSpan(BuildFile, FileSize, FileStartIdx, bIsUnixExe, SymlinkTarget);
			Hasher.GetHash(FileSpan.MD5Hash);
			Hasher.GetHash(FileSpan.SHA1Hash);
			Hasher.GetHash(FileSpan.SHA256Hash);
			Hasher.GetMime(FileSpan.MIMEType);
			FScopeLock ScopeLock(&FilesCS);
			Files.Add(FileSpan);
			FileReader->Close();
		}

		// Mark end of build
		DataStream.SetEndOfStream();

		// Deallocate our file read buffer
		delete[] FileReadBuffer;
	}

	IDirectoryBuildStreamer* FBuildStreamerFactory::Create(FDirectoryBuildStreamerConfig Config, FDirectoryBuildStreamerDependencies Dependencies)
	{
		check(Dependencies.StatsCollector != nullptr);
		check(Dependencies.FileSystem != nullptr);
		return new FDirectoryBuildStreamer(MoveTemp(Config), MoveTemp(Dependencies));
	}
}
