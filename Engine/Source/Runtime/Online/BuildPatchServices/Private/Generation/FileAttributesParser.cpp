// Copyright Epic Games, Inc. All Rights Reserved.
#include "Generation/FileAttributesParser.h"
#include "Misc/FileHelper.h"
#include "Templates/UniquePtr.h"

DECLARE_LOG_CATEGORY_EXTERN(LogFileAttributesParser, Log, All);
DEFINE_LOG_CATEGORY(LogFileAttributesParser);

namespace BuildPatchServices
{
	class FFileAttributesParserImpl
		: public FFileAttributesParser
	{
		typedef void(*SetKeyValueAttributeFunction)(FFileAttributes&, FString);

	public:
		FFileAttributesParserImpl(IPlatformFile& PlatformFile);
		virtual ~FFileAttributesParserImpl();

		virtual bool ParseFileAttributes(const FString& MetaFilename, TMap<FString, FFileAttributes>& FileAttributes) override;

	private:
		bool FileAttributesMetaToMap(const FString& AttributesList, TMap<FString, FFileAttributes>& FileAttributesMap);

	private:
		IPlatformFile& PlatformFile;
		TMap<FString, SetKeyValueAttributeFunction> AttributeSetters;
	};

	FFileAttributesParserImpl::FFileAttributesParserImpl(IPlatformFile& InPlatformFile)
		: PlatformFile(InPlatformFile)
	{
		AttributeSetters.Add(TEXT("readonly"),   [](FFileAttributes& Attributes, FString Value){ Attributes.bReadOnly = true; });
		AttributeSetters.Add(TEXT("compressed"), [](FFileAttributes& Attributes, FString Value){ Attributes.bCompressed = true; });
		AttributeSetters.Add(TEXT("executable"), [](FFileAttributes& Attributes, FString Value){ Attributes.bUnixExecutable = true; });
		AttributeSetters.Add(TEXT("tag"),        [](FFileAttributes& Attributes, FString Value){ Attributes.InstallTags.Add(MoveTemp(Value)); });
	}

	FFileAttributesParserImpl::~FFileAttributesParserImpl()
	{
	}

	bool FFileAttributesParserImpl::ParseFileAttributes(const FString& MetaFilename, TMap<FString, FFileAttributes>& FileAttributes)
	{
		TUniquePtr<IFileHandle> Handle(PlatformFile.OpenRead(*MetaFilename));
		if (Handle)
		{
			TArray<uint8> FileData;
			FileData.AddUninitialized(Handle->Size());
			if (Handle->Read(FileData.GetData(), FileData.Num()))
			{
				FString FileDataString;
				FFileHelper::BufferToString(FileDataString, FileData.GetData(), FileData.Num());
				return FileAttributesMetaToMap(FileDataString, FileAttributes);
			}
			else
			{
				UE_LOGF(LogFileAttributesParser, Error, "Could not read meta file %ls", *MetaFilename);
			}
		}
		else
		{
			UE_LOGF(LogFileAttributesParser, Error, "Could not open meta file %ls", *MetaFilename);
		}

		return false;
	}

	bool FFileAttributesParserImpl::FileAttributesMetaToMap(const FString& AttributesList, TMap<FString, FFileAttributes>& FileAttributesMap)
	{
		const TCHAR RegexId = TEXT('R');
		const TCHAR WildcardId = TEXT('W');
		const TCHAR Quote = TEXT('\"');
		const TCHAR EOFile = TEXT('\0');
		const TCHAR EOLine = TEXT('\n');

		bool Successful = true;
		bool FoundFilename = false;

		const TCHAR* CharPtr = *AttributesList;
		while (*CharPtr != EOFile)
		{
			// Parse filename
			while (FChar::IsWhitespace(*CharPtr) && *CharPtr != EOFile) { ++CharPtr; }
			if (*CharPtr == EOFile)
			{
				if (!FoundFilename)
				{
					UE_LOGF(LogFileAttributesParser, Error, "Did not find opening quote for filename!");
					return false;
				}
				break;
			}
			// make sure file starts with a quote
			bool bUsualName = *CharPtr == Quote;
			// make sure patterns starts with R" or W"
			bool bRegexName = *CharPtr == RegexId && *(CharPtr + 1) == Quote;
			bool bWildcardName = *CharPtr == WildcardId && *(CharPtr + 1) == Quote;
			const TCHAR* FilenameStart = nullptr;
			if (bUsualName)
			{
				// skip "
				FilenameStart = ++CharPtr;
			}
			else if (bRegexName || bWildcardName)
			{
				// keep R" and W" in filename
				FilenameStart = CharPtr;
				// skip R" and W"
				CharPtr += 2;
			}
			else
			{
				UE_LOGF(LogFileAttributesParser, Error, "Did not find opening quote for filename. Pos:%lld", CharPtr - *AttributesList);
				return false;
			}
			while (*CharPtr != Quote && *CharPtr != EOFile && *CharPtr != EOLine) { ++CharPtr; }
			// Check we didn't run out of file
			if (*CharPtr == EOFile)
			{
				UE_LOG(LogFileAttributesParser, Error, TEXT("Unexpected end of file before next quote! File names must be quoted. Pos:%" SIZE_T_FMT), CharPtr - *AttributesList);
				return false;
			}
			// Check we didn't run out of line
			if (*CharPtr == EOLine)
			{
				UE_LOG(LogFileAttributesParser, Error, TEXT("Unexpected end of line before next quote! File names must be quoted. Pos:%" SIZE_T_FMT), CharPtr - *AttributesList);
				return false;
			}
			// Save positions
			const TCHAR* FilenameEnd = CharPtr++;
			if (bRegexName || bWildcardName)
			{
				// we keep the closing " for pattern matchers
				FilenameEnd++;
				CharPtr++;
			}
			const TCHAR* AttributesStart = CharPtr;
			// Parse keywords
			while (*CharPtr != Quote && *CharPtr != EOFile && *CharPtr != EOLine){ ++CharPtr; }
			// Check we hit the end of the line or file, another quote it wrong
			if (*CharPtr == Quote)
			{
				UE_LOG(LogFileAttributesParser, Error, TEXT("Unexpected Quote before end of keywords! Pos:%" SIZE_T_FMT), CharPtr - *AttributesList);
				return false;
			}
			FoundFilename = true;
			// Save position
			const TCHAR* EndOfLine = CharPtr;
			// Grab info
			FString Filename = FString(FilenameEnd - FilenameStart, FilenameStart);
			FFileAttributes& FileAttributes = FileAttributesMap.FindOrAdd(Filename);
			TArray<FString> AttributeParamsArray;
			FString AttributeParams = FString::ConstructFromPtrSize(AttributesStart, EndOfLine - AttributesStart);
			AttributeParams.ParseIntoArrayWS(AttributeParamsArray);
			if (AttributeParamsArray.Num() == 0)
			{
				UE_LOGF(LogFileAttributesParser, Error, "No attributes have been set for %ls", *Filename);
				Successful = false;
			}

			for (const FString& AttributeParam : AttributeParamsArray)
			{
				FString Key, Value;
				if(!AttributeParam.Split(TEXT(":"), &Key, &Value))
				{
					Key = AttributeParam;
				}
				if (AttributeSetters.Contains(Key))
				{
					AttributeSetters[Key](FileAttributes, MoveTemp(Value));
				}
				else
				{
					UE_LOGF(LogFileAttributesParser, Error, "Unrecognised attribute %ls for %ls", *AttributeParam, *Filename);
					Successful = false;
				}
			}
			FileAttributes.InstallTags.Sort(TLess<FString>());
		}

		if (!Successful)
		{
			UE_LOGF(LogFileAttributesParser, Error, "Please provide the correct attribute. If no attribute is needed, remove the file from the input.");
		}

		return Successful;
	}

	FFileAttributesParserRef FFileAttributesParserFactory::Create(IPlatformFile& PlatformFile)
	{
		return MakeShareable(new FFileAttributesParserImpl(PlatformFile));
	}
}
