// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Generation/RegexFind.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"

namespace BuildPatchServices
{
	struct FFileAttributes
	{
		bool bReadOnly = false;
		bool bCompressed = false;
		bool bUnixExecutable = false;
		TSet<FString> InstallTags;
	};

	class FFileAttributesMap
	{
	public:
		FFileAttributesMap() = default;

		FFileAttributesMap(TMap<FString, FFileAttributes> ExtMap)
		{
			Emplace(MoveTemp(ExtMap));
		}

		void Emplace(TMap<FString, FFileAttributes> ExtMap)
		{
			Map = MoveTemp(ExtMap);
			SortPatterns();
		}

		const TMap<FString, FFileAttributes>& Raw() const
		{
			return Map;
		}

		const TMap<FString, FFileAttributes>& RawPatterns() const
		{
			return PatternMap;
		}

		FFileAttributes& FindExactOrAdd(const FString& Name)
		{
			return Map.FindOrAdd(Name);
		}

		FFileAttributes Find(const FString& Name) const
		{
			FFileAttributes Out;
			TArray<const FFileAttributes*> AllAttributes = RegexFind(Name, PatternMap);
			if (const FFileAttributes* Attributes = Map.Find(Name))
			{
				AllAttributes.Add(Attributes);
			}
			for (const FFileAttributes* Attributes : AllAttributes)
			{
				Out.bReadOnly |= Attributes->bReadOnly;
				Out.bCompressed |= Attributes->bCompressed;
				Out.bUnixExecutable |= Attributes->bUnixExecutable;
				Out.InstallTags.Append(Attributes->InstallTags);
			}
			return Out;
		}

	private:
		void SortPatterns()
		{
			TMap<FString, FFileAttributes> TmpMap = MoveTemp(Map);
			for (TPair<FString, FFileAttributes>& Item : TmpMap)
			{
				if (FRegexFinder::IsPattern(Item.Key))
				{
					PatternMap.Add(MoveTemp(Item));
				}
				else
				{
					FPaths::NormalizeFilename(Item.Key);
					Map.Add(MoveTemp(Item));
				}
			}
		}

	private:
		TMap<FString, FFileAttributes> Map;
		TMap<FString, FFileAttributes> PatternMap;
	};

	class FFileAttributesParser
	{
	public:
		/**
		 * Loads in the file attributes meta file and populates the given map.
		 * @param   MetaFilename        The amount of data to attempt to retrieve.
		 * @param   FileAttributes      The map to populate with the attributes
		 * @return  True if the file existed and parsed successfully
		 */
		virtual bool ParseFileAttributes(const FString& MetaFilename, TMap<FString, FFileAttributes>& FileAttributes) = 0;
	};

	typedef TSharedRef<FFileAttributesParser, ESPMode::ThreadSafe> FFileAttributesParserRef;
	typedef TSharedPtr<FFileAttributesParser, ESPMode::ThreadSafe> FFileAttributesParserPtr;

	class FFileAttributesParserFactory
	{
	public:
		static FFileAttributesParserRef Create(IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile());
	};
}
