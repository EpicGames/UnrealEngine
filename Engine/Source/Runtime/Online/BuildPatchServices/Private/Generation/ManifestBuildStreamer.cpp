// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/BuildStreamer.h"

#include "Algo/Transform.h"
#include "Containers/SortedMap.h"
#include "Containers/Ticker.h"
#include "Data/ChunkData.h"
#include "Misc/Paths.h"
#include "CoreMinimal.h"

#include "Core/Platform.h"
#include "Common/HttpManager.h"
#include "Common/FileSystem.h"
#include "Common/SpeedRecorder.h"
#include "Generation/ManifestBuilder.h"
#include "Installer/Statistics/CloudChunkSourceStatistics.h"
#include "Installer/Statistics/DownloadServiceStatistics.h"
#include "Installer/Statistics/MemoryChunkStoreStatistics.h"
#include "Installer/ChunkEvictionPolicy.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/CloudChunkSource.h"
#include "Installer/DownloadService.h"
#include "Installer/InstallerAnalytics.h"
#include "Installer/InstallerError.h"
#include "Installer/MemoryChunkStore.h"
#include "Installer/MessagePump.h"
#include "BuildPatchProgress.h"

DECLARE_LOG_CATEGORY_EXTERN(LogManifestBuildStreamer, Log, All);
DEFINE_LOG_CATEGORY(LogManifestBuildStreamer);

namespace BuildPatchServices
{
	class FManifestBuildStreamer
		: public IManifestBuildStreamer
	{
	public:
		FManifestBuildStreamer(FManifestBuildStreamerConfig Config, FManifestBuildStreamerDependencies Dependencies);
		~FManifestBuildStreamer();

		// IBuildStreamer interface begin.
		virtual uint32 DequeueData(uint8* Buffer, uint32 ReqSize, bool WaitForData = true) override;
		virtual bool IsEndOfData() const override;
		// IBuildStreamer interface end.

		// IManifestBuildStreamer interface begin.
		virtual const FBlockStructure& GetBlockStructure() const override;
		// IManifestBuildStreamer interface end.
		
		void Initialise();

	private:
		const FManifestBuildStreamerConfig Config;
		const FManifestBuildStreamerDependencies Dependencies;
		TUniquePtr<IChunkReferenceTracker> ChunkReferenceTracker;
		TUniquePtr<ICloudChunkSource> CloudChunkSource;
		TArray<FChunkPart> DataStack;
		uint32 TopStackOffset;
	};

	FManifestBuildStreamer::FManifestBuildStreamer(FManifestBuildStreamerConfig InConfig, FManifestBuildStreamerDependencies InDependencies)
		: Config(MoveTemp(InConfig))
		, Dependencies(MoveTemp(InDependencies))
		, TopStackOffset(0)
	{
	}

	FManifestBuildStreamer::~FManifestBuildStreamer()
	{
	}

	uint32 FManifestBuildStreamer::DequeueData(uint8* Buffer, uint32 ReqSize, bool WaitForData /*= true*/)
	{
		uint32 GrabbedBytes = 0;
		if (DataStack.Num() > 0)
		{
			while (DataStack.Num() && GrabbedBytes < ReqSize)
			{
				const FChunkPart& NextData = DataStack.Top();
				const uint32 DataOffset = NextData.Offset + TopStackOffset;
				const uint32 DataSize = FMath::Min(NextData.Size - TopStackOffset, ReqSize - GrabbedBytes);
				IChunkDataAccess* ChunkDataAccess = CloudChunkSource->Get(NextData.Guid);
				if (ChunkDataAccess != nullptr)
				{
					uint8* Data;
					ChunkDataAccess->GetDataLock(&Data, nullptr);
					FMemory::Memcpy(&Buffer[GrabbedBytes], &Data[DataOffset], DataSize);
					ChunkDataAccess->ReleaseDataLock();
					GrabbedBytes += DataSize;
					TopStackOffset += DataSize;
					if (TopStackOffset >= NextData.Size)
					{
						if (!ChunkReferenceTracker->PopReference(NextData.Guid))
						{
							UE_LOGF(LogManifestBuildStreamer, Fatal, "Ordering failure, lost track of references.");
						}
						TopStackOffset = 0;
						DataStack.Pop();
					}
				}
				else
				{
					UE_LOGF(LogManifestBuildStreamer, Error, "Could not get chunk data (%ls).", *NextData.Guid.ToString());
					break;
				}
			}
		}
		return GrabbedBytes;
	}

	bool FManifestBuildStreamer::IsEndOfData() const
	{
		return DataStack.Num() == 0;
	}

	const FBlockStructure& FManifestBuildStreamer::GetBlockStructure() const
	{
		return Config.DesiredBytes;
	}

	void FManifestBuildStreamer::Initialise()
	{
		TArray<FGuid> CustomChunkReferences;
		TArray<FString> BuildFiles;
		Dependencies.Manifest->GetFileList(BuildFiles);

		// Convert our desired bytes block structure into a stack of chunk parts.
		uint64 ChunkPartStart = 0;
		const FBlockEntry* BlockEntry = Config.DesiredBytes.GetHead();
		for (const FString& BuildFile : BuildFiles)
		{
			if (BlockEntry == nullptr)
			{
				break;
			}
			const FFileManifest* FileManifest = Dependencies.Manifest->GetFileManifest(BuildFile);
			if (FileManifest == nullptr)
			{
				UE_LOGF(LogManifestBuildStreamer, Fatal, "Could not get file manifest (%ls).", *BuildFile);
			}
			for (const FChunkPart& ChunkPart : FileManifest->ChunkParts)
			{
				const FBlockRange ChunkPartRange = FBlockRange::FromFirstAndSize(ChunkPartStart, ChunkPart.Size);
				if (BlockEntry == nullptr)
				{
					break;
				}
				while (BlockEntry != nullptr)
				{
					const FBlockRange& BlockEntryRange = BlockEntry->AsRange();

					// If BlockEntry entirely before ChunkPart, advance BlockEntry.
					if (BlockEntryRange.GetLast() < ChunkPartRange.GetFirst())
					{
						BlockEntry = BlockEntry->GetNext();
						continue;
					}
					// If ChunkPart entirely before BlockEntry, advance ChunkPart.
					else if(ChunkPartRange.GetLast() < BlockEntryRange.GetFirst())
					{
						break;
					}
					check(BlockEntryRange.Overlaps(ChunkPartRange));
					DataStack.Add(ChunkPart);
					FChunkPart& NewChunkPart = DataStack.Last();
					// Chopping start of chunk part?
					bool bUsedChunkPartEnd = true;
					if (ChunkPartRange.GetFirst() < BlockEntryRange.GetFirst())
					{
						const int64 ChopSize = BlockEntryRange.GetFirst() - ChunkPartRange.GetFirst();
						NewChunkPart.Offset += ChopSize;
						NewChunkPart.Size -= ChopSize;
					}
					// Chopping end of chunk part?
					if (ChunkPartRange.GetLast() > BlockEntryRange.GetLast())
					{
						const int64 ChopSize = ChunkPartRange.GetLast() - BlockEntryRange.GetLast();
						NewChunkPart.Size -= ChopSize;
						bUsedChunkPartEnd = false;
					}
					// If we used end of chunk part, advance chunk part.
					if (bUsedChunkPartEnd)
					{
						break;
					}
					// Else if we trimmed end of chunk part, advance block entry.
					else
					{
						BlockEntry = BlockEntry->GetNext();
						continue;
					}
				}
				ChunkPartStart += ChunkPart.Size;
			}
		}
		Algo::Transform(DataStack, CustomChunkReferences, &FChunkPart::Guid);
		Algo::Reverse(DataStack);

		ChunkReferenceTracker.Reset(Dependencies.ChunkReferenceTrackerFactory->Create(MoveTemp(CustomChunkReferences)));
		CloudChunkSource.Reset(Dependencies.CloudChunkSourceFactory->Create(ChunkReferenceTracker.Get()));
	}

	IManifestBuildStreamer* FBuildStreamerFactory::Create(FManifestBuildStreamerConfig Config, FManifestBuildStreamerDependencies Dependencies)
	{
		check(Dependencies.ChunkReferenceTrackerFactory != nullptr);
		check(Dependencies.CloudChunkSourceFactory != nullptr);
		check(Dependencies.StatsCollector != nullptr);
		check(Dependencies.Manifest != nullptr);
		FManifestBuildStreamer* ManifestBuildStreamer = new FManifestBuildStreamer(MoveTemp(Config), MoveTemp(Dependencies));
		ManifestBuildStreamer->Initialise();
		return ManifestBuildStreamer;
	}
}
