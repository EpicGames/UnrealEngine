// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/ManifestBuilder.h"

#include "Algo/Accumulate.h"
#include "BuildPatchServicesModule.h"
#include "BuildPatchUtil.h"
#include "Common/FileSystem.h"
#include "Data/ManifestData.h"
#include "Misc/Paths.h"

DECLARE_LOG_CATEGORY_EXTERN(LogManifestBuilder, Log, All);
DEFINE_LOG_CATEGORY(LogManifestBuilder);

namespace BuildPatchServices
{
	FManifestBuilderConfig::FManifestBuilderConfig()
		: bAllowEmptyBuilds(false)
	{
	}

	struct FFileBlock
	{
	public:
		FFileBlock(FGuid InChunkGuid, uint64 InFileOffset, uint64 InChunkOffset, uint64 InSize)
			: ChunkGuid(InChunkGuid)
			, FileOffset(InFileOffset)
			, ChunkOffset(InChunkOffset)
			, Size(InSize)
		{}

	public:
		FGuid ChunkGuid;
		uint64 FileOffset;
		uint64 ChunkOffset;
		uint64 Size;
	};

	class FManifestBuilder
		: public IManifestBuilder
	{
	public:
		FManifestBuilder(IFileSystem* FileSystem, const FManifestBuilderConfig& Config, const FManifestDetails& Details);
		virtual ~FManifestBuilder();

		virtual void AddChunkMatch(const FGuid& ChunkGuid, const FBlockStructure& Structure) override;
		virtual bool FinalizeData(const TArray<FFileSpan>& FileSpans, TArray<FChunkInfo> ChunkInfo) override;
		virtual bool SaveToFile(const FString& Filename, FSHAHash* OutSHA1Hash, FMD5Hash* OutMD5Hash) override;
		virtual FBuildPatchAppManifest& GetManifest() const override;

	private:
		TArray<FChunkPart> GetChunkPartsForFile(uint64 StartIdx, uint64 Size, TSet<FGuid>& ReferencedChunks);

	private:
		IFileSystem* FileSystem;
		const FManifestBuilderConfig Config;

		FBuildPatchAppManifestRef Manifest;
		FGuid ManifestEncryptionSecretId;
		TArray<uint8> ManifestEncryptionSecretKey;
		FFileAttributesMap FileAttributesMap;
		FBlockStructure BuildStructureAdded;

		TMap<FGuid, TArray<FBlockStructure>> AllMatches;

		bool bFinalizationSucceeded;
	};

	FManifestBuilder::FManifestBuilder(IFileSystem* InFileSystem, const FManifestBuilderConfig& InConfig, const FManifestDetails& InDetails)
		: FileSystem(InFileSystem)
		, Config(InConfig)
		, Manifest(MakeShareable(new FBuildPatchAppManifest()))
		, FileAttributesMap(InDetails.FileAttributesMap)
		, bFinalizationSucceeded(false)
	{
		Manifest->ManifestMeta.FeatureLevel = InDetails.FeatureLevel;
		Manifest->ManifestMeta.bIsFileData = false;
		Manifest->ManifestMeta.AppID = InDetails.AppId;
		Manifest->ManifestMeta.AppName = InDetails.AppName;
		Manifest->ManifestMeta.BuildVersion = InDetails.BuildVersion;
		Manifest->ManifestMeta.LaunchExe = InDetails.LaunchExe;
		Manifest->ManifestMeta.LaunchCommand = InDetails.LaunchCommand;
		Manifest->ManifestMeta.PrereqIds = InDetails.PrereqIds;
		Manifest->ManifestMeta.PrereqName = InDetails.PrereqName;
		Manifest->ManifestMeta.PrereqPath = InDetails.PrereqPath;
		Manifest->ManifestMeta.PrereqArgs = InDetails.PrereqArgs;
		Manifest->ManifestMeta.UninstallActionPath = InDetails.UninstallActionPath;
		Manifest->ManifestMeta.UninstallActionArgs = InDetails.UninstallActionArgs;
		ManifestEncryptionSecretId = InDetails.EncryptionSecretId;
		ManifestEncryptionSecretKey = InDetails.EncryptionSecretKey;

		for (const auto& CustomField : InDetails.CustomFields)
		{
			EVariantTypes VarType = CustomField.Value.GetType();
			if (VarType == EVariantTypes::Float || VarType == EVariantTypes::Double)
			{
				Manifest->SetCustomField(CustomField.Key, (double)CustomField.Value);
			}
			else if (VarType == EVariantTypes::Int8 || VarType == EVariantTypes::Int16 || VarType == EVariantTypes::Int32 || VarType == EVariantTypes::Int64 ||
				VarType == EVariantTypes::UInt8 || VarType == EVariantTypes::UInt16 || VarType == EVariantTypes::UInt32 || VarType == EVariantTypes::UInt64)
			{
				Manifest->SetCustomField(CustomField.Key, (int64)CustomField.Value);
			}
			else if (VarType == EVariantTypes::String)
			{
				Manifest->SetCustomField(CustomField.Key, CustomField.Value.GetValue<FString>());
			}
		}
	}

	FManifestBuilder::~FManifestBuilder()
	{
	}

	void FManifestBuilder::AddChunkMatch(const FGuid& ChunkGuid, const FBlockStructure& Structure)
	{
		// Make sure there is no intersection as that is not allowed.
		check(BuildStructureAdded.Intersect(Structure).GetHead() == nullptr);
		// Track full build matched.
		BuildStructureAdded.Add(Structure);
		// Add match to map. One chunk can have multiple matches.
		AllMatches.FindOrAdd(ChunkGuid).Add(Structure);
		UE_LOGF(LogManifestBuilder, Verbose, "Match added for chunk %ls.", *ChunkGuid.ToString());
	}

	bool FManifestBuilder::FinalizeData(const TArray<FFileSpan>& FileSpans, TArray<FChunkInfo> ChunkInfo)
	{
		// Keep track of referenced chunks so we can trim the list down.
		TSet<FGuid> ReferencedChunks;
		// For each file create its manifest.
		for (const FFileSpan& FileSpan : FileSpans)
		{
			FFileAttributes FileAttributes = FileAttributesMap.Find(FileSpan.Filename);
			Manifest->FileManifestList.FileList.AddDefaulted();
			FFileManifest& FileManifest = Manifest->FileManifestList.FileList.Last();
			FileManifest.Filename = FileSpan.Filename;
			FileManifest.MD5Hash = FileSpan.MD5Hash;
			FileManifest.SHA1Hash = FileSpan.SHA1Hash;
			FileManifest.SHA256Hash = FileSpan.SHA256Hash;
			FileManifest.MIMEType = FileSpan.MIMEType;
			FileManifest.InstallTags = FileAttributes.InstallTags.Array();
			FileManifest.SymlinkTarget = FileSpan.SymlinkTarget;
			if (FileAttributes.bReadOnly)
			{
				FileManifest.FileMetaFlags |= EFileMetaFlags::ReadOnly;
			}
			if (FileAttributes.bCompressed)
			{
				FileManifest.FileMetaFlags |= EFileMetaFlags::Compressed;
			}
			if (FileAttributes.bUnixExecutable || FileSpan.IsUnixExecutable)
			{
				FileManifest.FileMetaFlags |= EFileMetaFlags::UnixExecutable;
			}
			FileManifest.ChunkParts = GetChunkPartsForFile(FileSpan.StartIdx, FileSpan.Size, ReferencedChunks);
		}
		UE_LOGF(LogManifestBuilder, Verbose, "Manifest references %d chunks.", ReferencedChunks.Num());

		// Setup chunk list, removing all that were not referenced.
		Manifest->ChunkDataList.ChunkList = MoveTemp(ChunkInfo);
		int32 TotalChunkListNum = Manifest->ChunkDataList.ChunkList.Num();
		Manifest->ChunkDataList.ChunkList.RemoveAll([&](FChunkInfo& Candidate){ return ReferencedChunks.Contains(Candidate.Guid) == false; });
		UE_LOGF(LogManifestBuilder, Verbose, "Chunk info list trimmed from %d to %d.", TotalChunkListNum, Manifest->ChunkDataList.ChunkList.Num());

		// Call OnPostLoad for the file manifest list.
		Manifest->FileManifestList.OnPostLoad();
		// Init the manifest, and we are done.
		Manifest->InitLookups();

		// Sanity check expected file sizes.
		for (const FFileSpan& FileSpan : FileSpans)
		{
			check(Manifest->FileManifestLookup[FileSpan.Filename]->FileSize == FileSpan.Size);
		}

		// Sanity check all chunk info was provided.
		bool bHasAllInfo = true;
		for (const FGuid& ReferencedChunk : ReferencedChunks)
		{
			uint64 ChunkHash;
			if(Manifest->GetChunkHash(ReferencedChunk, ChunkHash) == false)
			{
				UE_LOGF(LogManifestBuilder, Error, "Generated manifest is missing ChunkInfo for chunk %ls.", *ReferencedChunk.ToString());
				bHasAllInfo = false;
			}
		}
		if (bHasAllInfo == false)
		{
			return false;
		}

		// Insert the legacy SHA-based prereq id if we have a prereq path specified but no prereq id.
		if (Manifest->ManifestMeta.PrereqIds.Num() == 0 && !Manifest->ManifestMeta.PrereqPath.IsEmpty())
		{
			UE_LOGF(LogManifestBuilder, Log, "Setting PrereqIds to be the SHA hash of the PrereqPath.");
			FSHAHash PrereqHash;
			Manifest->GetFileHash(Manifest->ManifestMeta.PrereqPath, PrereqHash);
			Manifest->ManifestMeta.PrereqIds.Add(PrereqHash.ToString());
		}

		// Some sanity checks for build integrity.
		// We allow an empty build only if config said to, but always require a whole one if not empty.
		const int64 BuildStructureSize = BuildStructureAdded.GetHead() == nullptr ? 0 : BuildStructureAdded.GetHead()->GetSize();
		// A structure is considered whole if there is either 0 or 1 block.
		const bool bBuildStructureIsWhole = BuildStructureAdded.GetHead() == nullptr ? true : BuildStructureAdded.GetHead()->GetNext() == nullptr;
		if (BuildStructureSize == 0 && !Config.bAllowEmptyBuilds)
		{
			UE_LOGF(LogManifestBuilder, Error, "Build structure is empty and was not explicitly allowed.");
			return false;
		}
		if (!bBuildStructureIsWhole)
		{
			UE_LOGF(LogManifestBuilder, Error, "Build structure added was not whole or complete.");
			return false;
		}
		if (BuildStructureSize != Manifest->GetBuildSize())
		{
			UE_LOGF(LogManifestBuilder, Error, "Generated manifest build size did not equal build structure added.");
			return false;
		}

		if (Manifest->ManifestMeta.FeatureLevel <= EFeatureLevel::UsesRuntimeGeneratedBuildId)
		{
			Manifest->ManifestMeta.BuildId = FBuildPatchUtils::GetBackwardsCompatibleBuildId(Manifest->ManifestMeta);
		}

		if (ManifestEncryptionSecretId.IsValid())
		{
			if (!Manifest->EncryptData(ManifestEncryptionSecretId, ManifestEncryptionSecretKey))
			{
				UE_LOGF(LogManifestBuilder, Error, "Generated manifest failed to encrypt properly.");
				return false;
			}
		}

		// Everything seems fine.
		bFinalizationSucceeded = true;
		return true;
	}

	bool FManifestBuilder::SaveToFile(const FString& Filename, FSHAHash* OutSHA1Hash, FMD5Hash* OutMD5Hash)
	{
		// Check that previous validation from FinalizeData succeeded, the error should have been picked up.
		bool bSuccess = false;
		if (bFinalizationSucceeded)
		{
			const FString TmpFilename = Filename + TEXT("tmp");
			bSuccess = Manifest->SaveToFile(TmpFilename, Manifest->ManifestMeta.FeatureLevel, OutSHA1Hash, OutMD5Hash);
			if (!bSuccess)
			{
				UE_LOGF(LogManifestBuilder, Error, "Could not export manifest to %ls.", *TmpFilename);
				return bSuccess;
			}
			bSuccess = FileSystem->MoveFile(*Filename, *TmpFilename);
			if (!bSuccess)
			{
				UE_LOGF(LogManifestBuilder, Error, "Could not move manifest from %ls to %ls.", *TmpFilename, *Filename);
			}
		}
		else
		{
			UE_LOGF(LogManifestBuilder, Error, "Cannot save manifest when build integrity check did not succeed.");
		}
		return bSuccess;
	}

	FBuildPatchAppManifest& FManifestBuilder::GetManifest() const
	{
		check(bFinalizationSucceeded);
		return *Manifest;
	}

	TArray<FChunkPart> FManifestBuilder::GetChunkPartsForFile(uint64 FileStart, uint64 FileSize, TSet<FGuid>& ReferencedChunks)
	{
		TArray<FChunkPart> FileChunkParts;
		// Short circuit empty files
		if (FileSize == 0)
		{
			return FileChunkParts;
		}
		// Collect all matching blocks.
		TArray<FFileBlock> MatchingBlocks;
		uint64 FileEnd = FileStart + FileSize;
		uint64 SizeCountCheck = 0;
		for (const TPair<FGuid, TArray<FBlockStructure>>& Match : AllMatches)
		{
			for (const FBlockStructure& BlockStructure : Match.Value)
			{
				const FBlockEntry* BlockEntry = BlockStructure.GetHead();
				uint64 ChunkOffset = 0;
				while (BlockEntry != nullptr)
				{
					uint64 BlockEnd = BlockEntry->GetOffset() + BlockEntry->GetSize();
					if (BlockEntry->GetOffset() < FileEnd && BlockEnd > FileStart)
					{
						uint64 IntersectStart = FMath::Max<uint64>(BlockEntry->GetOffset(), FileStart);
						uint64 IntersectEnd = FMath::Min<uint64>(BlockEnd, FileEnd);
						uint64 IntersectSize = IntersectEnd - IntersectStart;
						ChunkOffset += IntersectStart - BlockEntry->GetOffset();
						check(IntersectSize > 0);
						SizeCountCheck += IntersectSize;
						MatchingBlocks.Emplace(Match.Key, IntersectStart, ChunkOffset, IntersectSize);
						ReferencedChunks.Add(Match.Key);
						ChunkOffset += BlockEntry->GetSize() - (IntersectStart - BlockEntry->GetOffset());
					}
					else
					{
						ChunkOffset += BlockEntry->GetSize();
					}
					BlockEntry = BlockEntry->GetNext();
				}
			}
		}
		check(SizeCountCheck == FileSize);

		// Sort the matches by file position.
		struct FFileBlockSort
		{
			FORCEINLINE bool operator()(const FFileBlock& A, const FFileBlock& B) const
			{
				return A.FileOffset < B.FileOffset;
			}
		};
		MatchingBlocks.Sort(FFileBlockSort());

		// Add the info to the return array.
		for (const FFileBlock& MatchingBlock : MatchingBlocks)
		{
			FileChunkParts.AddDefaulted();
			FChunkPart& ChunkPart = FileChunkParts.Last();
			ChunkPart.Guid = MatchingBlock.ChunkGuid;
			ChunkPart.Offset = MatchingBlock.ChunkOffset;
			ChunkPart.Size = MatchingBlock.Size;
		}
		return FileChunkParts;
	}

	IManifestBuilderRef FManifestBuilderFactory::Create(IFileSystem* FileSystem, const FManifestBuilderConfig& Config, const FManifestDetails& Details)
	{
		return MakeShareable(new FManifestBuilder(FileSystem, Config, Details));
	}
}
