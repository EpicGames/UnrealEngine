// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Variant.h"
#include "BuildPatchManifest.h"
#include "Core/BlockStructure.h"
#include "Core/FileSpan.h"
#include "Generation/FileAttributesParser.h"

namespace BuildPatchServices
{
	struct FManifestDetails
	{
	public:
		// The feature level support the manifest should be serialised as.
		EFeatureLevel FeatureLevel;
		// The ID of the app of this build
		uint32 AppId;
		// The name of the app of this build
		FString AppName;
		// The version string for this build
		FString BuildVersion;
		// The local exe path that would launch this build
		FString LaunchExe;
		// The command line that would launch this build
		FString LaunchCommand;
		// The set of identifiers which the prerequisites satisfy
		TSet<FString> PrereqIds;
		// The display name of the prerequisites installer
		FString PrereqName;
		// The path to the prerequisites installer
		FString PrereqPath;
		// The command line arguments for the prerequisites installer
		FString PrereqArgs;
		// The path to the uninstall custom action executable.
		FString UninstallActionPath;
		// The arguments to the uninstall custom action executable.
		FString UninstallActionArgs;
		// Map of custom fields to add to the manifest
		TMap<FString, FVariant> CustomFields;
		// Map of file attributes
		FFileAttributesMap FileAttributesMap;
		// The encryption secret key to use for this manifest.
		TArray<uint8> EncryptionSecretKey;
		// The ID of the encryption secret key used by this manifest.
		FGuid EncryptionSecretId;
	};

	struct FManifestBuilderConfig
	{
	public:
		/**
		 * Default constructor. Initializes all members with default behavior values.
		 */
		FManifestBuilderConfig();

	public:
		// Whether or not we should allow build manifests with no data.
		bool bAllowEmptyBuilds;
	};

	class IManifestBuilder
	{
	public:
		virtual ~IManifestBuilder() = default;
		virtual void AddChunkMatch(const FGuid& ChunkGuid, const FBlockStructure& Structure) = 0;
		virtual bool FinalizeData(const TArray<FFileSpan>& FileSpans, TArray<FChunkInfo> ChunkInfo) = 0;
		virtual bool SaveToFile(const FString& Filename, FSHAHash* OutSHA1Hash, FMD5Hash* OutMD5Hash) = 0;
		virtual FBuildPatchAppManifest& GetManifest() const = 0;
	};

	typedef TSharedRef<IManifestBuilder> IManifestBuilderRef;
	typedef TSharedPtr<IManifestBuilder> IManifestBuilderPtr;

	class FManifestBuilderFactory
	{
	public:
		static IManifestBuilderRef Create(IFileSystem* FileSystem, const FManifestBuilderConfig& Config, const FManifestDetails& Details);
	};
}
