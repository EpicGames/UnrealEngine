// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generation/PackageChunkData.h"

#include "Async/Async.h"
#include "Async/Future.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/MemoryWriter.h"
#include "HttpModule.h"

#include "Core/Platform.h"
#include "Common/ChunkDataSizeProvider.h"
#include "Common/Crypto.h"
#include "Common/FileSystem.h"
#include "Common/HttpManager.h"
#include "Common/SpeedRecorder.h"
#include "Common/StatsCollector.h"
#include "Generation/ChunkDatabaseWriter.h"
#include "Installer/Statistics/CloudChunkSourceStat.h"
#include "Installer/Statistics/CloudChunkSourceStatistics.h"
#include "Installer/Statistics/DownloadServiceStatistics.h"
#include "Installer/Statistics/FileConstructorStatistics.h"
#include "Installer/Statistics/FileOperationTracker.h"
#include "Installer/Statistics/MemoryChunkStoreStatistics.h"
#include "Installer/ChunkEvictionPolicy.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/CloudChunkSource.h"
#include "Installer/DownloadService.h"
#include "Installer/InstallerAnalytics.h"
#include "Installer/InstallerError.h"
#include "Installer/InstallerSharedContext.h"
#include "Installer/MemoryChunkStore.h"
#include "Installer/MessagePump.h"
#include "Installer/OptimisedDelta.h"
#include "BuildPatchManifest.h"
#include "BuildPatchProgress.h"
#include "IBuildManifestSet.h"
#include "Core/AsyncHelpers.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPackageChunkData, Log, All);
DEFINE_LOG_CATEGORY(LogPackageChunkData);

// For the output file we'll use pretty json in debug, otherwise condensed.
#if UE_BUILD_DEBUG
typedef  TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FPackageChunksJsonWriter;
typedef  TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FPackageChunksJsonWriterFactory;
#else
typedef  TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FPackageChunksJsonWriter;
typedef  TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FPackageChunksJsonWriterFactory;
#endif //UE_BUILD_DEBUG

namespace PackageChunksHelpers
{
	int32 GetNumDigitsRequiredForInteger(int32 Integer)
	{
		// Technically, there are mathematical solutions to this, however there can be floating point errors in log that cause edge cases there.
		// We'll just use the obvious simple method.
		return FString::Printf(TEXT("%d"), Integer).Len();
	}

	TArray<FGuid> GetCustomChunkReferences(const TArray<TSet<FString>>& TagSetArray, const FBuildPatchAppManifestRef& NewManifest, const TSet<FString>& PrevTagSet, const FBuildPatchAppManifestRef& PrevManifest)
	{
		using namespace BuildPatchServices;
		TSet<FGuid> VisitedChunks;
		TArray<FGuid> UniqueChunkReferences;
		for (const TSet<FString>& TagSet : TagSetArray)
		{
			for (const FGuid& ChunkReference : CustomChunkReferencesHelpers::OrderedUniquePatchReferencesTagged(NewManifest, TagSet, PrevManifest, PrevTagSet))
			{
				bool bIsAlreadyInSet = false;
				VisitedChunks.Add(ChunkReference, &bIsAlreadyInSet);
				if (!bIsAlreadyInSet)
				{
					UniqueChunkReferences.Add(ChunkReference);
				}
			}
		}
		return UniqueChunkReferences;
	}

	TArray<FGuid> GetCustomChunkReferences(const TArray<TSet<FString>>& TagSetArray, const FBuildPatchAppManifestRef& NewManifest)
	{
		using namespace BuildPatchServices;
		TSet<FGuid> VisitedChunks;
		TArray<FGuid> UniqueChunkReferences;
		for (const TSet<FString>& TagSet : TagSetArray)
		{
			for (const FGuid& ChunkReference : CustomChunkReferencesHelpers::OrderedUniqueReferencesTagged(NewManifest, TagSet))
			{
				bool bIsAlreadyInSet = false;
				VisitedChunks.Add(ChunkReference, &bIsAlreadyInSet);
				if (!bIsAlreadyInSet)
				{
					UniqueChunkReferences.Add(ChunkReference);
				}
			}
		}
		return UniqueChunkReferences;
	}
}

namespace BuildPatchServices
{
	namespace ChunkDbHelpers
	{
		uint32 GetChunkDbHeaderSize()
		{
			static const uint32 ChunkDbHeaderSize = []()
			{
				TArray<uint8> HeaderData;
				FMemoryWriter HeaderWriter(HeaderData);
				FChunkDatabaseHeader ChunkDbHeader;
				HeaderWriter << ChunkDbHeader;
				return HeaderData.Num();
			}();
			return ChunkDbHeaderSize;
		}

		uint32 GetPerChunkEntryHeaderSize()
		{
			static const uint32 PerEntryHeaderSize = []()
			{
				TArray<uint8> HeaderData;
				FMemoryWriter HeaderWriter(HeaderData);
				FChunkDatabaseHeader ChunkDbHeader;
				ChunkDbHeader.Contents.Add({ FGuid::NewGuid(), 0, 0 });
				HeaderWriter << ChunkDbHeader;
				return HeaderData.Num() - GetChunkDbHeaderSize();
			}();
			return PerEntryHeaderSize;
		}
	}

	class FPackageChunks
		: public IPackageChunks
	{
	public:
		FPackageChunks(const FPackageChunksConfiguration& InConfiguration);
		~FPackageChunks();

		// IPackageChunks interface begin.
		virtual bool Run() override;
		// IPackageChunks interface end.

	private:
		void HandleDownloadComplete(int32 RequestId, const FDownloadRef& Download);
		void HandleManifestComplete();
		void HandleManifestSelection(const IOptimisedDelta::FResultValueOrError& ResultValueOrError);
		bool TagsAreValid(const FBuildPatchAppManifestPtr& ManifestPtr, const TArray<TSet<FString>>& ProvidedTagSetArray, const FString& BuildType);
		bool TagsAreValid(const FBuildPatchAppManifestPtr& ManifestPtr, const TSet<FString>& ProvidedTagSetArray, const FString& BuildType);
		void BeginPackageProcess();
		void OnPackageComplete(bool bInSuccess);

		typedef void(FPackageChunks::*PromiseCompleteFunc)();
		TFunction<void()> MakePromiseCompleteDelegate(PromiseCompleteFunc OnComplete);

		typedef void(FPackageChunks::*OptimiseCompleteFunc)(const IOptimisedDelta::FResultValueOrError&);
		TFunction<void(const IOptimisedDelta::FResultValueOrError&)> MakeOptimiseCompleteDelegate(OptimiseCompleteFunc OnComplete);

	private:
		// Configuration.
		const FPackageChunksConfiguration Configuration;

		// Dependencies.
		FTSTicker& CoreTicker;
		FDownloadCompleteDelegate DownloadCompleteDelegate;
		FDownloadProgressDelegate DownloadProgressDelegate;

		// Process control.
		bool bManifestsProcessed;
		FThreadSafeBool bShouldRun;
		FThreadSafeBool bSuccess;

		// Manifest acquisition.
		int32 RequestIdManifestFile;
		int32 RequestIdPrevManifestFile;
		TPromise<FBuildPatchAppManifestPtr> PromiseManifestFile;
		TPromise<FBuildPatchAppManifestPtr> PromisePrevManifestFile;
		TFuture<FBuildPatchAppManifestPtr> FutureManifestFile;
		TFuture<FBuildPatchAppManifestPtr> FuturePrevManifestFile;
		FBuildPatchAppManifestPtr Manifest;
		FBuildPatchAppManifestPtr PrevManifest;
		TUniquePtr<IBuildManifestSet> ManifestSet;
		bool bUsingOptimisedDelta;

		// Packaging.
		TArray<FChunkDatabaseFile> ChunkDbFiles;
		TArray<TArray<int32>> TagSetLookupTable;

		// Declare constructed systems last, to ensure they are destructed before our data members.
		TUniquePtr<IPlatform> Platform;
		TSharedPtr<IHttpManager> HttpManager;
		TSharedPtr<IFileSystem> FileSystem;
		TUniquePtr<ICrypto> Crypto;
		TUniquePtr<IMessagePump> MessagePump;
		TUniquePtr<IInstallerError> InstallerError;
		TSharedPtr<ISpeedRecorder> DownloadSpeedRecorder;
		TUniquePtr<ISpeedRecorder> DiskReadSpeedRecorder;
		TUniquePtr<ISpeedRecorder> DiskWriteSpeedRecorder;
		TSharedPtr<IChunkDataSizeProvider> ChunkDataSizeProvider;
		TSharedPtr<IInstallerAnalytics> InstallerAnalytics;
		TSharedPtr<IDownloadServiceStatistics> DownloadServiceStatistics;
		TUniquePtr<IDownloadService> DownloadService;
		TUniquePtr<IChunkReferenceTracker> ChunkReferenceTracker;
		TUniquePtr<IFileOperationTracker> FileOperationTracker;
		TSharedPtr<IOptimisedDelta> OptimisedDelta;
		FBuildPatchProgress BuildProgress;
		TUniquePtr<IMemoryChunkStoreStatistics> MemoryChunkStoreStatistics;
		TUniquePtr<ICloudChunkSourceStatistics> CloudChunkSourceStatistics;
		TUniquePtr<IFileConstructorStatistics> FileConstructorStatistics;
		TUniquePtr<IChunkDataSerialization> ChunkDataSerialization;
		TUniquePtr<IChunkEvictionPolicy> MemoryEvictionPolicy;
		TUniquePtr<IMemoryChunkStore> CloudChunkStore;
		TUniquePtr<IDownloadConnectionCount> DownloadConnectionCount;
		IBuildInstallerSharedContextPtr CloudSourceSharedContext;
		TUniquePtr<ICloudChunkSource> CloudChunkSource;
		TUniquePtr<IChunkDatabaseWriter> ChunkDatabaseWriter;
	};

	FPackageChunks::FPackageChunks(const FPackageChunksConfiguration& InConfiguration)
		: Configuration(InConfiguration)
		, CoreTicker(FTSTicker::GetCoreTicker())
		, DownloadCompleteDelegate(FDownloadCompleteDelegate::CreateRaw(this, &FPackageChunks::HandleDownloadComplete))
		, DownloadProgressDelegate()
		, bManifestsProcessed(false)
		, bShouldRun(true)
		, bSuccess(true)
		, RequestIdManifestFile(INDEX_NONE)
		, RequestIdPrevManifestFile(INDEX_NONE)
		, PromiseManifestFile(MakePromiseCompleteDelegate(&FPackageChunks::HandleManifestComplete))
		, PromisePrevManifestFile(MakePromiseCompleteDelegate(&FPackageChunks::HandleManifestComplete))
		, FutureManifestFile(PromiseManifestFile.GetFuture())
		, FuturePrevManifestFile(PromisePrevManifestFile.GetFuture())
		, bUsingOptimisedDelta(false)
		, Platform(FPlatformFactory::Create())
		, HttpManager(FHttpManagerFactory::Create())
		, FileSystem(FFileSystemFactory::Create())
		, Crypto(FCryptoFactory::Create())
		, MessagePump(FMessagePumpFactory::Create())
		, InstallerError(FInstallerErrorFactory::Create())
		, DownloadSpeedRecorder(FSpeedRecorderFactory::Create())
		, DiskReadSpeedRecorder(FSpeedRecorderFactory::Create())
		, DiskWriteSpeedRecorder(FSpeedRecorderFactory::Create())
		, ChunkDataSizeProvider(FChunkDataSizeProviderFactory::Create({}))
		, InstallerAnalytics(FInstallerAnalyticsFactory::Create(nullptr))
		, DownloadServiceStatistics(FDownloadServiceStatisticsFactory::Create(DownloadSpeedRecorder, ChunkDataSizeProvider, InstallerAnalytics))
		, DownloadService(FDownloadServiceFactory::Create(HttpManager, FileSystem, DownloadServiceStatistics, InstallerAnalytics))
		, FileOperationTracker(FFileOperationTrackerFactory::Create(CoreTicker))
	{
		// Make sure the cloud chunk source gets the abort signal if an error occurred.
		InstallerError->RegisterForErrors([this]()
		{
			AsyncHelpers::ExecuteOnGameThread<void>([this]()
			{
				if (CloudChunkSource.IsValid())
				{
					CloudChunkSource->Abort();
				}
			}).Wait();
		});
	}

	FPackageChunks::~FPackageChunks()
	{
	}

	bool FPackageChunks::Run()
	{
		// Run any core initialization required.
		FHttpModule::Get();

		// Kick off Manifest downloads.
		bShouldRun = bShouldRun && DownloadService->RequestFile(Configuration.ManifestFilePath, DownloadCompleteDelegate, DownloadProgressDelegate, &RequestIdManifestFile);
		bShouldRun = bShouldRun && DownloadService->RequestFile(Configuration.PrevManifestFilePath, DownloadCompleteDelegate, DownloadProgressDelegate, &RequestIdPrevManifestFile);
		if (!bShouldRun)
		{
			bSuccess = false;
		}

		// Main timers.
		double DeltaTime = 0.0;
		double LastTime = FPlatformTime::Seconds();

		// Setup desired frame times.
		float MainsFramerate = 100.0f;
		const float MainsFrameTime = 1.0f / MainsFramerate;

		// Run the main loop.
		while (bShouldRun)
		{
			// Increment global frame counter once for each app tick.
			GFrameCounter++;

			// Application tick.
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(DeltaTime);

			// Message pump.
			MessagePump->PumpMessages();

			// Flush any threaded logging.
			GLog->FlushThreadedLogs();

			// Control frame rate.
			FPlatformProcess::Sleep(FMath::Max<float>(0.0f, MainsFrameTime - (FPlatformTime::Seconds() - LastTime)));

			// Calculate deltas.
			const double AppTime = FPlatformTime::Seconds();
			DeltaTime = AppTime - LastTime;
			LastTime = AppTime;
		}
		GLog->FlushThreadedLogs();

		// Return success state.
		return bSuccess;
	}

	void FPackageChunks::HandleDownloadComplete(int32 RequestId, const FDownloadRef& Download)
	{
		TPromise<FBuildPatchAppManifestPtr>* RelevantPromisePtr = RequestId == RequestIdManifestFile ? &PromiseManifestFile : RequestId == RequestIdPrevManifestFile ? &PromisePrevManifestFile : nullptr;
		if (RelevantPromisePtr != nullptr)
		{
			if (Download->ResponseSuccessful())
			{
				Async(EAsyncExecution::ThreadPool, [Download, RelevantPromisePtr]()
				{
					FBuildPatchAppManifestPtr DownloadedManifest = MakeShareable(new FBuildPatchAppManifest());
					if (!DownloadedManifest->DeserializeFromData(Download->GetData()))
					{
						DownloadedManifest.Reset();
					}
					RelevantPromisePtr->SetValue(DownloadedManifest);
				});
			}
			else
			{
				RelevantPromisePtr->SetValue(FBuildPatchAppManifestPtr());
			}
		}
	}

	void FPackageChunks::HandleManifestComplete()
	{
		const bool bBothManifestsReady = FutureManifestFile.IsReady() && FuturePrevManifestFile.IsReady();
		if (bBothManifestsReady && !bManifestsProcessed)
		{
			bManifestsProcessed = true;
			Manifest = FutureManifestFile.Get();
			PrevManifest = FuturePrevManifestFile.Get();
			// Check required manifest was loaded ok.
			if (Manifest.IsValid() == false)
			{
				UE_LOGF(LogPackageChunkData, Error, "Could not download ManifestFilePath from %ls.", *Configuration.ManifestFilePath);
				bSuccess = false;
				bShouldRun = false;
			}
			// Check previous manifest was loaded ok if it was provided.
			if (Configuration.PrevManifestFilePath.IsEmpty() == false && PrevManifest.IsValid() == false)
			{
				UE_LOGF(LogPackageChunkData, Error, "Could not download PrevManifestFilePath from %ls.", *Configuration.PrevManifestFilePath);
				bSuccess = false;
				bShouldRun = false;
			}

			if ((Manifest.IsValid() && !Configuration.bIgnoreManifestFileInvalidTags && !TagsAreValid(Manifest, Configuration.TagSetArray, TEXT("build")))
				|| (PrevManifest.IsValid() && !Configuration.bIgnorePrevManifestFileInvalidTags && !TagsAreValid(PrevManifest, Configuration.PrevTagSet, TEXT("previous build"))))
			{
				bSuccess = false;
				bShouldRun = false;
			}

			if (bSuccess)
			{
				// Check for delta file, replacing Manifest if we find one
				FOptimisedDeltaConfiguration OptimisedDeltaConfiguration(Manifest.ToSharedRef());
				OptimisedDeltaConfiguration.SourceManifest = PrevManifest;
				OptimisedDeltaConfiguration.DeltaPolicy = Configuration.FeatureLevel >= EFeatureLevel::FirstOptimisedDelta ? EDeltaPolicy::TryFetchContinueWithout : EDeltaPolicy::Skip;
				OptimisedDeltaConfiguration.DeltaFilenameTrailer = Configuration.DeltaFilenameTrailer;
				TSharedRef<IUriProvider, ESPMode::ThreadSafe> UriProvider = FUriProviderFactory::Create(MessagePump.Get(), { FPaths::GetPath(Configuration.ManifestFilePath) });
				FOptimisedDeltaDependencies OptimisedDeltaDependencies(MoveTemp(UriProvider));
				OptimisedDeltaDependencies.DownloadService = DownloadService.Get();
				OptimisedDeltaDependencies.OnComplete = MakeOptimiseCompleteDelegate(&FPackageChunks::HandleManifestSelection);
				OptimisedDelta = FOptimisedDeltaFactory::Create(OptimisedDeltaConfiguration, MoveTemp(OptimisedDeltaDependencies));
			}
		}
	}

	bool FPackageChunks::TagsAreValid(const FBuildPatchAppManifestPtr& ManifestPtr, const TArray<TSet<FString>>& ProvidedTagSetArray, const FString& BuildType)
	{
		TSet<FString> ProvidedTags;
		for (const TSet<FString>& ProvidedTagSet : ProvidedTagSetArray)
		{
			ProvidedTags.Append(ProvidedTagSet);
		}
		return TagsAreValid(ManifestPtr, ProvidedTags, BuildType);
	}

	bool FPackageChunks::TagsAreValid(const FBuildPatchAppManifestPtr& ManifestPtr, const TSet<FString>& ProvidedTagSet, const FString& BuildType)
	{
		bool bTagsAreValid = true;
		TArray<FString> UnknownManifestTags;

		TSet<FString> ValidTagSet;
		ManifestPtr->GetFileTagList(ValidTagSet);
		for (const FString& ProvidedTag : ProvidedTagSet)
		{
			if (!ValidTagSet.Contains(ProvidedTag))
			{
				UnknownManifestTags.Add(ProvidedTag);
				bTagsAreValid = false;
			}
		}

		if (!bTagsAreValid)
		{
			UE_LOGF(LogPackageChunkData, Error, "The provided tags were invalid for the %ls: %ls", *BuildType, *FString::Join(UnknownManifestTags, TEXT(",")));
			UE_LOGF(LogPackageChunkData, Error, "Valid tags for the %ls are: %ls", *BuildType, *FString::Join(ValidTagSet.Array(), TEXT(",")));
		}

		return bTagsAreValid;
	}

	void FPackageChunks::HandleManifestSelection(const IOptimisedDelta::FResultValueOrError& ResultValueOrError)
	{
		FBuildPatchAppManifestPtr DeltaManifest = ResultValueOrError.IsValid() ? ResultValueOrError.GetValue() : nullptr;
		bUsingOptimisedDelta = Manifest.Get() != DeltaManifest.Get();
		if (DeltaManifest.IsValid())
		{
			Manifest = DeltaManifest;
		}
		ManifestSet.Reset(FBuildManifestSetFactory::Create({ FInstallerAction::MakeInstall(Manifest.ToSharedRef()) }));
		FileOperationTracker->OnManifestSelection(ManifestSet.Get());
		ChunkDataSizeProvider->AddManifestData(Manifest);
		BeginPackageProcess();
	}

	void FPackageChunks::BeginPackageProcess()
	{
		const TCHAR* StandardExtension = TEXT(".chunkdb");
		const TCHAR* DeltaExtension = TEXT(".delta.chunkdb");
		const TCHAR* ChunkDbExtension = bUsingOptimisedDelta ? DeltaExtension : StandardExtension;

		TArray<TSet<FString>> TagSetArray;
		TSet<FString> PrevTagSet;

		// If TagSetArray was not provided, we need to adjust it to contain an entry that uses all tags.
		if (Configuration.TagSetArray.Num() == 0)
		{
			TagSetArray.AddDefaulted();
			Manifest->GetFileTagList(TagSetArray.Last());
		}
		else
		{
			TagSetArray = Configuration.TagSetArray;
		}
		TagSetLookupTable.AddDefaulted(TagSetArray.Num());

		// Construct the chunk reference tracker, building our list of ordered unique chunk references.
		if (PrevManifest.IsValid())
		{
			// If PrevTagSet was not provided, we need to adjust it to contain an entry that uses all tags.
			if (Configuration.PrevTagSet.Num() == 0)
			{
				PrevManifest->GetFileTagList(PrevTagSet);
			}
			else
			{
				PrevTagSet = Configuration.PrevTagSet;
			}
			ChunkReferenceTracker.Reset(FChunkReferenceTrackerFactory::Create(PackageChunksHelpers::GetCustomChunkReferences(TagSetArray, Manifest.ToSharedRef(), PrevTagSet, PrevManifest.ToSharedRef())));
		}
		else
		{
			ChunkReferenceTracker.Reset(FChunkReferenceTrackerFactory::Create(PackageChunksHelpers::GetCustomChunkReferences(TagSetArray, Manifest.ToSharedRef())));
		}

		// Enumerate the chunks, allocating them to chunk db files.
		const uint64 ChunkDbHeaderSize = ChunkDbHelpers::GetChunkDbHeaderSize();
		const uint64 PerEntryHeaderSize = ChunkDbHelpers::GetPerChunkEntryHeaderSize();
		TArray<FGuid> FullDataSetArray;
		ChunkReferenceTracker->CopyOutOrderedUseList(FullDataSetArray);
		TSet<FGuid> FullDataSet(MoveTemp(FullDataSetArray));
		if (FullDataSet.Num() > 0)
		{
			// Create the data set for each tagset.
			int32 NumSetsWithData = 0;
			TArray<TSet<FGuid>> TaggedDataSets;
			TSet<FGuid> VisitedChunks;
			for (const TSet<FString>& TagSet : TagSetArray)
			{
				TSet<FGuid> TaggedChunks;
				TSet<FString> TaggedFiles;
				Manifest->GetTaggedFileList(TagSet, TaggedFiles);
				Manifest->GetChunksRequiredForFiles(TaggedFiles, TaggedChunks);
				TaggedChunks = TaggedChunks.Intersect(FullDataSet).Difference(VisitedChunks);
				if (TaggedChunks.Num() > 0)
				{
					++NumSetsWithData;
					VisitedChunks.Append(TaggedChunks);
				}
				TaggedDataSets.Add(MoveTemp(TaggedChunks));
			}
			const int32 NumDigitsForTagSets = NumSetsWithData > 1 ? PackageChunksHelpers::GetNumDigitsRequiredForInteger(TaggedDataSets.Num()) : 0;

			for (int32 ChunkDbTagSetIdx = 0; ChunkDbTagSetIdx < TaggedDataSets.Num(); ++ChunkDbTagSetIdx)
			{
				if (TaggedDataSets[ChunkDbTagSetIdx].Num() > 0)
				{
					const int32 FirstChunkDbFileIdx = ChunkDbFiles.Num();
					int32 ChunkDbPartCount = 0;
					int32 CurrentChunkDbFileIdx = FirstChunkDbFileIdx - 1;

					// Figure out the chunks to write per chunkdb file.
					uint64 AvailableFileSize = 0;
					for (const FGuid& DataId : TaggedDataSets[ChunkDbTagSetIdx])
					{
						const uint64 DataSize = Manifest->GetDataSize(DataId) + PerEntryHeaderSize;
						if (AvailableFileSize < DataSize &&
							((CurrentChunkDbFileIdx < FirstChunkDbFileIdx) || (ChunkDbFiles[CurrentChunkDbFileIdx].DataList.Num() > 0)))
						{
							ChunkDbFiles.AddDefaulted();
							++ChunkDbPartCount;
							++CurrentChunkDbFileIdx;
							AvailableFileSize = Configuration.MaxOutputFileSize - ChunkDbHeaderSize;
							TagSetLookupTable[ChunkDbTagSetIdx].Add(CurrentChunkDbFileIdx);
						}

						ChunkDbFiles[CurrentChunkDbFileIdx].DataList.Add(DataId);
						if (AvailableFileSize > DataSize)
						{
							AvailableFileSize -= DataSize;
						}
						else
						{
							AvailableFileSize = 0;
						}
					}

					// Figure out the filenames of each chunkdb.
					if (ChunkDbPartCount > 1)
					{
						// Figure out the per file filename
						FString FilenameFmt = Configuration.OutputFile;
						FilenameFmt.RemoveFromEnd(DeltaExtension);
						FilenameFmt.RemoveFromEnd(StandardExtension);
						if (NumDigitsForTagSets > 0)
						{
							FilenameFmt += FString::Printf(TEXT(".tagset%0*d"), NumDigitsForTagSets, ChunkDbTagSetIdx + 1);
						}
						// Technically, there are mathematical solutions to this, however there can be floating point errors in log that cause edge cases there.
						// We'll just use the obvious simple method.
						const int32 NumDigitsForParts = PackageChunksHelpers::GetNumDigitsRequiredForInteger(ChunkDbPartCount);
						for (int32 ChunkDbFileIdx = FirstChunkDbFileIdx; ChunkDbFileIdx < ChunkDbFiles.Num(); ++ChunkDbFileIdx)
						{
							ChunkDbFiles[ChunkDbFileIdx].DatabaseFilename = FString::Printf(TEXT("%s.part%0*d%s"), *FilenameFmt, NumDigitsForParts, (ChunkDbFileIdx - FirstChunkDbFileIdx) + 1, ChunkDbExtension);
						}
					}
					else if (ChunkDbPartCount == 1)
					{
						// Figure out the per file filename
						FString FilenameFmt = Configuration.OutputFile;
						FilenameFmt.RemoveFromEnd(DeltaExtension);
						FilenameFmt.RemoveFromEnd(StandardExtension);
						// Make sure we don't confuse any existing % characters when formatting (yes, % is a valid filename character).
						FilenameFmt.ReplaceInline(TEXT("%"), TEXT("%%"));
						if (NumDigitsForTagSets > 0)
						{
							FilenameFmt += FString::Printf(TEXT(".tagset%0*d"), NumDigitsForTagSets, ChunkDbTagSetIdx + 1);
						}
						FilenameFmt += ChunkDbExtension;
						ChunkDbFiles[CurrentChunkDbFileIdx].DatabaseFilename = FilenameFmt;
					}
				}
			}

			CloudSourceSharedContext = FBuildInstallerSharedContextFactory::Create(TEXT("PackageChunks"));
			CloudSourceSharedContext->PreallocateThreads(1);

			// Cloud config defaults.
			FCloudSourceConfig CloudSourceConfig({ Configuration.CloudDir });
			CloudSourceConfig.bBeginDownloadsOnFirstGet = false;
			CloudSourceConfig.MaxRetryCount = 30;

			// Load max download retry count from engine config.
			GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ChunkRetries"), CloudSourceConfig.MaxRetryCount, GEngineIni);
			CloudSourceConfig.MaxRetryCount = FMath::Clamp<int32>(CloudSourceConfig.MaxRetryCount, -1, 1000);

			// Load the number of retries before enabling detailed logging.
			GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("RetriesToEnableVerbose"), CloudSourceConfig.RetriesToEnableVerbose, GEngineIni);
			CloudSourceConfig.RetriesToEnableVerbose = FMath::Clamp<int32>(CloudSourceConfig.RetriesToEnableVerbose, 0, 1000);
			CloudSourceConfig.SharedContext = CloudSourceSharedContext.Get();

			TUniquePtr<FStatsCollector> StatsCollector(FStatsCollectorFactory::Create());
			TUniquePtr<ICloudChunkSourceStat> CloudChunkSourceStat(FCloudChunkSourceStatFactory::Create(StatsCollector.Get()));
			FChunkDataSerializationConfig ChunkDataSerializationConfig;
			// UE5 MERGE TODO: ChunkDataSerializationConfig.CloudChunkSourceStat = CloudChunkSourceStat.Get();

			// Create systems.
			const int32 CloudStoreId = 0;
			MemoryChunkStoreStatistics.Reset(FMemoryChunkStoreStatisticsFactory::Create(
				FileOperationTracker.Get()));
			CloudChunkSourceStatistics.Reset(FCloudChunkSourceStatisticsFactory::Create(
				InstallerAnalytics.Get(),
				&BuildProgress,
				FileOperationTracker.Get()));
			FileConstructorStatistics.Reset(FFileConstructorStatisticsFactory::Create(
				DiskReadSpeedRecorder.Get(),
				DiskWriteSpeedRecorder.Get(),
				&BuildProgress,
				FileOperationTracker.Get()));
			ChunkDataSerialization.Reset(FChunkDataSerializationFactory::Create(
				FileSystem.Get(), Crypto.Get(), ChunkDataSerializationConfig));
			MemoryEvictionPolicy.Reset(FChunkEvictionPolicyFactory::Create(
				ChunkReferenceTracker.Get()));
			CloudChunkStore.Reset(FMemoryChunkStoreFactory::Create(
				512,
				MemoryEvictionPolicy.Get(),
				nullptr,
				MemoryChunkStoreStatistics.Get(),
				nullptr));
			DownloadConnectionCount.Reset(FDownloadConnectionCountFactory::Create(FDownloadConnectionCountConfig(), DownloadServiceStatistics.Get()));
			CloudChunkSource.Reset(FCloudChunkSourceFactory::Create(
				CloudSourceConfig,
				Platform.Get(),
				CloudChunkStore.Get(),
				DownloadService.Get(),
				ChunkReferenceTracker.Get(),
				ChunkDataSerialization.Get(),
				MessagePump.Get(),
				InstallerError.Get(),
				DownloadConnectionCount.Get(),
				CloudChunkSourceStatistics.Get(),
				ManifestSet.Get(),
				FullDataSet));

			// Start an IO output thread which saves all the chunks to the chunkdbs.
			FChunkDbWriterConfig ChunkDbWriterConfig(ChunkDbFiles);
			ChunkDatabaseWriter.Reset(FChunkDatabaseWriterFactory::Create(
				ChunkDbWriterConfig,
				CloudChunkSource.Get(),
				FileSystem.Get(),
				InstallerError.Get(),
				ChunkReferenceTracker.Get(),
				ChunkDataSerialization.Get(),
				FileConstructorStatistics.Get(),
				[](float) {},
				[this](bool bInSuccess) { OnPackageComplete(bInSuccess); }));
		}
		// If there are no chunks required to install or patch the provided manifests and tags, we don't create any chunkdbs.
		else
		{
			UE_LOGF(LogPackageChunkData, Log, "No chunks were necessary for the provided manifest(s) and tagset(s). No chunkdb files were created.");
			OnPackageComplete(true);
		}
	}

	void FPackageChunks::OnPackageComplete(bool bInSuccess)
	{
		bSuccess = bSuccess && bInSuccess;

		// Check no errors were registered.
		if (InstallerError->HasError())
		{
			UE_LOGF(LogPackageChunkData, Error, "%ls: %ls", *InstallerError->GetErrorCode(), *InstallerError->GetErrorText().BuildSourceString());
			bSuccess = false;
		}
		else
		{
			UE_LOGF(LogPackageChunkData, Log, "Downloaded %ls at %ls/sec.", *FText::AsMemory(DownloadServiceStatistics->GetBytesDownloaded(), EMemoryUnitStandard::IEC).ToString(), *FText::AsMemory(DownloadSpeedRecorder->GetAverageSpeed(TNumericLimits<float>::Max()), EMemoryUnitStandard::IEC).ToString());
		}

		// Save the output.
		if (bSuccess && Configuration.ResultDataFilePath.IsEmpty() == false)
		{
			FString JsonOutput;
			TSharedRef<FPackageChunksJsonWriter> Writer = FPackageChunksJsonWriterFactory::Create(&JsonOutput);
			Writer->WriteObjectStart();
			{
				Writer->WriteArrayStart(TEXT("ChunkDbFilePaths"));
				for (const FChunkDatabaseFile& ChunkDbFile : ChunkDbFiles)
				{
					Writer->WriteValue(ChunkDbFile.DatabaseFilename);
				}
				Writer->WriteArrayEnd();
				if (Configuration.TagSetArray.Num() > 0)
				{
					Writer->WriteArrayStart(TEXT("TagSetLookupTable"));
					for (const TArray<int32>& TagSetLookup : TagSetLookupTable)
					{
						Writer->WriteArrayStart();
						for (const int32& ChunkDbFileIdx : TagSetLookup)
						{
							Writer->WriteValue(ChunkDbFileIdx);
						}
						Writer->WriteArrayEnd();
					}
					Writer->WriteArrayEnd();
				}
			}
			Writer->WriteObjectEnd();
			Writer->Close();
			bSuccess = FFileHelper::SaveStringToFile(JsonOutput, *Configuration.ResultDataFilePath);
			if (!bSuccess)
			{
				UE_LOGF(LogPackageChunkData, Error, "Could not save output to %ls", *Configuration.ResultDataFilePath);
			}
		}

		// Complete the process.
		bShouldRun = false;
	}

	TFunction<void()> FPackageChunks::MakePromiseCompleteDelegate(PromiseCompleteFunc OnComplete)
	{
		TFunction<void()> OnCompleteDelegate = [this, OnComplete]() { (this->*OnComplete)(); };
		TFunction<void()> GameThreadWrapper = [OnCompleteDelegate]() { AsyncHelpers::ExecuteOnGameThread<void>(OnCompleteDelegate); };
		return GameThreadWrapper;
	}

	TFunction<void(const IOptimisedDelta::FResultValueOrError&)> FPackageChunks::MakeOptimiseCompleteDelegate(OptimiseCompleteFunc OnComplete)
	{
		TFunction<void(const IOptimisedDelta::FResultValueOrError&)> GameThreadWrapper = [this, OnComplete](const IOptimisedDelta::FResultValueOrError& Result)
		{
			// This is boiler plate overcoming IOptimisedDelta::FResultValueOrError being non-copyable, and interface between async thread and main game thread.
			TSharedPtr<IOptimisedDelta::FResultValueOrError> NewResult = Result.IsValid() ? MakeShared<IOptimisedDelta::FResultValueOrError>(MakeValue(Result.GetValue())) : MakeShared<IOptimisedDelta::FResultValueOrError>(MakeError(Result.GetError()));
			TFunction<void()> OnCompleteDelegate = [this, OnComplete, NewResult = MoveTemp(NewResult)]() { (this->*OnComplete)(*NewResult); };
			AsyncHelpers::ExecuteOnGameThread<void>(OnCompleteDelegate);
		};
		return GameThreadWrapper;
	}

	IPackageChunks* FPackageChunksFactory::Create(const FPackageChunksConfiguration& Configuration)
	{
		return new FPackageChunks(Configuration);
	}
}