// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "BuildPatchSettings.h"

namespace BuildPatchServices
{
	namespace ChunkDbHelpers
	{
		uint32 GetChunkDbHeaderSize();
		uint32 GetPerChunkEntryHeaderSize();
	};

	class IPackageChunks
	{
	public:
		virtual ~IPackageChunks() = default;
		virtual bool Run() = 0;
	};

	class FPackageChunksFactory
	{
	public:
		static IPackageChunks* Create(const FPackageChunksConfiguration& Configuration);
	};
}
