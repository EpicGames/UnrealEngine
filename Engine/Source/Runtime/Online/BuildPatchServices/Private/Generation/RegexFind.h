// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/AnyOf.h"
#include "Algo/Compare.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Regex.h"
#include "Misc/WildcardString.h"

namespace BuildPatchServices
{	
	template<typename T>
	struct TMatchingType
	{
		static const FString& GetIdentification()
		{
			static const FString Identification{ T::CharIdentification };
			return Identification;
		}

		static bool StartsWithIdentification(const FString& String)
		{
			return String.StartsWith(GetIdentification());
		}

		static bool EndsWithQuote(const FString& String)
		{
			return String.EndsWith(TEXT("\""));
		}

		static bool IsPattern(const FString& String)
		{
			return StartsWithIdentification(String) && EndsWithQuote(String);
		}

		static FString ExtractPattern(const FString& String)
		{
			return String.Mid(GetIdentification().Len(), String.Len() - GetIdentification().Len() - 1);
		}

		static bool Match(const FString& Pattern, const FString& Value)
		{
			FString ActualPattern = ExtractPattern(Pattern);
			return T::PatternMatch(ActualPattern, Value);
		}
	};

	struct FRegexPatternType : TMatchingType<FRegexPatternType>
	{
		static constexpr const TCHAR* CharIdentification = TEXT("R\"");

		static bool PatternMatch(const FString& Pattern, const FString& Value)
		{
			const FRegexPattern KeyPattern(Pattern);
			FRegexMatcher NameRegexMatcher(KeyPattern, *Value);
			return NameRegexMatcher.FindNext();
		}
	};

	struct FWildcardPatternType : TMatchingType<FWildcardPatternType>
	{
		static constexpr const TCHAR* CharIdentification = TEXT("W\"");

		static bool PatternMatch(const FString& Pattern, const FString& Value)
		{
			const FWildcardString WildcardString(Pattern);
			return WildcardString.IsMatch(*Value);
		}
	};

	struct FRegexFinder
	{
		static bool StartsLikePattern(const FString& Pattern)
		{
			return FRegexPatternType::StartsWithIdentification(Pattern) || FWildcardPatternType::StartsWithIdentification(Pattern);
		}

		static bool EndsWithQuote(const FString& Pattern)
		{
			return FRegexPatternType::EndsWithQuote(Pattern) || FWildcardPatternType::EndsWithQuote(Pattern);
		}

		static bool IsPattern(const FString& Pattern)
		{
			return FRegexPatternType::IsPattern(Pattern) || FWildcardPatternType::IsPattern(Pattern);
		}

		static bool Match(const FString& Pattern, const FString& Value)
		{
			if (FRegexPatternType::IsPattern(Pattern))
			{
				return FRegexPatternType::Match(Pattern, Value);
			}

			if (FWildcardPatternType::IsPattern(Pattern))
			{
				return FWildcardPatternType::Match(Pattern, Value);
			}

			return false;
		}

		static bool Match(const FString& Pattern, const TSet<FString>& Set)
		{
			return Algo::AnyOf(Set, [&Pattern](const FString& Item) { return Match(Item, Pattern); });
		}
	};

	template<typename T>
	TArray<const T*> RegexFind(const FString& Name, const TMap<FString, T>& Map)
	{
		TArray<const T*> AllItems;
		for (const TPair<FString, T>& Item : Map)
		{
			if (FRegexFinder::Match(Item.Key, Name))
			{
				AllItems.Add(&Item.Value);
			}
		}
		return AllItems;
	}
}
