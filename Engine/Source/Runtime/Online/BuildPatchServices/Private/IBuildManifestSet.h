// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "BuildPatchSettings.h"

#include "BuildPatchManifest.h"

namespace BuildPatchServices
{
	struct FPreReqInfo
	{
		TSet<FString> IdSet;
		FString Name;
		FString Path;
		FString Args;
		FString AppName;
		FString VersionString;
		bool bIsRepair;
	};

	struct FUninstallActionInfo
	{
		FString Path;
		FString Args;
	};

	/**
	 * FBuildPatchInstallerAction
	 * This class decorates an FInstallerAction instance with downcasted functions for internal manifest implementation.
	 * It also allows editing of the install manifest in the case a more optimal version was produced from a delta download.
	 */
	struct FBuildPatchInstallerAction
	{
	public:
		FBuildPatchInstallerAction(const FInstallerAction& InInstallerAction)
			: InstallerAction(InInstallerAction)
			, CurrentManifest(StaticCastSharedPtr<FBuildPatchAppManifest>(InstallerAction.TryGetCurrentManifest()))
			, InstallManifest(StaticCastSharedPtr<FBuildPatchAppManifest>(InstallerAction.TryGetInstallManifest()))
		{
			// Pre-process install tags. Doing this logic here means it doesn't need repeating around lower level code.
			// No tags means full installation.
			InstallTags = InstallerAction.GetInstallTags();
			if (InstallTags.Num() == 0)
			{
				GetInstallOrCurrentManifest().GetFileTagList(InstallTags);
			}
			// Always require the empty tag.
			InstallTags.Add(TEXT(""));
			// Cache tagged files.
			if (InstallManifest.IsValid())
			{
				InstallManifest->GetTaggedFileList(InstallTags, TaggedFiles);
			}
		}

		bool IsInstall() const { return InstallerAction.IsInstall(); }
		bool IsUpdate() const { return InstallerAction.IsUpdate(); }
		bool IsRepair() const { return InstallerAction.IsRepair(); }
		bool IsVerifyOnly() const { return InstallerAction.IsVerifyOnly(); }
		bool IsUninstall() const { return InstallerAction.IsUninstall(); }
		const TSet<FString>& GetInstallTags() const { return InstallTags; }
		const FString& GetInstallSubdirectory() const { return InstallerAction.GetInstallSubdirectory(); }
		const FString& GetCloudSubdirectory() const { return InstallerAction.GetCloudSubdirectory(); }
		FString GetUniqueBuildId() const { return InstallManifest.IsValid() ? InstallManifest->GetUniqueBuildId() : FString(); }
		const FBuildPatchAppManifest& GetCurrentManifest() const { return *CurrentManifest; }
		const FBuildPatchAppManifest& GetInstallManifest() const { return *InstallManifest; }
		const FBuildPatchAppManifest* TryGetCurrentManifest() const { return CurrentManifest.Get(); }
		const FBuildPatchAppManifest* TryGetInstallManifest() const { return InstallManifest.Get(); }
		const FBuildPatchAppManifest& GetCurrentOrInstallManifest() const { return *(CurrentManifest.IsValid() ? CurrentManifest : InstallManifest); }
		const FBuildPatchAppManifest& GetInstallOrCurrentManifest() const { return *(InstallManifest.IsValid() ? InstallManifest : CurrentManifest); }
		const FBuildPatchAppManifestRef GetSharedCurrentManifest() const { return CurrentManifest.ToSharedRef(); }
		const FBuildPatchAppManifestRef GetSharedInstallManifest() const { return InstallManifest.ToSharedRef(); }
		const FBuildPatchAppManifestPtr TryGetSharedCurrentManifest() const { return CurrentManifest; }
		const FBuildPatchAppManifestPtr TryGetSharedInstallManifest() const { return InstallManifest; }
		const FBuildPatchAppManifestRef GetSharedCurrentOrInstallManifest() const { return (CurrentManifest.IsValid() ? CurrentManifest : InstallManifest).ToSharedRef(); }
		const FBuildPatchAppManifestRef GetSharedInstallOrCurrentManifest() const { return (InstallManifest.IsValid() ? InstallManifest : CurrentManifest).ToSharedRef(); }

		void SetDeltaManifest(const FBuildPatchAppManifestRef& DeltaManifest) { if (IsUpdate()) { InstallManifest = DeltaManifest; } }

	private:
		// Hide tagged file accessors, they are only expected to be used by the manifest set, to prevent other systems from confusing
		// flat manifest filenames, and filenames including the action subdirectory.
		friend class FBuildPatchManifestSet;
		const TSet<FString>& GetTaggedFiles() const { return TaggedFiles; }
		void SetTaggedFiles(TSet<FString> OverrideTaggedFiles) { TaggedFiles = MoveTemp(OverrideTaggedFiles); }

	private:
		const FInstallerAction InstallerAction;
		FBuildPatchAppManifestPtr CurrentManifest;
		FBuildPatchAppManifestPtr InstallManifest;
		TSet<FString> InstallTags;
		TSet<FString> TaggedFiles;
	};

	class IBuildManifestSet
	{
	public:
		virtual ~IBuildManifestSet() = default;

		// Manifest info
		virtual void GetInstallResumeIds(TSet<FString>& ResumeIds, bool bIncludeLegacy) const = 0;
		virtual void GetInstallResumeIdsForFile(const FString& BuildFile, TSet<FString>& ResumeIds, bool bIncludeLegacy) const = 0;

		// Data enumeration
		virtual void GetReferencedChunks(TSet<FGuid>& DataGuids) const = 0;
		virtual void GetReferencedChunks(TArray<FGuid>& DataGuids) const = 0;
		virtual uint64 GetDownloadSize(const FGuid& DataGuid) const = 0;
		virtual uint64 GetDownloadSize(const TSet<FGuid>& DataGuids) const = 0;
		virtual FChunkInfo const* GetChunkInfo(const FGuid& DataGuid) const = 0;
		virtual bool GetChunkShaHash(const FGuid& ChunkGuid, FSHAHash& OutHash) const = 0;
		virtual FGuid GetDataSecretId(const FGuid& DataGuid) const = 0;
		virtual FString GetChunkCloudSubdirectory(const FGuid& DataGuid) const = 0;
		virtual FString GetChunkFilename(const FGuid& DataGuid) const = 0;
		virtual FString GetChunkFilenameWithCloudSubdirectory(const FGuid& DataGuid) const = 0;
		virtual FString GetChunkUniqueBuildId(const FGuid& DataGuid) const = 0;
		virtual bool GetChunkFromDeltaStatus(const FGuid& DataGuid) const = 0;

		// File enumeration
		virtual int32 GetNumExpectedFiles() const = 0;
		virtual void GetExpectedFiles(TSet<FString>& Filenames) const = 0;
		virtual void GetOutdatedFiles(const FString& InstallDirectory, TSet<FString>& OutdatedFiles) const = 0;
		// Gets files from the current manifest that are missing from disk.
		virtual void GetMissedFiles(const FString& InstallDirectory, TSet<FString>& OutMissedFiles) const = 0;
		virtual void GetRemovableFiles(TSet<FString>& FilesToRemove) const = 0;
		virtual const FFileManifest* GetCurrentFileManifest(const FString& BuildFile) const = 0;
		virtual const FFileManifest* GetNewFileManifest(const FString& BuildFile) const = 0;
		virtual uint64 GetTotalNewFileSize(const TArray<FString>& Filenames) const = 0;
		virtual uint64 GetTotalNewFileSize(const TSet<FString>& Filenames) const = 0;
		virtual void GetFilesTaggedForRepair(TSet<FString>& Filenames) const = 0;
		virtual bool IsFileRepairAction(const FString& Filename) const = 0;

		// Misc actions
		virtual bool ContainsUpdate() const = 0;
		virtual bool IsExclusivelyRepairing() const = 0;
		virtual bool HasFileAttributes() const = 0;
		virtual bool IsExclusivelyVerifying() const = 0;

		// Custom install support
		virtual void GetPreReqInfo(TArray<FPreReqInfo>& PreReqInfo) const = 0;
		virtual void GetUninstallActionInfo(TArray<FUninstallActionInfo>& UninstallActionsInfo) const = 0;
	};

	class FBuildManifestSetFactory
	{
	public:
		static IBuildManifestSet* Create(TArray<FBuildPatchInstallerAction> InstallerActions);
	};
}