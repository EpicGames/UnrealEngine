// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/BuildStatistics.h"
#include "Installer/Statistics/FileOperationTracker.h"
#include "Installer/Statistics/ChunkDbChunkSourceStatistics.h"
#include "Installer/Statistics/CloudChunkSourceStatistics.h"
#include "Installer/Statistics/FileConstructorStatistics.h"
#include "Installer/Statistics/MemoryChunkStoreStatistics.h"
#include "Installer/Statistics/DownloadServiceStatistics.h"
#include "Installer/Statistics/InstallChunkSourceStatistics.h"
#include "Installer/Statistics/VerifierStatistics.h"

namespace BuildPatchServices
{
	FFileOperation::FFileOperation(FString InFilename, const FGuid& InDataId, uint64 InOffest, uint64 InSize, EFileOperationState InCurrentState)
		: Filename(MoveTemp(InFilename))
		, DataId(InDataId)
		, Offest(InOffest)
		, Size(InSize)
		, CurrentState(InCurrentState)
	{
	}

	FFileOperation::~FFileOperation()
	{
	}

	class FBuildStatistics
		: public IBuildStatistics
	{
	public:
		FBuildStatistics(FBuildPatchInstallerRef Installer);
		~FBuildStatistics() {}

	public:
		// IBuildStatistics interface begin.
		virtual const FBuildInstallerConfiguration& GetConfiguration() const override;
		virtual int64 GetDownloadSize() const override;
		virtual int64 GetBuildSize() const override;
		virtual FString GetCurrentWorkingFileName() const override;
		virtual float GetCurrentWorkingFileProgress() const override;
		virtual FGuid GetCurrentWorkingData() const override;
		virtual TArray<FDownload> GetCurrentDownloads() const override;
		virtual bool IsDownloadActive() const override;
		virtual bool IsHardDiskActiveAdministering() const override;
		virtual bool IsHardDiskActiveWrite() const override;
		virtual bool IsHardDiskActiveRead() const override;
		virtual const TArray<FFileOperation>& GetFileOperationStates() const override;
		virtual double GetDownloadByteSpeed() const override;
		virtual double GetDiskReadByteSpeed() const override;
		virtual double GetDiskWriteByteSpeed() const override;
		virtual int64 GetDownloadedBytes() const override;
		virtual int64 GetDiskReadBytes() const override;
		virtual int64 GetDiskWrittenBytes() const override;
		virtual int32 GetNumVerifyErrors() const override;
		virtual TMap<EVerifyError, int32> GetVerifyErrorCounts() const override;
		// IBuildStatistics interface end.

	private:
		FBuildPatchInstallerRef Installer;
	};

	FBuildStatistics::FBuildStatistics(FBuildPatchInstallerRef InInstaller)
		: Installer(MoveTemp(InInstaller))
	{
	}

	const FBuildInstallerConfiguration& FBuildStatistics::GetConfiguration() const
	{
		return Installer->GetConfiguration();
	}

	int64 FBuildStatistics::GetDownloadSize() const
	{
		return Installer->GetTotalDownloadRequired();
	}

	int64 FBuildStatistics::GetBuildSize() const
	{
		return Installer->GetFileConstructorStatistics()->GetRequiredConstructSize();
	}

	FString FBuildStatistics::GetCurrentWorkingFileName() const
	{
		return Installer->GetFileConstructorStatistics()->GetCurrentFile();
	}

	float FBuildStatistics::GetCurrentWorkingFileProgress() const
	{
		return Installer->GetFileConstructorStatistics()->GetCurrentFileProgress();
	}

	FGuid FBuildStatistics::GetCurrentWorkingData() const
	{
		return Installer->GetFileConstructorStatistics()->GetCurrentChunk();
	}

	TArray<FDownload> FBuildStatistics::GetCurrentDownloads() const
	{
		return Installer->GetDownloadServiceStatistics()->GetCurrentDownloads();
	}

	bool FBuildStatistics::IsDownloadActive() const
	{
		return Installer->GetDownloadServiceStatistics()->GetNumCurrentDownloads() > 0;
	}

	bool FBuildStatistics::IsHardDiskActiveAdministering() const
	{
		return Installer->GetFileConstructorStatistics()->IsCurrentlyAdministering();
	}

	bool FBuildStatistics::IsHardDiskActiveWrite() const
	{
		return Installer->GetFileConstructorStatistics()->IsCurrentlyWriting();
	}

	bool FBuildStatistics::IsHardDiskActiveRead() const
	{
		return Installer->GetFileConstructorStatistics()->IsCurrentlyReading()
			|| Installer->GetInstallChunkSourceStatistics()->IsCurrentlyReading()
			|| Installer->GetVerifierStatistics()->IsCurrentlyReading()
			|| Installer->GetChunkDbChunkSourceStatistics()->IsCurrentlyReading();
	}

	const TArray<FFileOperation>& FBuildStatistics::GetFileOperationStates() const
	{
		return Installer->GetFileOperationTracker()->GetStates();
	}

	double FBuildStatistics::GetDownloadByteSpeed() const
	{
		return Installer->GetDownloadSpeed();
	}

	double FBuildStatistics::GetDiskReadByteSpeed() const
	{
		return Installer->GetDiskReadSpeedRecorder()->GetAverageSpeed(10.0f, true);
	}

	double FBuildStatistics::GetDiskWriteByteSpeed() const
	{
		return Installer->GetDiskWriteSpeedRecorder()->GetAverageSpeed(10.0f, true);
	}

	int64 FBuildStatistics::GetDownloadedBytes() const
	{
		return Installer->GetCloudChunkSourceStatistics()->GetDownloadedBytes();
	}

	int64 FBuildStatistics::GetDiskReadBytes() const
	{
		return Installer->GetInstallChunkSourceStatistics()->GetBytesRead()
		     + Installer->GetVerifierStatistics()->GetBytesVerified();
	}

	int64 FBuildStatistics::GetDiskWrittenBytes() const
	{
		return Installer->GetFileConstructorStatistics()->GetBytesConstructed();
	}

	int32 FBuildStatistics::GetNumVerifyErrors() const
	{
		return Installer->GetVerifierStatistics()->GetNumFailedFilesVerified();
	}

	TMap<EVerifyError, int32> FBuildStatistics::GetVerifyErrorCounts() const
	{
		return Installer->GetVerifierStatistics()->GetVerifyErrorCounts();
	}

	IBuildStatistics* FBuildStatisticsFactory::Create(FBuildPatchInstallerRef Installer)
	{
		return new FBuildStatistics(MoveTemp(Installer));
	}
}
