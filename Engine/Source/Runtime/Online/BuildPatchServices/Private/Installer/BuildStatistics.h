// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/IBuildStatistics.h"
#include "BuildPatchInstaller.h"

namespace BuildPatchServices
{
	/**
	 * A factory for creating an IBuildStatistics instance.
	 */
	class FBuildStatisticsFactory
	{
	public:
		static IBuildStatistics* Create(FBuildPatchInstallerRef Installer);
	};
}