// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/ChunkDbChunkSource.h"
#include "Misc/Guid.h"
#include "Async/Async.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Queue.h"
#include "Algo/Transform.h"
#include "Core/Platform.h"
#include "Common/FileSystem.h"
#include "Common/StatsCollector.h"
#include "Data/ChunkData.h"
#include "Installer/ChunkStore.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/MessagePump.h"
#include "Installer/InstallerError.h"
#include "Installer/InstallerSharedContext.h"
#include "Memory/SharedBuffer.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Tasks/Task.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "Misc/Compression.h"
#include "Compression/OodleDataCompressionUtil.h"
#include "Misc/Paths.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
namespace ChunkDbSourceHelpers
{
	bool DisableOsIntervention(uint32& OutPreviousOsIntervention)
	{
		// This prevents dialogs from popping up if we try to read a chunkdb from a removable drive
		// with no media. 

		// We only call this during startup on a single thread so we can limit this to 
		// just our thread.				
		return ::SetThreadErrorMode(SEM_FAILCRITICALERRORS, (DWORD*)&OutPreviousOsIntervention);		
	}

	void ResetOsIntervention(uint32 Previous)
	{
		::SetThreadErrorMode((DWORD)Previous, NULL);
	}
}
#else
namespace ChunkDbSourceHelpers
{
	bool DisableOsIntervention(uint32& OutPreviousOsIntervention)
	{
		OutPreviousOsIntervention = 0;
		return false;
	}

	void ResetOsIntervention(uint32 Previous)
	{
	}
}
#endif

DEFINE_LOG_CATEGORY_STATIC(LogChunkDbChunkSource, Log, All);

namespace BuildPatchServices
{
	/**
	 * Class holding variables for accessing a chunkdb file's data.
	 */
	class FChunkDbDataAccess
	{
	public:
		FChunkDbDataAccess(FChunkDatabaseHeader InHeader,
			FString InChunkDbFileName, TUniquePtr<FArchive> InArchive, IFileHandle* InFileHandler)
			: Header(MoveTemp(InHeader))
			, ChunkDbFileName(MoveTemp(InChunkDbFileName))
			, Archive(MoveTemp(InArchive))
			, FileHandler(InFileHandler)
		{
		}

		FChunkDatabaseHeader Header;
		FString ChunkDbFileName;

		// For truncation, we track when each chunk in the chunkdb is retired.
		// If the system is enabled, then this array will have the same count as the Header.Contents array.
		TArray<int32> ChunkRetiresAt;

		// When the reference tracker gets below this watermark, then we know we are done with this file and we can
		// close/retire it.
		int32 RetireAt = 0;

		// If we're retired then any access is invalid and fatal as the file has been closed and could be deleted.
		bool bIsRetired = false;

		int64 TotalSize() const
		{
			// FArchiveFileReaderGeneric::Size uses a cached value and does not re-evaluate the real size after truncation.
			return FileHandler ? FileHandler->Size() : 0L;
		}

		TUniquePtr<UE::TScopeLock<UE::FMutex>> LockArchive(FArchive** OutArchive) const
		{
			check(OutArchive != nullptr)
			// Wrap to unique ptr to return it. Original TScopeLock does not support move semantic.
			TUniquePtr<UE::TScopeLock<UE::FMutex>> Lock{};
			*OutArchive = Archive.Get();
			return Lock;
		}

		void Retire(IFileSystem* FileSystem, bool bDelete)
		{
			bIsRetired = true;
			
			{
				Archive.Reset();
				FileHandler = nullptr;
			}
			if (bDelete && FileSystem)
			{
				if (FileSystem->DeleteFile(*ChunkDbFileName))
				{
					UE_LOGF(LogChunkDbChunkSource, Display, "Chunkdb deleted upon retirement: %ls", *FPaths::GetCleanFilename(ChunkDbFileName));
				}
				else
				{
					UE_LOGF(LogChunkDbChunkSource, Error, "Failed to delete chunkdb upon retirement: %ls", *ChunkDbFileName);
				}
			}
		}

		void TruncateIfNecessary(IFileSystem* FileSystem, int64 RequiredChunkDbFileSize)
		{
			if (FileHandler && FileHandler->Size() > RequiredChunkDbFileSize)
			{
				FileHandler->Truncate(RequiredChunkDbFileSize);
			}
		}
	private:
		TUniquePtr<FArchive> Archive;
		IFileHandle* FileHandler;
	};

	// Holds where to get the chunk data from. File + location in file.
	struct FChunkAccessLookup
	{
		FChunkLocation* Location;
		FChunkDbDataAccess* DbFile;
	};

	class FChunkDbChunkSource : public IConstructorChunkDbChunkSource
	{
	public:
		FChunkDbChunkSource(FChunkDbSourceConfig Configuration, IFileSystem* FileSystem, const TArray<FGuid>& ChunkAccessOrderList, IChunkDataSerialization* ChunkDataSerialization, IChunkDbChunkSourceStat* ChunkDbChunkSourceStat);
		~FChunkDbChunkSource()
		{
		}

		virtual void ReportFileCompletion(int32 RemainingChunkCount) override
		{
			//
			// Since we've completed a file we know we won't need to resume/retry it and can delete
			// the source chunkdb that it used.
			//
			for (FChunkDbDataAccess& ChunkDbDataAccess : ChunkDbDataAccesses)
			{
				if (!ChunkDbDataAccess.bIsRetired &&
					ChunkDbDataAccess.RetireAt >= RemainingChunkCount)
				{
					ChunkDbDataAccess.Retire(FileSystem, Configuration.bDeleteChunkDBAfterUse);
				}
			}
		}
		virtual void TryTruncateFiles(int32 RemainingChunkCount) override
		{
			if (!Configuration.bTruncateChunkDbs || LastRemainingChunkCount == RemainingChunkCount)
			{
				return;
			}
			LastRemainingChunkCount = RemainingChunkCount;
			//
			// We are truncating for critical saving of disk space, so once a chunk has been used from chunkdb, we release space from the chunkdb files if we can.
			//
			for (FChunkDbDataAccess& ChunkDbDataAccess : ChunkDbDataAccesses)
			{
				if (ChunkDbDataAccess.ChunkRetiresAt.Num() == 0)
				{
					continue;
				}
				
				const int32 LastChunkPartRequired = ChunkDbDataAccess.ChunkRetiresAt.FindLastByPredicate([RemainingChunkCount](int32 Value) { return Value <= RemainingChunkCount; });
				if (LastChunkPartRequired == (ChunkDbDataAccess.ChunkRetiresAt.Num() - 1))
				{
					continue;
				}

				ChunkDbDataAccess.ChunkRetiresAt.SetNum(LastChunkPartRequired + 1, EAllowShrinking::No);
				int64 RequiredChunkDbFileSize = 0;
				if (LastChunkPartRequired != INDEX_NONE)
				{
					const FChunkLocation& LastChunkLocation = ChunkDbDataAccess.Header.Contents[LastChunkPartRequired];
					RequiredChunkDbFileSize = LastChunkLocation.ByteStart + LastChunkLocation.ByteSize;
					UE_LOGF(LogChunkDbChunkSource, Log, "Truncating file %ls up to chunk part %d (tracker index: %d), file size: %lld. Remaining chunk parts in tracker: %d", *ChunkDbDataAccess.ChunkDbFileName, LastChunkPartRequired, ChunkDbDataAccess.ChunkRetiresAt[LastChunkPartRequired], RequiredChunkDbFileSize, RemainingChunkCount);
				}
				else
				{
					UE_LOGF(LogChunkDbChunkSource, Log, "Truncating file %ls up to chunk part %d (tracker index: <unknown>), file size: %lld. Remaining chunk parts in tracker: %d", *ChunkDbDataAccess.ChunkDbFileName, LastChunkPartRequired, RequiredChunkDbFileSize, RemainingChunkCount);
				}
				ChunkDbDataAccess.TruncateIfNecessary(FileSystem, RequiredChunkDbFileSize);
			}
		}

		virtual int32 GetChunkUnavailableAt(const FGuid& DataId) const override
		{
			// While technically the chunks retire as a result of delete-during-install, we only do this
			// when they aren't needed any more, so we can set this to "never retires"
			return TNumericLimits<int32>::Max();
		}

		virtual FRequestProcessFn CreateRequest(const FGuid& DataId, FMutableMemoryView DestinationBuffer, void* UserPtr, FChunkRequestCompleteDelegate CompleteFn) override;
		virtual const TSet<FGuid>& GetAvailableChunks() const override 
		{
			return AvailableChunks; 
		}

		virtual uint64 GetChunkDbSizesAtIndexes(const TArray<int32>& FileCompletionIndexes, TArray<uint64>& OutChunkDbSizesAtCompletion) const override;

		static void LoadChunkDbFiles(
			const TArray<FString>& ChunkDbFiles, IFileSystem* FileSystem, const TArray<FGuid>& ChunkAccessOrderedList,
			TArray<FChunkDbDataAccess>& OutChunkDbDataAccesses, TMap<FGuid, FChunkAccessLookup>& OutChunkDbDataAccessLookup, TSet<FGuid>* OutOptionalAvailableStore, bool bTruncateChunkDbs);

	private:

		// Configuration.
		const FChunkDbSourceConfig Configuration;
		// Dependencies.
		IFileSystem* FileSystem = nullptr;
		IChunkReferenceTracker* ChunkReferenceTracker = nullptr;
		IChunkDataSerialization* ChunkDataSerialization = nullptr;
		IChunkDbChunkSourceStat* ChunkDbChunkSourceStat = nullptr;
		// Storage of our chunkdb and enumerated available chunks lookup.
		TArray<FChunkDbDataAccess> ChunkDbDataAccesses;
		TMap<FGuid, FChunkAccessLookup> ChunkDbDataAccessLookup;
		TSet<FGuid> AvailableChunks;

		// Number of chunks to process in this manifest when we started.
		int32 OriginalChunkCount = 0;
		int32 LastRemainingChunkCount = TNumericLimits<int32>::Max();
	};

	// Read in the headers, evalutate the list of chunks, and determine when we'll be done with our chunk dbs.
	void FChunkDbChunkSource::LoadChunkDbFiles(
		const TArray<FString>& ChunkDbFiles, IFileSystem* FileSystem, const TArray<FGuid>& ChunkAccessOrderedList, 
		TArray<FChunkDbDataAccess>& OutChunkDbDataAccesses, TMap<FGuid, FChunkAccessLookup>& OutChunkDbDataAccessLookup, TSet<FGuid>* OutOptionalAvailableStore, bool bTruncateChunkDbs)
	{

		// Allow OS intervention only once.
		bool bResetOsIntervention = false;
		bool bHasPreviousOsIntervention = false;
		uint32 PreviousOsIntervention = 0;
		// Load each chunkdb's TOC to enumerate available chunks.
		for (const FString& ChunkDbFilename : ChunkDbFiles)
		{
			IFileHandle* FileHandler = nullptr;
			TUniquePtr<FArchive> ChunkDbArchive(FileSystem->CreateFileReaderEx(*ChunkDbFilename, BuildPatchServices::EReadFlags::None, &FileHandler));
			if (ChunkDbArchive.IsValid())
			{
				// Load header.
				FChunkDatabaseHeader Header;
				*ChunkDbArchive << Header;
				if (ChunkDbArchive->IsError())
				{
					GLog->Logf(TEXT("Failed to load chunkdb header for %s"), *ChunkDbFilename);
				}
				else if (Header.Contents.Num() == 0)
				{
					GLog->Logf(TEXT("Loaded empty chunkdb %s"), *ChunkDbFilename);
				}
				else
				{
					// Hold on to the handle and header info.
					FChunkDbDataAccess DataSource(MoveTemp(Header), ChunkDbFilename, MoveTemp(ChunkDbArchive), FileHandler);
					if (bTruncateChunkDbs)
					{
						// Initialise with int32 max, so that any element that does not get filled out later, will evaluate as retired.
						DataSource.ChunkRetiresAt.AddUninitialized(DataSource.Header.Contents.Num());
						for (int32& i : DataSource.ChunkRetiresAt)
						{
							i = TNumericLimits<int32>::Max();
						}
					}
					OutChunkDbDataAccesses.Add(MoveTemp(DataSource));
				}
			}
			else if (!bResetOsIntervention)
			{
				bResetOsIntervention = true;
				bHasPreviousOsIntervention = ChunkDbSourceHelpers::DisableOsIntervention(PreviousOsIntervention);
			}
		}
		// Reset OS intervention if we disabled it.
		if (bResetOsIntervention && bHasPreviousOsIntervention)
		{
			ChunkDbSourceHelpers::ResetOsIntervention(PreviousOsIntervention);
		}

		// Index all chunks to their location info.
		for (FChunkDbDataAccess& ChunkDbDataAccess : OutChunkDbDataAccesses)
		{
			const int64 ChunkDbFileSize = ChunkDbDataAccess.TotalSize();

			for (FChunkLocation& ChunkLocation : ChunkDbDataAccess.Header.Contents)
			{
				// Check this chunk actually exists in the chunkdb, we clamp to avoid unsigned mismatch.
				const int64 ChunkLastByte = int64(ChunkLocation.ByteStart + ChunkLocation.ByteSize) - 1;
				const bool bChunkExists = ChunkLocation.ByteSize > 0 && ChunkLastByte <= ChunkDbFileSize;
				if (bChunkExists && !OutChunkDbDataAccessLookup.Contains(ChunkLocation.ChunkId))
				{
					OutChunkDbDataAccessLookup.Add(ChunkLocation.ChunkId, { &ChunkLocation, &ChunkDbDataAccess });

					if (OutOptionalAvailableStore)
					{
						OutOptionalAvailableStore->Add(ChunkLocation.ChunkId);
					}
				}
			}
		}

		TMap<FString, int32> FileLastSeenAt;

		for (int32 GuidIndex = 0; GuidIndex < ChunkAccessOrderedList.Num(); GuidIndex++)
		{
			const FGuid& Guid = ChunkAccessOrderedList[GuidIndex];

			FChunkAccessLookup* SourceForGuid = OutChunkDbDataAccessLookup.Find(Guid);
			if (!SourceForGuid)
			{
				continue;
			}

			FileLastSeenAt.FindOrAdd(SourceForGuid->DbFile->ChunkDbFileName) = GuidIndex;

			if (bTruncateChunkDbs)
			{
				const TArray<FChunkLocation>& ChunkDBContents = SourceForGuid->DbFile->Header.Contents;
				const int32 IndexOfChunkPart = SourceForGuid->Location - ChunkDBContents.GetData();
				check(ChunkDBContents.IsValidIndex(IndexOfChunkPart)); // programmer error if this fires
				SourceForGuid->DbFile->ChunkRetiresAt[IndexOfChunkPart] = ChunkAccessOrderedList.Num() - (GuidIndex + 1);
			}
		}

		for (FChunkDbDataAccess& ChunkDbDataAccess : OutChunkDbDataAccesses)
		{
			// Default to retired immediately (this is functionally after the first file completes)
			ChunkDbDataAccess.RetireAt = ChunkAccessOrderedList.Num();
			int32* LastAt = FileLastSeenAt.Find(ChunkDbDataAccess.ChunkDbFileName);
			if (LastAt)
			{
				// The reference stack gets popped rather than advanced, so we need to reverse the ordering.
				// LastAt is the chunk index that last used the file, which means when we are at
				// LastAt + 1 through the stack we can delete the file.
				ChunkDbDataAccess.RetireAt = ChunkAccessOrderedList.Num() - (*LastAt + 1);
			}
		}
	}
	
	// Get how many bytes of chunkdbs will still exist on disk after the given indexes (i.e. will not have been retired)
	// Return is the total size of given chunkdbs.
	static uint64 GetChunkDbSizesAtIndexesInternal(const TArray<FChunkDbDataAccess>& InOpenedChunkDbs, int32 InOriginalChunkCount, const TArray<int32>& InFileCompletionIndexes, TArray<uint64>& OutChunkDbSizesAtCompletion)
	{
		uint64 AllChunkDbSize = 0;
		for (const FChunkDbDataAccess& ChunkFile : InOpenedChunkDbs)
		{
			if (!ChunkFile.bIsRetired)
			{
				AllChunkDbSize += ChunkFile.TotalSize();
			}
		}
		
		// Go over the list of completions and evalute how many chunk dbs are left over.
		for (int32 FileCompletionIndex : InFileCompletionIndexes)
		{
			// retiring happens as the list is _popped_, so everything is backwards.
			int32 RetireAtEquivalent = InOriginalChunkCount - FileCompletionIndex;

			uint64 TotalSizeAtIndex = 0;
			for (const FChunkDbDataAccess& ChunkFile : InOpenedChunkDbs)
			{
				if (!ChunkFile.bIsRetired &&
					ChunkFile.RetireAt < RetireAtEquivalent)
				{
					TotalSizeAtIndex += ChunkFile.TotalSize();
				}
			}

			OutChunkDbSizesAtCompletion.Add(TotalSizeAtIndex);
		}
		return AllChunkDbSize;
	}


	uint64 FChunkDbChunkSource::GetChunkDbSizesAtIndexes(const TArray<int32>& FileCompletionIndexes, TArray<uint64>& OutChunkDbSizesAtCompletion) const
	{
		return GetChunkDbSizesAtIndexesInternal(ChunkDbDataAccesses, OriginalChunkCount, FileCompletionIndexes, OutChunkDbSizesAtCompletion);
	}

	uint64 IConstructorChunkDbChunkSource::GetChunkDbSizesAtIndexes(const TArray<FString>& ChunkDbFiles, IFileSystem* FileSystem, const TArray<FGuid>& ChunkAccessOrderedList, const TArray<int32>& FileCompletionIndexes, TArray<uint64>& OutChunkDbSizesAtCompletion)
	{
		TArray<FChunkDbDataAccess> ChunkFiles;
		TMap<FGuid, FChunkAccessLookup> ChunkGuidLookup;

		FChunkDbChunkSource::LoadChunkDbFiles(ChunkDbFiles, FileSystem, ChunkAccessOrderedList, ChunkFiles, ChunkGuidLookup, nullptr, false);

		return GetChunkDbSizesAtIndexesInternal(ChunkFiles, ChunkAccessOrderedList.Num(), FileCompletionIndexes, OutChunkDbSizesAtCompletion);
	}

	FChunkDbChunkSource::FChunkDbChunkSource(FChunkDbSourceConfig InConfiguration, IFileSystem* InFileSystem, const TArray<FGuid>& InChunkAccessOrderList, 
		IChunkDataSerialization* InChunkDataSerialization, IChunkDbChunkSourceStat* InChunkDbChunkSourceStat)
		: Configuration(MoveTemp(InConfiguration))
		, FileSystem(InFileSystem)
		, ChunkDataSerialization(InChunkDataSerialization)
		, ChunkDbChunkSourceStat(InChunkDbChunkSourceStat)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FChunkDbChunkSource_ctor);

		OriginalChunkCount = InChunkAccessOrderList.Num();

		LoadChunkDbFiles(Configuration.ChunkDbFiles, FileSystem, InChunkAccessOrderList, ChunkDbDataAccesses, ChunkDbDataAccessLookup, &AvailableChunks, Configuration.bTruncateChunkDbs);

		// Immediately retire any chunkdbs we don't need so they don't eat disk space during the first file.
		FChunkDbChunkSource::ReportFileCompletion(OriginalChunkCount);
	}


	IConstructorChunkSource::FRequestProcessFn FChunkDbChunkSource::CreateRequest(const FGuid& DataId, FMutableMemoryView DestinationBuffer, void* UserPtr, FChunkRequestCompleteDelegate CompleteFn)
	{
		FChunkAccessLookup* ChunkInfo = ChunkDbDataAccessLookup.Find(DataId);
		if (!ChunkInfo)
		{
			CompleteFn.Execute(DataId, false, true, UserPtr);
			return [](bool) {return;};
		}

		return [ChunkInfo, DataId, DestinationBuffer, UserPtr, CompleteFn, ChunkDataSerialization = ChunkDataSerialization, ChunkDbChunkSourceStat = ChunkDbChunkSourceStat, bTruncateChunkDbs = Configuration.bTruncateChunkDbs](bool bIsAborted)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChunkDbRead);
			if (bIsAborted)
			{
				CompleteFn.Execute(DataId, true, false, UserPtr);
				return;
			}
			FChunkLocation& ChunkLocation = *ChunkInfo->Location;
			ISpeedRecorder::FRecord ActivityRecord;
			bool Result = false;
			FUniqueBuffer CompressedBuffer;
			IChunkDbChunkSourceStat::ELoadResult LoadResult = IChunkDbChunkSourceStat::ELoadResult::Success;
			FChunkHeader Header;
			FChunkDbDataAccess& ChunkDbDataAccess = *ChunkInfo->DbFile;
			{
				FArchive* ChunkDbFile;
				TUniquePtr<UE::TScopeLock<UE::FMutex>> LockGuard = ChunkDbDataAccess.LockArchive(&ChunkDbFile);
				if (!ChunkDbFile || ChunkDbFile->IsError())
				{
					CompleteFn.Execute(DataId, false, true, UserPtr);
					return;
				}
				
				ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
				ChunkDbChunkSourceStat->OnLoadStarted(DataId);

				if (static_cast<uint64>(ChunkDbFile->TotalSize()) < ChunkLocation.ByteStart)
				{
					GLog->Logf(ELogVerbosity::Type::Error, TEXT("Total file size (%lld) < Chunk start position (%lld). The file may have been truncated before resuming unlock: %s"), ChunkDbFile->TotalSize(), ChunkLocation.ByteStart, *ChunkDbDataAccess.ChunkDbFileName);
					CompleteFn.Execute(DataId, false, true, UserPtr);
					return;
				}
				// We'd love to read direct in to the destination if we don't have
				// any compression. However we don't know if it's compressed until we read the header which
				// is tiny and dependent - we don't know how big it is until we read part of it.
				ChunkDbFile->Seek(ChunkLocation.ByteStart);

				// If it's uncompressed, we can read direct to the destination.
				Result = ChunkDataSerialization->ValidateAndRead(*ChunkDbFile, DestinationBuffer, Header, CompressedBuffer);

				// Save this here so we only include the IO time and not the hash/decompress time.
				ActivityRecord.Size = ChunkDbFile->Tell() - ChunkLocation.ByteStart;
			}
			ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
			ChunkDbChunkSourceStat->OnReadComplete(ActivityRecord);

			if (!Result)
			{
				LoadResult = IChunkDbChunkSourceStat::ELoadResult::SerializationError;
				ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
				ChunkDbChunkSourceStat->OnLoadComplete(DataId, LoadResult);

				// The header or chunk data was bad.
				CompleteFn.Execute(DataId, false, true, UserPtr);
				return;
			}

			// We either need to hash the chunk for validation or decompress it into the destination buffer - don't
			// block IO for this.
			UE::Tasks::Launch(TEXT("ChunkDbDecompressionAndHash"), 
				[DataId, CompleteFn, UserPtr, Header, DestinationBuffer, ChunkDataSerialization = ChunkDataSerialization, 
				CompressedBuffer = MoveTemp(CompressedBuffer), ChunkDbChunkSourceStat = ChunkDbChunkSourceStat]()
				{
					bool bDecompressSucceeded = ChunkDataSerialization->DecompressAndDecryptValidatedRead(Header, DestinationBuffer, CompressedBuffer);

					ChunkDbChunkSourceStat->OnLoadComplete(DataId, 
						bDecompressSucceeded ? IChunkDbChunkSourceStat::ELoadResult::Success : IChunkDbChunkSourceStat::ELoadResult::CorruptedData);

					CompleteFn.Execute(DataId, false, !bDecompressSucceeded, UserPtr);
				}
			);
		};
	}

	IConstructorChunkDbChunkSource* IConstructorChunkDbChunkSource::CreateChunkDbSource(FChunkDbSourceConfig&& Configuration, IFileSystem* FileSystem, 
		const TArray<FGuid>& ChunkAccessOrderList, IChunkDataSerialization* ChunkDataSerialization, IChunkDbChunkSourceStat* ChunkDbChunkSourceStat)
	{
		return new FChunkDbChunkSource(MoveTemp(Configuration), FileSystem, ChunkAccessOrderList, ChunkDataSerialization, ChunkDbChunkSourceStat);
	}

	const TCHAR* ToString(const IChunkDbChunkSourceStat::ELoadResult& LoadResult)
	{
		switch(LoadResult)
		{
			case IChunkDbChunkSourceStat::ELoadResult::Success:
				return TEXT("Success");
			case IChunkDbChunkSourceStat::ELoadResult::CorruptedData:
				return TEXT("CorruptedData");
			case IChunkDbChunkSourceStat::ELoadResult::SerializationError:
				return TEXT("SerializationError");
			default:
				return TEXT("Unknown");
		}
	}
}
