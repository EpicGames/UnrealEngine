// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/ChunkSource.h"
#include "Installer/Controllable.h"
#include "Containers/Set.h"
#include "Common/SpeedRecorder.h"

class IBuildInstallerSharedContext;

namespace BuildPatchServices
{
	class IFileSystem;
	class IChunkReferenceTracker;
	class IChunkDataSerialization;
	class IChunkDbChunkSourceStat;
	class IBuildManifestSet;

	
	/**
	* A struct containing the configuration values for a chunkdb chunk source.
	*/
	struct FChunkDbSourceConfig
	{
		// An array of chunkdb full file paths.
		TArray<FString> ChunkDbFiles;

		// If true, once we complete a file we delete all the chunkdbs used to create it.
		bool bDeleteChunkDBAfterUse = false;

		// If we are loading from a chunkDB, in reverse order, we can truncate the chunkDB files as we go if desired.
		// This is for preload support.
		bool bTruncateChunkDbs = false;

		/**
		* Constructor which sets usual defaults, and takes params for values that cannot use a default.
		* @param InChunkDbFiles    The chunkdb filename array.
		*/
		FChunkDbSourceConfig(TArray<FString> InChunkDbFiles)
			: ChunkDbFiles(MoveTemp(InChunkDbFiles))
		{}
	};

	/**
	 * The interface for a chunkdb chunk source, which provides access to chunk data retrieved from chunkdb files.
	 */
	class IConstructorChunkDbChunkSource : public IConstructorChunkSource
	{
	public:
		virtual ~IConstructorChunkDbChunkSource() = default;

		/**
		 * Get the set of chunks available in the chunkdbs which were provided to the source.
		 * @return the set of chunks available.
		 */
		virtual const TSet<FGuid>& GetAvailableChunks() const = 0;

		// Fill out how many bytes of chunkdbs are left if we delete all the ones that
		// are no longer necessary at the given FileCompletionIndexes in to ChunkAccessOrderedList
		static uint64 GetChunkDbSizesAtIndexes(const TArray<FString>& ChunkDbFiles, IFileSystem* FileSystem, const TArray<FGuid>& ChunkAccessOrderedList, const TArray<int32>& FileCompletionIndexes, TArray<uint64>& OutChunkDbSizesAtCompletion);

		// As above, except use the remaining open chunkdbs for progressive disk space checking.		
		virtual uint64 GetChunkDbSizesAtIndexes(const TArray<int32>& FileCompletionIndexes, TArray<uint64>& OutChunkDbSizesAtCompletion) const = 0;
		
		virtual void ReportFileCompletion(int32 RemainingChunkCount) =0;
		virtual void TryTruncateFiles(int32 RemainingChunkCount) =0;

		static IConstructorChunkDbChunkSource* CreateChunkDbSource(FChunkDbSourceConfig&& Configuration, IFileSystem* FileSystem, const TArray<FGuid>& ChunkAccessOrderList, 
			IChunkDataSerialization* ChunkDataSerialization, IChunkDbChunkSourceStat* ChunkDbChunkSourceStat);
	};
	
	/**
	 * This interface defines the statistics class required by the chunkdb chunk source. It should be implemented in order to collect
	 * desired information which is being broadcast by the system.
	 */
	class IChunkDbChunkSourceStat
	{
	public:
		/**
		 * Enum which describes success, or the reason for failure when loading a chunk.
		 */
		enum class ELoadResult : uint8
		{
			Success = 0,

			// The hash information was missing.
			MissingHashInfo_DEPRECATED,

			// The expected data hash for the chunk did not match.
			HashCheckFailed_DEPRECATED,

			// The chunkdb header specified an invalid chunk location offset or size.
			LocationOutOfBounds_DEPRECATED,

			// An unexpected error during serialization. This includes header validation
			// checks like whether the hash is present.
			SerializationError,
			
			// Either the hash didn't match or the decompression call failed.
			CorruptedData
		};

	public:
		virtual ~IChunkDbChunkSourceStat() = default;

		/**
		 * Called when a batch of chunks are going to be loaded.
		 * @param ChunkIds  The ids of each chunk.
		 */
		UE_DEPRECATED(5.6, "No longer batch loaded")
		virtual void OnBatchStarted(const TArray<FGuid>& ChunkIds) {};

		/**
		 * Called each time a chunk load begins.
		 * @param ChunkId   The id of the chunk.
		 */
		virtual void OnLoadStarted(const FGuid& ChunkId) = 0;

		/**
		 * Called each time a chunk load completes.
		 * @param ChunkId   The id of the chunk.
		 * @param Result    The load result.
		 */
		virtual void OnLoadComplete(const FGuid& ChunkId, ELoadResult Result) = 0;

		// Called when the read for the load is complete and we're moving on to decompression/hashing.
		virtual void OnReadComplete(const ISpeedRecorder::FRecord& Record) = 0;
	};
	
	/**
	 * A ToString implementation for IChunkDbChunkSourceStat::ELoadResult.
	 */
	const TCHAR* ToString(const IChunkDbChunkSourceStat::ELoadResult& LoadResult);
}
