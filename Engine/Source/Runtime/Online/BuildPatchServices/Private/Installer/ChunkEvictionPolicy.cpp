// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/ChunkEvictionPolicy.h"
#include "Installer/ChunkReferenceTracker.h"

namespace BuildPatchServices
{
	class FChunkEvictionPolicy : public IChunkEvictionPolicy
	{
	public:
		FChunkEvictionPolicy(IChunkReferenceTracker* InChunkReferenceTracker);

		~FChunkEvictionPolicy();

		// IChunkEvictionPolicy interface begin.
		virtual void Query(const TMap<FGuid, TUniquePtr<IChunkDataAccess>>& CurrentMap, int32 DesiredMax, TSet<FGuid>& OutCleanable, TSet<FGuid>& OutBootable) const override;
		// IChunkEvictionPolicy interface end.

	private:
		IChunkReferenceTracker* ChunkReferenceTracker;
	};

	FChunkEvictionPolicy::FChunkEvictionPolicy(IChunkReferenceTracker* InChunkReferenceTracker)
		: ChunkReferenceTracker(InChunkReferenceTracker)
	{
	}

	FChunkEvictionPolicy::~FChunkEvictionPolicy()
	{
	}

	void FChunkEvictionPolicy::Query(const TMap<FGuid, TUniquePtr<IChunkDataAccess>>& CurrentMap, int32 DesiredMax, TSet<FGuid>& OutCleanable, TSet<FGuid>& OutBootable) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Eviction_Query);
		for (const TPair<FGuid, TUniquePtr<IChunkDataAccess>>& Pair : CurrentMap)
		{
			if (ChunkReferenceTracker->GetReferenceCount(Pair.Key) == 0)
			{
				OutCleanable.Add(Pair.Key);
			}
		}
		int32 BootsNeeded = (CurrentMap.Num() - OutCleanable.Num()) - DesiredMax;
		if (BootsNeeded > 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Eviction_BootSort);

			TArray<FGuid> ChunkUseOrder;
			for (const TPair<FGuid, TUniquePtr<IChunkDataAccess>>& Pair : CurrentMap)
			{
				if (!OutCleanable.Contains(Pair.Key))
				{
					ChunkUseOrder.Add(Pair.Key);
				}
			}

			// This sorts the entirety of the reference list even if we only need 1 boot :/ feels like there's a better way.
			// this was taking like 40-50ms for each put that overflows.
			ChunkReferenceTracker->SortByUseOrder(ChunkUseOrder, IChunkReferenceTracker::ESortDirection::Descending);
			for (int32 ChunkUseOrderIdx = 0; ChunkUseOrderIdx < ChunkUseOrder.Num() && BootsNeeded > 0; ++ChunkUseOrderIdx)
			{
				--BootsNeeded;
				OutBootable.Add(ChunkUseOrder[ChunkUseOrderIdx]);
			}
		}
	}

	IChunkEvictionPolicy* FChunkEvictionPolicyFactory::Create(IChunkReferenceTracker* ChunkReferenceTracker)
	{
		check(ChunkReferenceTracker != nullptr);
		return new FChunkEvictionPolicy(ChunkReferenceTracker);
	}
}
