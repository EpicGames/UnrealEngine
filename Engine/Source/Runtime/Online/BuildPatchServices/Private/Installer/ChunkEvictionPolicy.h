// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/UniquePtr.h"
#include "Data/ChunkData.h"

namespace BuildPatchServices
{
	class IChunkReferenceTracker;

	/**
	 * An interface providing decisions for cleaning and booting chunk data from stores.
	 */
	class IChunkEvictionPolicy
	{
	public:
		virtual ~IChunkEvictionPolicy() = default;

		/**
		 * Query which chunks in a given set can be removed, or if necessary booted, in order to achieve the desired max count.
		 * It is possible to receive a decision of not evicting any data and thus expanding the data set past the desired max.
		 * @param CurrentMap    Reference to the full set of data to be considered.
		 * @param DesiredMax    The maximum number of chunks desired for the set of data.
		 * @param OutCleanable  Receives IDs for chunks which should be abandoned as they are no longer required.
		 * @param OutBootable   Receives IDs for chunks which would be preferable to remove, but are still required later.
		 */
		virtual void Query(const TMap<FGuid, TUniquePtr<IChunkDataAccess>>& CurrentMap, int32 DesiredMax, TSet<FGuid>& OutCleanable, TSet<FGuid>& OutBootable) const = 0;
	};

	/**
	 * A factory for creating an IChunkEvictionPolicy instance.
	 */
	class FChunkEvictionPolicyFactory
	{
	public:
		/**
		 * This implementation uses a chunk reference tracker in order to make decisions about cleaning up unreferenced chunks, and
		 * booting out chunks that are not required for the longest time.
		 * @param ChunkReferenceTracker     Required ptr to a chunk reference tracker which will be used to make decisions.
		 * @return the new IChunkEvictionPolicy created.
		 */
		static IChunkEvictionPolicy* Create(IChunkReferenceTracker* ChunkReferenceTracker);
	};
}
