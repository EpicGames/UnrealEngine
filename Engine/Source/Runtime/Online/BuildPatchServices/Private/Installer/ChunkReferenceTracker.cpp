// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/ChunkReferenceTracker.h"
#include "Templates/Greater.h"
#include "Algo/Sort.h"
#include "Misc/ScopeLock.h"
#include "IBuildManifestSet.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChunkReferenceTracker, Warning, All);
DEFINE_LOG_CATEGORY(LogChunkReferenceTracker);

namespace BuildPatchServices
{
	class FChunkReferenceTracker : public IChunkReferenceTracker
	{
	public:
		// Construct the list of chunks from a manifest and an ordered list of files to construct. 
		FChunkReferenceTracker(const IBuildManifestSet* ManifestSet, const TSet<FString>& FilesToConstruct);

		// Pass in a direct ordered list of guids to use as chunks.
		FChunkReferenceTracker(TArray<FGuid> CustomUseList);

		~FChunkReferenceTracker();

		// IChunkReferenceTracker interface begin.
		virtual void CopyOutOrderedUseList(TArray<FGuid>& OutUseList) const override
		{
			int32 LocalCurrentPosition = CurrentPosition.load(std::memory_order_acquire);
			OutUseList.Append(UseList.GetData() + LocalCurrentPosition, UseList.Num() - LocalCurrentPosition);
		}
		virtual TSet<FGuid> GetReferencedChunks() const override;
		virtual int32 GetReferenceCount(const FGuid& ChunkId) const override;
		virtual void SortByUseOrder(TArray<FGuid>& ChunkList, ESortDirection Direction) const override;
		virtual TArray<FGuid> GetNextReferences(int32 Count, const TFunction<bool(const FGuid&)>& SelectPredicate) const override;
		virtual TArray<FGuid> SelectFromNextReferences(int32 Count, const TFunction<bool(const FGuid&)>& SelectPredicate) const override;
		virtual bool PopReference(const FGuid& ChunkId) override;
		virtual int32 GetRemainingChunkCount() const override;
		virtual int32 GetNextUsageForChunk(const FGuid& ChunkId, int32& OutLastUsageIndex) const override;
		virtual int32 GetCurrentUsageIndex() const override
		{
			return CurrentPosition.load(std::memory_order_relaxed);
		}

		// IChunkReferenceTracker interface end.

	private:
		// Index of the next chunk to be used in UseList.
		std::atomic<int> CurrentPosition = 0;

		// Ordered list of guids in order of consumption.
		TArray<FGuid> UseList;

		// A sorted array (guid then index), where the indeces are the location of the guid in UseList, in ascending
		// order.
		// i.e. for all X in GuidUsagePositions, UseList[X.Value] = X.Key;
		TArray<TPair<FGuid, int32>> GuidUsagePositions;		
	};	

	FChunkReferenceTracker::FChunkReferenceTracker(const IBuildManifestSet* ManifestSet, const TSet<FString>& FilesToConstruct)
	{
		// Iterate each file in reference order to construct the ordered list of chunks we
		// will need to construct the files, and track when we are going to use them.
		int32 UsageIndex = 0;
		for (const FString& File : FilesToConstruct)
		{
			const FFileManifest* NewFileManifest = ManifestSet->GetNewFileManifest(File);
			if (NewFileManifest != nullptr)
			{
				for (const FChunkPart& ChunkPart : NewFileManifest->ChunkParts)
				{
					UseList.Add(ChunkPart.Guid);
					GuidUsagePositions.Add({ChunkPart.Guid, UsageIndex});
					UsageIndex++;
				}
			}
		}

		Algo::Sort(GuidUsagePositions);
	}

	FChunkReferenceTracker::FChunkReferenceTracker(TArray<FGuid> CustomChunkReferences)
		: UseList(MoveTemp(CustomChunkReferences))
	{
		int32 ChunkIndex = 0;
		for (const FGuid& Chunk : UseList)
		{
			GuidUsagePositions.Add({Chunk, ChunkIndex});
			ChunkIndex++;
		}

		Algo::Sort(GuidUsagePositions);
	}

	FChunkReferenceTracker::~FChunkReferenceTracker()
	{

	}
	
	TSet<FGuid> FChunkReferenceTracker::GetReferencedChunks() const
	{
		int32 LocalCurrentPosition = CurrentPosition.load(std::memory_order_acquire);

		TSet<FGuid> ReferencedChunks;

		FGuid CurrentGuid;
		for (const TPair<FGuid, int32>& ReferencedChunk : GuidUsagePositions)
		{
			if (ReferencedChunk.Value < LocalCurrentPosition)
			{
				continue;
			}
			if (CurrentGuid != ReferencedChunk.Key)
			{
				ReferencedChunks.Add(ReferencedChunk.Key);
				CurrentGuid = ReferencedChunk.Key;
			}
		}
		
		return ReferencedChunks;
	}

	int32 FChunkReferenceTracker::GetReferenceCount(const FGuid& ChunkId) const
	{
		int32 LocalCurrentPosition = CurrentPosition.load(std::memory_order_acquire);
		int32 GuidPosition = Algo::LowerBound(GuidUsagePositions, TPair<FGuid,int32> { ChunkId, CurrentPosition });

		int32 StartGuidPosition = GuidPosition;
		while (GuidPosition < GuidUsagePositions.Num() && GuidUsagePositions[GuidPosition].Key == ChunkId)
		{
			GuidPosition++;
		}

		return GuidPosition - StartGuidPosition;
	}

	void FChunkReferenceTracker::SortByUseOrder(TArray<FGuid>& ChunkList, ESortDirection Direction) const
	{
		int32 LocalCurrentPosition = CurrentPosition.load(std::memory_order_acquire);

		// Get the next index for each chunk.
		TMap<FGuid, int32> NextUsageIndexes;
		for (FGuid& Guid : ChunkList)
		{
			int32 GuidPosition = Algo::LowerBound(GuidUsagePositions, TPair<FGuid,int32> { Guid, CurrentPosition });

			int32 UsageIndex = TNumericLimits<int32>::Max(); // Unused chunks need to be sorted as though they are never used
			if (GuidPosition < GuidUsagePositions.Num() &&
				GuidUsagePositions[GuidPosition].Key == Guid)
			{
				UsageIndex =  GuidUsagePositions[GuidPosition].Value;
			}

			NextUsageIndexes.Add(Guid, UsageIndex);
		}

		switch (Direction)
		{
		case ESortDirection::Ascending:
			{
				Algo::SortBy(ChunkList, [&NextUsageIndexes](const FGuid& Id) { return NextUsageIndexes[Id]; }, TLess<int32>());
				break;
			}
		case ESortDirection::Descending:
			{
				Algo::SortBy(ChunkList, [&NextUsageIndexes](const FGuid& Id) { return NextUsageIndexes[Id]; }, TGreater<int32>());
				break;
			}
		}
	}

	TArray<FGuid> FChunkReferenceTracker::GetNextReferences(int32 Count, const TFunction<bool(const FGuid&)>& SelectPredicate) const
	{
		// Returns "Count" unique references that match SelectPredicate.

		int32 LocalCurrentPosition = CurrentPosition.load(std::memory_order_acquire);
		TSet<FGuid> AddedIds;
		TArray<FGuid> NextReferences;

		for (; LocalCurrentPosition < UseList.Num() && NextReferences.Num() < Count; LocalCurrentPosition++)
		{
			const FGuid& UseId = UseList[LocalCurrentPosition];
			if (AddedIds.Contains(UseId) == false && SelectPredicate(UseId))
			{
				AddedIds.Add(UseId);
				NextReferences.Add(UseId);
			}
		}
		
		return NextReferences;
	}
	
	int32 FChunkReferenceTracker::GetNextUsageForChunk(const FGuid& ChunkId, int32& OutLastUsageIndex) const
	{
		int32 LocalCurrentPosition = CurrentPosition.load(std::memory_order_acquire);

		int32 GuidPosition = Algo::LowerBound(GuidUsagePositions, TPair<FGuid,int32> { ChunkId, LocalCurrentPosition });
		if (GuidPosition >= GuidUsagePositions.Num() ||
			GuidUsagePositions[GuidPosition].Key != ChunkId)
		{
			return -1;
		}
		int32 NextUsage = GuidUsagePositions[GuidPosition].Value;

		while (GuidPosition + 1 < GuidUsagePositions.Num() &&
			GuidUsagePositions[GuidPosition + 1].Key == ChunkId)
		{
			GuidPosition++;
		}

		OutLastUsageIndex = GuidUsagePositions[GuidPosition].Value;
		return NextUsage;
	}

	int32 FChunkReferenceTracker::GetRemainingChunkCount() const
	{
		int32 LocalCurrentPosition = CurrentPosition.load(std::memory_order_acquire);
		return UseList.Num() - LocalCurrentPosition;
	}

	TArray<FGuid> FChunkReferenceTracker::SelectFromNextReferences(int32 Count, const TFunction<bool(const FGuid&)>& SelectPredicate) const
	{
		// Original code 
		/*
			for (int32 UseStackIdx = UseStack.Num() - 1; UseStackIdx >= 0 && Count > 0; --UseStackIdx)
			{
				const FGuid& UseId = UseStack[UseStackIdx];
				if (AddedIds.Contains(UseId) == false)
				{
					--Count;
					if (SelectPredicate(UseId))
					{
						AddedIds.Add(UseId);
						NextReferences.Add(UseId);
					}
				}
			}
		*/
		// This is obfuscated a bit but is actually exactly the same as GetNextReferences. Since we can only decrease Count
		// on a new unique chunk and we only add a new unique chunk when we add a reference, this is the same.
		return GetNextReferences(Count, SelectPredicate);
	}

	bool FChunkReferenceTracker::PopReference(const FGuid& ChunkId)
	{
		int32 LocalCurrentPosition = CurrentPosition.load(std::memory_order_acquire);
		do
		{
			if (LocalCurrentPosition >= UseList.Num() || UseList[LocalCurrentPosition] != ChunkId)
			{
				return false;
			}
		} while (!CurrentPosition.compare_exchange_weak(LocalCurrentPosition, LocalCurrentPosition + 1, std::memory_order_acq_rel));

		return true;
	}

	IChunkReferenceTracker* FChunkReferenceTrackerFactory::Create(const IBuildManifestSet* ManifestSet, const TSet<FString>& FilesToConstruct)
	{
		return new FChunkReferenceTracker(ManifestSet, FilesToConstruct);
	}

	IChunkReferenceTracker* FChunkReferenceTrackerFactory::Create(TArray<FGuid> CustomChunkReferences)
	{
		return new FChunkReferenceTracker(MoveTemp(CustomChunkReferences));
	}
}