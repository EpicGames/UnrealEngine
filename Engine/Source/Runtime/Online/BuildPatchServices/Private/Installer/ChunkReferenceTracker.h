// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BuildPatchManifest.h"
#include "IBuildManifestSet.h"

namespace BuildPatchServices
{
	/**
	 * An interface for tracking references to chunks used throughout an installation. It is used to share across systems the
	 * chunks that are still required and when.
	 */
	class IChunkReferenceTracker
	{
	public:
		enum class ESortDirection : uint8
		{
			Ascending = 0,
			Descending
		};

	public:
		virtual ~IChunkReferenceTracker() = default;

		/**
		 * Gets a set of all chunks referenced by the installation this tracker refers to.
		 * This returns the _remaining_ referenced chunks!
		 * Note that this does NOT return chunks in usage order anymore! If you were tacitly relying
		 * on the enumeration order of an unordered TSet, this likely broke! Use CopyOutOrderedUseList.
		 * @return set of GUID containing every chunk used.
		 */
		virtual TSet<FGuid> GetReferencedChunks() const = 0;

		/**
		 * Gets the number of times a specific chunk is still referenced for the associated installation.
		 * @param ChunkId           The id for the chunk in question.
		 * @return the number of references remaining.
		 */
		virtual int32 GetReferenceCount(const FGuid& ChunkId) const = 0;

		/**
		 * Sorts a given array of chunk ids by the order in which they are required for the installation.
		 * @param ChunkList         The array to be sorted.
		 * @param Direction         The direction of sort. Ascending places soonest required chunk first.
		 */
		virtual void SortByUseOrder(TArray<FGuid>& ChunkList, ESortDirection Direction) const = 0;

		/**
		 * Find the next X chunk references that match the provided predicate.
		 * @param Count             The number of chunk entries that are desired.
		 * @param SelectPredicate   The predicate used to select each chunk.
		 * @return an array of unique chunk id entries, in the order in which they are required.
		 */
		virtual TArray<FGuid> GetNextReferences(int32 Count, const TFunction<bool(const FGuid&)>& SelectPredicate) const = 0;

		/**
		 * Select from the next X chunk references, entries that match the provided predicate.
		 * @param Count             The number of chunk entries to search through.
		 * @param SelectPredicate   The predicate used to select each chunk.
		 * @return an array of unique chunk id entries, in the order in which they are required.
		 */
		virtual TArray<FGuid> SelectFromNextReferences(int32 Count, const TFunction<bool(const FGuid&)>& SelectPredicate) const = 0;

		/**
		* Poorly named - Should be named "DoneWithChunk". The expectation is that we finish with chunks in the expected
		* use order.
		* 
		 * Pop the top reference from the tracker, indicating that operation has been performed.
		 * It is not valid to pop anything but the top guid, so it must be provided for verification of behavior.
		 * @param ChunkId           The id of the top chunk, this is used to verify behavior.
		 * @return true if the correct guid was provided and the reference was popped, false if the wrong guid
		 *         was provided and thus no change was made.
		 */
		virtual bool PopReference(const FGuid& ChunkId) = 0;

		// Return how many chunks are left to process.
		virtual int32 GetRemainingChunkCount() const = 0;

		// Return the ordered list we are consuming.
		// copies the _remaining_ items!
		virtual void CopyOutOrderedUseList(TArray<FGuid>& OutUseList) const = 0;

		// Returns the next usage index in _advancing order_ for the chunkid, or -1 if it won't
		// be used again.
		virtual int32 GetNextUsageForChunk(const FGuid& ChunkId, int32& OutLastUsageIndex) const = 0;

		virtual int32 GetCurrentUsageIndex() const = 0;
	};

	/**
	 * A factory for creating an IChunkReferenceTracker instance.
	 */
	class FChunkReferenceTrackerFactory
	{
	public:
		/**
		 * This implementation takes the install manifest and generates the internal data and chunk reference tracking based
		 * off of a set of files that will be constructed.
		 * @param InstallManifest   The install manifest to enumerate references from.
		 * @param FilesToConstruct  The set of files to be installed, other files will not be considered.
		 * @return the new IChunkReferenceTracker instance created.
		 */
		static IChunkReferenceTracker* Create(const IBuildManifestSet* Manifest, const TSet<FString>& FilesToConstruct);
		
		/**
		 * This implementation takes custom chunk references to track. The array should be every chunk reference, including duplicates, in order of use.
		 * See namespace CustomChunkReferencesHelpers for common setup examples to use.
		 * @param CustomChunkReferences    The custom chunk references to track.
		 * @return the new IChunkReferenceTracker instance created.
		 */
		static IChunkReferenceTracker* Create(TArray<FGuid> CustomChunkReferences);
	};


	/**
	 * Helpers for creating a CustomChunkReferences array for use with the equivalent FChunkReferenceTrackerFactory.
	 */
	namespace CustomChunkReferencesHelpers
	{
		/**
		 * This implementation takes the install manifest and generates the CustomChunkReferences needed for a chunk reference tracker based
		 * on caching data and so using each chunk once in the order that would be required to install the build.
		 * @param InstallManifest   The install manifest to enumerate references from.
		 * @return the chunk use references for FChunkReferenceTrackerFactory::Create.
		 */
		FORCEINLINE TArray<FGuid> OrderedUniqueReferences(const FBuildPatchAppManifestRef& InstallManifest)
		{
			TArray<FGuid> ChunkReferences;
			// Create our full list of chunks, no dupes, just one reference per chunk in the correct order.
			TArray<FString> AllFiles;
			TSet<FGuid> AllChunks;
			InstallManifest->GetFileList(AllFiles);
			for (const FString& File : AllFiles)
			{
				const FFileManifest* NewFileManifest = InstallManifest->GetFileManifest(File);
				if (NewFileManifest != nullptr)
				{
					for (const FChunkPart& ChunkPart : NewFileManifest->ChunkParts)
					{
						bool bWasAlreadyInSet = false;
						AllChunks.Add(ChunkPart.Guid, &bWasAlreadyInSet);
						if (!bWasAlreadyInSet)
						{
							ChunkReferences.Add(ChunkPart.Guid);
						}
					}
				}
			}
			return ChunkReferences;
		}
		
		/**
		 * This implementation takes a new install manifest and a current manifest. It generates the CustomChunkReferences needed for a chunk reference tracker based
		 * on caching data for a patch only, and so using the chunks in InstallManifest, which are not in CurrentManifest, once each in the order that they
		 * would be required to patch the build.
		 * @param InstallManifest   The install manifest to enumerate chunk references from.
		 * @param CurrentManifest   The current manifest to exclude chunk references from.
		 * @return the chunk use references for FChunkReferenceTrackerFactory::Create.
		 */
		FORCEINLINE TArray<FGuid> OrderedUniquePatchReferences(const FBuildPatchAppManifestRef& InstallManifest, const FBuildPatchAppManifestRef& CurrentManifest)
		{
			TArray<FGuid> ChunkReferences;
			// Create our list of chunks, no dupes, just one reference per chunk which appears only in InstallManifest, and in the correct order of use.
			TArray<FString> AllFiles;
			TSet<FGuid> OldChunks;
			TSet<FGuid> NewChunks;
			CurrentManifest->GetDataList(OldChunks);
			InstallManifest->GetFileList(AllFiles);
			for (const FString& File : AllFiles)
			{
				const FFileManifest* NewFileManifest = InstallManifest->GetFileManifest(File);
				if (NewFileManifest != nullptr)
				{
					for (const FChunkPart& ChunkPart : NewFileManifest->ChunkParts)
					{
						const bool bIsNewChunk = !OldChunks.Contains(ChunkPart.Guid);
						if (bIsNewChunk)
						{
							bool bWasAlreadyInSet = false;
							NewChunks.Add(ChunkPart.Guid, &bWasAlreadyInSet);
							if (!bWasAlreadyInSet)
							{
								ChunkReferences.Add(ChunkPart.Guid);
							}
						}
					}
				}
			}
			return ChunkReferences;
		}
		
		/**
		 * This implementation takes the install manifest and a tagset. It generates the CustomChunkReferences needed for a chunk reference tracker based
		 * on caching data and so using each chunk once in the order that would be required to install the build when using the same tagset provided.
		 * @param InstallManifest   The install manifest to enumerate references from.
		 * @param InstallTagSet     The tagset that would be used with install manifest, this will filter down required file list and thus required chunk list.
		 * @return the chunk use references for FChunkReferenceTrackerFactory::Create.
		 */
		FORCEINLINE TArray<FGuid> OrderedUniqueReferencesTagged(const FBuildPatchAppManifestRef& InstallManifest, const TSet<FString>& InstallTagSet)
		{
			TArray<FGuid> ChunkReferences;
			// Create our full list of chunks, no dupes, just one reference per chunk in the correct order.
			TArray<FString> TaggedFiles;
			TSet<FGuid> TaggedChunks;
			InstallManifest->GetTaggedFileList(InstallTagSet, TaggedFiles);
			for (const FString& File : TaggedFiles)
			{
				const FFileManifest* NewFileManifest = InstallManifest->GetFileManifest(File);
				if (NewFileManifest != nullptr)
				{
					for (const FChunkPart& ChunkPart : NewFileManifest->ChunkParts)
					{
						bool bWasAlreadyInSet = false;
						TaggedChunks.Add(ChunkPart.Guid, &bWasAlreadyInSet);
						if (!bWasAlreadyInSet)
						{
							ChunkReferences.Add(ChunkPart.Guid);
						}
					}
				}
			}
			return ChunkReferences;
		}
		
		/**
		 * This implementation takes a new install manifest, a current manifest, and a tagset for each. It generates the CustomChunkReferences needed for a chunk reference tracker based
		 * on caching data for a patch only, and so using the chunks in InstallManifest, which are not in CurrentManifest, once each in the order that would be required
         * to patch the build when using the tagsets provided.
		 * @param InstallManifest   The install manifest to enumerate chunk references from.
		 * @param InstallTagSet     The tagset that would be used with install manifest, this will filter down required file list and thus required chunk list.
		 * @param CurrentManifest   The current manifest if considering an update, which reduces the number of chunks required.
		 * @param CurrentTagSet     The tagset filtering files from CurrentManifest, so that chunks would not be ignored for files not present on disk. This would often be the same as InstallTagSet unless
		 *                          InstallManifest changed the file tag names, or you are calling for purpose of adding/removing tags.
		 * @return the chunk use references for FChunkReferenceTrackerFactory::Create.
		 */
		FORCEINLINE TArray<FGuid> OrderedUniquePatchReferencesTagged(const FBuildPatchAppManifestRef& InstallManifest, const TSet<FString>& InstallTagSet, const FBuildPatchAppManifestRef& CurrentManifest, const TSet<FString>& CurrentTagSet)
		{
			// Enumerate what is available in the current build.
			TSet<FString> FilesInstalled;
			TSet<FGuid> ChunksRequired;
			TSet<FGuid> ChunksInstalled;
			CurrentManifest->GetTaggedFileList(CurrentTagSet, FilesInstalled);
			CurrentManifest->GetChunksRequiredForFiles(FilesInstalled, ChunksRequired);
			CurrentManifest->EnumerateProducibleChunks(CurrentTagSet, ChunksRequired, ChunksInstalled);

			TArray<FGuid> ChunkReferences;
			// Create our list of chunks, no dupes, just one reference per chunk which appears only in InstallManifest, and in the correct order of use.
			TArray<FString> TaggedFiles;
			TSet<FGuid> TaggedNewChunks;
			InstallManifest->GetTaggedFileList(InstallTagSet, TaggedFiles);
			for (const FString& TaggedFile : TaggedFiles)
			{
				const FFileManifest* const OldFileManifest = CurrentManifest->GetFileManifest(TaggedFile);
				const FFileManifest* const NewFileManifest = InstallManifest->GetFileManifest(TaggedFile);
				if (!OldFileManifest || OldFileManifest->SHA1Hash != NewFileManifest->SHA1Hash || !FilesInstalled.Contains(TaggedFile))
				{
					for (const FChunkPart& ChunkPart : NewFileManifest->ChunkParts)
					{
						const bool bIsNewChunk = !ChunksInstalled.Contains(ChunkPart.Guid);
						if (bIsNewChunk)
						{
							bool bWasAlreadyInSet = false;
							TaggedNewChunks.Add(ChunkPart.Guid, &bWasAlreadyInSet);
							if (!bWasAlreadyInSet)
							{
								ChunkReferences.Add(ChunkPart.Guid);
							}
						}
					}
				}
			}
			return ChunkReferences;
		}
	}
}
