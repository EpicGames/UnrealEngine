// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/Function.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "Memory/MemoryFwd.h"
#include "Misc/Guid.h"

namespace BuildPatchServices
{
	class IChunkDataAccess;

	// This class is for providing chunks to a threaded file constructor that manages its own backing store.
	class IConstructorChunkSource
	{
	public:
		virtual ~IConstructorChunkSource() = default;

		DECLARE_DELEGATE_FourParams(FChunkRequestCompleteDelegate, const FGuid& DataId, bool bAborted, bool bFailedToRead, void* UserPtr);

		typedef TUniqueFunction<void(bool bIsAborted)> FRequestProcessFn;

		virtual FRequestProcessFn CreateRequest(const FGuid& DataId, FMutableMemoryView DestinationBuffer, void* UserPtr, FChunkRequestCompleteDelegate CompleteFn) = 0;

		// Return the reference index at which we can no longer get this data id from this source.
		virtual int32 GetChunkUnavailableAt(const FGuid& DataId) const = 0;
	};

	/**
	 * An interface providing basic access to retrieving chunk data.
	 */
	class IChunkSource
	{
	public:
		virtual ~IChunkSource() = default;

		/**
		 * Gets the chunk data for the given id if this source has that chunk.
		 * @param DataId    The id for the chunk.
		 * @return pointer to the chunk data. or nullptr if this source does not contain the requested chunk.
		 */
		virtual IChunkDataAccess* Get(const FGuid& DataId) = 0;

		/**
		 * Adds additional chunk requirements to the source, the implementation will return the resulting set of chunks that could not be accessed
		 * via this source.
		 * @param NewRequirements   The set of chunks that are now additionally desired.
		 * @return the set of chunks provided which can not be acquired via this chunk.
		 */
		virtual TSet<FGuid> AddRuntimeRequirements(TSet<FGuid> NewRequirements) = 0;

		/**
		 * Adds a requirement to reacquire a chunk that may have already been acquired before by this source. This allows the source implementation to
		 * to support forward reading of chunks, and track which it would not need to request.
		 * @param RepeatRequirement     The chunk that needs reacquiring.
		 * @return true if the chunk provided can be acquired by this source.
		 */
		virtual bool AddRepeatRequirement(const FGuid& RepeatRequirement) = 0;

		/**
		 * Sets a callback to be used when chunks that are being fetched by this source are no longer available.
		 * @param Callback          The function to call with the set of chunks no longer still available.
		 */
		virtual void SetUnavailableChunksCallback(TFunction<void(TSet<FGuid>)> Callback) = 0;
	};
}
