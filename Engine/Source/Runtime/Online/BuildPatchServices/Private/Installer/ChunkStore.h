// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/Function.h"

namespace BuildPatchServices
{
	class IChunkDataAccess;

	/**
	 * An interface providing access to storage of chunk data instances.
	 */
	class IChunkStore
	{
	public:
		virtual ~IChunkStore() = default;

		/**
		 * Put chunk data into this store. Chunk data unique ptr must be moved in, the store becomes the owner
		 * of the memory and its lifetime.
		 * Whether or not the call involves actually storing the data provided is implementation specific. It is possible to implement
		 * readonly/null IChunkStore.
		 * @param DataId    The GUID for the data.
		 * @param ChunkData The instance of the data. This must be moved in.
		 */
		virtual void Put(const FGuid& DataId, TUniquePtr<IChunkDataAccess> ChunkData) = 0;

		/**
		 * Get access to chunk data contained in this store.
		 * The returned data ptr is only valid until the next Get call, or a remove call for the same data.
		 * @param DataId    The GUID for the data.
		 * @return ptr to the data instance, or nullptr if this GUID was not in the store.
		 */
		virtual IChunkDataAccess* Get(const FGuid& DataId) = 0;

		/**
		 * Remove chunk data from this store. The data access is returned, this will cause destruction once out of scope.
		 * Whether or not the call involves actual data destruction is implementation specific. It is possible to implement
		 * readonly/null IChunkStore.
		 * @param DataId    The GUID for the data.
		 * @return the data instance referred to, or invalid if this GUID was not in the store.
		 */
		virtual TUniquePtr<IChunkDataAccess> Remove(const FGuid& DataId) = 0;

		/**
		 * Get the number of chunks this store can hold. For unsized stores, it is expected to return max int32 value.
		 * @return the store size.
		 */
		virtual int32 GetSize() const = 0;

		/**
		 * Sets a callback to be used when chunks which have been Put, are lost.
		 * Examples of why this may occur:
		 *     An eviction policy instructs the store to boot a chunk, but this store has no overflow store provided. (see IChunkEvictionPolicy::Query).
		 *     The system backing this store (e.g. a file on disk storage) experiences a failure and the chunk could not be held.
		 * NB: The callback is not executed for a standard Clean instruction from an eviction policy.
		 * @param Callback  The function to call with the chunk that is no longer available.
		 */
		virtual void SetLostChunkCallback(TFunction<void(const FGuid&)> Callback) = 0;
	};
}
