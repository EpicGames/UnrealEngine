// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/CloudChunkSource.h"
#include "CoreMinimal.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Queue.h"
#include "Misc/ScopeLock.h"
#include "Algo/AllOf.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "Core/MeanValue.h"
#include "Core/Platform.h"
#include "Http.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/ChunkStore.h"
#include "Installer/DownloadService.h"
#include "Installer/InstallerError.h"
#include "Installer/DownloadConnectionCount.h"
#include "Installer/MessagePump.h"
#include "Common/StatsCollector.h"
#include "Interfaces/IBuildInstaller.h"
#include "Interfaces/IBuildInstallerSharedContext.h"
#include "BuildPatchUtil.h"
#include "Installer/Statistics/DownloadServiceStatistics.h"
#include "Memory/SharedBuffer.h"
#include "Tasks/Task.h"
#include "ProfilingDebugging/CountersTrace.h"
#include <atomic>
DECLARE_LOG_CATEGORY_EXTERN(LogCloudChunkSource, Log, All);
DEFINE_LOG_CATEGORY(LogCloudChunkSource);

namespace BuildPatchServices
{
	/**
	 * A class used to monitor the download success rate.
	 */
	class FChunkSuccessRate
	{
	public:
		FChunkSuccessRate();

		double GetOverall();
		double GetImmediate();
		void AddSuccess();
		void AddFail();

	private:
		double LastAverage;
		double ImmediateAccumulatedValue;
		double ImmediateValueCount;
		double TotalAccumulatedValue;
		double TotalValueCount;
	};

	FChunkSuccessRate::FChunkSuccessRate()
		: LastAverage(1.0L)
		, ImmediateAccumulatedValue(0.0L)
		, ImmediateValueCount(0.0L)
		, TotalAccumulatedValue(0.0L)
		, TotalValueCount(0.0L)
	{
	}

	double FChunkSuccessRate::GetOverall()
	{
		if (!(TotalValueCount > 0.0L))
		{
			return 0.0L;
		}
		return TotalAccumulatedValue / TotalValueCount;
	}
	double FChunkSuccessRate::GetImmediate()
	{
		static const uint32 MinimumCount = 3U;

		if (ImmediateValueCount >= MinimumCount)
		{
			LastAverage = ImmediateAccumulatedValue / ImmediateValueCount;
			ImmediateAccumulatedValue = ImmediateValueCount = 0.0L;
		}

		return LastAverage;
	}
	void FChunkSuccessRate::AddSuccess()
	{
		ImmediateAccumulatedValue += 1.0L;
		ImmediateValueCount += 1.0L;
		TotalAccumulatedValue += 1.0L;
		TotalValueCount += 1.0L;
	}

	void FChunkSuccessRate::AddFail()
	{
		ImmediateValueCount += 1.0L;
		TotalValueCount += 1.0L;
	}

	static float GetRetryDelay(const TArray<float>& RetryDelayTimes, int32 RetryNum)
	{
		const int32 RetryTimeIndex = FMath::Clamp<int32>(RetryNum - 1, 0, RetryDelayTimes.Num() - 1);
		return RetryDelayTimes[RetryTimeIndex];
	}
	
	static EBuildPatchDownloadHealth GetDownloadHealth(bool bIsDisconnected, const TArray<float>& HealthPercentages, float ChunkSuccessRate)
	{
		EBuildPatchDownloadHealth DownloadHealth;
		if (bIsDisconnected)
		{
			DownloadHealth = EBuildPatchDownloadHealth::Disconnected;
		}
		else if (ChunkSuccessRate >= HealthPercentages[(int32)EBuildPatchDownloadHealth::Excellent])
		{
			DownloadHealth = EBuildPatchDownloadHealth::Excellent;
		}
		else if (ChunkSuccessRate >= HealthPercentages[(int32)EBuildPatchDownloadHealth::Good])
		{
			DownloadHealth = EBuildPatchDownloadHealth::Good;
		}
		else if (ChunkSuccessRate >= HealthPercentages[(int32)EBuildPatchDownloadHealth::OK])
		{
			DownloadHealth = EBuildPatchDownloadHealth::OK;
		}
		else
		{
			DownloadHealth = EBuildPatchDownloadHealth::Poor;
		}
		return DownloadHealth;
	}


	/**
	 * Helper class for limiting download rate using an implementation of the token bucket algorithm.
	 * 
	 * One 'token' in this context is a byte that we can schedule for download. On tick, we increase
	 * the number of available tokens according to the configured download rate limit. When we want
	 * to schedule a chunk for download we must confirm that enough tokens are available and consume
	 * them appropriately. The result is that the average download rate cannot exceed the configured
	 * limit, although short bursts of higher rates are likely.
	 */
	class FDownloadRateLimiter
	{
	public:
		FDownloadRateLimiter(uint64 InRateLimitBps)
			: RateLimitBps(InRateLimitBps)
			, TokenCount(InRateLimitBps)
			, LastTickCycleCount(0) {}

		/**
		 * Tick this component. Must be called regularly from a single thread.
		 */
		void Tick()
		{
			// Sample the max burst size once now since it could be
			// changed by another thread calling SetRateLimitBps.
			const uint64 MaxBurstValue = MaxBurstSize();

			uint64 CurrentTickCycleCount = FStatsCollector::GetCycles();

			// On the first tick the only thing we should do is update LastTickCycleCount.
			if (LastTickCycleCount != 0)
			{
				// Count the time elapsed since last tick and increment TokenCount appropriately.
				uint64 ExpectedTokenCount, NewTokenCount;
				do
				{
					ExpectedTokenCount = TokenCount.Load();
					CurrentTickCycleCount = FStatsCollector::GetCycles();
					const uint64 Increment = (uint64)(RateLimitBps.Load() * FStatsCollector::CyclesToSeconds(CurrentTickCycleCount - LastTickCycleCount));
					NewTokenCount = FMath::Min(ExpectedTokenCount + Increment, MaxBurstValue);

					// If the compare exchange fails because another thread
					// changed the value in between, try again.
				}
				while (!TokenCount.CompareExchange(ExpectedTokenCount, NewTokenCount));
			}
			LastTickCycleCount = CurrentTickCycleCount;
		}

		/**
		 * Update the target rate limit.
		 * @param InRateLimitBps Rate limit in bytes per second. 0 means unlimited.
		 */
		void SetRateLimitBps(uint64 InRateLimitBps)
		{
			RateLimitBps.Exchange(InRateLimitBps);
			TokenCount.Exchange(InRateLimitBps);
		}

		/**
		 * Attempt to schedule a chunk download of ChunkSizeBytes.
		 * @param ChunkSizeBytes Size of the chunk.
		 * Returns 0 if the chunk can be scheduled, or the estimated ms until it can.
		 */
		uint64 TrySchedule(uint64 ChunkSizeBytes)
		{
			// Sample the max burst size once now since it could be
			// changed by another thread calling SetRateLimitBps.
			const uint64 MaxBurstValue = MaxBurstSize();

			// Compute the token cost for this chunk. There are two possible cases:
			// 1. ChunkSizeBytes is <= maximum burst size.
			//    The cost is simply equal to ChunkSizeBytes.
			// 2. ChunkSizeBytes is > maximum burst size.
			//    The cost is equal to the maximum burst size. This avoids deadlocks
			//    regardless of the relationship between the rate limit and chunk size,
			//    even if it means that the rate limit cannot be strictly adhered to.
			const uint64 TokenCost = ChunkSizeBytes > MaxBurstValue ? MaxBurstValue : ChunkSizeBytes;

			//UE_LOGF(LogTemp, Warning, "Scheduling: %llu bytes, cost %llu, tokens %llu", ChunkSizeBytes, TokenCost, TokenCount.Load());
			uint64 BytesPerSecondLimit = RateLimitBps.Load();

			if (BytesPerSecondLimit != 0)
			{
				// Do an atomic compare exchange to make sure we don't double decrement
				// and end up with e.g. negative TokenCount.
				uint64 ExpectedTokenCount, NewTokenCount;
				do
				{
					// If the cost is greater than what's available, return false.
					ExpectedTokenCount = TokenCount.Load();
					if (ExpectedTokenCount < TokenCost)
					{
						// We know the rate and how many we are missing, estimate how long until it's ready.
						uint64 NeededBytes = TokenCost - ExpectedTokenCount;
						uint64 BytesPerMsLimit = FMath::Max(1ULL, BytesPerSecondLimit / 1000);

						// 0 means it can be scheduled, return a min of 1 ms delay
						uint64 MsUntilToken = FMath::Max(1ULL, NeededBytes / BytesPerMsLimit);
						return MsUntilToken;
					}

					// Otherwise atomically decrement and return true.
					NewTokenCount = ExpectedTokenCount - TokenCost;

					// If the compare exchange fails because another thread
					// changed the value in between, try again.
				}
				while (!TokenCount.CompareExchange(ExpectedTokenCount, NewTokenCount));
			}
			return 0;
		}

		/**
		 * Refunds token based on the ChunkSizeBytes.
		 * @param ChunkSizeBytes Size of the chunk.
		 */
		void Refund(uint64 ChunkSizeBytes)
		{
			if (RateLimitBps.Load() != 0)
			{
				uint64 ExpectedTokenCount, NewTokenCount;
				do
				{
					ExpectedTokenCount = TokenCount.Load();
					NewTokenCount = FMath::Min(MaxBurstSize(), ExpectedTokenCount + ChunkSizeBytes);
				}
				while (!TokenCount.CompareExchange(ExpectedTokenCount, NewTokenCount));
			}
		}

	private:
		/**
		 * Returns the current maximum burst size, which is also
		 * the maximum number of tokens that may be stored.
		 */
		uint64 MaxBurstSize(void) const { return RateLimitBps.Load() * BurstMultiplier; }

	private:
		static const uint64 BurstMultiplier = 4;
		TAtomic<uint64> RateLimitBps;
		TAtomic<uint64> TokenCount;
		uint64 LastTickCycleCount;
	};
	
	/**
	 * The concrete implementation of ICloudChunkSource
	 */
	class FCloudChunkSource
		: public ICloudChunkSource
	{
	private:
		/**
		 * This is a mediator class for binding thread safe shared ptr delegates for the download service, without having to enforce that
		 * this service should be made using TShared* reference controllers.
		 */
		class FDownloadDelegates
		{
		public:
			void OnDownloadProgress(int32 RequestId, uint64 BytesSoFar);
			void OnDownloadComplete(int32 RequestId, const FDownloadRef& Download);

			TMap<int32, FDownloadRef> GetCompletedDownloads();

			// Returns the number of seconds since the last data was received.
			// Tracking health and connection state
			double GetSecondsSinceLastData() const;
		private:
			// The point in time when the latest data was received;
			std::atomic<int64> CyclesAtLastData{0};

			// Communication from download thread to processing thread.
			FCriticalSection CompletedDownloadsCS;
			TMap<int32, FDownloadRef> CompletedDownloads;
		};
		
		/**
		 * This struct holds variable for each individual task.
		 */
		struct FTaskInfo
		{
		public:
			FTaskInfo();

		public:
			// Are we currently trying to downloads?
			bool bQueuedForDownload = false;
			int32 CloudDirUsed = 0;
			FString UrlUsed;
			int32 RetryNum;
			int32 ExpectedSize;
			double SecondsAtRequested;
			double SecondsAtFail;
		};

	public:
		FCloudChunkSource(FCloudSourceConfig InConfiguration, IPlatform* Platform, IChunkStore* InChunkStore, IDownloadService* InDownloadService, IChunkReferenceTracker* InChunkReferenceTracker, IChunkDataSerialization* InChunkDataSerialization, IMessagePump* InMessagePump, IInstallerError* InInstallerError, IDownloadConnectionCount* InDownloadConnectionCount, ICloudChunkSourceStat* InCloudChunkSourceStat, IBuildManifestSet* ManifestSet, TSet<FGuid> InInitialDownloadSet);
		~FCloudChunkSource();

		// IControllable interface begin.
		virtual void SetPaused(bool bInIsPaused) override;
		virtual void Abort() override;
		virtual void Reset() override;
		// IControllable interface end.

		// IChunkSource interface begin.
		virtual IChunkDataAccess* Get(const FGuid& DataId) override;
		virtual TSet<FGuid> AddRuntimeRequirements(TSet<FGuid> NewRequirements) override;
		virtual bool AddRepeatRequirement(const FGuid& RepeatRequirement) override;
		virtual void SetUnavailableChunksCallback(TFunction<void(TSet<FGuid>)> Callback) override;
		// IChunkSource interface end.

		// ICloudChunkSource interface begin.
		virtual void SetDownloadBytesPerSecondLimit(uint64 InDownloadRateLimitBps) override;
		virtual void ThreadRun() override;
		// ICloudChunkSource interface end.

	private:
		void EnsureAquiring(const FGuid& DataId);

		FGuid GetNextTask(const TMap<FGuid, FTaskInfo>& TaskInfos, const TMap<int32, FGuid>& InFlightDownloads, const TSet<FGuid>& TotalRequiredChunks, const TSet<FGuid>& PriorityRequests, const TSet<FGuid>& FailedDownloads, const TSet<FGuid>& Stored, TArray<FGuid>& DownloadQueue, EBuildPatchDownloadHealth DownloadHealth);

	private:
		const FCloudSourceConfig Configuration;
		IPlatform* Platform;
		IChunkStore* ChunkStore;
		IDownloadService* DownloadService;
		IChunkReferenceTracker* ChunkReferenceTracker;
		IChunkDataSerialization* ChunkDataSerialization;
		IMessagePump* MessagePump;
		IInstallerError* InstallerError;
		ICloudChunkSourceStat* CloudChunkSourceStat;
		IBuildManifestSet* ManifestSet;
		const TSet<FGuid> InitialDownloadSet;
		TPromise<void> Promise;
		TFuture<void> Future;
		IBuildInstallerThread* Thread = nullptr;

		// Communication from external process requesting pause/abort.
		FThreadSafeBool bIsPaused;
		FThreadSafeBool bShouldAbort;

		// Download throttling helper.
		FDownloadRateLimiter DownloadRateLimiter;

		// Communication from request threads to processing thread.
		FCriticalSection RequestedDownloadsCS;
		TArray<FGuid> RequestedDownloads;

		// Communication and storage of incoming additional requirements.
		TQueue<TSet<FGuid>, EQueueMode::Mpsc> RuntimeRequestMessages;

		// Communication and storage of incoming repeat requirements.
		TQueue<FGuid, EQueueMode::Mpsc> RepeatRequirementMessages;

		// Determine if additional download requests should be initiated.
		IDownloadConnectionCount* DownloadCount;

		// If we start getting failures on our downloads, we track which ones
		// fail and avoid them until everything goes bad. Initially we just hit the
		// first directory.
		int32 CurrentBestCloudDir = 0;
		TArray<int32> CloudDirFailureCount;
	};

	void FCloudChunkSource::FDownloadDelegates::OnDownloadProgress(int32 RequestId, uint64 BytesSoFar)
	{
		CyclesAtLastData = FStatsCollector::GetCycles();
	}

	void FCloudChunkSource::FDownloadDelegates::OnDownloadComplete(int32 RequestId, const FDownloadRef& Download)
	{
		FScopeLock ScopeLock(&CompletedDownloadsCS);
		CompletedDownloads.Add(RequestId, Download);
	}

	double FCloudChunkSource::FDownloadDelegates::GetSecondsSinceLastData() const
	{
		return FStatsCollector::CyclesToSeconds(FStatsCollector::GetCycles() - CyclesAtLastData);
	}

	TMap<int32, FDownloadRef> FCloudChunkSource::FDownloadDelegates::GetCompletedDownloads()
	{
		FScopeLock ScopeLock(&CompletedDownloadsCS);
		return MoveTemp(CompletedDownloads);
	}

	FCloudChunkSource::FTaskInfo::FTaskInfo()
		: UrlUsed()
		, RetryNum(0)
		, ExpectedSize(0)
		, SecondsAtFail(0)
	{
	}

	FCloudChunkSource::FCloudChunkSource(FCloudSourceConfig InConfiguration, IPlatform* InPlatform, IChunkStore* InChunkStore, IDownloadService* InDownloadService, IChunkReferenceTracker* InChunkReferenceTracker, IChunkDataSerialization* InChunkDataSerialization, IMessagePump* InMessagePump, IInstallerError* InInstallerError, IDownloadConnectionCount* InDownloadConnectionCount, ICloudChunkSourceStat* InCloudChunkSourceStat, IBuildManifestSet* InManifestSet, TSet<FGuid> InInitialDownloadSet)
		: Configuration(MoveTemp(InConfiguration))
		, Platform(InPlatform)
		, ChunkStore(InChunkStore)
		, DownloadService(InDownloadService)
		, ChunkReferenceTracker(InChunkReferenceTracker)
		, ChunkDataSerialization(InChunkDataSerialization)
		, MessagePump(InMessagePump)
		, InstallerError(InInstallerError)
		, CloudChunkSourceStat(InCloudChunkSourceStat)
		, ManifestSet(InManifestSet)
		, InitialDownloadSet(MoveTemp(InInitialDownloadSet))
		, Promise()
		, Future()
		, Thread(nullptr)
		, bIsPaused(false)
		, bShouldAbort(false)
		, DownloadRateLimiter(InConfiguration.DownloadRateLimitBps)
		, RequestedDownloadsCS()
		, RequestedDownloads()
		, DownloadCount(InDownloadConnectionCount)
	{
		CloudDirFailureCount.SetNumZeroed(Configuration.CloudRoots.Num());

		Future = Promise.GetFuture();
		if (Configuration.bRunOwnThread)
		{
			check(Configuration.SharedContext);
			Thread = Configuration.SharedContext->CreateThread();
			Thread->RunTask([this]() { ThreadRun(); });
		}
	}

	FCloudChunkSource::~FCloudChunkSource()
	{
		bShouldAbort = true;
		Future.Wait();

		if (Thread)
		{
			Configuration.SharedContext->ReleaseThread(Thread);
			Thread = nullptr;
		}
	}

	void FCloudChunkSource::SetPaused(bool bInIsPaused)
	{
		bIsPaused = bInIsPaused;
	}

	void FCloudChunkSource::Abort()
	{
		bShouldAbort = true;
	}

	void FCloudChunkSource::Reset()
	{
		bShouldAbort = false;
	}

	IChunkDataAccess* FCloudChunkSource::Get(const FGuid& DataId)
	{
		IChunkDataAccess* ChunkData = ChunkStore->Get(DataId);
		if (ChunkData == nullptr)
		{
			// Make sure we are trying to download this chunk before waiting for it to complete.
			EnsureAquiring(DataId);

			// Wait for the chunk to be available.
			while ((ChunkData = ChunkStore->Get(DataId)) == nullptr && !bShouldAbort)
			{
				Platform->Sleep(0.01f);
			}
		}
		return ChunkData;
	}

	TSet<FGuid> FCloudChunkSource::AddRuntimeRequirements(TSet<FGuid> NewRequirements)
	{
		CloudChunkSourceStat->OnAcceptedNewRequirements(NewRequirements);
		RuntimeRequestMessages.Enqueue(MoveTemp(NewRequirements));
		// We don't have a concept of being unavailable yet.
		return TSet<FGuid>();
	}

	bool FCloudChunkSource::AddRepeatRequirement(const FGuid& RepeatRequirement)
	{
		RepeatRequirementMessages.Enqueue(RepeatRequirement);
		// We don't have a concept of being unavailable yet.
		return true;
	}

	void FCloudChunkSource::SetUnavailableChunksCallback(TFunction<void(TSet<FGuid>)> Callback)
	{
		// We don't have a concept of being unavailable yet.
	}

	void FCloudChunkSource::SetDownloadBytesPerSecondLimit(uint64 InDownloadRateLimitBps)
	{
		DownloadRateLimiter.SetRateLimitBps(InDownloadRateLimitBps);
	}

	void FCloudChunkSource::EnsureAquiring(const FGuid& DataId)
	{
		FScopeLock ScopeLock(&RequestedDownloadsCS);
		RequestedDownloads.Add(DataId);
	}

	FGuid FCloudChunkSource::GetNextTask(const TMap<FGuid, FTaskInfo>& TaskInfos, const TMap<int32, FGuid>& InFlightDownloads, const TSet<FGuid>& TotalRequiredChunks, const TSet<FGuid>& PriorityRequests, const TSet<FGuid>& FailedDownloads, const TSet<FGuid>& Stored, TArray<FGuid>& DownloadQueue, EBuildPatchDownloadHealth DownloadHealth)
	{
		// Check for aborting.
		if (bShouldAbort)
		{
			return FGuid();
		}

		// Check priority request.
		if (PriorityRequests.Num() > 0)
		{
			return *PriorityRequests.CreateConstIterator();
		}

		// Check retries.
		const double SecondsNow = FStatsCollector::GetSeconds();
		FGuid ChunkToRetry;
		for (auto FailedIt = FailedDownloads.CreateConstIterator(); FailedIt && !ChunkToRetry.IsValid(); ++FailedIt)
		{
			const FTaskInfo& FailedDownload = TaskInfos[*FailedIt];
			const double SecondsSinceFailure = SecondsNow - FailedDownload.SecondsAtFail;
			if (SecondsSinceFailure >= GetRetryDelay(Configuration.RetryDelayTimes, FailedDownload.RetryNum))
			{
				ChunkToRetry = *FailedIt;
			}
		}
		if (ChunkToRetry.IsValid())
		{
			return ChunkToRetry;
		}

		// Check if we can start more.
		uint32 NumProcessing = InFlightDownloads.Num() + FailedDownloads.Num();
		const uint32 MaxDownloads = DownloadCount->GetAdjustedCount(InFlightDownloads.Num(), DownloadHealth);
		
		if ( NumProcessing < MaxDownloads)
		{
			// Find the next chunks to get if we completed the last batch.
			if (DownloadQueue.Num() == 0)
			{
				// Select the next X chunks that we initially instructed to download.
				TFunction<bool(const FGuid&)> SelectPredicate = [&TotalRequiredChunks](const FGuid& ChunkId) { return TotalRequiredChunks.Contains(ChunkId); };
				// Grab all the chunks relevant to this source to fill the store.
				int32 SearchLength = FMath::Max(ChunkStore->GetSize(), Configuration.PreFetchMinimum);
				DownloadQueue = ChunkReferenceTracker->SelectFromNextReferences(SearchLength, SelectPredicate);
				// Remove already downloading or complete chunks.
				TFunction<bool(const FGuid&)> RemovePredicate = [&TaskInfos, &FailedDownloads, &Stored](const FGuid& ChunkId)
				{
					const FTaskInfo* TaskInfo = TaskInfos.Find(ChunkId);
					return (TaskInfo && TaskInfo->bQueuedForDownload) || FailedDownloads.Contains(ChunkId) || Stored.Contains(ChunkId);
				};

				DownloadQueue.RemoveAll(RemovePredicate);
				// Clamp to configured max.
				DownloadQueue.SetNum(FMath::Min(DownloadQueue.Num(), Configuration.PreFetchMaximum), EAllowShrinking::No);
				// Reverse so the array is a stack for popping.
				Algo::Reverse(DownloadQueue);
			}

			// Return the next chunk in the queue
			if (DownloadQueue.Num() > 0)
			{
				return DownloadQueue.Last();
			}
		}

		return FGuid();
	}

	void FCloudChunkSource::ThreadRun()
	{
		TMap<FGuid, FTaskInfo> TaskInfos;
		TMap<int32, FGuid> InFlightDownloads;
		TSet<FGuid> FailedDownloads;
		TSet<FGuid> PlacedInStore;
		TSet<FGuid> PriorityRequests;
		TArray<FGuid> DownloadQueue;
		bool bDownloadsStarted = Configuration.bBeginDownloadsOnFirstGet == false;
		bool bTotalRequiredTrimmed = false;
		FMeanValue MeanChunkTime;
		FChunkSuccessRate ChunkSuccessRate;
		EBuildPatchDownloadHealth TrackedDownloadHealth = EBuildPatchDownloadHealth::Excellent;
		int32 TrackedActiveRequestCount = 0;
		int32 TrackedActiveRequestAverageSum = 0;
		int32 TrackedActiveRequestAverageCount = 0;
		double SecondsAtLastTrackedAverage = 0;
		TSet<FGuid> TotalRequiredChunks = InitialDownloadSet;
		uint64 TotalRequiredChunkSize = ManifestSet->GetDownloadSize(TotalRequiredChunks);
		uint64 TotalReceivedData = 0;
		uint64 RepeatRequirementSize = 0;
		const TSharedRef<FDownloadDelegates, ESPMode::ThreadSafe> DownloadDelegates = MakeShared<FDownloadDelegates, ESPMode::ThreadSafe>();

		FDownloadProgressDelegate OnDownloadProgressDelegate(FDownloadProgressDelegate::CreateThreadSafeSP(DownloadDelegates, &FDownloadDelegates::OnDownloadProgress));
		FDownloadCompleteDelegate OnDownloadCompleteDelegate(FDownloadCompleteDelegate::CreateThreadSafeSP(DownloadDelegates, &FDownloadDelegates::OnDownloadComplete));

		// Chunk Uri Processing
		typedef TTuple<FGuid, FChunkUriResponse> FGuidUriResponse;
		typedef TQueue<FGuidUriResponse, EQueueMode::Mpsc> FGuidUriResponseQueue;
		TSharedRef<FGuidUriResponseQueue, ESPMode::ThreadSafe> ChunkUriResponsesRef = MakeShared<FGuidUriResponseQueue>(); // use Mpsc, message pump callback may be on this thread or message pump thread
		TWeakPtr<FGuidUriResponseQueue, ESPMode::ThreadSafe> WeakChunkUriResponses = ChunkUriResponsesRef;
		FGuidUriResponseQueue& ChunkUriResponses = ChunkUriResponsesRef.Get();
		TSet<FGuid> RequestedChunkUris;
		TMap<FGuid, FChunkUriResponse> ChunkUris;

		// Provide initial stat values.
		CloudChunkSourceStat->OnRequiredDataUpdated(TotalRequiredChunkSize + RepeatRequirementSize);
		CloudChunkSourceStat->OnReceivedDataUpdated(TotalReceivedData);
		CloudChunkSourceStat->OnDownloadHealthUpdated(TrackedDownloadHealth);
		CloudChunkSourceStat->OnSuccessRateUpdated(ChunkSuccessRate.GetOverall());
		CloudChunkSourceStat->OnActiveRequestCountUpdated(TrackedActiveRequestCount);

		while (!bShouldAbort)
		{
			bool bRequiredDataUpdated = false;
			DownloadRateLimiter.Tick();

			// 'Forget' any repeat requirements.
			FGuid RepeatRequirement;
			while (RepeatRequirementMessages.Dequeue(RepeatRequirement))
			{
				if (PlacedInStore.Remove(RepeatRequirement) > 0)
				{
					RequestedChunkUris.Remove(RepeatRequirement);
					RepeatRequirementSize += ManifestSet->GetDownloadSize(RepeatRequirement);
					bRequiredDataUpdated = true;
				}
			}
			// Process new runtime requests.
			TSet<FGuid> Temp;
			while (RuntimeRequestMessages.Dequeue(Temp))
			{
				Temp = Temp.Intersect(ChunkReferenceTracker->GetReferencedChunks());
				Temp = Temp.Difference(TotalRequiredChunks);
				if (Temp.Num() > 0)
				{
					TotalRequiredChunkSize += ManifestSet->GetDownloadSize(Temp);
					TotalRequiredChunks.Append(MoveTemp(Temp));
					bRequiredDataUpdated = true;
				}
			}
			// Grab incoming requests as a priority.
			TArray<FGuid> FrameRequestedDownloads;
			RequestedDownloadsCS.Lock();
			FrameRequestedDownloads = MoveTemp(RequestedDownloads);
			RequestedDownloadsCS.Unlock();
			TSet<FGuid> PriorityUrlRequests;
			for (const FGuid& FrameRequestedDownload : FrameRequestedDownloads)
			{
				bDownloadsStarted = true;
				if (!TaskInfos.Contains(FrameRequestedDownload) && !PlacedInStore.Contains(FrameRequestedDownload))
				{
					PriorityRequests.Add(FrameRequestedDownload);
					if (!RequestedChunkUris.Contains(FrameRequestedDownload))
					{
						PriorityUrlRequests.Add(FrameRequestedDownload);
						UE_LOGF(LogCloudChunkSource, Log, "New chunk uri priority request: %ls", *FrameRequestedDownload.ToString());
					}
					else
					{
						UE_LOGF(LogCloudChunkSource, Log, "New priority request: %ls", *FrameRequestedDownload.ToString());
					}
					if (!TotalRequiredChunks.Contains(FrameRequestedDownload))
					{
						TotalRequiredChunks.Add(FrameRequestedDownload);
						TotalRequiredChunkSize += ManifestSet->GetDownloadSize(FrameRequestedDownload);
						bRequiredDataUpdated = true;
					}
				}
			}
			// Select the next X chunks that are for downloading, so we can request URIs.
			TFunction<bool(const FGuid&)> SelectPredicate = [&TotalRequiredChunks, &RequestedChunkUris](const FGuid& ChunkId)
			{
				// if we requre it and we haven't already requested it.
				return TotalRequiredChunks.Contains(ChunkId) && !RequestedChunkUris.Contains(ChunkId);
			};

			TSet<FGuid> ChunkUrisToRequest;
			// Don't take the lock over the reference stack if we can't ever pass our selection predicate
			if (TotalRequiredChunks.Num())
			{
				ChunkUrisToRequest = TSet<FGuid> { ChunkReferenceTracker->SelectFromNextReferences(FGenericPlatformMath::Max(1, Configuration.PreFetchMaximum - PriorityUrlRequests.Num()), SelectPredicate) };
			}
			ChunkUrisToRequest.Append(MoveTemp(PriorityUrlRequests));

			for (const FGuid& ChunkUriToRequest : ChunkUrisToRequest)
			{
				RequestedChunkUris.Add(ChunkUriToRequest);
				FChunkUriRequest ChunkUriRequest;

				FTaskInfo& Info = TaskInfos.FindOrAdd(ChunkUriToRequest);
				Info.CloudDirUsed = CurrentBestCloudDir;
				ChunkUriRequest.CloudDirectory = Configuration.CloudRoots[CurrentBestCloudDir];
				ChunkUriRequest.ChunkCloudSubdirectory = ManifestSet->GetChunkCloudSubdirectory(ChunkUriToRequest);
				ChunkUriRequest.ChunkCloudSubdirectory.RemoveFromStart(TEXT("/"));
				ChunkUriRequest.ChunkPath = ManifestSet->GetChunkFilename(ChunkUriToRequest);
				ChunkUriRequest.UniqueBuildId = ManifestSet->GetChunkUniqueBuildId(ChunkUriToRequest);
				ChunkUriRequest.bChunkFromDelta = ManifestSet->GetChunkFromDeltaStatus(ChunkUriToRequest);

				bool bMessageSent = MessagePump->SendRequest(ChunkUriRequest, [WeakChunkUriResponses, ChunkUriToRequest](FChunkUriResponse Response)
				{
					TSharedPtr<FGuidUriResponseQueue, ESPMode::ThreadSafe> ChunkUriResponsesPtr = WeakChunkUriResponses.Pin();
					if (ChunkUriResponsesPtr.IsValid())
					{
						ChunkUriResponsesPtr->Enqueue(FGuidUriResponse(ChunkUriToRequest, MoveTemp(Response)));
					}
				});
				if (!bMessageSent)
				{
					TSharedPtr<FGuidUriResponseQueue, ESPMode::ThreadSafe> ChunkUriResponsesPtr = WeakChunkUriResponses.Pin();
					if (ChunkUriResponsesPtr.IsValid())
					{
						// If no handler is registered SendRequest does nothing - make our own default response
						FChunkUriResponse Response;
						Response.Uri = ChunkUriRequest.CloudDirectory / ChunkUriRequest.ChunkCloudSubdirectory / ChunkUriRequest.ChunkPath;
						ChunkUriResponsesPtr->Enqueue(FGuidUriResponse(ChunkUriToRequest, MoveTemp(Response)));
					}
				}
			}
			// Process new chunk uri responses.
			FGuidUriResponse ChunkUriResponse;
			while (ChunkUriResponses.Dequeue(ChunkUriResponse))
			{
				if (ChunkUriResponse.Get<1>().bFailed)
				{
					// We couldn't get a valid url for the chunk and so the chunk should be considered a failed
					// download. This is considered a hard failure for the chunk (i.e. don't try other CDNs).
					InstallerError->SetError(EBuildPatchInstallError::DownloadError, DownloadErrorCodes::FailedUriRequest);
					bShouldAbort = true;
				}
				else
				{
					ChunkUris.Add(MoveTemp(ChunkUriResponse.Get<0>()), MoveTemp(ChunkUriResponse.Get<1>()));
				}
			}
			// Trim our initial download list on first begin.
			if (!bTotalRequiredTrimmed && bDownloadsStarted)
			{
				bTotalRequiredTrimmed = true;
				TotalRequiredChunks = TotalRequiredChunks.Intersect(ChunkReferenceTracker->GetReferencedChunks());
				const int64 NewChunkSize = ManifestSet->GetDownloadSize(TotalRequiredChunks);
				if (NewChunkSize != TotalRequiredChunkSize)
				{
					TotalRequiredChunkSize = NewChunkSize;
					bRequiredDataUpdated = true;
				}
			}
			// Update required data spec.
			if (bRequiredDataUpdated)
			{
				CloudChunkSourceStat->OnRequiredDataUpdated(TotalRequiredChunkSize + RepeatRequirementSize);
			}

			// Process completed downloads.
			const TMap<int32, FDownloadRef> FrameCompletedDownloads = DownloadDelegates->GetCompletedDownloads();
			
			for (const TPair<int32, FDownloadRef>& FrameCompletedDownload : FrameCompletedDownloads)
			{
				const int32& RequestId = FrameCompletedDownload.Key;
				const FDownloadRef& Download = FrameCompletedDownload.Value;
				const FGuid& DownloadId = InFlightDownloads[RequestId];
				FTaskInfo& TaskInfo = TaskInfos.FindOrAdd(DownloadId);
				TaskInfo.bQueuedForDownload = false;
				bool bDownloadSuccess = Download->ResponseSuccessful();
				if (bDownloadSuccess)
				{
					// HTTP module gives const access to downloaded data, and we need to change it.
					// @TODO: look into refactor serialization it can already know SHA list? Or consider adding SHA params to public API.
					TArray<uint8> DownloadedData = Download->GetData();

					// If we know the SHA for this chunk, inject to data for verification.
					FSHAHash ChunkShaHash;
					if (ManifestSet->GetChunkShaHash(DownloadId, ChunkShaHash))
					{
						// Older chunks might not have a sha hash internally, but the manifest 
						// is new and have it, so we can inject it.
						bDownloadSuccess = ChunkDataSerialization->InjectShaToChunkData(DownloadedData, ChunkShaHash);
						if (!bDownloadSuccess)
						{
							CloudChunkSourceStat->OnDownloadCorrupt(DownloadId, TaskInfo.UrlUsed, EChunkLoadResult::SerializationError);
							UE_LOGF(LogCloudChunkSource, Log, "Chunk download request succeeded but the data header was not valid to inject ShaHash. DownloadedData size: %d for chunk: %ls", DownloadedData.Num(), *TaskInfo.UrlUsed);
						}
					}

					if (bDownloadSuccess)
					{
						EChunkLoadResult LoadResult;
						TUniquePtr<IChunkDataAccess> ChunkDataAccess(ChunkDataSerialization->LoadFromMemory(DownloadedData, LoadResult));
						if (LoadResult == EChunkLoadResult::Success || LoadResult == EChunkLoadResult::MissingSecretKey)
						{
							TotalReceivedData += TaskInfo.ExpectedSize;
							TaskInfos.Remove(DownloadId);
							PlacedInStore.Add(DownloadId);
							ChunkStore->Put(DownloadId, MoveTemp(ChunkDataAccess));
							CloudChunkSourceStat->OnDownloadSuccess(DownloadId);
							CloudChunkSourceStat->OnReceivedDataUpdated(TotalReceivedData);
							CloudChunkSourceStat->OnCloudDirectoryDataReceived(Configuration.CloudRoots[TaskInfo.CloudDirUsed], TaskInfo.ExpectedSize);
						}
						else
						{
							CloudChunkSourceStat->OnDownloadCorrupt(DownloadId, TaskInfo.UrlUsed, LoadResult);
							bDownloadSuccess = false;
							UE_LOGF(LogCloudChunkSource, Log, "Chunk download request succeeded but the data was not valid. Load result %ls (%d) for chunk: %ls", ToString(LoadResult), EnumToUnderlyingType(LoadResult), *TaskInfo.UrlUsed);
						}
					}

					// Return the verbosity back to the default value if it was changed
					if (TaskInfo.RetryNum > Configuration.RetriesToEnableVerbose && UE_GET_LOG_VERBOSITY(LogHttp) != ELogVerbosity::Error)
					{
						UE_SET_LOG_VERBOSITY(LogHttp, Error);
					}
				}
				else
				{
					CloudChunkSourceStat->OnDownloadFailed(DownloadId, TaskInfo.UrlUsed);
					FChunkSourceEvent ChunkDownloadFailEvent{ FChunkSourceEvent::EType::HttpRequestFailed, TaskInfo.UrlUsed };
					MessagePump->SendMessage(ChunkDownloadFailEvent);
					UE_LOGF(LogCloudChunkSource, Log, "FAILED: %ls", *TaskInfo.UrlUsed);
				}

				// Handle failed (note this also launches a retry on a bad serialization, not just download.
				if (!bDownloadSuccess)
				{
					ChunkSuccessRate.AddFail();
					FailedDownloads.Add(DownloadId);
					if (Configuration.MaxRetryCount >= 0 && TaskInfo.RetryNum >= Configuration.MaxRetryCount)
					{
						InstallerError->SetError(EBuildPatchInstallError::DownloadError, DownloadErrorCodes::OutOfChunkRetries);
						bShouldAbort = true;
					}
					++TaskInfo.RetryNum;
					
					// Change logs verbosity to print the detailed error
					if (TaskInfo.RetryNum > Configuration.RetriesToEnableVerbose)
					{
						UE_SET_LOG_VERBOSITY(LogHttp, Verbose);
					}


					// Mark this CDN as failed.
					{
						CloudDirFailureCount[TaskInfo.CloudDirUsed]++;
						if (CloudDirFailureCount[TaskInfo.CloudDirUsed] > (100 << 20))
						{
							// Cap to prevent wrap. I think this is technically impossible due to the time it would take to 
							// get here but...
							CloudDirFailureCount[TaskInfo.CloudDirUsed] = 100 << 20;
						}

						// Find who has failed the least, be sure to take equivalents in the initial specified order.
						// We expect this to be like 5 entries.
						int32 MinFailCount = CloudDirFailureCount[0];
						int32 MinAtIndex = 0;
						for (int32 CloudDirSeek = 1; CloudDirSeek < CloudDirFailureCount.Num(); CloudDirSeek++)
						{
							if (CloudDirFailureCount[CloudDirSeek] < MinFailCount)
							{
								MinFailCount = CloudDirFailureCount[CloudDirSeek];
								MinAtIndex = CloudDirSeek;
							}
						}

						CurrentBestCloudDir = MinAtIndex;
						UE_LOGF(LogCloudChunkSource, Warning, "CDN %ls failed download, updating CDN selection to: %ls", *Configuration.CloudRoots[TaskInfo.CloudDirUsed], *Configuration.CloudRoots[MinAtIndex]);
						CloudChunkSourceStat->OnCloudDirectoryError(Configuration.CloudRoots[TaskInfo.CloudDirUsed]);
					}

					TaskInfo.SecondsAtFail = FStatsCollector::GetSeconds();
					const int32 UnusedDownloadTokenSize = TaskInfo.ExpectedSize - Download->GetData().Num();
					if (UnusedDownloadTokenSize > 0)
					{
						DownloadRateLimiter.Refund(UnusedDownloadTokenSize);
					}
					RequestedChunkUris.Remove(DownloadId);

					UE_LOGF(LogCloudChunkSource, Log, "Retry '%d' for chunk '%ls'", TaskInfo.RetryNum, *TaskInfo.UrlUsed);
				}
				else
				{
					const double ChunkTime = FStatsCollector::GetSeconds() - TaskInfo.SecondsAtRequested;
					MeanChunkTime.AddSample(ChunkTime);
					ChunkSuccessRate.AddSuccess();
				}
				ChunkUris.Remove(DownloadId);
				InFlightDownloads.Remove(RequestId);
			}

			// Update connection status and health.
			bool bAllDownloadsRetrying = FailedDownloads.Num() > 0 || InFlightDownloads.Num() > 0;
			for (auto InFlightIt = InFlightDownloads.CreateConstIterator(); InFlightIt && bAllDownloadsRetrying; ++InFlightIt)
			{
				if (TaskInfos.FindOrAdd(InFlightIt.Value()).RetryNum == 0)
				{
					bAllDownloadsRetrying = false;
					break;
				}
			}
			const double SecondsSinceData = DownloadDelegates->GetSecondsSinceLastData();
			const bool bReportAsDisconnected = (bAllDownloadsRetrying && SecondsSinceData > Configuration.DisconnectedDelay);
			const float SuccessRate = ChunkSuccessRate.GetOverall();
			EBuildPatchDownloadHealth OverallDownloadHealth = GetDownloadHealth(bReportAsDisconnected, Configuration.HealthPercentages, SuccessRate);
			if (TrackedDownloadHealth != OverallDownloadHealth)
			{
				TrackedDownloadHealth = OverallDownloadHealth;
				CloudChunkSourceStat->OnDownloadHealthUpdated(TrackedDownloadHealth);
			}
			if (FrameCompletedDownloads.Num() > 0)
			{
				CloudChunkSourceStat->OnSuccessRateUpdated(SuccessRate);
			}
			const float ImmediateSuccessRate = ChunkSuccessRate.GetImmediate();
			EBuildPatchDownloadHealth ImmediateDownloadHealth = GetDownloadHealth(bReportAsDisconnected, Configuration.HealthPercentages, ImmediateSuccessRate);
			// Kick off new downloads.
			if (bDownloadsStarted)
			{
				FGuid NextTask;
				while ((NextTask = GetNextTask(TaskInfos, InFlightDownloads, TotalRequiredChunks, PriorityRequests, FailedDownloads, PlacedInStore, DownloadQueue, ImmediateDownloadHealth)).IsValid())
				{
					FChunkUriResponse* ChunkUri = ChunkUris.Find(NextTask);
					uint64 DownloadSize = ManifestSet->GetDownloadSize(NextTask);
					if (ChunkUri && DownloadRateLimiter.TrySchedule(DownloadSize) == 0)
					{
						FTaskInfo& TaskInfo = TaskInfos.FindOrAdd(NextTask);
						TaskInfo.bQueuedForDownload = true;
						TaskInfo.UrlUsed = ChunkUri->Uri;
						TaskInfo.ExpectedSize = DownloadSize;
						TaskInfo.SecondsAtRequested = FStatsCollector::GetSeconds();
						int32 RequestId = 0;
						DownloadService->RequestFileWithHeaders(TaskInfo.UrlUsed, ChunkUri->AdditionalHeaders, OnDownloadCompleteDelegate, OnDownloadProgressDelegate, &RequestId);
						InFlightDownloads.Add(RequestId, NextTask);
						PriorityRequests.Remove(NextTask);
						FailedDownloads.Remove(NextTask);
						DownloadQueue.Remove(NextTask);
						CloudChunkSourceStat->OnDownloadRequested(NextTask);
					}
					else
					{
						break;
					}
				}
			}

			// Update request count.
			int32 ActiveRequestCount = InFlightDownloads.Num() + FailedDownloads.Num();;
			if (TrackedActiveRequestCount != ActiveRequestCount)
			{
				TrackedActiveRequestCount = ActiveRequestCount;
				CloudChunkSourceStat->OnActiveRequestCountUpdated(TrackedActiveRequestCount);
			}

			// Update Average request count.
			const double CurrentSeconds = FStatsCollector::GetSeconds();
			// Track the average once per second
			if (SecondsAtLastTrackedAverage == 0 || CurrentSeconds - SecondsAtLastTrackedAverage > 1.0f)
			{
				SecondsAtLastTrackedAverage = CurrentSeconds;
				// Start tracking the average only once we have started downloading
				if (TrackedActiveRequestAverageSum > 0 || TrackedActiveRequestCount > 0)
				{
					TrackedActiveRequestAverageSum += TrackedActiveRequestCount;
					TrackedActiveRequestAverageCount++;
					if (TrackedActiveRequestAverageCount > 0)
					{
						CloudChunkSourceStat->OnActiveRequestCountAvgUpdated(TrackedActiveRequestAverageSum / TrackedActiveRequestAverageCount);
					}
				}
			}

			// Check for abnormally slow downloads. This was originally implemented as a temporary measure to fix major stall anomalies and zero size tcp window issue.
			// It remains until proven unrequired.
			if (MeanChunkTime.IsReliable())
			{
				bool bResetMeanChunkTime = false;
				for (const TPair<int32, FGuid>& InFlightDownload : InFlightDownloads)
				{
					FTaskInfo& TaskInfo = TaskInfos.FindOrAdd(InFlightDownload.Value);
					if (TaskInfo.RetryNum == 0)
					{
						double DownloadTime = FStatsCollector::GetSeconds() - TaskInfo.SecondsAtRequested;
						double DownloadTimeMean, DownloadTimeStd;
						MeanChunkTime.GetValues(DownloadTimeMean, DownloadTimeStd);
						// The point at which we decide the chunk is delayed, with a sane minimum
						double BreakingPoint = FMath::Max<double>(Configuration.TcpZeroWindowMinimumSeconds, DownloadTimeMean + (DownloadTimeStd * 4.0));
						if (DownloadTime > BreakingPoint && TaskInfo.UrlUsed.EndsWith(TEXT(".chunk")))
						{
							bResetMeanChunkTime = true;
							DownloadService->RequestCancel(InFlightDownload.Key);
							CloudChunkSourceStat->OnDownloadAborted(InFlightDownload.Value, TaskInfo.UrlUsed, DownloadTimeMean, DownloadTimeStd, DownloadTime, BreakingPoint);
						}
					}
				}
				if (bResetMeanChunkTime)
				{
					MeanChunkTime.Reset();
				}
			}

			// Wait while paused
			while (bIsPaused && !bShouldAbort)
			{
				Platform->Sleep(0.1f);
			}

			// Give other threads some time.
			Platform->Sleep(0.01f);
		}

		// Abandon in flight downloads if should abort.
		if (bShouldAbort)
		{
			for (const TPair<int32, FGuid>& InFlightDownload : InFlightDownloads)
			{
				DownloadService->RequestAbandon(InFlightDownload.Key);
			}
		}

		// Provide final stat values.
		CloudChunkSourceStat->OnDownloadHealthUpdated(TrackedDownloadHealth);
		CloudChunkSourceStat->OnSuccessRateUpdated(ChunkSuccessRate.GetOverall());
		CloudChunkSourceStat->OnActiveRequestCountUpdated(0);

		// The promise should always be set, even if not needed as destruction of an unset promise will assert.
		Promise.SetValue();
	}

	
	ICloudChunkSource* FCloudChunkSourceFactory::Create(FCloudSourceConfig Configuration, IPlatform* Platform, IChunkStore* ChunkStore, IDownloadService* DownloadService, IChunkReferenceTracker* ChunkReferenceTracker, IChunkDataSerialization* ChunkDataSerialization, IMessagePump* MessagePump, IInstallerError* InstallerError, IDownloadConnectionCount* ConnectionCount, ICloudChunkSourceStat* CloudChunkSourceStat, IBuildManifestSet* ManifestSet, TSet<FGuid> InitialDownloadSet)
	{
		UE_LOGF(LogCloudChunkSource, Verbose, "FCloudChunkSourceFactory::Create for %d roots", Configuration.CloudRoots.Num());

		check(Platform != nullptr);
		check(ChunkStore != nullptr);
		check(DownloadService != nullptr);
		check(ChunkReferenceTracker != nullptr);
		check(ChunkDataSerialization != nullptr);
		check(MessagePump != nullptr);
		check(InstallerError != nullptr);
		check(ConnectionCount != nullptr)
		check(CloudChunkSourceStat != nullptr);
		return new FCloudChunkSource(MoveTemp(Configuration), Platform, ChunkStore, DownloadService, ChunkReferenceTracker, ChunkDataSerialization, MessagePump, InstallerError, ConnectionCount, CloudChunkSourceStat, ManifestSet, MoveTemp(InitialDownloadSet));
	}

	//-------------------------------------------------------------------------
	//
	// ConstructorCloudChunkSource below here.
	//
	//-------------------------------------------------------------------------
	class FConstructorCloudChunkSource : public IConstructorCloudChunkSource
	{
	private:
		/**
		* This is a wrapper class for binding thread safe shared ptr delegates for the download service, without having to enforce that
		* this service should be made using TShared* reference controllers.
		*/
		class FDownloadDelegates
		{
		public:
			FDownloadDelegates(FConstructorCloudChunkSource& InCloudChunkSource)
			: CloudSource(InCloudChunkSource)
			{
			}

		public:
			void OnDownloadProgress(int32 RequestId, uint64 BytesSoFar)
			{
				CloudSource.OnDownloadProgress(RequestId, BytesSoFar);
			}
			void OnDownloadComplete(int32 RequestId, const FDownloadRef& Download)
			{
				CloudSource.OnDownloadComplete(RequestId, Download);
			}

		private:
			FConstructorCloudChunkSource& CloudSource;
		};
		
		struct FCloudRead
		{
			FGuid DataId;
			FMutableMemoryView Destination;
			FChunkRequestCompleteDelegate CompleteFn;
			void* UserPtr;

			bool bQueuedForDownload = false;
			int32 CloudDirUsed = 0;
			FString UrlUsed;
			int32 RetryNum = 0;
			uint64 ExpectedSize = 0;
			int32 RequestId = 0;
			double SecondsAtRequested = 0;
			int32 ReadId = 0;

			// The time when we want to launch this retry, if RetryNum != 0
			double RetryTime = 0;
			bool bDecompressionSuccess = false; // Flag indicating that the chunk data was successfully downloaded, decoded, and decompressed.
		};

	public:
		FConstructorCloudChunkSource(FConstructorCloudChunkSourceConfig&& InConfiguration, IDownloadService* InDownloadService, IChunkDataSerialization* InChunkDataSerialization,
			ICloudChunkSourceStat* InCloudChunkSourceStat, IBuildManifestSet* InManifestSet, IDownloadConnectionCount* InDownloadCount, IMessagePump* InMessagePump)
			: Configuration(MoveTemp(InConfiguration))
			, DownloadDelegates(MakeShareable(new FDownloadDelegates(*this)))
			, OnDownloadProgressDelegate(FDownloadProgressDelegate::CreateThreadSafeSP(DownloadDelegates, &FDownloadDelegates::OnDownloadProgress))
			, OnDownloadCompleteDelegate(FDownloadCompleteDelegate::CreateThreadSafeSP(DownloadDelegates, &FDownloadDelegates::OnDownloadComplete))
			, DownloadService(InDownloadService)
			, ChunkDataSerialization(InChunkDataSerialization)
			, CloudChunkSourceStat(InCloudChunkSourceStat)
			, ManifestSet(InManifestSet)
			, DownloadCount(InDownloadCount)
			, MessagePump(InMessagePump)
			, DownloadRateLimiter(InConfiguration.DownloadRateLimitBps)
		{
			CloudDirFailureCount.SetNumZeroed(Configuration.CloudRoots.Num());
		}

		virtual ~FConstructorCloudChunkSource()
		{
			Abort();
		}
		
		virtual FRequestProcessFn CreateRequest(const FGuid& DataId, FMutableMemoryView DestinationBuffer, void* UserPtr, FChunkRequestCompleteDelegate CompleteFn) override
		{
			//
			// This function can get called from any thread as the failure case for reading a chunk is to
			// request it off the cloud.
			//

			// We don't have a request that can be serviced directly because of how we have multiple in flight,
			// so we return an empty request function and let the tick do the work.
			FCloudRead* Read = new FCloudRead();
			Read->DataId = DataId;
			Read->Destination = DestinationBuffer;
			Read->CompleteFn = CompleteFn;
			Read->UserPtr = UserPtr;
			Read->ReadId = NextReadId.fetch_add(1, std::memory_order_relaxed);

			ReadQueueCS.Lock();
			ReadQueue.Add(Read);
			ReadQueueCS.Unlock();

			// We need to make sure that we get ticked to start the request as we might not be called from
			// the dispatch thread.
			WakeupMainThreadFn();

			return [](bool){return;};
		}
	
		virtual int32 GetChunkUnavailableAt(const FGuid& DataId) const override
		{
			// We can always redownload but it's never kept local so it's immediately unavailable
			return 0;
		}

		virtual void Abort() override;
		virtual void Tick(bool bStartNewDownloads, uint32& OutTimeToNextTickMs, uint32 InMaxDownloads) override;
		virtual void SetWakeupFunction(TUniqueFunction<void()>&& InWakeupMainThreadFn) override
		{
			WakeupMainThreadFn = MoveTemp(InWakeupMainThreadFn);
		}

		virtual void PostRequiredByteCount(uint64 InDownloadExpected) override
		{
			CloudChunkSourceStat->OnRequiredDataUpdated(InDownloadExpected);
		}

		virtual void SetDownloadBytesPerSecondLimit(uint64 BytesPerSecondLimit) override
		{
			DownloadRateLimiter.SetRateLimitBps(BytesPerSecondLimit);

			// If the limit changed, we need to wake up and try to see if we can schedule
			// more downloads
			WakeupMainThreadFn();
		}

	private:

		bool GetDisconnectionStatus()
		{
			{
				FScopeLock _(&ReadQueueCS);

				if (ReadQueue.Num() == 0 || !Algo::AllOf(ReadQueue, [](const FCloudRead* CloudRead) { return CloudRead->RetryNum > 0; }))
				{
					AllDownloadsRetryingSince.store(-1.0f);
					return false;
				}
			}

			// Store the timestamp when all chunks downloads are in retrying state
			if (AllDownloadsRetryingSince < 0.0f)
			{
				AllDownloadsRetryingSince.store(FStatsCollector::GetCycles());
				return false;
			}

			// When all chunks are retrying for a while the connection status is set to disconnected
			const double SecondsSinceAllRetrying = FStatsCollector::CyclesToSeconds(FStatsCollector::GetCycles() - AllDownloadsRetryingSince.load(std::memory_order_relaxed));
			return SecondsSinceAllRetrying >= Configuration.DisconnectedDelay;
		}
		
		void OnDownloadProgress(int32 RequestId, uint64 BytesSoFar)
		{
			CyclesAtLastData.store(FStatsCollector::GetCycles(), std::memory_order_relaxed);
		}

		void OnDownloadComplete(int32 RequestId, const FDownloadRef& Download)
		{
			{
				FScopeLock ScopeLock(&CompletedRequestsCS);
				CompletedDownloads.Add(RequestId, Download);
			}
			WakeupMainThreadFn();
		}

		void UpdateCloudDirFailure(FCloudRead* Read)
		{
			// Mark this CDN as failed.
			// Cap to prevent wrap. I think this is technically impossible due to the time it would take to get here but...
			CloudDirFailureCount[Read->CloudDirUsed] = FMath::Min(CloudDirFailureCount[Read->CloudDirUsed] + 1, 100 << 20);
			// Find who has failed the least, be sure to take equivalents in the initial specified order.
			// We expect this to be like 5 entries.
			const int32* MinFailedCloudDir = Algo::MinElement(CloudDirFailureCount);

			const int32 MinAtIndex = MinFailedCloudDir - CloudDirFailureCount.GetData();

			if (CurrentBestCloudDir != MinAtIndex)
			{
				CurrentBestCloudDir = MinAtIndex;
				CurrentBestCloudDir = MinAtIndex;
				const TCHAR* Reason = Read->bDecompressionSuccess ? TEXT("download") : TEXT("decompress");
				UE_LOGF(LogCloudChunkSource, Warning, "CDN %ls failed %ls, updating CDN selection to: %ls", 
					*Configuration.CloudRoots[Read->CloudDirUsed],
					Reason,
					*Configuration.CloudRoots[MinAtIndex]);
			}

			MessagePump->SendMessage({ FGenericMessage::EType::CDNDownloadFailed, Read->DataId, Configuration.CloudRoots[Read->CloudDirUsed], Configuration.CloudRoots[MinAtIndex] });
		}

		void OnDownloadOrDecompressionFailed(FCloudRead* Read)
		{
			UpdateCloudDirFailure(Read);

			ChunkSuccessTracker.AddFail();

			DownloadRateLimiter.Refund(Read->ExpectedSize);

			// Update retry.
			Read->RetryNum++;
			if (Configuration.MaxRetryCount >= 0 && Read->RetryNum >= Configuration.MaxRetryCount)
			{
				// Fail the request.
				Read->CompleteFn.Execute(Read->DataId, false, true, Read->UserPtr);
				delete Read;
			}
			else
			{
				// Set retry time and put the task back in the queue.
				Read->RetryTime = FStatsCollector::GetSeconds() + GetRetryDelay(Configuration.RetryDelayTimes, Read->RetryNum);
				FScopeLock _(&ReadQueueCS);
				ReadQueue.Add(Read);
			}
		}

		FConstructorCloudChunkSourceConfig Configuration;
		TSharedRef<FDownloadDelegates, ESPMode::ThreadSafe> DownloadDelegates;
		FDownloadProgressDelegate OnDownloadProgressDelegate;
		FDownloadCompleteDelegate OnDownloadCompleteDelegate;

		// Handles to various systems/data that we require to be valid across execution duration.
		IDownloadService* DownloadService;
		IChunkDataSerialization* ChunkDataSerialization;
		ICloudChunkSourceStat* CloudChunkSourceStat;
		IBuildManifestSet* ManifestSet;
		IDownloadConnectionCount* DownloadCount;
		IMessagePump* MessagePump;

		// Called when we get a completed download and we want to get ticked again.
		TUniqueFunction<void()> WakeupMainThreadFn;

		// Tracking health and connection state
		std::atomic<int64> CyclesAtLastData = 0;
		std::atomic_int32_t NextReadId = 0;
		std::atomic<int64> AllDownloadsRetryingSince = -1.0f;

		// Communication from download thread to processing thread.
		FCriticalSection CompletedRequestsCS;
		TMap<int32, FDownloadRef> CompletedDownloads; // Keyed off the download request id
		TMap<int32, FChunkUriResponse> CompletedRequests; // Keyed off the read id
		TMap<int32, FCloudRead*> CompletedDecompressions; // Keyed off the completed decompression task id

		// If we start getting failures on our downloads, we track which ones
		// fail and avoid them until everything goes bad. Initially we just hit the
		// first directory.
		int32 CurrentBestCloudDir = 0;
		TArray<int32> CloudDirFailureCount;

		// Read pointers are in one of these until they are either destroyed or passed
		// to the validation task.
		FCriticalSection ReadQueueCS;
		TArray<FCloudRead*> ReadQueue;
		TMap<int32, FCloudRead*> InFlightDownloads; // Keyed off the download request id
		TMap<int32, FCloudRead*> InFlightURLRequests; // Keyed off the read id

		// \todo does this thing have any recency bias? Surely we only care about the last few
		// seconds of success...
		FChunkSuccessRate ChunkSuccessTracker;

		FDownloadRateLimiter DownloadRateLimiter;

		// Change recognition
		EBuildPatchDownloadHealth LastSeenHealth = EBuildPatchDownloadHealth::NUM_Values;
		float LastSeenSuccessRate = -1.0f;

		double StartTime = -1;
		uint64 TotalBytes = 0;
	};

	// Must be called from the same thread as Tick. Can be called multiple times.
	void FConstructorCloudChunkSource::Abort()
	{
		// Release any unqueued reads.
		{
			FScopeLock _(&ReadQueueCS);
			for (FCloudRead* Read : ReadQueue)
			{
				Read->CompleteFn.Execute(Read->DataId, true, false, Read->UserPtr);
				delete Read;
			}
			ReadQueue.Empty();
		}

		// We have to wait until ALL our uri resposes come back since we can't
		// delete the lambda references out from under them.
		if (InFlightURLRequests.Num())
		{
			double LastReport = FPlatformTime::Seconds();
			double FirstReport = LastReport;
			UE_LOGF(LogCloudChunkSource, Display, "Draining outstanding url requests on cancel... (%d)", InFlightURLRequests.Num());
			while (InFlightURLRequests.Num())
			{
				double CurrentTime = FPlatformTime::Seconds();
				if (CurrentTime - LastReport > 5)
				{
					// We want it to be clear to whoever is looking at the logs that we're waiting on
					// client code if we are hung here.
					UE_LOGF(LogCloudChunkSource, Display, "Still waiting on outstanding url requests, %.1f seconds, %d outstanding", CurrentTime - FirstReport, InFlightURLRequests.Num());
					LastReport = CurrentTime;
				}

				TMap<int32, FChunkUriResponse> FrameCompletedRequests;
				CompletedRequestsCS.Lock();
				FrameCompletedRequests = MoveTemp(CompletedRequests);
				CompletedRequestsCS.Unlock();

				for (const TPair<int32, FChunkUriResponse>& Pair : FrameCompletedRequests)
				{
					FCloudRead* Read = InFlightURLRequests.FindAndRemoveChecked(Pair.Key);
					Read->CompleteFn.Execute(Read->DataId, true, false, Read->UserPtr);
					delete Read;
				}

				// Ideally we get them all more or less instantly, but otherwise we have to wait. We expect
				// this to be rare - user initiated cancels should have completed these already, and internal
				// cancels are not common, so we just use a sleep rather than set up an event.
				if (InFlightURLRequests.Num())
				{
					FPlatformProcess::Sleep(.002f);
				}
			}
		}

		// Abort any downloads.
		for (TPair<int32, FCloudRead*> Pair : InFlightDownloads)
		{
			FCloudRead* Read = Pair.Value;
			DownloadService->RequestAbandon(Read->RequestId);
			Read->CompleteFn.Execute(Read->DataId, true, false, Read->UserPtr);
			delete Read;
		}
		InFlightDownloads.Empty();
	}
	//
	// Since we need to have a lot of requests in flight at the same time _and_ we need to
	// synchronize across them to manage how many are in flight at any given moment, we have
	// to have a touchpoint on the main thread to dispatch and manage everything. This thread
	// should do very little work! It should just be brokering things, not accomplishing anything.
	//
	void FConstructorCloudChunkSource::Tick(bool bStartNewDownloads, uint32& OutTimeToNextTickMs, uint32 InMaxDownloads)
	{
		int32 StartingDownloadCount = InFlightDownloads.Num();

		DownloadRateLimiter.Tick();
		
		{
			int32 PendingCount;
			{
				FScopeLock _(&ReadQueueCS);
				PendingCount = ReadQueue.Num();
			}

			TRACE_INT_VALUE(TEXT("BPS.Cloud.ActiveURLRequests"), InFlightURLRequests.Num());
			TRACE_INT_VALUE(TEXT("BPS.Cloud.ActiveReads"), InFlightDownloads.Num());
			TRACE_INT_VALUE(TEXT("BPS.Cloud.PendingReads"), PendingCount);

			UE_LOGF(LogCloudChunkSource, Verbose, "Cloud: Active: %d, Pending %d, URLS: %d", InFlightDownloads.Num(), PendingCount, InFlightURLRequests.Num());
		}

		// Process completed downloads.
		TMap<int32, FDownloadRef> FrameCompletedDownloads;
		TMap<int32, FChunkUriResponse> FrameCompletedRequests;
		TMap<int32, FCloudRead*> FrameCompletedDecompressions;
		{
			FScopeLock ScopeLock(&CompletedRequestsCS);
			FrameCompletedRequests = MoveTemp(CompletedRequests);
			FrameCompletedDownloads = MoveTemp(CompletedDownloads);
			FrameCompletedDecompressions = MoveTemp(CompletedDecompressions);
		}

		{
			// Finalize completed FCloudRead;
			for (const TPair<int32, FCloudRead*>& CompletedDecompression : FrameCompletedDecompressions)
			{
				FCloudRead* Read = CompletedDecompression.Value;
				if (Read->bDecompressionSuccess)
				{
					ChunkSuccessTracker.AddSuccess();
					CloudChunkSourceStat->OnDownloadSuccess(Read->DataId);
					Read->CompleteFn.Execute(Read->DataId, false, false, Read->UserPtr);
					delete Read;
				}
				else
				{
					// \todo this wants to know what the actual internal error was for some reason. idk why.
					CloudChunkSourceStat->OnDownloadCorrupt(Read->DataId, Read->UrlUsed, EChunkLoadResult::HashCheckFailed);
					OnDownloadOrDecompressionFailed(Read);
				}

			}
		}

		{
			for (const TPair<int32, FChunkUriResponse>& FrameCompletedRequest : FrameCompletedRequests)
			{
				FCloudRead* Read = InFlightURLRequests.FindAndRemoveChecked(FrameCompletedRequest.Key);

				const FChunkUriResponse& Response = FrameCompletedRequest.Value;
				if (Response.bFailed)
				{
					// Failed to get an auth url means failed to read
					Read->CompleteFn.Execute(Read->DataId, false, true, Read->UserPtr);
					delete Read;
				}
				else if (!bStartNewDownloads)
				{
					// if we can't start new downloads it counts as an abort
					Read->CompleteFn.Execute(Read->DataId, true, false, Read->UserPtr);
					delete Read;
				}
				else
				{
					// Only override the URL if set; the default handler does nothing.
					if (Response.Uri.Len())
					{
						Read->UrlUsed = Response.Uri;
					}
					DownloadService->RequestFileWithHeaders(Read->UrlUsed, Response.AdditionalHeaders, OnDownloadCompleteDelegate, OnDownloadProgressDelegate, &Read->RequestId);
					InFlightDownloads.Add(Read->RequestId, Read);

					CloudChunkSourceStat->OnDownloadRequested(Read->DataId);
				}
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CDT_ProcessCompleted);
			for (const TPair<int32, FDownloadRef>& FrameCompletedDownload : FrameCompletedDownloads)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(CDT_ProcessOne);
				const int32& RequestId = FrameCompletedDownload.Key;
				const FDownloadRef& Download = FrameCompletedDownload.Value;

				FCloudRead** ReadPtr = InFlightDownloads.Find(RequestId);
				if (ReadPtr == nullptr)
				{
					// We can get here when we abort. We still get a completion callback when we abandon the download
					// and we no longer have any data structure reference.
					continue;
				}

				FCloudRead* Read = *ReadPtr;
				InFlightDownloads.Remove(RequestId);

				Read->bQueuedForDownload = false;

				UE_LOGF(LogCloudChunkSource, VeryVerbose, "Downloaded chunk: %ls", *WriteToString<40>(Read->DataId));

				bool bDownloadSuccess = Download->ResponseSuccessful();
				if (bDownloadSuccess)
				{
					const TArray<uint8>& DownloadedData = Download->GetData();

					TotalBytes += DownloadedData.Num();

					double MegaBytesPerSecond = (TotalBytes / (FStatsCollector::GetSeconds() - StartTime)) / (1024*1024);
					//UE_LOGF(LogTemp, Warning, "Rate: %.2f MB/s", MegaBytesPerSecond);

					CloudChunkSourceStat->OnReceivedDataUpdated(TotalBytes);
					CloudChunkSourceStat->OnCloudDirectoryDataReceived(Configuration.CloudRoots[Read->CloudDirUsed], DownloadedData.Num());

					FMemoryReader ReaderThunk(DownloadedData);

					// If the chunk is uncompressed it can be directly copied to the output here, otherwise it'll get decompressed
					// during the verification task;
					FChunkHeader Header;
					FUniqueBuffer CompressedBuffer;
					bool Result = ChunkDataSerialization->ValidateAndRead(ReaderThunk, Read->Destination, Header, CompressedBuffer);

					if (!Result)
					{
						// The header or chunk data was bad.
						UE_LOGF(LogCloudChunkSource, Error, "Failed to ValidateAndRead chunk %ls, download size %d", *WriteToString<64>(Read->DataId), DownloadedData.Num());
						bDownloadSuccess = false;
					}
					else
					{
						// Older chunks might not have a sha hash internally, but the manifest is new and might have it,
						// so we can inject it.
						if (!EnumHasAnyFlags(Header.HashType, EChunkHashFlags::Sha1))
						{
							if (ManifestSet->GetChunkShaHash(Read->DataId, Header.SHAHash))
							{
								EnumAddFlags(Header.HashType, EChunkHashFlags::Sha1);
							}
						}
					
						// We either need to hash the chunk for validation or decompress it in to the destination buffer - don't
						// block IO for this.
						UE::Tasks::Launch(TEXT("CloudDecompressionAndHash"), [this, Read, Header, CompressedBuffer = MoveTemp(CompressedBuffer)]()
							{
								Read->bDecompressionSuccess = ChunkDataSerialization->DecompressAndDecryptValidatedRead(Header, Read->Destination, CompressedBuffer);
								{
									FScopeLock ScopeLock(&CompletedRequestsCS);
									CompletedDecompressions.Add(Read->ReadId, Read);
								}
								WakeupMainThreadFn();
							}
						);
					}
				}
				else
				{
					CloudChunkSourceStat->OnDownloadFailed(Read->DataId, Read->UrlUsed);
					CloudChunkSourceStat->OnCloudDirectoryError(Configuration.CloudRoots[Read->CloudDirUsed]);
					UE_LOGF(LogCloudChunkSource, Error, "FAILED: %ls", *Read->UrlUsed);
				}

				// Handle failed (note this also launches a retry on a bad serialization, not just download.
				if(!bDownloadSuccess)
				{
					OnDownloadOrDecompressionFailed(Read);
				}
			} // end each completed
		} // end process completed

		// Update connection status and health.
		EBuildPatchDownloadHealth DownloadHealth;
		{
			const bool bReportAsDisconnected = GetDisconnectionStatus();
			const float SuccessRate = ChunkSuccessTracker.GetOverall();
			EBuildPatchDownloadHealth OverallDownloadHealth = GetDownloadHealth(bReportAsDisconnected, Configuration.HealthPercentages, SuccessRate);
			if (LastSeenHealth != OverallDownloadHealth)
			{
				LastSeenHealth = OverallDownloadHealth;
				CloudChunkSourceStat->OnDownloadHealthUpdated(LastSeenHealth);
			}
			if (LastSeenSuccessRate != SuccessRate)
			{
				LastSeenSuccessRate = SuccessRate;
				CloudChunkSourceStat->OnSuccessRateUpdated(SuccessRate);
			}

			DownloadHealth = GetDownloadHealth(bReportAsDisconnected, Configuration.HealthPercentages, ChunkSuccessTracker.GetImmediate());
		}

		// Start new downloads, if we aren't gated or paused
		int32 WaitedForOldDownloads = 0;
		int32 OldestReadId = TNumericLimits<int32>::Max();
		int32 NewestReadId = 0;
		for (TPair<int32, FCloudRead*>& InFlight : InFlightDownloads)
		{
			if (InFlight.Value->ReadId < OldestReadId)
			{
				OldestReadId = InFlight.Value->ReadId;
			}
			if (InFlight.Value->ReadId > NewestReadId)
			{
				NewestReadId = InFlight.Value->ReadId;
			}
		}
		if (OldestReadId > NewestReadId)
		{
			OldestReadId = NewestReadId;
		}

		uint64 WaitingForTokensUntilMs = 0;
		while (bStartNewDownloads)
		{
			const uint32 MaxDownloads = FMath::Min(InMaxDownloads, FMath::Max(1U, DownloadCount->GetAdjustedCount(InFlightDownloads.Num(), DownloadHealth)));
			if (((uint32)InFlightDownloads.Num() + (uint32)InFlightURLRequests.Num()) >= MaxDownloads)
			{
				break;
			}

			FCloudRead* Read = nullptr;
			uint64 DownloadSize;

			{
				FScopeLock _(&ReadQueueCS);

				if (ReadQueue.Num() == 0)
				{
					break;
				}

				Read = ReadQueue[0];
				DownloadSize = ManifestSet->GetDownloadSize(Read->DataId);
				uint64 MsUntilScheduled = DownloadRateLimiter.TrySchedule(DownloadSize);
				if (MsUntilScheduled)
				{
					WaitingForTokensUntilMs = MsUntilScheduled;
					break;
				}
				ReadQueue.RemoveAt(0);
			}

			// We have to provide clients of the API a way to remap URLs for one reason or another. This is often
			// due to appending auth keys to the URL itself, or adding auth headers. This can be (very) latent or instantaneous.
			FChunkUriRequest ChunkUriRequest;
			ChunkUriRequest.CloudDirectory = Configuration.CloudRoots[CurrentBestCloudDir];
			ChunkUriRequest.ChunkCloudSubdirectory = ManifestSet->GetChunkCloudSubdirectory(Read->DataId);
			ChunkUriRequest.ChunkCloudSubdirectory.RemoveFromStart(TEXT("/"));
			ChunkUriRequest.ChunkPath = ManifestSet->GetChunkFilename(Read->DataId);
			ChunkUriRequest.UniqueBuildId = ManifestSet->GetChunkUniqueBuildId(Read->DataId);
			ChunkUriRequest.bChunkFromDelta = ManifestSet->GetChunkFromDeltaStatus(Read->DataId);
			
			Read->CloudDirUsed = CurrentBestCloudDir;
			Read->bQueuedForDownload = true;
			Read->ExpectedSize = ManifestSet->GetDownloadSize(Read->DataId);
			Read->SecondsAtRequested = FStatsCollector::GetSeconds();
			Read->UrlUsed = ChunkUriRequest.CloudDirectory / ChunkUriRequest.ChunkCloudSubdirectory / ChunkUriRequest.ChunkPath;
			UE_LOGF(LogCloudChunkSource, VeryVerbose, "Downloading: %ls", *Read->UrlUsed);

			InFlightURLRequests.Add(Read->ReadId, Read);
			bool bMessageSent = MessagePump->SendRequest(ChunkUriRequest, [this, ReadId = Read->ReadId](FChunkUriResponse Response)
				{
					CompletedRequestsCS.Lock();
					CompletedRequests.Add(ReadId, MoveTemp(Response));
					CompletedRequestsCS.Unlock();

					WakeupMainThreadFn();
				});

			if (!bMessageSent)
			{
				// If no message handler was registered to do any modification of the URL, start the download immediately.
				InFlightURLRequests.Remove(Read->ReadId);

				DownloadService->RequestFileWithHeaders(Read->UrlUsed, {}, OnDownloadCompleteDelegate, OnDownloadProgressDelegate, &Read->RequestId);
				InFlightDownloads.Add(Read->RequestId, Read);

				CloudChunkSourceStat->OnDownloadRequested(Read->DataId);
			}

			if (StartTime == -1)
			{
				StartTime = Read->SecondsAtRequested;
			}
		}

		TRACE_INT_VALUE(TEXT("BPS.Cloud.OldestDownload"), NewestReadId - OldestReadId);

		if (StartingDownloadCount != InFlightDownloads.Num())
		{
			CloudChunkSourceStat->OnActiveRequestCountUpdated(InFlightDownloads.Num());
		}

		
		// Determine how long we should wait. Normally we want to just wait until we get triggered,
		// either because we completed a download or because a new request came in. However if we have 
		// a retry that we want to requeue in a bit we just need to check back in.
		uint32 WaitTimeMs = TNumericLimits<uint32>::Max(); // default infinite
		double CurrentTime = FStatsCollector::GetSeconds();
		{
			FScopeLock _(&ReadQueueCS);
			if (ReadQueue.Num())
			{
				// This should never be that long. We should only have what has been requested
				// for the current block the constructor is writing.
				//
				// Note that we only check retries - for reads that have never queued, the reason they
				// aren't queued is because we ran out of download slots and we will check those
				// when we get woken up by a completed download.
				double ClosestRetry = TNumericLimits<double>::Max();
				for (FCloudRead* Read : ReadQueue)
				{					
					if (Read->RetryNum && Read->RetryTime < ClosestRetry)
					{
						ClosestRetry = Read->RetryTime;
					}
				}

				if (ClosestRetry != TNumericLimits<double>::Max())
				{
					double TimeToClosestRetry = ClosestRetry - CurrentTime;
					if (TimeToClosestRetry <= 0)
					{
						WaitTimeMs = 0;
					}
					else
					{
						// Don't wait longer than 30 seconds just for sanity. We're waking up after a timeout
						// anyway, computers don't care between 30 seconds and 30 minutes here.
						WaitTimeMs = FMath::Min(30U, (uint32)(TimeToClosestRetry * 1000));
					}
				}
			}
		}

		if (InFlightDownloads.Num())
		{
			// If we have in flight downloads we want to wake up every so often to
			// check for disconnections. Every second should do the trick.
			if (WaitTimeMs == TNumericLimits<uint32>::Max() ||
				WaitTimeMs > 1000)
			{
				WaitTimeMs = 1000;
			}
		}

		// If we aren't kicking reads because we don't have the quota, we need to check back soon.
		if (WaitingForTokensUntilMs)
		{
			WaitTimeMs = WaitingForTokensUntilMs;
		}

		OutTimeToNextTickMs = WaitTimeMs;
	}


	IConstructorCloudChunkSource* IConstructorCloudChunkSource::CreateCloudSource(FConstructorCloudChunkSourceConfig&& Configuration, IDownloadService* InDownloadService, 
		IChunkDataSerialization* InChunkDataSerialization, IDownloadConnectionCount* InDownloadCount, IMessagePump* InMessagePump, ICloudChunkSourceStat* InCloudChunkSourceStat, 
		IBuildManifestSet* InManifestSet)
	{
		return new FConstructorCloudChunkSource(MoveTemp(Configuration), InDownloadService, InChunkDataSerialization, InCloudChunkSourceStat, InManifestSet, InDownloadCount, InMessagePump);
	}
}