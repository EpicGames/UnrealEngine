// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/ChunkSource.h"
#include "Installer/DownloadConnectionCount.h"
#include "Installer/Controllable.h"
#include "BuildPatchManifest.h"
#include "Interfaces/IBuildInstaller.h"

enum class EBuildPatchDownloadHealth;
class IBuildInstallerSharedContext;

namespace BuildPatchServices
{
	class IPlatform;
	class IChunkStore;
	class IDownloadService;
	class IChunkReferenceTracker;
	class IChunkDataSerialization;
	class IMessagePump;
	class IInstallerError;
	class ICloudChunkSourceStat;
	class IDownloadServiceStatistics;
	class IBuildManifestSet;

	struct FConstructorCloudChunkSourceConfig
	{
		// An array of cloud root paths, supporting HTTP(s) and file access. HTTP(s) roots must begin with the protocol.
		TArray<FString> CloudRoots;
		// The maximum number of times that a single chunk should retry, before registering a fatal error.
		// Infinite can be specified as < 0.
		int32 MaxRetryCount = 6;
		// Array of times in seconds, representing the time between each retry upon failure. The last entry will be used
		// indefinitely once it is reached.
		TArray<float> RetryDelayTimes;
		// Configures what success rate values apply to which EBuildPatchDownloadHealth value, there should be
		// EBuildPatchDownloadHealth::NUM_Values entries in this array.
		TArray<float> HealthPercentages;
		// When all requests are failing, how many seconds before a success until we determine the state as disconnected.
		float DisconnectedDelay = 5.0f;

		// Maximum download speed in bytes per second.
		// Infinite can be specified as 0.
		uint64 DownloadRateLimitBps = 0;

		FConstructorCloudChunkSourceConfig(const TArray<FString>& InCloudRoots)
		: CloudRoots(InCloudRoots)
		{
			const float RetryFloats[] = {0.5f, 1.0f, 1.0f, 3.0f, 3.0f, 10.0f, 10.0f, 20.0f, 20.0f, 30.0f};
			RetryDelayTimes.Empty(UE_ARRAY_COUNT(RetryFloats));
			RetryDelayTimes.Append(RetryFloats, UE_ARRAY_COUNT(RetryFloats));
			const float HealthFloats[] = {0.0f, 0.0f, 0.9f, 0.99f, 1.0f};
			check((int32)EBuildPatchDownloadHealth::NUM_Values == UE_ARRAY_COUNT(HealthFloats));
			HealthPercentages.Empty(UE_ARRAY_COUNT(HealthFloats));
			HealthPercentages.Append(HealthFloats, UE_ARRAY_COUNT(HealthFloats));
		}
	};

	class IConstructorCloudChunkSource : public IConstructorChunkSource
	{
	public:
		virtual ~IConstructorCloudChunkSource() = default;


		virtual void Abort() = 0;

		virtual void Tick(bool bStartNewDownloads, uint32& OutTimeToNextTickMs, uint32 InMaxDownloads) = 0;

		// When downloads are complete we need a way to alert the main thread.
		virtual void SetWakeupFunction(TUniqueFunction<void()>&& WakeupMainThreadFn) = 0;

		// Pass in total download requirements for the install (including already completed downloads)
		virtual void PostRequiredByteCount(uint64 InDownloadExpected) = 0;

		virtual void SetDownloadBytesPerSecondLimit(uint64 BytesPerSecondLimit) = 0;

		static IConstructorCloudChunkSource* CreateCloudSource(FConstructorCloudChunkSourceConfig&& Configuration, IDownloadService* InDownloadService, 
			IChunkDataSerialization* InChunkDataSerialization, IDownloadConnectionCount* InDownloadCount, IMessagePump* InMessagePump, ICloudChunkSourceStat* InCloudChunkSourceStat, 
			IBuildManifestSet* InManifestSet);
	};

	/**
	 * The interface for a cloud chunk source, which provides access to chunk data retrieved from provided cloud roots.
	 */
	class ICloudChunkSource
		: public IChunkSource
		, public IControllable
	{
	public:
		virtual ~ICloudChunkSource() = default;

		/**
		 * Update the download rate limit.
		 * @param InDownloadRateLimitBps    The new maximum download rate in bytes per second. Infinite can be specified as 0.
		 */
		virtual void SetDownloadBytesPerSecondLimit(uint64 InDownloadRateLimitBps) = 0;

		/**
		 * Should be called from the desired thread if bRunOwnThread is false.
		 * It will block until until aborted by the owning system.
		 */
		virtual void ThreadRun() = 0;
	};

	/**
	 * A struct containing the configuration values for a cloud chunk source.
	 */
	struct FCloudSourceConfig
	{
		// An array of cloud root paths, supporting HTTP(s) and file access. HTTP(s) roots must begin with the protocol.
		TArray<FString> CloudRoots;
		// The maximum number of times that a single chunk should retry, before registering a fatal error.
		// Infinite can be specified as < 0.
		int32 MaxRetryCount = 6;
		// Number of retries to enable logging of http warnings.
		int32 RetriesToEnableVerbose;
		// The minimum number of chunks to request ahead of what is required, depending on store slack.
		int32 PreFetchMinimum  = 16;
		// The maximum number of chunks to request ahead of what is required, depending on store slack.
		// This shouldn't be too much because we'll queue too many on the same CDN which might not be good and we have to chew 
		// though a ton of requests before we swap over, and we only really want there to be one queued + one ready to go per connection anyway.
		int32 PreFetchMaximum = 32;
		// Maximum download speed in bytes per second.
		// Infinite can be specified as 0.
		uint64 DownloadRateLimitBps = 0;
		// Array of times in seconds, representing the time between each retry upon failure. The last entry will be used
		// indefinitely once it is reached.
		TArray<float> RetryDelayTimes;
		// Configures what success rate values apply to which EBuildPatchDownloadHealth value, there should be
		// EBuildPatchDownloadHealth::NUM_Values entries in this array.
		TArray<float> HealthPercentages;
		// When all requests are failing, how many seconds before a success until we determine the state as disconnected.
		float DisconnectedDelay = 5.0f;
		// If true, the downloads will not begin until the first Get request is made. It is fairly fundamental to stop downloads of chunks until resume
		// data is processed, but can be special case disabled.
		bool bBeginDownloadsOnFirstGet = true;
		// The minimum time to allow a http download before assessing it as affected by TCP zero window issue.
		float TcpZeroWindowMinimumSeconds = 20.0f;
		// The context for allocating shared resources.
		IBuildInstallerSharedContext* SharedContext = nullptr;
		// Whether cloud source should run its own thread. If true, SharedContext is required.
		bool bRunOwnThread = true;

		/**
		 * Constructor which sets usual defaults, and takes params for values that cannot use a default.
		 * @param InCloudRoots  The cloud roots array
		 */
		FCloudSourceConfig(const TArray<FString>& InCloudRoots)
			: CloudRoots(InCloudRoots)
			, RetriesToEnableVerbose(MaxRetryCount - 1)
		{
			const float RetryFloats[] = {0.5f, 1.0f, 1.0f, 3.0f, 3.0f, 10.0f, 10.0f, 20.0f, 20.0f, 30.0f};
			RetryDelayTimes.Empty(UE_ARRAY_COUNT(RetryFloats));
			RetryDelayTimes.Append(RetryFloats, UE_ARRAY_COUNT(RetryFloats));
			const float HealthFloats[] = {0.0f, 0.0f, 0.9f, 0.99f, 1.0f};
			check((int32)EBuildPatchDownloadHealth::NUM_Values == UE_ARRAY_COUNT(HealthFloats));
			HealthPercentages.Empty(UE_ARRAY_COUNT(HealthFloats));
			HealthPercentages.Append(HealthFloats, UE_ARRAY_COUNT(HealthFloats));
		}
	};

	/**
	 * A factory for creating an ICloudChunkSource instance.
	 */
	class FCloudChunkSourceFactory
	{
	public:

		/**
		 * This implementation requests chunks from a list of cloud sources, which is iterated as failures occur. It supports a configurable
		 * number of simultaneous requests, pre-fetching of chunks which are required next, depending on store slack, and tracking of success rates.
		 * A chunk can be requested which was not in the initial set, and it will be downloaded as a priority.
		 * @param Configuration             The configuration struct for this cloud source.
		 * @param Platform                  The platform access interface.
		 * @param ChunkStore                The chunk store where received chunks will be put.
		 * @param DownloadService           The service used to request each chunk via url.
		 * @param ChunkReferenceTracker     The reference tracker for the installation, used to decide which chunks to fetch and when.
		 * @param ChunkDataSerialization    Chunk data serialization implementation for converting downloaded bytes into chunk data.
		 * @param MessagePump               The message pump to receive messages about source events.
		 * @param InstallerError            Error tracker where fatal errors will be reported.
		 * @param ConnectionCount           The class that provides an approximately optimal number of simultaneous download connections
		 * @param CloudChunkSourceStat      The class to receive statistics and event information.
		 * @param InstallManifest           The manifest that chunks are required for.
		 * @param InitialDownloadSet        The initial set of chunks to be sourced from cloud.
		 * @return the new ICloudChunkSource instance created.
		 */
		static ICloudChunkSource* Create(FCloudSourceConfig Configuration, 
			IPlatform* Platform, 
			IChunkStore* ChunkStore, 
			IDownloadService* DownloadService, 
			IChunkReferenceTracker* ChunkReferenceTracker, 
			IChunkDataSerialization* ChunkDataSerialization, 
			IMessagePump* MessagePump, 
			IInstallerError* InstallerError, 
			IDownloadConnectionCount* InConnectionCount, 
			ICloudChunkSourceStat* CloudChunkSourceStat, 
			IBuildManifestSet* ManifestSet, 
			TSet<FGuid> InitialDownloadSet);
	};

	/**
	 * This interface defines the statistics class required by the cloud source. It should be implemented in order to collect
	 * desired information which is being broadcast by the system.
	 */
	class ICloudChunkSourceStat
	{
	public:
		virtual ~ICloudChunkSourceStat() = default;

		/**
		 * Called whenever a chunk download request is made.
		 * @param ChunkId           The id of the chunk.
		 */
		virtual void OnDownloadRequested(const FGuid& ChunkId) = 0;

		/**
		 * Called whenever a chunk download request has succeeded.
		 * @param ChunkId           The id of the chunk.
		 */
		virtual void OnDownloadSuccess(const FGuid& ChunkId) = 0;

		/**
		 * Called whenever a chunk download request has failed.
		 * @param ChunkId           The id of the chunk.
		 * @param Url               The url used to request the chunk.
		 */
		virtual void OnDownloadFailed(const FGuid& ChunkId, const FString& Url) = 0;

		/**
		 * Called whenever a chunk download request succeeded but the data was not valid.
		 * @param ChunkId           The id of the chunk.
		 * @param Url               The url used to request the chunk.
		 * @param LoadResult        The result from attempting to serialize the downloaded data.
		 */
		virtual void OnDownloadCorrupt(const FGuid& ChunkId, const FString& Url, EChunkLoadResult LoadResult) = 0;

		/**
		 * Called whenever a chunk was aborted because it was determined as taking too long.
		 * @param ChunkId           The id of the chunk.
		 * @param Url               The url used to request the chunk.
		 * @param DownloadTimeMean  The current mean time for chunk downloads when this abort was made.
		 * @param DownloadTimeStd   The current standard deviation for chunk download times when this abort was made.
		 * @param DownloadTime      The time this request had been running for.
		 * @param BreakingPoint     The calculated breaking point which this request has overran.
		 */
		virtual void OnDownloadAborted(const FGuid& ChunkId, const FString& Url, double DownloadTimeMean, double DownloadTimeStd, double DownloadTime, double BreakingPoint) = 0;

		/**
		 * Called to update the amount of bytes which have been downloaded.
		 * @param TotalBytes        The number of bytes downloaded so far.
		 */
		virtual void OnReceivedDataUpdated(int64 TotalBytes) = 0;

		/**
		 * Called whenever the total number of bytes intended to download updates.
		 * This can occur after initialization if a chunk is requested which was not in the original list.
		 * @param TotalBytes        The number of bytes downloaded so far.
		 */
		virtual void OnRequiredDataUpdated(int64 TotalBytes) = 0;

		/**
		 * Called whenever the perceived download health changes, according to the provided configuration.
		 * @param DownloadHealth    The new download health value.
		 */
		virtual void OnDownloadHealthUpdated(EBuildPatchDownloadHealth DownloadHealth) = 0;

		/**
		 * Called whenever the success rate of requests has been updated.
		 * @param SuccessRate       The new success rate value, when 1.0 is all requests made so far have been successful,
		 *                          and 0.0 is all requests failed so far.
		 */
		virtual void OnSuccessRateUpdated(float SuccessRate) = 0;

		/**
		 * Called whenever the current number of active requests updates.
		 * @param RequestCount      The number of currently active requests, this will range between 0 and NumSimultaneousDownloads config.
		 */
		virtual void OnActiveRequestCountUpdated(uint32 RequestCount) = 0;

		/**
		 * Called once per second whenever there has been an active request.
		 * @param AverageRequestCount      The average number of active requests that there has currently been.
		 */
		virtual void OnActiveRequestCountAvgUpdated(int32 AverageRequestCount) = 0;

		/**
		 * Called when a batch of chunks are added and accepted via IChunkSource::AddRuntimeRequirements.
		 * @param ChunkIds  The ids of each chunk.
		 */
		virtual void OnAcceptedNewRequirements(const TSet<FGuid>& ChunkIds) = 0;

		/**
		 * Called when a chunk is decrypted.
		 * @param DecryptionCycles  The amount of cycles spent on decryption a chunk.
		 */
		virtual void OnChunkDecrypted(uint64 DecryptionCycles) = 0;

		/**
		 * Called when new data has been received from the given cloud directory.
		 * @param CloudDirectory  The identifier for this cloud directory, e.g. base URL.
		 * @param ReceivedData    The number of bytes received from this cloud directory.
		 */
		virtual void OnCloudDirectoryDataReceived(const FString& CloudDirectory, uint64 ReceivedData) = 0;

		/**
		 * Called when a download from the given cloud directory has failed.
		 * @param CloudDirectory  The identifier for this cloud directory, e.g. base URL.
		 */
		virtual void OnCloudDirectoryError(const FString& CloudDirectory) = 0;
	};
}
