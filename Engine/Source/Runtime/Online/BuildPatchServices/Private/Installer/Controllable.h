// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

namespace BuildPatchServices
{
	/**
	 * Defines the interface to be implemented by any controllable class.
	 */
	class IControllable
	{
	public:
		virtual ~IControllable() = default;

		/**
		 * Sets whether the class should pause current activities and wait.
		 * @param bIsPaused     True if the class should pause.
		 */
		virtual void SetPaused(bool bIsPaused) = 0;

		/**
		 * Called to instruct the class to cease all activity, and perform any shutdown.
		 */
		virtual void Abort() = 0;

		/**
		 * Return to default state without destructing the object.
		 */
		virtual void Reset() = 0;
	};
}
