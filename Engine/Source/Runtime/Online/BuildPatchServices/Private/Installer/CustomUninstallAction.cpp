// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomUninstallAction.h"
#include "BuildPatchManifest.h"
#include "BuildPatchProgress.h"
#include "BuildPatchSettings.h"
#include "BuildPatchState.h"
#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "Core/Platform.h"
#include "Common/FileSystem.h"
#include "Interfaces/IBuildInstaller.h"
#include "Installer/InstallerError.h"
#include "IBuildManifestSet.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCustomUninstallAction, Log, All);
DEFINE_LOG_CATEGORY(LogCustomUninstallAction);

namespace BuildPatchServices
{
	class FCustomUninstallAction
		: public ICustomUninstallAction
	{
	public:
		/**
		 * Constructor with configuration values.
		 * @param InstallerError         A pointer to the class used to record errors during installation.
		 * @param FileSystem             A pointer to a filesystem abstraction used to interact with the filesystem.
		 * @param Platform               A pointer to a platform abstraction used to perform platform operations.
		 */
		FCustomUninstallAction(IInstallerError* InstallerError, IFileSystem* FileSystem, IPlatform* Platform);

		// ICustomUninstallAction interface begin.
		bool RunAction(const IBuildManifestSet* ManifestSet, const FBuildInstallerConfiguration& Configuration, const FString& InstallStagingDir, FBuildPatchProgress& BuildProgress) const override;
		// ICustomUninstallAction interface end.

	private:
		// A pointer to the instance of IInstallerError which records errors during the installation process.
		IInstallerError* InstallerError;

		// A pointer to an abstraction representing the filesystem.
		IFileSystem* FileSystem;

		// A pointer to an abstraction representing the platform.
		IPlatform* Platform;
	};

	FCustomUninstallAction::FCustomUninstallAction(IInstallerError* InInstallerError, IFileSystem* InFileSystem, IPlatform* InPlatform)
		: InstallerError(InInstallerError)
		, FileSystem(InFileSystem)
		, Platform(InPlatform)
	{
	}

	bool FCustomUninstallAction::RunAction(const IBuildManifestSet* ManifestSet, const FBuildInstallerConfiguration& Configuration, const FString& InstallStagingDir, FBuildPatchProgress& BuildProgress) const
	{
		bool bRunSuccessfull = true;
		TArray<FUninstallActionInfo> UninstallActionsInfo;
		ManifestSet->GetUninstallActionInfo(UninstallActionsInfo);

		for (const FUninstallActionInfo& UninstallActionInfo: UninstallActionsInfo)
		{
			BuildProgress.SetStateProgress(EBuildPatchState::RunningCustomUninstallAction, 0.0f);

			const FString InstallDirWithSlash = Configuration.InstallDirectory / TEXT("");
			const FString StageDirWithSlash = InstallStagingDir / TEXT("");

			FString FullActionPath;

			bool bUsingInstallRoot = false;
			bool bUsingStageRoot = false;
			// Constructing the full path to the executable and making sure it does exist.
			if (Configuration.InstallMode == EInstallMode::StageFiles)
			{
				FullActionPath = StageDirWithSlash + UninstallActionInfo.Path;
				if (FileSystem->FileExists(*FullActionPath))
				{
					bUsingStageRoot = true;
				}
			}	
			if (!bUsingStageRoot)
			{
				FullActionPath = InstallDirWithSlash + UninstallActionInfo.Path;
				if (FileSystem->FileExists(*FullActionPath))
				{
					bUsingInstallRoot = true;
				}
			}

			// If we failed to find the action executable above, then we have nothing to run.
			// This was likely an incompleted installation or an error in the shipped build.
			if (!bUsingInstallRoot && !bUsingStageRoot)
			{
				UE_LOGF(LogCustomUninstallAction, Error, "Could not find the custom action executable file %ls on disk.", *FullActionPath);
			}
			else
			{
				UE_LOGF(LogCustomUninstallAction, Log, "Running the custom uninstall action executable %ls %ls", *FullActionPath, *UninstallActionInfo.Args);

				// Custom uninstall action executable have to be ran elevated to be able to access restricted resources
				// on some OSs will result in a minimised or un-focused request.
				int32 ReturnCode = INDEX_NONE;
				if (Platform->ExecElevatedProcess(*FullActionPath, *UninstallActionInfo.Args, &ReturnCode))
				{
					if (ReturnCode != 0)
					{
						UE_LOGF(LogCustomUninstallAction, Error, "Custom uninstall action executable failed with code %u", ReturnCode);
						InstallerError->SetError(EBuildPatchInstallError::CustomUninstallActionError, *FString::Printf(TEXT("%s%u"), UninstallCustomActionErrorPrefixes::ReturnCode, ReturnCode));
						bRunSuccessfull = false;
					}
					else
					{
						UE_LOGF(LogCustomUninstallAction, Log, "Custom uninstall action complete");
					}
				}
				else
				{
					ReturnCode = Platform->GetLastError();
					UE_LOGF(LogCustomUninstallAction, Error, "Failed to start the custom uninstall action process %u", ReturnCode);
					InstallerError->SetError(EBuildPatchInstallError::CustomUninstallActionError, *FString::Printf(TEXT("%s%u"), UninstallCustomActionErrorPrefixes::ExecuteCode, ReturnCode));
					bRunSuccessfull = false;
				}
			}
		}

		BuildProgress.SetStateProgress(EBuildPatchState::RunningCustomUninstallAction, 1.0f);

		return bRunSuccessfull;
	}

	ICustomUninstallAction* FCustomUninstallActionFactory::Create(IInstallerError* InstallerError, IFileSystem* FileSystem, IPlatform* Platform)
	{
		return new FCustomUninstallAction(InstallerError, FileSystem, Platform);
	}
};
