// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BuildPatchManifest.h"


namespace BuildPatchServices
{
	class IInstallerError;
	class IFileSystem;
	class IPlatform;
	struct FBuildInstallerConfiguration;
	struct FBuildPatchProgress;
	class IBuildManifestSet;

	class ICustomUninstallAction
	{
	public:

		virtual ~ICustomUninstallAction() { }

		/**
		 * ICustomUninstallAction - RunAction
		 * Runs custom uninstall action associated with the installation.
		 * @param BuildManifest       The manifest containing details of the installer.
		 * @param Configuration       The installer configuration structure.
		 * @param InstallStagingDir   The directory within staging to construct install files to.
		 * @param BuildProgress       Used to keep track of install progress.
		 * @return                    Returns true if the custom uninstall action executable succeeded, false otherwise.
		 */
		virtual bool RunAction(const IBuildManifestSet* ManifestSet, const FBuildInstallerConfiguration& Configuration, const FString& InstallStagingDir, FBuildPatchProgress& BuildProgress) const = 0;
	};

	class FCustomUninstallActionFactory
	{
	public:
		/**
		 * FCustomUninstallActionFactory - Create
		 * Creates an instance of ICustomUninstallAction.
		 * @param InstallerError          The error handling implementation which any installation errors will be reported to.
		 * @param FileSystem              An abstraction representing the filesystem.
		 * @param Platform                An abstraction providing access to platform operations.
		 * @return the new ICustomUninstallAction instance created.
		 */
		static ICustomUninstallAction* Create(IInstallerError* InstallerError, IFileSystem* FileSystem, IPlatform* Platform);
	};
}
