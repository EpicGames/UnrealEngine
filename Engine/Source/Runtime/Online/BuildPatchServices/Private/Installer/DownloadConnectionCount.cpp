// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/DownloadConnectionCount.h"
#include "Installer/CloudChunkSource.h"
#include "CoreMinimal.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Queue.h"
#include "Misc/ScopeLock.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "Core/MeanValue.h"
#include "Core/Platform.h"
#include "Installer/ChunkReferenceTracker.h"
#include "Installer/ChunkStore.h"
#include "Installer/DownloadService.h"
#include "Installer/InstallerError.h"
#include "Common/StatsCollector.h"
#include "Interfaces/IBuildInstaller.h"
#include "BuildPatchUtil.h"
#include "Installer/Statistics/DownloadServiceStatistics.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDownloadConnectionCount, Log, All);
DEFINE_LOG_CATEGORY(LogDownloadConnectionCount);

namespace BuildPatchServices
{
	class FDownloadConnectionCount : public IDownloadConnectionCount
	{
		enum class EDownloadSpeedStatus
		{
			TransientIncrease, TransientDecrease, Steady, Headroom, Saturated
		};

	public:
		FDownloadConnectionCount(FDownloadConnectionCountConfig InConfiguration, IDownloadServiceStatistics* InDownloadStatistics);
		virtual ~FDownloadConnectionCount();
		virtual uint32 GetAdjustedCount(uint32 NumProcessing, EBuildPatchDownloadHealth CurrentHealth);
		void SetConnectionScaling(bool bEnableScaling) override;

	private:
		EDownloadSpeedStatus GetSpeedStatus(double Speed) const;
		EDownloadSpeedStatus GetSpeedIncreaseStatus() const;
		EDownloadSpeedStatus GetSpeedDecreaseStatus() const;
		bool IsHealthAcceptable(EBuildPatchDownloadHealth CurrentHealth) const;
		bool IsCountAcceptable(uint32 NumProcessing) const;
		bool IsSpeedIncreasing(double Speed) const;
		bool IsSpeedDecreasing(double Speed) const;
		bool ShouldIncreaseAllowance(EDownloadSpeedStatus SpeedStatus) const;
		bool ShouldDecreaseAllowance(EDownloadSpeedStatus SpeedStatus) const;
		uint32 GetAllowance() const;
		void UpdateTrackingSpeed(double Speed);
		void UpdateSpeedStatus(EDownloadSpeedStatus SpeedStatus);
		void UpdateAllowance(int32 Adjustment);
		void UpdateHealth(EBuildPatchDownloadHealth CurrentHealth);
		void Reset();

	private:
		IDownloadServiceStatistics* DownloadStatistics;
		FDownloadConnectionCountConfig Configuration;
		uint32 ConsecutiveDecreases;
		uint32 RequestAllowance;
		uint32 ConsecutiveIncreases;
		uint32 ConsecutiveFailedHealth;
		double HighBandwidthLevel;
		double LowBandwidthLevel;
		uint32 AveragesSeen;
		const uint32 MinimumAveragesBeforeHighSet = 4;
		const uint32 MaximumAllowanceExcess = 16;
		TAtomic<bool> bConnectionScalingEnabled;
		bool bPreviousConnectionScalingEnabledStatus;
	};

	FDownloadConnectionCount::FDownloadConnectionCount(FDownloadConnectionCountConfig InConfiguration, IDownloadServiceStatistics* InDownloadStatistics)
		: DownloadStatistics(InDownloadStatistics)
		, Configuration(InConfiguration)
		, ConsecutiveDecreases(0U)
		, RequestAllowance(Configuration.FallbackCount)
		, ConsecutiveIncreases(0U)
		, ConsecutiveFailedHealth(0U)
		, HighBandwidthLevel(0.0)
		, LowBandwidthLevel(0.0)
		, AveragesSeen(0U)
		, bConnectionScalingEnabled(!Configuration.bDisableConnectionScaling)
		, bPreviousConnectionScalingEnabledStatus(!Configuration.bDisableConnectionScaling)
	{
	}

	void FDownloadConnectionCount::Reset()
	{
		ConsecutiveDecreases = 0U;
		RequestAllowance = Configuration.FallbackCount;
		ConsecutiveIncreases = 0U;
		ConsecutiveFailedHealth = 0U;
		HighBandwidthLevel = 0.0;
		LowBandwidthLevel = 0.0;
		AveragesSeen = 0U;

		// Calling with 0 resets the internal values.
		DownloadStatistics->GetImmediateAverageSpeedPerRequest(0);
	}

	FDownloadConnectionCount::~FDownloadConnectionCount()
	{
	}

	uint32 FDownloadConnectionCount::GetAdjustedCount(uint32 NumProcessing, EBuildPatchDownloadHealth CurrentHealth)
	{
		if (bConnectionScalingEnabled == false)
		{
			bPreviousConnectionScalingEnabledStatus = false;
			return Configuration.FallbackCount;
		}
		else if (!bPreviousConnectionScalingEnabledStatus)
		{
			Reset();
		}
		bPreviousConnectionScalingEnabledStatus = true;

		uint32 OldCount = GetAllowance();
		// Prevent request allowance getting too far ahead of what is being processed currently.
		if ((RequestAllowance - NumProcessing) < MaximumAllowanceExcess)
		{
			const bool bShouldInitiate = IsCountAcceptable(NumProcessing) && IsHealthAcceptable(CurrentHealth);
			if (!bShouldInitiate)
			{
				UpdateAllowance(-1);
			}

			UpdateHealth(CurrentHealth);

			if ((nullptr != DownloadStatistics) && bShouldInitiate)
			{
				const TPair<double, uint32> SpeedPair = DownloadStatistics->GetImmediateAverageSpeedPerRequest(Configuration.AverageSpeedMinCount);
				const double Speed = SpeedPair.Get<0U>();
				const uint32 SampleCount = SpeedPair.Get<1U>();
				if ((SampleCount >= Configuration.AverageSpeedMinCount))
				{
					// Mac has a tendency to have outlandishly high speeds at startup
					// that are never again exceeded. This locks in the connection
					// allowance to the minimum. Prevent this by not updating maximum
					// encountered speed until things settle out. This is the purpose
					// of the 'AveragesSeen' member
					if (AveragesSeen < MinimumAveragesBeforeHighSet)
					{
						AveragesSeen += 1U;
					}
					else
					{
						const EDownloadSpeedStatus SpeedStatus = GetSpeedStatus(Speed);
						if (ShouldIncreaseAllowance(SpeedStatus))
						{
							UpdateAllowance(1);
							UpdateTrackingSpeed(Speed);
						}
						else if (ShouldDecreaseAllowance(SpeedStatus))
						{
							UpdateAllowance(-1);
							UpdateTrackingSpeed(Speed);
						}
						UpdateSpeedStatus(SpeedStatus);
					}
				}
			}
			else if (bShouldInitiate)
			{
				UpdateAllowance(1);
			}
		}
		else
		{
			UpdateAllowance(-1);
		}

		bool bLog = (OldCount != GetAllowance());

		if (bLog)
		{
			UE_LOGF(LogDownloadConnectionCount, Verbose, "New Request Count = %d", GetAllowance());
		}

		return GetAllowance();
	}

	void FDownloadConnectionCount::SetConnectionScaling(bool bEnableScaling)
	{
		bConnectionScalingEnabled = bEnableScaling;
	}

	bool FDownloadConnectionCount::IsHealthAcceptable(EBuildPatchDownloadHealth CurrentHealth) const
	{
		bool bIsOk = true;
		switch (CurrentHealth)
		{
			case EBuildPatchDownloadHealth::Good:
			case EBuildPatchDownloadHealth::Excellent:
				bIsOk = true;
				break;
			default:
				if (0 == (ConsecutiveFailedHealth % Configuration.HealthHysteresis))
				{
					if (ConsecutiveFailedHealth > 0)
					{
						bIsOk = false;
					}
				}
				break;
		}
		return bIsOk;
	}

	bool FDownloadConnectionCount::IsCountAcceptable(uint32 NumProcessing) const
	{
		return (NumProcessing < Configuration.MaxLimit);
	}

	void FDownloadConnectionCount::UpdateAllowance(int32 Adjustment)
	{
		if (Adjustment < 0 && RequestAllowance <= Configuration.MinLimit)
		{
			Adjustment = 0;
		}
		RequestAllowance += Adjustment;
		RequestAllowance = FMath::Clamp(RequestAllowance, Configuration.MinLimit, Configuration.MaxLimit);
	}

	uint32 FDownloadConnectionCount::GetAllowance() const
	{
		return RequestAllowance;
	}

	bool FDownloadConnectionCount::ShouldIncreaseAllowance(EDownloadSpeedStatus SpeedStatus) const
	{
		return EDownloadSpeedStatus::Headroom == SpeedStatus;
	}

	bool FDownloadConnectionCount::ShouldDecreaseAllowance(EDownloadSpeedStatus SpeedStatus) const
	{
		return EDownloadSpeedStatus::Saturated == SpeedStatus;
	}

	FDownloadConnectionCount::EDownloadSpeedStatus FDownloadConnectionCount::GetSpeedStatus(double Speed) const
	{
		if (IsSpeedIncreasing(Speed))
		{
			return GetSpeedIncreaseStatus();

		}
		else if (IsSpeedDecreasing(Speed))
		{
			return GetSpeedDecreaseStatus();
		}

		return EDownloadSpeedStatus::Steady;
	}

	bool FDownloadConnectionCount::IsSpeedIncreasing(double Speed) const
	{
		return Speed > (HighBandwidthLevel * Configuration.HighBandwidthFactor);
	}

	bool FDownloadConnectionCount::IsSpeedDecreasing(double Speed) const
	{
		return Speed < LowBandwidthLevel;
	}

	FDownloadConnectionCount::EDownloadSpeedStatus FDownloadConnectionCount::GetSpeedIncreaseStatus() const
	{
		// If we have sustained headroom, ramp up.
		if (ConsecutiveIncreases >= Configuration.PositiveHysteresis)
		{
			return EDownloadSpeedStatus::Headroom;
		}
		else
		{
			return EDownloadSpeedStatus::TransientIncrease;
		}
	}

	FDownloadConnectionCount::EDownloadSpeedStatus FDownloadConnectionCount::GetSpeedDecreaseStatus() const
	{
		// If we have sustained low bandwidth, ramp down.
		if (ConsecutiveDecreases >= Configuration.NegativeHysteresis)
		{
			return EDownloadSpeedStatus::Saturated;
		}
		else
		{
			return EDownloadSpeedStatus::TransientDecrease;
		}
	}

	void FDownloadConnectionCount::UpdateSpeedStatus(EDownloadSpeedStatus SpeedStatus)
	{
		switch (SpeedStatus)
		{
		case EDownloadSpeedStatus::Steady:
		case EDownloadSpeedStatus::Headroom:
		case EDownloadSpeedStatus::Saturated:
			// Once we decrease or increase the request count, reset the hysteresis counters
			// so request count is not changed again until
			// the configured number of consecutive changes are seen.
			ConsecutiveIncreases = 0U;
			ConsecutiveDecreases = 0U;
			break;
		case EDownloadSpeedStatus::TransientIncrease:
			ConsecutiveIncreases += 1U;
			ConsecutiveDecreases = 0U;
			break;
		case EDownloadSpeedStatus::TransientDecrease:
			ConsecutiveIncreases = 0U;
			ConsecutiveDecreases += 1U;
			break;
		}
	}

	void FDownloadConnectionCount::UpdateTrackingSpeed(double Speed)
	{
		if (Speed > HighBandwidthLevel)
		{
			HighBandwidthLevel = Speed;
			LowBandwidthLevel = HighBandwidthLevel * Configuration.LowBandwidthFactor;
		}
		else if (Speed < LowBandwidthLevel)
		{
			LowBandwidthLevel = Speed;
			HighBandwidthLevel = LowBandwidthLevel / Configuration.LowBandwidthFactor;
			if (ensure(HighBandwidthLevel > LowBandwidthLevel) == false)
			{
				LowBandwidthLevel = HighBandwidthLevel * 0.6L;
			}
		}
	}
	void FDownloadConnectionCount::UpdateHealth(EBuildPatchDownloadHealth CurrentHealth)
	{
		switch (CurrentHealth)
		{
			case EBuildPatchDownloadHealth::Good:
			case EBuildPatchDownloadHealth::Excellent:
				ConsecutiveFailedHealth = 0;
				break;
			default:
				ConsecutiveFailedHealth += 1;
				break;
		}
	}
	IDownloadConnectionCount* FDownloadConnectionCountFactory::Create(FDownloadConnectionCountConfig Configuration, IDownloadServiceStatistics* InDownloadStatistics)
	{
		return new FDownloadConnectionCount(Configuration, InDownloadStatistics);
	}
}

