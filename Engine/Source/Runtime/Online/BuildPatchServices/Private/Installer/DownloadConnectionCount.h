// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Installer/ChunkSource.h"
#include "Installer/Controllable.h"
#include "BuildPatchManifest.h"
#include "Interfaces/IBuildInstaller.h"

enum class EBuildPatchDownloadHealth;

namespace BuildPatchServices
{
	class IDownloadServiceStatistics;
	/*
	 * An interface for scaling the number of simultaneous download connections
	 */
	class IDownloadConnectionCount
	{
	public:
		/**
		 * Get the number of simultaneous download connections
		 * @param NumProcessing -- the current number of downloads in-flight
		 * @param CurrentHealth -- percentage of downloads that have failed since last call to this function, in terms of "health"
		 * @return number of connections that should be approximately optimal.
		 */
		virtual uint32 GetAdjustedCount(uint32 NumProcessing, EBuildPatchDownloadHealth CurrentHealth) = 0;

		/**
		 * Set whether connection scaling is enabled.
		 * @param bEnableScaling -- Enables download scaling, true to enable.
		 */
		virtual void SetConnectionScaling(bool bEnableScaling) = 0;

		virtual ~IDownloadConnectionCount() = default;
	};


	struct FDownloadConnectionCountConfig
	{
		// Maximum number of simultaneous connections to be used.
		uint32 MaxLimit;
		// Minimum number of connections to be used.
		uint32 MinLimit;
		// How many consecutive decreases in speed must be encountered before the count is changed.
		uint32 NegativeHysteresis;
		// Minimum number of samples needed that constitutes a useful average speed.
		uint32 AverageSpeedMinCount;
		// How many consecutive increases in speed must be encountered before the count is changed.
		uint32 PositiveHysteresis;
		// Percentage of highest seen speed that must be exceeded before the count is changed
		double HighBandwidthFactor;
		// Percentage of highest seen speed that must be fallen short of before the count is changed
		double LowBandwidthFactor;
		// Enable or disable download scaling.
		bool bDisableConnectionScaling;
		// The number of connections to allow when dynamic scaling is disabled
		uint32 FallbackCount;
		// How many consecutive poor download health status checks must be encountered before the count is changed.
		uint32 HealthHysteresis;

		FDownloadConnectionCountConfig()
			: MaxLimit(100U)
			, MinLimit(8U)
			, NegativeHysteresis(8U)
			, AverageSpeedMinCount(8U)
			, PositiveHysteresis(4U)
			, HighBandwidthFactor(0.85L)
			, LowBandwidthFactor(0.65L)
			, bDisableConnectionScaling(false)
			, FallbackCount(16)
			, HealthHysteresis(8)
		{
		}
		FDownloadConnectionCountConfig(const FDownloadConnectionCountConfig& CopyThis)
			: MaxLimit(CopyThis.MaxLimit)
			, MinLimit(CopyThis.MinLimit)
			, NegativeHysteresis(CopyThis.NegativeHysteresis)
			, AverageSpeedMinCount(CopyThis.AverageSpeedMinCount)
			, PositiveHysteresis(CopyThis.PositiveHysteresis)
			, HighBandwidthFactor(CopyThis.HighBandwidthFactor)
			, LowBandwidthFactor(CopyThis.LowBandwidthFactor)
			, bDisableConnectionScaling(CopyThis.bDisableConnectionScaling)
			, FallbackCount(CopyThis.FallbackCount)
			, HealthHysteresis(CopyThis.HealthHysteresis)
		{

		}
	};

	class FDownloadConnectionCountFactory
	{
	public:
		/*
		 * Create a download connection count calculator.
		 * @param Configuration -- configuration values.
		 * @param InDownloadStatistics -- A download statistics object. Can be null. If null, the object will use an implementation that does not take into account download speed changes.
		*/
		static IDownloadConnectionCount* Create(FDownloadConnectionCountConfig Configuration, IDownloadServiceStatistics* InDownloadStatistics);
	};
}