// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/DownloadService.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/ScopeLock.h"
#include "Core/AsyncHelpers.h"
#include "Tasks/Task.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "Common/StatsCollector.h"
#include "Installer/InstallerAnalytics.h"
#include "Common/HttpManager.h"
#include "Common/FileSystem.h"
#include "Stats/Stats.h"
#include "Containers/Ticker.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDownloadService, Warning, All);
DEFINE_LOG_CATEGORY(LogDownloadService);

namespace BuildPatchServices
{
	// 2MB buffer for reading from disk/network.
	const int32 FileReaderBufferSize = 2097152;
	// Empty array representation for no data.
	const TArray<uint8> NoData;

	struct FFileRequest
	{
		UE::Tasks::FTask Task;
		UE::Tasks::FCancellationToken ShouldCancel;
	};

	class FHttpDownload
		: public IDownload
	{
	public:
		FHttpDownload(FHttpRequestPtr InHttpRequest, bool bInSuccess)
			: HttpRequest(MoveTemp(InHttpRequest))
			, HttpResponse(HttpRequest ? HttpRequest->GetResponse() : nullptr)
			, bSuccess(bInSuccess)
		{}

		// IDownload interface begin.
		virtual bool RequestSuccessful() const override { return bSuccess; }
		virtual bool ResponseSuccessful() const override { return EHttpResponseCodes::IsOk(GetResponseCode()); }
		virtual int32 GetResponseCode() const override { return HttpResponse.IsValid() ? HttpResponse->GetResponseCode() : INDEX_NONE; }
		virtual const TArray<uint8>& GetData() const override { return HttpResponse.IsValid() ? HttpResponse->GetContent() : NoData; }
		// IDownload interface end.

	private:
		// Make sure to hold on the the request pointer because the response could reference it
		FHttpRequestPtr HttpRequest;
		FHttpResponsePtr HttpResponse;
		bool bSuccess = false;
	};

	class FFileDownload
		: public IDownload
	{
	public:
		FFileDownload(TArray<uint8> InDataArray, bool bInSuccess)
			: DataArray(MoveTemp(InDataArray))
			, bSuccess(bInSuccess)
		{}

		// IDownload interface begin.
		virtual bool RequestSuccessful() const override { return bSuccess; }
		virtual bool ResponseSuccessful() const override { return EHttpResponseCodes::IsOk(GetResponseCode()); }
		virtual int32 GetResponseCode() const override { return RequestSuccessful() ? EHttpResponseCodes::Ok : EHttpResponseCodes::NotFound; }
		virtual const TArray<uint8>& GetData() const override { return DataArray; }
		// IDownload interface end.

	private:
		TArray<uint8> DataArray;
		bool bSuccess = false;
	};

	struct FDownloadDelegates
	{
	public:
		FDownloadDelegates() = default;
		FDownloadDelegates(const FDownloadCompleteDelegate& InOnCompleteDelegate, const FDownloadProgressDelegate& InOnProgressDelegate);

	public:
		FDownloadCompleteDelegate OnCompleteDelegate;
		FDownloadProgressDelegate OnProgressDelegate;
	};

	FDownloadDelegates::FDownloadDelegates(const FDownloadCompleteDelegate& InOnCompleteDelegate, const FDownloadProgressDelegate& InOnProgressDelegate)
		: OnCompleteDelegate(InOnCompleteDelegate)
		, OnProgressDelegate(InOnProgressDelegate)
	{
	}

	/**
	 * Use Pimpl pattern for binding thread safe shared ptr delegates for the http module, without having to enforce that
	 * this service should be made using TShared* reference controllers.
	 */
	class FDownloadServiceImpl
		: public TSharedFromThis<FDownloadServiceImpl>
	{
	public:
		FDownloadServiceImpl(TSharedPtr<IHttpManager> HttpManager, TSharedPtr<IFileSystem> FileSystem, TSharedPtr<IDownloadServiceStat> DownloadServiceStat, TSharedPtr<IInstallerAnalytics> InstallerAnalytics);
		~FDownloadServiceImpl() = default;

		void CancelAllRequests();

		// IDownloadService interface begin.
		bool RequestFileWithHeaders(const FString& FileUri, const TMap<FString, FString>& RequestHeaders, const FDownloadCompleteDelegate& OnCompleteDelegate, const FDownloadProgressDelegate& OnProgressDelegate, int32* OutRequestId = nullptr);
		void RequestCancel(int32 RequestId);
		void RequestAbandon(int32 RequestId);
		// IDownloadService interface end.

	private:
		UE::Tasks::FTask MakeFileLoadTask(int32 RequestId, const FString& FileUri, FFileRequest* FileRequest);

		IDownloadServiceStat::FDownloadRecord MakeDownloadRecord(int32 RequestId, FString Uri);

		void OnRequestProgress(FHttpRequestPtr Request, uint64 BytesSent, uint64 BytesReceived, const int32 RequestId);

		void OnProcessRequestComplete(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bSucceeded, IDownloadServiceStat::FDownloadRecord DownloadRecord);

	private:
		TSharedPtr<IHttpManager> HttpManager;
		TSharedPtr<IFileSystem> FileSystem;
		TSharedPtr<IDownloadServiceStat> DownloadServiceStat;
		TSharedPtr<IInstallerAnalytics> InstallerAnalytics;

		std::atomic<int32> RequestIdCounter;

		FCriticalSection ActiveRequestsCS;
		TMap<int32, TSharedRef<IHttpRequest, ESPMode::ThreadSafe>> ActiveHttpRequests;
		TMap<int32, TUniquePtr<FFileRequest>> ActiveFileRequests;

		FCriticalSection RequestDelegatesCS;
		TMap<int32, FDownloadDelegates> RequestDelegates;
	};

	class FDownloadService
		: public IDownloadService
	{
	public:
		FDownloadService(TSharedPtr<IHttpManager> HttpManager, TSharedPtr<IFileSystem> FileSystem, TSharedPtr<IDownloadServiceStat> DownloadServiceStat, TSharedPtr<IInstallerAnalytics> InstallerAnalytics)
			: Impl(MakeShared<FDownloadServiceImpl>(HttpManager, FileSystem, DownloadServiceStat, InstallerAnalytics))
		{}
		~FDownloadService()
		{
			Impl->CancelAllRequests();
		}

		// IDownloadService interface begin.
		virtual bool RequestFile(const FString& FileUri, const FDownloadCompleteDelegate& OnCompleteDelegate, const FDownloadProgressDelegate& OnProgressDelegate, int32* OutRequestId = nullptr) override
		{
			TMap<FString, FString> RequestHeaders;
			return Impl->RequestFileWithHeaders(FileUri, RequestHeaders, OnCompleteDelegate, OnProgressDelegate, OutRequestId);
		}
		virtual bool RequestFileWithHeaders(const FString& FileUri, const TMap<FString, FString>& RequestHeaders, const FDownloadCompleteDelegate& OnCompleteDelegate, const FDownloadProgressDelegate& OnProgressDelegate, int32* OutRequestId = nullptr) override
		{
			return Impl->RequestFileWithHeaders(FileUri, RequestHeaders, OnCompleteDelegate, OnProgressDelegate, OutRequestId);
		}
		virtual void RequestCancel(int32 RequestId) override { Impl->RequestCancel(RequestId); }
		virtual void RequestAbandon(int32 RequestId) override { Impl->RequestAbandon(RequestId); }
		// IDownloadService interface end.

	private:
		TSharedRef<FDownloadServiceImpl> Impl;
	};

	FDownloadServiceImpl::FDownloadServiceImpl(TSharedPtr<IHttpManager> InHttpManager, TSharedPtr<IFileSystem> InFileSystem, TSharedPtr<IDownloadServiceStat> InDownloadServiceStat, TSharedPtr<IInstallerAnalytics> InInstallerAnalytics)
		: HttpManager(InHttpManager)
		, FileSystem(InFileSystem)
		, DownloadServiceStat(InDownloadServiceStat)
		, InstallerAnalytics(InInstallerAnalytics)
	{}

	void FDownloadServiceImpl::CancelAllRequests()
	{
		// Cancel all HTTP requests and wait for all file downloads threads to exit.
		{
			FScopeLock ScopeLock(&ActiveRequestsCS);
			for (const TPair<int32, TSharedRef<IHttpRequest>>& ActiveHttpRequest : ActiveHttpRequests)
			{
				ActiveHttpRequest.Value->CancelRequest();
			}
			for (const TPair<int32, TUniquePtr<FFileRequest>>& ActiveFileRequest : ActiveFileRequests)
			{
				ActiveFileRequest.Value->ShouldCancel.Cancel();
			}
		}

		// callbacks and removal handled by completion delegates
	}

	bool FDownloadServiceImpl::RequestFileWithHeaders(const FString& FileUri, const TMap<FString, FString>& RequestHeaders, const FDownloadCompleteDelegate& OnCompleteDelegate, const FDownloadProgressDelegate& OnProgressDelegate, int32* OutRequestId)
	{
		const int32 RequestId = ++RequestIdCounter;
		if (OutRequestId)
		{
			*OutRequestId = RequestId;
		}


		// Save the delegates.
		{
			FScopeLock ScopeLock(&RequestDelegatesCS);
			RequestDelegates.Emplace(RequestId, FDownloadDelegates(OnCompleteDelegate, OnProgressDelegate));
		}

		const bool bIsHttpRequest = FileUri.StartsWith(TEXT("http"), ESearchCase::IgnoreCase);
		if (bIsHttpRequest)
		{
			// Kick off http request.
			FHttpRequestPtr HttpRequest;
			{
				FScopeLock ScopeLock(&ActiveRequestsCS);
				HttpRequest = ActiveHttpRequests.Emplace(RequestId, HttpManager->CreateRequest());
			}

			HttpRequest->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);
			HttpRequest->SetURL(FileUri);
			HttpRequest->SetVerb(TEXT("GET"));

			for (const TPair<FString, FString>& Header : RequestHeaders)
			{
				HttpRequest->SetHeader(Header.Key, Header.Value);
			}

			// We have to use BingSP here because BindSPLambda is not making sure that the context object is alive till the end of callback execution.
			HttpRequest->OnRequestProgress64().BindSP(this, &FDownloadServiceImpl::OnRequestProgress, RequestId);
			HttpRequest->OnProcessRequestComplete().BindSP(this, &FDownloadServiceImpl::OnProcessRequestComplete, MakeDownloadRecord(RequestId, FileUri));
			HttpRequest->ProcessRequest();
		}
		else
		{
			// Load file from drive/network.
			TUniquePtr<FFileRequest> FileRequest = MakeUnique<FFileRequest>();
			FFileRequest* FileRequestPtr = FileRequest.Get();
			{
				FScopeLock ScopeLock(&ActiveRequestsCS);
				ActiveFileRequests.Add(RequestId, MoveTemp(FileRequest));
			}

			UE::Tasks::FTask FileLoadTask = MakeFileLoadTask(RequestId, FileUri, FileRequestPtr);
			{
				FScopeLock ScopeLock(&ActiveRequestsCS);
				if (ActiveFileRequests.Contains(RequestId))
				{
					FileRequestPtr->Task = MoveTemp(FileLoadTask);
				}
			}
		}
		DownloadServiceStat->OnDownloadStarted(RequestId, FileUri);
		return true;
	}

	void FDownloadServiceImpl::RequestCancel(int32 RequestId)
	{
		FScopeLock ScopeLock(&ActiveRequestsCS);
		
		if (FHttpRequestRef* MaybeRequest = ActiveHttpRequests.Find(RequestId))
		{
			(*MaybeRequest)->CancelRequest();
		}
		if (TUniquePtr<FFileRequest>* MaybeRequest = ActiveFileRequests.Find(RequestId))
		{
			(*MaybeRequest)->ShouldCancel.Cancel();
		}

		// callbacks and removal handled by completion delegates
	}

	void FDownloadServiceImpl::RequestAbandon(int32 RequestId)
	{
		{
			FScopeLock ScopeLock(&RequestDelegatesCS);
			RequestDelegates.Remove(RequestId);
		}

		RequestCancel(RequestId);
	}

	UE::Tasks::FTask FDownloadServiceImpl::MakeFileLoadTask(const int32 RequestId, const FString& FileUri, FFileRequest* FileRequest)
	{
		return UE::Tasks::Launch(UE_SOURCE_LOCATION,
			[this, WeakThis = AsWeak(), RequestId, FileUri, FileRequest]
			{
				TSharedPtr<FDownloadServiceImpl> PinThis = WeakThis.Pin();
				if (!PinThis)
				{
					return;
				}

				TArray<uint8> FileDataArray;
				bool bSuccess = !FileRequest->ShouldCancel.IsCanceled();
				IDownloadServiceStat::FDownloadRecord DownloadRecord = MakeDownloadRecord(RequestId, FileUri);
				if (bSuccess)
				{
					TUniquePtr<FArchive> Reader = FileSystem->CreateFileReader(*FileUri);
					bSuccess = Reader.IsValid();
					if (bSuccess)
					{
						const int64 FileSize = Reader->TotalSize();
						FileDataArray.Reset();
						FileDataArray.AddUninitialized(FileSize);
						int64 BytesRead = 0;
						while (BytesRead < FileSize)
						{
							bool bIsCanceled = FileRequest->ShouldCancel.IsCanceled();
							if (bIsCanceled)
							{
								bSuccess = false;
								break;
							}

							const int64 ReadLen = FMath::Min<int64>(FileReaderBufferSize, FileSize - BytesRead);
							Reader->Serialize(FileDataArray.GetData() + BytesRead, ReadLen);
							BytesRead += ReadLen;

							{
								FScopeLock ScopeLock(&RequestDelegatesCS);
								if (FDownloadDelegates* MaybeDelegates = RequestDelegates.Find(RequestId))
								{
									MaybeDelegates->OnProgressDelegate.ExecuteIfBound(RequestId, BytesRead);
								}
							}

							DownloadServiceStat->OnDownloadProgress(RequestId, BytesRead);
						}
						DownloadRecord.SpeedRecord.Size = BytesRead;
						bSuccess = bSuccess && Reader->Close() && BytesRead == FileSize;
					}
					DownloadRecord.SpeedRecord.CyclesEnd = FStatsCollector::GetCycles();
					DownloadRecord.bSuccess = bSuccess;
					DownloadRecord.ResponseCode = bSuccess ? EHttpResponseCodes::Ok : EHttpResponseCodes::NotFound;
				}

				if (!bSuccess)
				{
					FileDataArray.Empty();
				}

				FDownloadDelegates Delegates;
				bool bFoundDelegates = false;
				{
					FScopeLock ScopeLock(&RequestDelegatesCS);
					bFoundDelegates = RequestDelegates.RemoveAndCopyValue(RequestId, Delegates);
				}

				if (bFoundDelegates)
				{
					TSharedRef<FFileDownload> CompletedDownload = MakeShared<FFileDownload>(MoveTemp(FileDataArray), bSuccess);
					Delegates.OnCompleteDelegate.ExecuteIfBound(RequestId, CompletedDownload);
				}
				
				{
					FScopeLock ScopeLock(&ActiveRequestsCS);
					ActiveFileRequests.Remove(RequestId);

					// FileRequest is dangling at this point
				}

				DownloadServiceStat->OnDownloadComplete(DownloadRecord);
			}, UE::Tasks::ETaskPriority::BackgroundHigh);
	}

	IDownloadServiceStat::FDownloadRecord FDownloadServiceImpl::MakeDownloadRecord(int32 RequestId, FString Uri)
	{
		IDownloadServiceStat::FDownloadRecord DownloadRecord;
		DownloadRecord.RequestId = RequestId;
		DownloadRecord.Uri = MoveTemp(Uri);
		DownloadRecord.bSuccess = false;
		DownloadRecord.ResponseCode = INDEX_NONE;
		DownloadRecord.SpeedRecord.CyclesStart = FStatsCollector::GetCycles();
		DownloadRecord.SpeedRecord.CyclesEnd = DownloadRecord.SpeedRecord.CyclesStart;
		DownloadRecord.SpeedRecord.Size = 0;
		return DownloadRecord;
	}

	void FDownloadServiceImpl::OnRequestProgress(FHttpRequestPtr Request, uint64 BytesSent, uint64 BytesReceived, const int32 RequestId)
	{
#if USING_ADDRESS_SANITISER
		// Force use the Request parameter to work around some MSVC-specific compiler issue in ASan.
		UE_LOGF(LogDownloadService, Verbose, "[FDownloadServiceImpl::RequestFile] %ls %ls received %d bytes", *Request->GetVerb(), *Request->GetURL(), BytesReceived);
#endif
		{
			FScopeLock ScopeLock(&RequestDelegatesCS);
			if (FDownloadDelegates* MaybeDelegates = RequestDelegates.Find(RequestId))
			{
				MaybeDelegates->OnProgressDelegate.ExecuteIfBound(RequestId, BytesReceived);
			}
		}

		DownloadServiceStat->OnDownloadProgress(RequestId, BytesReceived);
	}

	void FDownloadServiceImpl::OnProcessRequestComplete(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bSucceeded, IDownloadServiceStat::FDownloadRecord DownloadRecord)
	{
		InstallerAnalytics->TrackRequest(Request);

		DownloadRecord.bSuccess = bSucceeded;
		DownloadRecord.ResponseCode = Response.IsValid() ? Response->GetResponseCode() : (int32)Request->GetFailureReason();
		DownloadRecord.SpeedRecord.CyclesEnd = FStatsCollector::GetCycles();
		DownloadRecord.SpeedRecord.Size = Response.IsValid() ? Response->GetContent().Num() : 0;
		DownloadRecord.FailureReason = Request->GetFailureReason();

		FDownloadDelegates Delegates;
		bool bFoundDelegates = false;
		{
			FScopeLock ScopeLock(&RequestDelegatesCS);
			bFoundDelegates = RequestDelegates.RemoveAndCopyValue(DownloadRecord.RequestId, Delegates);
		}

		if (bFoundDelegates)
		{
			TSharedRef<FHttpDownload> CompletedDownload = MakeShared<FHttpDownload>(Request, bSucceeded);
			Delegates.OnCompleteDelegate.ExecuteIfBound(DownloadRecord.RequestId, CompletedDownload);
		}

		{
			FScopeLock ScopeLock(&ActiveRequestsCS);
			ActiveHttpRequests.Remove(DownloadRecord.RequestId);
		}

		DownloadServiceStat->OnDownloadComplete(DownloadRecord);
	}

	IDownloadService* FDownloadServiceFactory::Create(TSharedPtr<IHttpManager> HttpManager, TSharedPtr<IFileSystem> FileSystem, TSharedPtr<IDownloadServiceStat> DownloadServiceStat, TSharedPtr<IInstallerAnalytics> InstallerAnalytics)
	{
		check(HttpManager != nullptr);
		check(FileSystem != nullptr);
		check(DownloadServiceStat != nullptr);
		check(InstallerAnalytics != nullptr);
		return new FDownloadService(HttpManager, FileSystem, DownloadServiceStat, InstallerAnalytics);
	}
}
