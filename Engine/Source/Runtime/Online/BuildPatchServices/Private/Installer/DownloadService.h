// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "Common/SpeedRecorder.h"
#include "Interfaces/IHttpBase.h"

namespace BuildPatchServices
{
	class IDownloadServiceStat;
	class IInstallerAnalytics;
	class IHttpManager;
	class IFileSystem;

	/**
	 * An interface providing access to the result of a download.
	 */
	class IDownload
	{
	public:
		virtual ~IDownload() = default;

		/**
		 * Gets whether the request was successfully made.
		 * @return true for success.
		 */
		virtual bool RequestSuccessful() const = 0;

		/**
		 * Gets whether the response was received and indicated good status.
		 * @return true for success.
		 */
		virtual bool ResponseSuccessful() const = 0;

		/**
		 * Gets the response code, or INDEX_NONE if response failed.
		 * @return the response code.
		 */
		virtual int32 GetResponseCode() const = 0;

		/**
		 * Get a reference to the downloaded data held by this class.
		 * @return reference to downloaded bytes.
		 */
		virtual const TArray<uint8>& GetData() const = 0;
	};
	typedef TSharedRef<IDownload, ESPMode::ThreadSafe> FDownloadRef;

	/**
	 * Delegate called for download progress updates.
	 * @param RequestId     The id that was returned by RequestFile(..).
	 * @param BytesSoFar    The number of bytes received so far.
	 */
	DECLARE_DELEGATE_TwoParams(FDownloadProgressDelegate, int32 /* RequestId */, uint64 /* BytesSoFar */);

	/**
	 * Delegate called for download complete.
	 * @param RequestId     The id that was returned by RequestFile(..).
	 * @param Download      The complete download instance.
	 */
	DECLARE_DELEGATE_TwoParams(FDownloadCompleteDelegate, int32 /* RequestId */, const FDownloadRef& /* Download */);

	/**
	 * An interface providing access to download files, supporting http(s) and network protocols.
	 */
	class IDownloadService
	{
	public:
		virtual ~IDownloadService() = default;

		/**
		 * Starts a new request for a file.
		 * @param FileUri               The uri for the file request. For http(s), this should begin with the protocol.
		 * @param OnCompleteDelegate    The delegate that will be called with the completed download.
		 * @param OnProgressDelegate    The delegate that will be called with updates to the ongoing download.
		 * @param OutRequestId          The request id for this download, which will match that provided to the delegates.
		 * @return if the request was queued successfully.
		 */
		virtual bool RequestFile(const FString& FileUri, const FDownloadCompleteDelegate& OnCompleteDelegate, const FDownloadProgressDelegate& OnProgressDelegate, int32* OutRequestId = nullptr) = 0;

		/** As per RequestFile, except adds the given headers to the request as a <Name, Value> pair. */
		virtual bool RequestFileWithHeaders(const FString& FileUri, const TMap<FString, FString>& RequestHeaders, const FDownloadCompleteDelegate& OnCompleteDelegate, const FDownloadProgressDelegate& OnProgressDelegate, int32* OutRequestId = nullptr) = 0;

		/**
		 * Requests the cancellation of a requested file.
		 * @param RequestId             The id that was returned by the call to RequestFile which should be canceled.
		 */
		virtual void RequestCancel(int32 RequestId) = 0;

		/**
		 *  Requests the abandoning of a requested file (cancelling without calling complete/progress handlers)
		 * @param RequestId             The id that was returned by the call to RequestFile which should be canceled.
		 */
		virtual void RequestAbandon(int32 RequestId) = 0;
	};

	/**
	 * A factory for creating the default implementation of IDownloadService.
	 */
	class FDownloadServiceFactory
	{
	public:
		/**
		 * Instantiates an instance of an IDownloadService, using the HTTP module, and platform file API.
		 * @param HttpManager           The HTTP manager interface for making HTTP(s) requests.
		 * @param FileSystem            The file system interface for network and disk file loading.
		 * @param DownloadServiceStat   The class to receive statistics and event information.
		 * @param InstallerAnalytics    The analytics implementation for tracking HTTP requests.
		 * @return the new IDownloadService instance created.
		 */
		static IDownloadService* Create(TSharedPtr<IHttpManager> HttpManager, TSharedPtr<IFileSystem> FileSystem, TSharedPtr<IDownloadServiceStat> DownloadServiceStat, TSharedPtr<IInstallerAnalytics> InstallerAnalytics);
	};

	/**
	 * This interface defines the statistics class required by the download service. It should be implemented in order to collect
	 * desired information which is being broadcast by the system.
	 */
	class IDownloadServiceStat
	{
	public:
		virtual ~IDownloadServiceStat() = default;

		/**
		 * A struct containing the information about a completed request.
		 */
		struct FDownloadRecord
		{
			// The id for the request that was made.
			int32 RequestId;
			// The uri used when making the request.
			FString Uri;
			// Whether the request completed successfully.
			bool bSuccess;
			// The response code for the request.
			int32 ResponseCode;
			// The timing record.
			ISpeedRecorder::FRecord SpeedRecord;
			// If failed, the reason why the request failed
			EHttpFailureReason FailureReason;
		};

		/**
		 * Called as each request has started.
		 * @param RequestId             The id for the request.
		 * @param Uri                   The uri for the request.
		 */
		virtual void OnDownloadStarted(int32 RequestId, const FString& Uri) = 0;

		/**
		 * Called for each request completion.
		 * @param RequestId             The id for the request.
		 * @param BytesReceived         The bytes received so far.
		 */
		virtual void OnDownloadProgress(int32 RequestId, uint64 BytesReceived) = 0;

		/**
		 * Called for each request completion.
		 * @param DownloadRecord        The struct containing the stats for this request.
		 */
		virtual void OnDownloadComplete(const FDownloadRecord& DownloadRecord) = 0;
	};
}
