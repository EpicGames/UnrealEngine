// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/FileAttribution.h"
#include "Common/FileSystem.h"
#include "BuildPatchProgress.h"
#include "IBuildManifestSet.h"
#include "BuildPatchUtil.h"
#include "Core/AsyncHelpers.h"
#include <atomic>

namespace BuildPatchServices
{
	class FFileAttribution : public IFileAttribution
	{
	public:
		FFileAttribution(IFileSystem* FileSystem, IBuildManifestSet* ManifestSet, TSet<FString> TouchedFiles, const FString& InstallDirectory, 
			const FString& StagedFileDirectory, bool bUseStageDirectory, FBuildPatchProgress* BuildProgress, const TMap<FString, FString>& PerFileSubdirectories);

		virtual ~FFileAttribution() {}

		// IControllable interface begin.
		virtual void SetPaused(bool bInIsPaused) override;
		virtual void Abort() override;
		virtual void Reset() override;
		// IControllable interface end.

		// IFileAttribution interface begin.
		virtual bool ApplyAttributes() override;
		// IFileAttribution interface end.

	private:
		FString SelectFullFilePath(const FString& BuildFile) const;
		bool HasSameAttributes(const FFileManifest* NewFileManifest, const FFileManifest* OldFileManifest) const;
		void SetupFileAttributes(const FString& FilePath, const FFileManifest& FileManifest, bool bForce) const;

	private:
		IFileSystem* FileSystem;
		IBuildManifestSet* ManifestSet;
		TSet<FString> TouchedFiles;
		const FString InstallDirectory;
		const FString StagedFileDirectory;
		const bool bUseStageDirectory;
		FBuildPatchProgress* BuildProgress;
		FThreadSafeBool bIsPaused;
		FThreadSafeBool bShouldAbort;
		const TMap<FString, FString>& PerFileSubdirectories;
	};

	FFileAttribution::FFileAttribution(IFileSystem* InFileSystem, IBuildManifestSet* InManifestSet, TSet<FString> InTouchedFiles, const FString& InInstallDirectory, 
		const FString& InStagedFileDirectory, bool bInUseStageDirectory, FBuildPatchProgress* InBuildProgress, const TMap<FString, FString>& InPerFileSubdirectories)
		: FileSystem(InFileSystem)
		, ManifestSet(InManifestSet)
		, TouchedFiles(MoveTemp(InTouchedFiles))
		, InstallDirectory(InInstallDirectory)
		, StagedFileDirectory(InStagedFileDirectory)
		, bUseStageDirectory(bInUseStageDirectory)
		, BuildProgress(InBuildProgress)
		, bIsPaused(false)
		, bShouldAbort(false)
		, PerFileSubdirectories(InPerFileSubdirectories)
	{
		BuildProgress->SetStateProgress(EBuildPatchState::SettingAttributes, 0.0f);
	}

	void FFileAttribution::SetPaused(bool bInIsPaused)
	{
		bIsPaused = bInIsPaused;
	}

	void FFileAttribution::Abort()
	{
		bShouldAbort = true;
	}

	void FFileAttribution::Reset()
	{
		bShouldAbort = false;
	}

	bool FFileAttribution::ApplyAttributes()
	{
		// We need to set attributes for all files in the new build that require it
		TSet<FString> BuildFileList;
		ManifestSet->GetExpectedFiles(BuildFileList);
		BuildProgress->SetStateProgress(EBuildPatchState::SettingAttributes, 0.0f);

		FTaskCoresScheduler<void> ApplyAttributesTasks = BuildPatchServices::CreateTaskSchedulerLimitedByCoresNumber(EAsyncExecution::TaskGraph);

		std::atomic<int32> BuildFileIdx{ 0 };
		float FilesNum = BuildFileList.Num();
		for (const FString& BuildFile : BuildFileList)
		{
			if (bShouldAbort)
			{
				break;
			}
			auto TaskFunc = [&]()
				{
					const FFileManifest* NewFileManifest = ManifestSet->GetNewFileManifest(BuildFile);
					const FFileManifest* OldFileManifest = ManifestSet->GetCurrentFileManifest(BuildFile);
					const bool bForce = ManifestSet->IsFileRepairAction(BuildFile);
					bool bHasChanged = bForce || (TouchedFiles.Contains(BuildFile) && !HasSameAttributes(NewFileManifest, OldFileManifest));
					if (NewFileManifest != nullptr && bHasChanged)
					{
						SetupFileAttributes(SelectFullFilePath(BuildFile), *NewFileManifest, bForce);
					}
					BuildProgress->SetStateProgress(EBuildPatchState::SettingAttributes, float(BuildFileIdx.fetch_add(1)) / FilesNum);
				};
			ApplyAttributesTasks.ScheduleTask(MoveTemp(TaskFunc));
			// Wait while paused
			while (bIsPaused && !bShouldAbort)
			{
				FPlatformProcess::Sleep(0.5f);
			}
		}

		ApplyAttributesTasks.WaitAll();
		BuildProgress->SetStateProgress(EBuildPatchState::SettingAttributes, 1.0f);

		// We don't fail on this step currently
		return true;
	}

	FString FFileAttribution::SelectFullFilePath(const FString& BuildFile) const
	{
		FString InstallFilename;
		if (bUseStageDirectory)
		{
			InstallFilename = StagedFileDirectory / BuildFile;
			int64 FileSize;
			if (FileSystem->GetFileSize(*InstallFilename, FileSize))
			{
				return InstallFilename;
			}
		}

		return FBuildPatchUtils::ResolveInstallationFileName(InstallDirectory, PerFileSubdirectories, BuildFile);
	}

	bool FFileAttribution::HasSameAttributes(const FFileManifest* NewFileManifest, const FFileManifest* OldFileManifest) const
	{
		// Currently it is not supported to rely on this, as the update process always makes new files when a file changes.
		// This can be reconsidered when the patching process changes.
		return false;

		// return (NewFileManifest != nullptr && OldFileManifest != nullptr)
		//     && (NewFileManifest->bIsUnixExecutable == OldFileManifest->bIsUnixExecutable)
		//     && (NewFileManifest->bIsReadOnly == OldFileManifest->bIsReadOnly)
		//     && (NewFileManifest->bIsCompressed == OldFileManifest->bIsCompressed);
	}

	void FFileAttribution::SetupFileAttributes(const FString& FilePath, const FFileManifest& FileManifest, bool bForce) const
	{
		using namespace BuildPatchServices;
		EAttributeFlags FileAttributes = EAttributeFlags::None;

		// First check file attributes as it's much faster to read and do nothing
		bool bKnownAttributes = FileSystem->GetAttributes(*FilePath, FileAttributes);
		const bool bFileExists = EnumHasAllFlags(FileAttributes, EAttributeFlags::Exists);
		bool bIsReadOnly = EnumHasAllFlags(FileAttributes, EAttributeFlags::ReadOnly);
		const bool bIsCompressed = EnumHasAllFlags(FileAttributes, EAttributeFlags::Compressed);
		const bool bIsUnixExecutable = EnumHasAllFlags(FileAttributes, EAttributeFlags::Executable);
		const bool bFileManifestIsReadOnly = EnumHasAllFlags(FileManifest.FileMetaFlags, EFileMetaFlags::ReadOnly);
		const bool bFileManifestIsCompressed = EnumHasAllFlags(FileManifest.FileMetaFlags, EFileMetaFlags::Compressed);
		const bool bFileManifestIsUnixExecutable = EnumHasAllFlags(FileManifest.FileMetaFlags, EFileMetaFlags::UnixExecutable);

		// If we know the file is missing, skip out
		if (bKnownAttributes && !bFileExists)
		{
			return;
		}

		// If we are forcing, say we don't know existing so that all calls are made
		if (bForce)
		{
			bKnownAttributes = false;
		}

		// Set compression attribute
		if (!bKnownAttributes || bIsCompressed != bFileManifestIsCompressed)
		{
			// Must make not readonly if required
			if (!bKnownAttributes || bIsReadOnly)
			{
				bIsReadOnly = false;
				FileSystem->SetReadOnly(*FilePath, false);
			}
			FileSystem->SetCompressed(*FilePath, bFileManifestIsCompressed);
		}

		// Set executable attribute
		if (!bKnownAttributes || bIsUnixExecutable != bFileManifestIsUnixExecutable)
		{
			// Must make not readonly if required
			if (!bKnownAttributes || bIsReadOnly)
			{
				bIsReadOnly = false;
				FileSystem->SetReadOnly(*FilePath, false);
			}
			FileSystem->SetExecutable(*FilePath, bFileManifestIsUnixExecutable);
		}

		// Set readonly attribute
		if (!bKnownAttributes || bIsReadOnly != bFileManifestIsReadOnly)
		{
			FileSystem->SetReadOnly(*FilePath, bFileManifestIsReadOnly);
		}
	}

	IFileAttribution* FFileAttributionFactory::Create(IFileSystem* FileSystem, IBuildManifestSet* ManifestSet, TSet<FString> TouchedFiles, 
		const FString& InstallDirectory, const FString& StagedFileDirectory, FBuildPatchProgress* BuildProgress, const TMap<FString, FString>& PerFileSubdirectories)
	{
		check(BuildProgress != nullptr);
		return new FFileAttribution(FileSystem, ManifestSet, MoveTemp(TouchedFiles), InstallDirectory, StagedFileDirectory, !StagedFileDirectory.IsEmpty(), BuildProgress, PerFileSubdirectories);
	}
}