// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Installer/Controllable.h"
#include "BuildPatchManifest.h"


namespace BuildPatchServices
{
	class IFileSystem;
	struct FBuildPatchProgress;
	class IBuildManifestSet;

	class IFileAttribution
		: public IControllable
	{
	public:
		/**
		 * Applies the attributes to the files in the Staging directory, or installation directory.
		 * @return true is successful.
		 */
		virtual bool ApplyAttributes() = 0;
	};

	class FFileAttributionFactory
	{
	public:
		/**
		 * Creates a file attribute class that can be used to setup file attributes that are contained in the build manifest, optionally
		 * taking account of a staging directory where alternative files are.
		 * @param FileSystem            The file system interface.
		 * @param NewManifest           The manifest describing the build data.
		 * @param OldManifest           The manifest describing the build data.
		 * @param TouchedFiles          The set of files that were touched by the installation, these will filter the list of actions performed when not forcing.
		 * @param InstallDirectory      The directory to analyze.
		 * @param StagedFileDirectory   A stage directory for updated files, ignored if empty string. If a file exists here, it will be
		 *                              checked instead of the one in InstallDirectory.
		 * @param BuildProgress         The build progress class to update with progress information.
		 * @return     Ref of an object that can be used to perform the operation.
		 */
		static IFileAttribution* Create(IFileSystem* FileSystem, IBuildManifestSet* ManifestSet, TSet<FString> TouchedFiles, const FString& InstallDirectory, 
			const FString& StagedFileDirectory, FBuildPatchProgress* BuildProgress, const TMap<FString, FString>& PerFileSubdirectories);
	};
}
