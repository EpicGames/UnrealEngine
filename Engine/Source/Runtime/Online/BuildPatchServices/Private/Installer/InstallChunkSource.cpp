// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/InstallChunkSource.h"

#include "Algo/Transform.h"
#include "Async/Mutex.h"
#include "BuildPatchHash.h"
#include "Common/FileSystem.h"
#include "Common/StatsCollector.h"
#include "Core/BlockStructure.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryReader.h"
#include "Tasks/Task.h"
#include "BuildPatchUtil.h"

DECLARE_LOG_CATEGORY_EXTERN(LogInstallChunkSource, Log, All);
DEFINE_LOG_CATEGORY(LogInstallChunkSource);

namespace BuildPatchServices
{
	static FSHAHash GetShaHashForDataSet(const void* ChunkData, const uint32 ChunkSize)
	{
		FSHAHash ShaHashCheck;
		FSHA1::HashBuffer(ChunkData, ChunkSize, ShaHashCheck.Hash);
		return ShaHashCheck;
	}
	
	class FInstallChunkSource : public IConstructorInstallChunkSource
	{
	public:
		FInstallChunkSource(IFileSystem* FileSystem, IInstallChunkSourceStat* InInstallChunkSourceStat, const TMultiMap<FString, FBuildPatchAppManifestRef>& InInstallationSources, 
			const TSet<FGuid>& ChunksThatWillBeNeeded, bool bInAllowLoadingAnyChunk, const TMap<FString, FString>& PerFileSubdirectories);
		~FInstallChunkSource();
		
		virtual FRequestProcessFn CreateRequest(const FGuid& DataId, FMutableMemoryView DestinationBuffer, void* UserPtr, FChunkRequestCompleteDelegate CompleteFn) override;
		virtual const TSet<FGuid>& GetAvailableChunks() const override
		{
			return AvailableInBuilds;
		}
		virtual void OnBeforeDeleteFile(const FString& FilePath) override
		{
			// Make sure we close our handle before the deletion occurs.
			// With multiple files in flight in the constructor we can be deleting a file at the same time as we are
			// reading chunks for other files, which means we can hit OpenedFileHandles from multiple threads.
			UE::TUniqueLock _(OpenedFileHandlesLock);
			{
				TUniquePtr<FOpenedFile>* File = OpenedFileHandles.Find(FilePath);
				if (File)
				{
					if ((*File)->FileHandleLock.TryLock() == false)
					{
						// We can't ever be trying to delete a file when a read is active on it,
						// as this means that a file is trying to read from it after harvesting has
						// completed, which should ensure no chunks are read.
						UE_LOGF(LogInstallChunkSource, Error, "Critical error: trying to delete %ls while read lock is held.", *FilePath);

						// We don't want to just crash, so take the lock in order to wait for the
						// other thread to finish. Then we hope that nothing else is queued :grimacing:
						(*File)->FileHandleLock.Lock();
					}

					OpenedFileHandles.Remove(FilePath);
				}
			}
		}
		virtual int32 GetChunkUnavailableAt(const FGuid& DataId) const override;
		virtual void SetFileRetirementPositions(TMap<FString, int32>&& InFileRetirementPositions) override
		{
			FileRetirementPositions = MoveTemp(InFileRetirementPositions);
		}

		virtual void GetChunksForFile(const FString& InstallationDirectory, const FString& FilenameInManifest, TSet<FGuid>& OutChunks) const override;

		virtual void EnumerateFilesForChunk(const FGuid& DataId, TUniqueFunction<void(const FString& NormalizedInstallDirectory, const FString& NormalizedFileName)>&& Callback) const override
		{
			const FString* FoundInstallDirectory;
			const FBuildPatchAppManifest* FoundInstallManifest;
			FindChunkLocation(DataId, &FoundInstallDirectory, &FoundInstallManifest);

			const TArray<FChunkSourceDetails>* ChunkSource = ChunkSources.Find(DataId);

			if (FoundInstallDirectory == nullptr || FoundInstallManifest == nullptr || ChunkSource == nullptr)
			{
				return;
			}

			// The installation directory starts off normalized by then appends a directory which
			// might be empty, leaving a trailing slash. Rather than chase down all possibilities we
			// just re normalize.
			FString NormalizedInstallDirectory = *FoundInstallDirectory;
			FPaths::NormalizeDirectoryName(NormalizedInstallDirectory);

			for (const FChunkSourceDetails& ChunkDetails : *ChunkSource)
			{
				// afaict the file manifest filename is normalized because in the manifest builder it
				// generates it from file spans, which are created in directorybuildstreamer which makes them
				// relative, and internal to that function they are normalized.
				Callback(NormalizedInstallDirectory, ChunkDetails.InstalledFileManifest->Filename);
			}
		}

	private:
		void FindChunkLocation(const FGuid& DataId, const FString** FoundInstallDirectory, const FBuildPatchAppManifest** FoundInstallManifest) const;

	private:
	
		IFileSystem* FileSystem;
		IInstallChunkSourceStat* InstallChunkSourceStat;

		// Storage of enumerated chunks.
		TSet<FGuid> AvailableInBuilds;
		TArray<TPair<FString, FBuildPatchAppManifestRef>> InstallationSources;

		struct FOpenedFile
		{
			FOpenedFile(TUniquePtr<FArchive>&& InFileHandle) : FileHandle(MoveTemp(InFileHandle)) {}

			TUniquePtr<FArchive> FileHandle;
			UE::FMutex FileHandleLock;
		};

		UE::FMutex OpenedFileHandlesLock; 
		// This is keyed off the resolved file name (including subdirectory mapping)
		TMap<FString, TUniquePtr<FOpenedFile>> OpenedFileHandles;

		// The index (ChunkReferenceTracker->GetCurrentUsageIndex) at which our files will get deleted due to destructive install to make room for
		// the new file. 
		TMap<FString, int32> FileRetirementPositions;

		// Contains the info to build the chunk from various pieces strewn across the installation.
		struct FChunkSourceDetails
		{
			const FFileManifest* InstalledFileManifest;
			uint32 OffsetInChunk;
			uint32 SizeOfChunkPart;
			uint64 OffsetInInstalledFile;
		};

		// Contains the list of pieces to assemble chunks from, for each chunk.
		TMap<FGuid, TArray<FChunkSourceDetails>> ChunkSources;

		// When we try and recreate chunks from an existing installation, there are chunks that don't get enumerated
		// as producible. Additionally, the chunks might not fill the full window. Setting this allows any request for
		// a chunk where we have any parts to succeed and be zero filled for any missing parts.
		bool bAllowLoadingAnyChunk = false;

		// Contains files that have a custom subdirectory to install into. This is accessed from multiple threads
		// but should be static across execution and so requires no lock.
		const TMap<FString, FString>& PerFileSubdirectories;
	};

	FInstallChunkSource::FInstallChunkSource(IFileSystem* InFileSystem, IInstallChunkSourceStat* InInstallChunkSourceStat, 
		const TMultiMap<FString, FBuildPatchAppManifestRef>& InInstallationSources, const TSet<FGuid>& ChunksThatWillBeNeeded,
		bool bInAllowLoadingAnyChunk, const TMap<FString, FString>& InPerFileSubdirectories)
		: FileSystem(InFileSystem)
		, InstallChunkSourceStat(InInstallChunkSourceStat)
		, bAllowLoadingAnyChunk(bInAllowLoadingAnyChunk)
		, PerFileSubdirectories(InPerFileSubdirectories)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(InstallChunkSource_ctor);

		// InstallationSources is a map of installation directory to the manifest that was installed there.
		for (const TPair<FString, FBuildPatchAppManifestRef>& InstallationSourcePair : InInstallationSources)
		{
			const FString& InstallationDirectory = InstallationSourcePair.Key;
			const FBuildPatchAppManifestRef& InstalledManifest = InstallationSourcePair.Value;

			// Patches are only allowed to pull chunks from the install of the same tags, otherwise they must be
			// supplied via another source. However we can't get the tag list at this point; all we have is a list of
			// "registered installations" from the caller, so we get the full list of files and filter based on chunks we care
			// about. This may result in us attempting to pull chunks from files outside our tag if it can be serviced
			// by a file in a different install tag, however if the file doesn't exist, we'll continue to check other
			// files until we presumably find the correct file in our own tag.
			TSet<FString> InstalledFiles;
			InstalledManifest->GetFileList(InstalledFiles);

			struct FChunkAssemblyInfo
			{
				TArray<FChunkSourceDetails> SourceDetails;
				bool bComplete;
			};

			// Find the chunks we can get from these files.
			TMap<FGuid, FBlockStructure> ChunkStructures;
			TMap<FGuid, FChunkAssemblyInfo> ChunkAssemblyInfos;

			int32 MaxChunkStructures = 0;

			for (const FString& InstalledFileName : InstalledFiles)
			{
				const FFileManifest* InstalledFileManifest = InstalledManifest->GetFileManifest(InstalledFileName);
				if (!InstalledFileManifest)
				{
					// Should never happen?
					UE_LOGF(LogInstallChunkSource, Error, "Missing file manifest from file provided from manifest? %ls", *InstalledFileName);
					continue;
				}

				bool bExistenceCheckPassed = false;

				int32 ChunksForFile = 0;

				uint32 ChunkPartSize = 0;
				uint64 FileOffset = 0;
				for (int32 ChunkPartIndex = 0; ChunkPartIndex < InstalledFileManifest->ChunkParts.Num(); ChunkPartIndex++, FileOffset += ChunkPartSize)
				{
					const FChunkPart& ChunkPart = InstalledFileManifest->ChunkParts[ChunkPartIndex];
					ChunkPartSize = ChunkPart.Size;

					if (!ChunksThatWillBeNeeded.Contains(ChunkPart.Guid))
					{
						continue;
					}

					// We do the file existence check _after_ we know we need the chunk so that we avoid
					// hitting the disk for every single file in the manifest when the tag might only be installing
					// 5 of them.
					if (!bExistenceCheckPassed)
					{
						const FString DiskFileName = FBuildPatchUtils::ResolveInstallationFileName(InstallationDirectory, PerFileSubdirectories, InstalledFileName);

						int64 SizeOnDisk = 0;
						bool bGotFileSizeSuccessfully = InFileSystem->GetFileSize(*DiskFileName, SizeOnDisk);
						uint64 ExpectedSize = InstalledManifest->GetFileSize(InstalledFileName);
						if (bGotFileSizeSuccessfully &&
							(uint64)SizeOnDisk == ExpectedSize)
						{
							bExistenceCheckPassed = true;
						}
						else
						{
							// We can't get any chunks from this file, so bail.
							break;
						}
					}
					

					// If we get the full chunk here, we don't need to assemble anything.
					FChunkAssemblyInfo& ChunkAssemblyInfo = ChunkAssemblyInfos.FindOrAdd(ChunkPart.Guid);
					if (ChunkAssemblyInfo.SourceDetails.Num() == 0)
					{
						// First time we've seen this chunk - check if we are getting it all at once (often)
						const FChunkInfo* ChunkInfoPtr = InstalledManifest->GetChunkInfo(ChunkPart.Guid);
						if (!ChunkInfoPtr)
						{
							UE_LOGF(LogInstallChunkSource, Display, "Missing chunk info for guid: %ls", *LexToString(ChunkPart.Guid));
							continue;
						}

						if (ChunkPart.Offset == 0 &&
							ChunkInfoPtr->DataSizeUncompressed == ChunkPart.Size)
						{
							// Got the whole thing, don't need to fill out the chunk parts.
							FChunkSourceDetails NewInfo;
							NewInfo.InstalledFileManifest = InstalledFileManifest;
							NewInfo.OffsetInInstalledFile = FileOffset;
							NewInfo.OffsetInChunk = ChunkPart.Offset;
							NewInfo.SizeOfChunkPart = ChunkPart.Size;
							ChunkAssemblyInfo.SourceDetails.Add(NewInfo);
							ChunkAssemblyInfo.bComplete = true;
							ChunksForFile++;
							continue;
						}

						// If it's just a part of the chunk we fall through
					}

					if (ChunkAssemblyInfo.bComplete)
					{
						// We already have all we need to make this chunk, anything else is redundant.
						continue;
					}

					FBlockStructure& PartialStructure = ChunkStructures.FindOrAdd(ChunkPart.Guid);
					PartialStructure.Add(ChunkPart.Offset, ChunkPart.Size, ESearchDir::FromEnd);
					
					if (ChunkStructures.Num() > MaxChunkStructures)
					{
						MaxChunkStructures = ChunkStructures.Num();
					}

					FChunkSourceDetails PartInfo;
					PartInfo.InstalledFileManifest = InstalledFileManifest;
					PartInfo.OffsetInInstalledFile = FileOffset;
					PartInfo.OffsetInChunk = ChunkPart.Offset;
					PartInfo.SizeOfChunkPart = ChunkPart.Size;
					ChunkAssemblyInfo.SourceDetails.Add(PartInfo);

					const FChunkInfo* ChunkInfoPtr = InstalledManifest->GetChunkInfo(ChunkPart.Guid);

					if (PartialStructure.GetHead() == PartialStructure.GetTail() &&
						PartialStructure.GetHead()->GetSize() == ChunkInfoPtr->DataSizeUncompressed)
					{
						// Done with this chunk, free the structures space.
						ChunkAssemblyInfo.bComplete = true;
						ChunkStructures.Remove(ChunkPart.Guid);
						ChunksForFile++;
					}
				}

				UE_CLOGF(ChunksForFile, LogInstallChunkSource, Verbose, "%ls produced %d chunks", *InstalledFileName, ChunksForFile);
			}

			// Move the full chunks we have available over to our internal structures.
			uint32 IncompleteChunks = 0;
			uint32 ChunksInAnotherSource = 0;
			uint32 CompleteChunks = 0;
			for (TPair<FGuid, FChunkAssemblyInfo>& ChunkAssemblyPair : ChunkAssemblyInfos)
			{
				if (!ChunkAssemblyPair.Value.bComplete)
				{
					IncompleteChunks++;
					// Skip incomplete chunks unless we allow loading incomplete chunks
					if (!bAllowLoadingAnyChunk)
					{
						continue;
					}
				}

				TArray<FChunkSourceDetails>& ChunkSource = ChunkSources.FindOrAdd(ChunkAssemblyPair.Key);
				if (ChunkSource.Num())
				{
					ChunksInAnotherSource++;
					continue;
				}

				ChunkSource = MoveTemp(ChunkAssemblyPair.Value.SourceDetails);
				if (ChunkAssemblyPair.Value.bComplete)
				{
					CompleteChunks++;
				}

				AvailableInBuilds.Add(ChunkAssemblyPair.Key);
			}

			UE_LOGF(LogInstallChunkSource, Verbose, "Got %u complete chunks, %u incomplete chunks, %u repeat chunks", CompleteChunks, IncompleteChunks, ChunksInAnotherSource);
			if (CompleteChunks)
			{
				InstallationSources.Add(InstallationSourcePair);
			}

		}

		UE_LOGF(LogInstallChunkSource, Log, "Useful Sources:%d/%d. Available Chunks:%d/%d.", InstallationSources.Num(), InInstallationSources.Num(), AvailableInBuilds.Num(), ChunksThatWillBeNeeded.Num());
	}

	FInstallChunkSource::~FInstallChunkSource()
	{
		
	}

	void FInstallChunkSource::GetChunksForFile(const FString& InstallationDirectory, const FString& FileInManifest, TSet<FGuid>& OutChunks) const
	{
		const FFileManifest* FileManifest = nullptr;
		FString NormalisedInstallDirectory = InstallationDirectory;
		FPaths::NormalizeDirectoryName(NormalisedInstallDirectory);
		for (const TPair<FString, FBuildPatchAppManifestRef>& Pair : InstallationSources)
		{
			FString NormaliasedInstallSourcePath = Pair.Key;
			FPaths::NormalizeDirectoryName(NormaliasedInstallSourcePath);
			if (FPaths::IsSamePath(NormalisedInstallDirectory, NormaliasedInstallSourcePath))
			{
				FileManifest = Pair.Value->GetFileManifest(FileInManifest);
				break;
			}
		}
		if (FileManifest != nullptr)
		{
			for (const TPair<FGuid, TArray<FChunkSourceDetails>>& ChunkSource : ChunkSources)
			{
				bool bNeedsThisFile = false;
				for (const FChunkSourceDetails& Details : ChunkSource.Value)
				{
					if (Details.InstalledFileManifest == FileManifest)
					{
						bNeedsThisFile = true;
						break;
					}
				}

				if (bNeedsThisFile)
				{
					OutChunks.Add(ChunkSource.Key);
				}
			}
		}
	}

	IConstructorChunkSource::FRequestProcessFn FInstallChunkSource::CreateRequest(const FGuid& DataId, FMutableMemoryView DestinationBuffer, void* UserPtr, FChunkRequestCompleteDelegate CompleteFn)
	{
		const FString* FoundInstallDirectory;
		const FBuildPatchAppManifest* FoundInstallManifest;
		FindChunkLocation(DataId, &FoundInstallDirectory, &FoundInstallManifest);
		if (FoundInstallDirectory == nullptr || FoundInstallManifest == nullptr)
		{
			CompleteFn.Execute(DataId, false, true, UserPtr);
			return [](bool) {return;};
		}

		// This request can be called from multiple threads, including on the same source file.
		return [this, FoundInstallManifest, FoundInstallDirectory = *FoundInstallDirectory, DataId, DestinationBuffer, UserPtr, CompleteFn](bool bIsAborted)
		{
			ISpeedRecorder::FRecord ActivityRecord;
			ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
			InstallChunkSourceStat->OnLoadStarted(DataId);

			TRACE_CPUPROFILER_EVENT_SCOPE(InstallRead);
			if (bIsAborted)
			{
				ActivityRecord.CyclesEnd = ActivityRecord.CyclesStart;
				InstallChunkSourceStat->OnLoadComplete(DataId, IInstallChunkSourceStat::ELoadResult::Aborted, ActivityRecord);

				CompleteFn.Execute(DataId, true, false, UserPtr);
				return;
			}

			const TArray<FChunkSourceDetails>* ChunkSource = ChunkSources.Find(DataId);
			const FChunkInfo* ChunkInfoPtr = FoundInstallManifest->GetChunkInfo(DataId);
			if (!ChunkInfoPtr || !ChunkSource)
			{
				if (!ChunkSource)
				{
					UE_LOGF(LogInstallChunkSource, Error, "Chunk %ls not found in sources.", *DataId.ToString());
				}
				else
				{
					UE_LOGF(LogInstallChunkSource, Error, "Chunk info for %ls not found in install manifest.", *DataId.ToString());
				}
				ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
				InstallChunkSourceStat->OnLoadComplete(DataId, IInstallChunkSourceStat::ELoadResult::MissingPartInfo, ActivityRecord);

				CompleteFn.Execute(DataId, false, true, UserPtr);
				return;
			}

			FBlockStructure ChunkBlocks;

			IInstallChunkSourceStat::ELoadResult Result = IInstallChunkSourceStat::ELoadResult::Success;
	
			bool bLoadedWholeChunk = false;
			for (int32 FileChunkPartsIdx = 0; FileChunkPartsIdx < ChunkSource->Num(); ++FileChunkPartsIdx)
			{
				if (bLoadedWholeChunk)
				{
					// The manifest gave us more chunk parts than we needed to generate the full chunk. This shouldn't happen,
					// and so conceptually is an error, but since we have all the data we can technically proceed.
					// This seems to happen with some regularity - need to understand why.
					//UE_LOGF(LogInstallChunkSource, Display, "Chunk %ls had more chunk sources than necessary to re-assemble", *WriteToString<40>(DataId));
					break;
				}
				const FChunkSourceDetails& FileChunkPart = ChunkSource->operator[](FileChunkPartsIdx);

				// Validate the chunk can load into the destination.
				uint32 ChunkEndLocation = FileChunkPart.OffsetInChunk + FileChunkPart.SizeOfChunkPart;
				if (ChunkEndLocation > DestinationBuffer.GetSize())
				{
					// The chunk metadata tried to assemble larger than the chunk itself - error
					UE_LOGF(LogInstallChunkSource, Error, "Chunk %ls assembled larger than the actual chunk size (chunk wanted end %u vs buffer size %llu", *WriteToString<40>(DataId), ChunkEndLocation, DestinationBuffer.GetSize());
					Result = IInstallChunkSourceStat::ELoadResult::InvalidChunkParts;
					break;
				}

				// We use the internal FArchive pointer so that we don't have to hold the file handle
				// lock over the read - the uniqueptr might move around, but the managed pointer will not.
				// We do know that our specific file won't get deleted until we are done, but it's possible that
				// multiple threads are here looking for the same file.
				FOpenedFile* OpenedFile = nullptr;
				{
					const FString DiskFileName = FBuildPatchUtils::ResolveInstallationFileName(FoundInstallDirectory, PerFileSubdirectories, FileChunkPart.InstalledFileManifest->Filename);
					OpenedFileHandlesLock.Lock();
					TUniquePtr<FOpenedFile>* OpenedFilePtr = OpenedFileHandles.Find(DiskFileName);
					if (OpenedFilePtr == nullptr)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(Install_OpenSource);
						TUniquePtr<FArchive> NewReader = FileSystem->CreateFileReader(*DiskFileName);
						if (NewReader.IsValid())
						{
							TUniquePtr<FOpenedFile> NewOpenedFile = MakeUnique<FOpenedFile>(MoveTemp(NewReader));

							OpenedFileHandles.Add(DiskFileName, MoveTemp(NewOpenedFile));
							OpenedFilePtr = OpenedFileHandles.Find(DiskFileName);
						}
					}
					if (OpenedFilePtr)
					{
						OpenedFile = OpenedFilePtr->Get();
					}
					OpenedFileHandlesLock.Unlock();
				}

				if (OpenedFile == nullptr)
				{
					UE_LOGF(LogInstallChunkSource, Error, "Failed to open: %ls on chunk %ls", *FileChunkPart.InstalledFileManifest->Filename, *LexToString(DataId));
					Result = IInstallChunkSourceStat::ELoadResult::OpenFileFail;
					break;
				}

				{
					UE::TUniqueLock HandleLock(OpenedFile->FileHandleLock);

					TRACE_CPUPROFILER_EVENT_SCOPE(Install_Serialize);

					OpenedFile->FileHandle->Seek(FileChunkPart.OffsetInInstalledFile);
					OpenedFile->FileHandle->Serialize((uint8*)DestinationBuffer.GetData() + FileChunkPart.OffsetInChunk, FileChunkPart.SizeOfChunkPart);

					ActivityRecord.Size += FileChunkPart.SizeOfChunkPart;

					FBlockStructure NewChunk;
					NewChunk.Add(FileChunkPart.OffsetInChunk, FileChunkPart.SizeOfChunkPart);
					if (NewChunk.Intersect(ChunkBlocks).GetHead() != nullptr)
					{
						// This used to be allowed but in advance of multi threaded reading we want to make sure this
						// doesn't happen anymore (already shouldn't be...)
						OpenedFile->FileHandleLock.Unlock();
						UE_LOGF(LogInstallChunkSource, Error, "Chunk %ls had overlapping chunk parts", *WriteToString<40>(DataId));
						Result = IInstallChunkSourceStat::ELoadResult::InvalidChunkParts;
						break;
					}
					ChunkBlocks.Add(FileChunkPart.OffsetInChunk, FileChunkPart.SizeOfChunkPart);
				}

				// The expectation is that we get the full chunk only once we've assembled all of the parts
				// provided by the manifest, so the last iteration should set this to true. If it isn't the last
				// iteration, then we'll hit the faux-error case at the top of the loop.
				bLoadedWholeChunk = ChunkBlocks.GetHead() && ChunkBlocks.GetHead() == ChunkBlocks.GetTail() && ChunkBlocks.GetHead()->GetSize() == ChunkInfoPtr->DataSizeUncompressed;
			}

			// If we didn't have enough parts but we're allowing loading partial chunks, succeed the chunk
			// and zero fill what we didn't have.
			if (!bLoadedWholeChunk && 
				Result == IInstallChunkSourceStat::ELoadResult::Success &&
				bAllowLoadingAnyChunk)
			{
				FBlockStructure Remnants;
				Remnants.Add(0, ChunkInfoPtr->DataSizeUncompressed);
				Remnants.Remove(ChunkBlocks);

				const FBlockEntry* Entry = Remnants.GetHead();
				while (Entry)
				{
					FMemory::Memset((uint8*)DestinationBuffer.GetData() + Entry->GetOffset(), 0, Entry->GetSize());
					Entry = Entry->GetNext();
				}
				bLoadedWholeChunk = true;
			}
			
			if (!bLoadedWholeChunk)
			{
				if (Result == IInstallChunkSourceStat::ELoadResult::Success)
				{
					// If we failed without hitting a different case, then we just didn't have enough parts.
					Result = IInstallChunkSourceStat::ELoadResult::InvalidChunkParts;
				}

				ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
				InstallChunkSourceStat->OnLoadComplete(DataId, IInstallChunkSourceStat::ELoadResult::MissingPartInfo, ActivityRecord);

				CompleteFn.Execute(DataId, false, !bLoadedWholeChunk, UserPtr);
				return;
			}

			// We set this here because it is used to compute the IO speeds, however we can't call OnLoadComplete because we don't know
			// the hash result yet.
			ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();

			// Check chunk hash. 
			UE::Tasks::Launch(TEXT("Install_Hash"), [DataId, UserPtr, DestinationBuffer, FoundInstallManifest, CompleteFn, ActivityRecord, InstallChunkSourceStat=InstallChunkSourceStat]()
				{
					IInstallChunkSourceStat::ELoadResult Result = IInstallChunkSourceStat::ELoadResult::Success;
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(InstallHash);
						FSHAHash ChunkShaHash;
						uint64 ChunkRollingHash = 0;

						if (FoundInstallManifest->GetChunkShaHash(DataId, ChunkShaHash))
						{
							if (GetShaHashForDataSet(DestinationBuffer.GetData(), DestinationBuffer.GetSize()) != ChunkShaHash)
							{
								Result = IInstallChunkSourceStat::ELoadResult::HashCheckFailed;
							}
						}
						else if (FoundInstallManifest->GetChunkHash(DataId, ChunkRollingHash))
						{
							if (FRollingHash::GetHashForDataSet((const uint8*)DestinationBuffer.GetData(), DestinationBuffer.GetSize()) != ChunkRollingHash)
							{
								Result = IInstallChunkSourceStat::ELoadResult::HashCheckFailed;
							}
						}
						else
						{
							Result = IInstallChunkSourceStat::ELoadResult::MissingHashInfo;
						}
					}

					InstallChunkSourceStat->OnLoadComplete(
						DataId, 
						Result, 
						ActivityRecord);

					CompleteFn.Execute(DataId, false, Result != IInstallChunkSourceStat::ELoadResult::Success, UserPtr);
				}
			);
		};
	}
	
	int32 FInstallChunkSource::GetChunkUnavailableAt(const FGuid& DataId) const
	{
		if (FileRetirementPositions.Num() == 0)
		{
			// If we aren't destructive then it's always available.
			return TNumericLimits<int32>::Max();
		}

		const FString* FoundInstallDirectory;
		const FBuildPatchAppManifest* FoundInstallManifest;
		FindChunkLocation(DataId, &FoundInstallDirectory, &FoundInstallManifest);
		if (FoundInstallDirectory == nullptr || FoundInstallManifest == nullptr)
		{
			return TNumericLimits<int32>::Max();
		}

		// This chunk is no longer available as soon as the first file containing a part is complete (if destructive install)
		int32 ChunkUnavailableAt = TNumericLimits<int32>::Max();

		const TArray<FChunkSourceDetails>* ChunkSource = ChunkSources.Find(DataId);
		if (ChunkSource)
		{
			for (const FChunkSourceDetails& Part : *ChunkSource)
			{
				const int32* FirstIndexAfterFile = FileRetirementPositions.Find(Part.InstalledFileManifest->Filename);
				if (FirstIndexAfterFile &&
					*FirstIndexAfterFile < ChunkUnavailableAt)
				{
					ChunkUnavailableAt = *FirstIndexAfterFile;
				}
			}
		}

		return ChunkUnavailableAt;
	}
	
	void FInstallChunkSource::FindChunkLocation(const FGuid& DataId, const FString** FoundInstallDirectory, const FBuildPatchAppManifest** FoundInstallManifest) const
	{
		uint64 ChunkHash;
		*FoundInstallDirectory = nullptr;
		*FoundInstallManifest = nullptr;
		for (const TPair<FString, FBuildPatchAppManifestRef>& Pair : InstallationSources)
		{
			// GetChunkHash can be used as a check for whether this manifest references this chunk.
			if (Pair.Value->GetChunkHash(DataId, ChunkHash))
			{
				*FoundInstallDirectory = &Pair.Key;
				*FoundInstallManifest = &Pair.Value.Get();
				return;
			}
		}
	}

	IConstructorInstallChunkSource* IConstructorInstallChunkSource::CreateInstallSource(IFileSystem* FileSystem, IInstallChunkSourceStat* InstallChunkSourceStat, 
		const TMultiMap<FString, FBuildPatchAppManifestRef>& InstallationSources, const TSet<FGuid>& ChunksThatWillBeNeeded, bool bAllowLoadingAnyChunk, const TMap<FString, FString>& PerFileSubdirectories)
	{
		return new FInstallChunkSource(FileSystem, InstallChunkSourceStat, InstallationSources, ChunksThatWillBeNeeded, bAllowLoadingAnyChunk, PerFileSubdirectories);
	}

	const TCHAR* ToString(const IInstallChunkSourceStat::ELoadResult& LoadResult)
	{
		switch(LoadResult)
		{
			case IInstallChunkSourceStat::ELoadResult::Success:
				return TEXT("Success");
			case IInstallChunkSourceStat::ELoadResult::MissingHashInfo:
				return TEXT("MissingHashInfo");
			case IInstallChunkSourceStat::ELoadResult::MissingPartInfo:
				return TEXT("MissingPartInfo");
			case IInstallChunkSourceStat::ELoadResult::OpenFileFail:
				return TEXT("OpenFileFail");
			case IInstallChunkSourceStat::ELoadResult::HashCheckFailed:
				return TEXT("HashCheckFailed");
			case IInstallChunkSourceStat::ELoadResult::Aborted:
				return TEXT("Aborted");
			case IInstallChunkSourceStat::ELoadResult::InvalidChunkParts:
				return TEXT("InvalidChunkParts");
			default:
				return TEXT("Unknown");
		}
	}
}
