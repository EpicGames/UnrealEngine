// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/ChunkSource.h"
#include "Installer/Controllable.h"
#include "Common/SpeedRecorder.h"
#include "BuildPatchManifest.h"

class IBuildInstallerSharedContext;

namespace BuildPatchServices
{
	class IChunkStore;
	class IChunkReferenceTracker;
	class IInstallerError;
	class IInstallChunkSourceStat;
	class IFileSystem;
	class IBuildManifestSet;


	/**
	* The interface for an installation chunk source, which provides access to chunk data retrieved from known local installations.
	*/
	class IConstructorInstallChunkSource : public IConstructorChunkSource
	{
	public:
		virtual ~IConstructorInstallChunkSource() = default;

		// Return the chunks this source can provide.
		virtual const TSet<FGuid>& GetAvailableChunks() const = 0;

		// Notification that a file is about to be deleted so we need to clear out any references.
		virtual void OnBeforeDeleteFile(const FString& FilePath) = 0;

		// Return the list of chunks for a file so that the constructor can harvest them before the file
		// is deleted
		virtual void GetChunksForFile(const FString& InstallationDirectory, const FString& FilenameInManifest, TSet<FGuid>& OutChunks) const = 0;

		// Call the lambda on each file the chunk needs to load bits from.
		virtual void EnumerateFilesForChunk(const FGuid& DataId, TUniqueFunction<void(const FString& NormalizedInstallDirectory, const FString& NormalizedFileName)>&& Callback) const = 0;

		// Tell the install source when files will be going away so we can report when chunks are unavailable.
		// This should be the ChunkReferenceTracker->GetCurrentUsageIndex at which the files will get deleted.
		virtual void SetFileRetirementPositions(TMap<FString, int32>&& FileRetirementPositions) = 0;

		// InstallationSources is the install manifest for each installed app that we can pull from. This
		// is expected to be a single entry.
		// LONGTERM question -- how does this work during delta generation? It is expected that all apps
		// are installed and chunks are pulled from everything?
		static IConstructorInstallChunkSource* CreateInstallSource(
			IFileSystem* FileSystem, 
			IInstallChunkSourceStat* InstallChunkSourceStat, 
			const TMultiMap<FString, FBuildPatchAppManifestRef>& InstallationSources,
			const TSet<FGuid>& ChunksThatWillBeNeeded, 
			bool bAllowLoadingAnyChunk,
			const TMap<FString, FString>& InPerFileSubdirectories);
	};

	/**
	 * This interface defines the statistics class required by the install chunk source. It should be implemented in order to collect
	 * desired information which is being broadcast by the system.
	 */
	class IInstallChunkSourceStat
	{
	public:
		/**
		 * Enum which describes success, or the reason for failure when loading a chunk.
		 */
		enum class ELoadResult : uint8
		{
			Success = 0,

			// The hash information was missing.
			MissingHashInfo,

			// Chunk part information was missing.
			MissingPartInfo,

			// Failed to open a source file.
			OpenFileFail,

			// The expected source file size was not matched.
			IncorrectFileSize_DEPRECATED,

			// The expected data hash for the chunk did not match.
			HashCheckFailed,

			// The process has been aborted.
			Aborted,

			// Invalid assemble structure (i.e. overlapping chunk parts)
			InvalidChunkParts
		};

	public:
		virtual ~IInstallChunkSourceStat() = default;

		/**
		 * Called when a batch of chunks are going to be loaded.
		 * @param ChunkIds  The ids of each chunk.
		 */
		UE_DEPRECATED(5.6, "No longer batch loaded")
		virtual void OnBatchStarted(const TArray<FGuid>& ChunkIds) {};

		/**
		 * Called each time a chunk load begins.
		 * @param ChunkId   The id of the chunk.
		 */
		virtual void OnLoadStarted(const FGuid& ChunkId) = 0;

		/**
		 * Called each time a chunk load completes.
		 * @param ChunkId   The id of the chunk.
		 * @param Result    The load result.
		 * @param Record    The recording for the received data.
		 */
		virtual void OnLoadComplete(const FGuid& ChunkId, const ELoadResult& Result, const ISpeedRecorder::FRecord& Record) = 0;

		/**
		 * Called when a batch of chunks are added and accepted via IChunkSource::AddRuntimeRequirements.
		 * @param ChunkIds  The ids of each chunk.
		 */
		UE_DEPRECATED(5.6, "No longer batch loaded")
		virtual void OnAcceptedNewRequirements(const TSet<FGuid>& ChunkIds) {};
	};
	
	/**
	 * A ToString implementation for IInstallChunkSourceStat::ELoadResult.
	 */
	const TCHAR* ToString(const IInstallChunkSourceStat::ELoadResult& LoadResult);
}