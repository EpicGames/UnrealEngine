// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/InstallerAnalytics.h"
#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Misc/Guid.h"
#include "Misc/ScopeLock.h"
#include "Stats/Stats.h"

#define ERROR_EVENT_SEND_LIMIT 20

namespace BuildPatchServices
{
	/**
	 * A simple struct to hold details required to record an analytics event.
	 */
	struct FAnalyticsEventInfo
	{
	public:
		FAnalyticsEventInfo(FString InEventName, TArray<FAnalyticsEventAttribute> InAttributes);

	public:
		// The analytics event name
		FString EventName;
		// The list of attributes
		TArray<FAnalyticsEventAttribute> Attributes;
	};

	FAnalyticsEventInfo::FAnalyticsEventInfo(FString InEventName, TArray<FAnalyticsEventAttribute> InAttributes)
		: EventName(MoveTemp(InEventName))
		, Attributes(MoveTemp(InAttributes))
	{
	}

	class FNullInstallerAnalytics
		: public IInstallerAnalytics
	{
	public:
		// IInstallerAnalytics interface begin.
		virtual void RecordChunkDownloadError(const FString& ChunkUrl, int32 ResponseCode, const FString& ErrorString) override {}
		virtual void RecordMissingDelta(const FString& DeltaUrl, const FString& BaseVersion, const FString& PatchVersion) override {}
		virtual void RecordChunkDownloadAborted(const FString& ChunkUrl, double ChunkTime, double ChunkMean, double ChunkStd, double BreakingPoint) override {}
		virtual void RecordChunkCacheError(const FGuid& ChunkGuid, const FString& Filename, int32 LastError, const FString& SystemName, const FString& ErrorString) override {}
		virtual void RecordConstructionError(const FString& Filename, int32 LastError, const FString& ErrorString) override {}
		virtual void RecordPrereqInstallationError(const FString& AppName, const FString& AppVersion, const FString& Filename, const FString& CommandLine, int32 ErrorCode, const FString& ErrorString) override {}
		virtual void TrackRequest(const FHttpRequestPtr& Request) override {}
		virtual void Flush() override {}
		// IInstallerAnalytics interface end.
	};

	class FInstallerAnalytics
		: public IInstallerAnalytics
	{
	public:
		FInstallerAnalytics(IAnalyticsProvider* InAnalyticsProvider);
		~FInstallerAnalytics();

		// IInstallerAnalytics interface begin.
		virtual void RecordChunkDownloadError(const FString& ChunkUrl, int32 ResponseCode, const FString& ErrorString) override;
		virtual void RecordMissingDelta(const FString& DeltaUrl, const FString& BaseVersion, const FString& PatchVersion) override;
		virtual void RecordChunkDownloadAborted(const FString& ChunkUrl, double ChunkTime, double ChunkMean, double ChunkStd, double BreakingPoint) override;
		virtual void RecordChunkCacheError(const FGuid& ChunkGuid, const FString& Filename, int32 LastError, const FString& SystemName, const FString& ErrorString) override;
		virtual void RecordConstructionError(const FString& Filename, int32 LastError, const FString& ErrorString) override;
		virtual void RecordPrereqInstallationError(const FString& AppName, const FString& AppVersion, const FString& Filename, const FString& CommandLine, int32 ErrorCode, const FString& ErrorString) override;
		virtual void TrackRequest(const FHttpRequestPtr& Request) override;
		virtual void Flush() override;
		// IInstallerAnalytics interface end.

	private:
		void QueueAnalyticsEvent(FString EventName, TArray<FAnalyticsEventAttribute> Attributes);
		bool Tick(float Delta);

	private:
		IAnalyticsProvider* Analytics;
		FThreadSafeCounter DownloadErrors;
		FThreadSafeCounter CacheErrors;
		FThreadSafeCounter ConstructionErrors;
		FCriticalSection AnalyticsEventQueueCS;
		TArray<FAnalyticsEventInfo> AnalyticsEventQueue;
		FTSTicker::FDelegateHandle TickerHandle;
	};

	FInstallerAnalytics::FInstallerAnalytics(IAnalyticsProvider* InAnalyticsProvider)
		: Analytics(InAnalyticsProvider)
		, DownloadErrors(0)
		, CacheErrors(0)
		, ConstructionErrors(0)
		, AnalyticsEventQueueCS()
		, AnalyticsEventQueue()
	{
		check(Analytics);
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FInstallerAnalytics::Tick));
	}

	FInstallerAnalytics::~FInstallerAnalytics()
	{
		// Remove ticker.
		FTSTicker::RemoveTicker(TickerHandle);
	}

	void FInstallerAnalytics::RecordChunkDownloadError(const FString& ChunkUrl, int32 ResponseCode, const FString& ErrorString)
	{
		if (DownloadErrors.Increment() <= ERROR_EVENT_SEND_LIMIT)
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("ChunkURL"), ChunkUrl));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("ResponseCode"), ResponseCode));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("ErrorString"), ErrorString));
			QueueAnalyticsEvent(TEXT("Patcher.Error.Download"), MoveTemp(Attributes));
		}
	}

	void FInstallerAnalytics::RecordMissingDelta(const FString& DeltaUrl, const FString& BaseVersion, const FString& PatchVersion)
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("DeltaUrl"), DeltaUrl));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("BaseVersion"), BaseVersion));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("PatchVersion"), PatchVersion));
		QueueAnalyticsEvent(TEXT("Patcher.Error.Delta"), MoveTemp(Attributes));
	}

	void FInstallerAnalytics::RecordChunkDownloadAborted(const FString& ChunkUrl, double ChunkTime, double ChunkMean, double ChunkStd, double BreakingPoint)
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ChunkURL"), ChunkUrl));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ChunkTime"), ChunkTime));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ChunkMean"), ChunkMean));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ChunkStd"), ChunkStd));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("BreakingPoint"), BreakingPoint));
		QueueAnalyticsEvent(TEXT("Patcher.Warning.ChunkAborted"), MoveTemp(Attributes));
	}

	void FInstallerAnalytics::RecordChunkCacheError(const FGuid& ChunkGuid, const FString& Filename, int32 LastError, const FString& SystemName, const FString& ErrorString)
	{
		if (CacheErrors.Increment() <= ERROR_EVENT_SEND_LIMIT)
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("ChunkGuid"), ChunkGuid.ToString()));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Filename"), Filename));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("LastError"), LastError));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("SystemName"), SystemName));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("ErrorString"), ErrorString));
			QueueAnalyticsEvent(TEXT("Patcher.Error.Cache"), MoveTemp(Attributes));
		}
	}

	void FInstallerAnalytics::RecordConstructionError(const FString& Filename, int32 LastError, const FString& ErrorString)
	{
		if (ConstructionErrors.Increment() <= ERROR_EVENT_SEND_LIMIT)
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Add(FAnalyticsEventAttribute(TEXT("Filename"), Filename));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("LastError"), LastError));
			Attributes.Add(FAnalyticsEventAttribute(TEXT("ErrorString"), ErrorString));
			QueueAnalyticsEvent(TEXT("Patcher.Error.Construction"), MoveTemp(Attributes));
		}
	}

	void FInstallerAnalytics::RecordPrereqInstallationError(const FString& AppName, const FString& AppVersion, const FString& Filename, const FString& CommandLine, int32 ErrorCode, const FString& ErrorString)
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AppName"), AppName));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AppVersion"), AppVersion));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Filename"), Filename));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("CommandLine"), CommandLine));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ReturnCode"), ErrorCode));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ErrorString"), ErrorString));
		QueueAnalyticsEvent(TEXT("Patcher.Error.Prerequisites"), MoveTemp(Attributes));
	}

	void FInstallerAnalytics::TrackRequest(const FHttpRequestPtr& Request)
	{
		//static const FName EndpointName = "CDN.Chunk";
	}

	void FInstallerAnalytics::Flush()
	{
		check(IsInGameThread());
		check(Analytics);

        QUICK_SCOPE_CYCLE_COUNTER(STAT_FInstallerAnalytics_Tick);

		// Process the Analytics Event queue
		FScopeLock ScopeLock(&AnalyticsEventQueueCS);
		for (FAnalyticsEventInfo& AnalyticsEvent : AnalyticsEventQueue)
		{
			Analytics->RecordEvent(AnalyticsEvent.EventName, AnalyticsEvent.Attributes);
		}
		AnalyticsEventQueue.Reset();
	}

	void FInstallerAnalytics::QueueAnalyticsEvent(FString EventName, TArray<FAnalyticsEventAttribute> Attributes)
	{
		FScopeLock ScopeLock(&AnalyticsEventQueueCS);
		AnalyticsEventQueue.Emplace(MoveTemp(EventName), MoveTemp(Attributes));
	}

	bool FInstallerAnalytics::Tick(float Delta)
	{
		Flush();
		return true;
	}

	IInstallerAnalytics* FInstallerAnalyticsFactory::Create(IAnalyticsProvider* AnalyticsProvider)
	{
		if (AnalyticsProvider)
		{
			return new FInstallerAnalytics(AnalyticsProvider);
		}

		return new FNullInstallerAnalytics();
	}
}
