// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "Interfaces/IHttpRequest.h"

struct FAnalyticsEventAttribute;
class IAnalyticsProvider;

namespace BuildPatchServices
{
	/**
	 * An interface to event recording implementation.
	 */
	class IInstallerAnalytics
	{
	public:
		virtual ~IInstallerAnalytics() = default;

		/**
		 * @EventName Patcher.Error.Download
		 *
		 * @Trigger Fires for the first 20 chunk download errors that occur. Deprecated: Not recommended for use.
		 *
		 * @Type Client
		 *
		 * @EventParam ChunkURL     (string) The url used to request the chunk.
		 * @EventParam ResponseCode (int32)  The HTTP response code received, or OS error code for file operation.
		 * @EventParam ErrorString  (string) The stage at which the chunk was deemed error. "Download Fail", "Uncompress Fail", "Verify Fail", or "SaveToDisk Fail".
		 *
		 * @Owner Portal Team
		 */
		virtual void RecordChunkDownloadError(const FString& ChunkUrl, int32 ResponseCode, const FString& ErrorString) = 0;

		/**
		 * @EventName Patcher.Error.Delta
		 *
		 * @Trigger Fires for delta files that return 403/404 http response codes
		 *
		 * @Type Client
		 *
		 * @EventParam DeltaUrl		(string) The url used to request the delta file
		 * @EventParam BaseVersion	(string) The base (installed) version of the app
		 * @EventParam PatchVersion	(string) The version of the app that is being upgraded to
		 * @Owner Portal Team
		 */
		virtual void RecordMissingDelta(const FString& DeltaUrl, const FString& BaseVersion, const FString& PatchVersion) = 0;

		/**
		 * @EventName Patcher.Warning.ChunkAborted
		 *
		 * @Trigger Fires for chunks that were aborted as deemed failing. This is a prediction that the chunk is suffering from 0-byte TCP window issue. Deprecated: Not recommended for use.
		 *
		 * @Type Client
		 *
		 * @EventParam ChunkURL      (string) The url used to request the chunk.
		 * @EventParam ChunkTime     (double) The amount of time the chunk has been downloading for in seconds.
		 * @EventParam ChunkMean     (double) The current mean average chunk download time in seconds.
		 * @EventParam ChunkStd      (double) The current standard deviation of chunk download times.
		 * @EventParam BreakingPoint (double) The time at which a chunk will be aborted based on the current mean and std.
		 *
		 * @Owner Portal Team
		 */
		virtual void RecordChunkDownloadAborted(const FString& ChunkUrl, double ChunkTime, double ChunkMean, double ChunkStd, double BreakingPoint) = 0;

		/**
		 * @EventName Patcher.Error.Cache
		 *
		 * @Trigger Fires for the first 20 chunks which experienced a cache error. Deprecated: Not recommended for use.
		 *
		 * @Type Client
		 *
		 * @EventParam ChunkGuid   (string) The guid of the effected chunk.
		 * @EventParam Filename    (string) The filename used if appropriate, or -1.
		 * @EventParam LastError   (int32)  The OS error code if appropriate.
		 * @EventParam SystemName  (string) The cache sub-system that is reporting this. "ChunkBooting", "ChunkRecycle", "DriveCache".
		 * @EventParam ErrorString (string) The stage at which the chunk was deemed error. "Hash Check Failed", "Incorrect File Size", "Datasize/Hashtype Mismatch", "Corrupt Header", "Open File Fail", "Source File Missing", "Source File Too Small", "Chunk Hash Fail", or "Chunk Save Failed".
		 *
		 * @Owner Portal Team
		 */
		virtual void RecordChunkCacheError(const FGuid& ChunkGuid, const FString& Filename, int32 LastError, const FString& SystemName, const FString& ErrorString) = 0;

		/**
		 * @EventName Patcher.Error.Construction
		 *
		 * @Trigger Fires for the first 20 errors experienced while constructing an installation file. Deprecated: Not recommended for use.
		 *
		 * @Type Client
		 *
		 * @EventParam Filename    (string) The filename of the installation file.
		 * @EventParam LastError   (int32)  The OS error code if appropriate, or -1.
		 * @EventParam ErrorString (string) The reason the error occurred. "Missing Chunk", "Not Enough Disk Space", "Could Not Create File", "Missing File Manifest", "Serialised Verify Fail", "File Data Missing", "File Data Uncompress Fail", "File Data Verify Fail", "File Data GUID Mismatch", "File Data Part OOB", "File Data Unknown Storage", or "Failed To Move".
		 *
		 * @Owner Portal Team
		 */
		virtual void RecordConstructionError(const FString& Filename, int32 LastError, const FString& ErrorString) = 0;

		/**
		 * @EventName Patcher.Error.Prerequisites
		 *
		 * @Trigger Fires for errors experienced while installing prerequisites. Deprecated: Not recommended for use.
		 *
		 * @Type Client
		 *
		 * @EventParam AppName     (string) The app name for the installation that is running.
		 * @EventParam AppVersion  (string) The app version for the installation that is running.
		 * @EventParam Filename    (string) The filename used when running prerequisites.
		 * @EventParam CommandLine (string) The command line used when running prerequisites.
		 * @EventParam ReturnCode  (int32)  The error or return code from running the prerequisite.
		 * @EventParam ErrorString (string) The action that failed. "Failed to start installer", or "Failed to install".
		 *
		 * @Owner Portal Team
		 */
		virtual void RecordPrereqInstallationError(const FString& AppName, const FString& AppVersion, const FString& Filename, const FString& CommandLine, int32 ErrorCode, const FString& ErrorString) = 0;

		/**
		 * @EventName CDN.Chunk
		 *
		 * @Trigger Fires aggregate events for HTTP requests made for BuildPatchServices installers.
		 *
		 * @Type Dynamic
		 *
		 * @EventParam Code-XXX                  (int32)  The count for this HTTP response code, where XXX in the event name is the code.
		 * @EventParam DomainName                (string) The domain name used for the requests in this aggregate.
		 * @EventParam SuccessCount              (int32)  The count of request successes.
		 * @EventParam FailCount                 (int32)  The count of request failures.
		 * @EventParam DownloadBytesSuccessTotal (int64)  The number of bytes received by successful responses.
		 * @EventParam DownloadBytesFailTotal    (int64)  The number of bytes received by failed responses.
		 * @EventParam ElapsedTimeSuccessTotal   (float)  The total seconds elapsed for successful requests in this aggregate.
		 * @EventParam ElapsedTimeSuccessMin     (float)  The minimum seconds elapsed for successful request in this aggregate.
		 * @EventParam ElapsedTimeSuccessMax     (float)  The maximum seconds elapsed for successful request in this aggregate.
		 * @EventParam ElapsedTimeFailTotal      (float)  The total seconds elapsed for failed requests in this aggregate.
		 * @EventParam ElapsedTimeFailMin        (float)  The minimum seconds elapsed for failed request in this aggregate.
		 * @EventParam ElapsedTimeFailMax        (float)  The maximum seconds elapsed for failed request in this aggregate.
		 *
		 * @Owner Portal Team
		 */
		virtual void TrackRequest(const FHttpRequestPtr& Request) = 0;

		/**
		 * Ensures that any queued analytics are sent immediately.
		 */
		virtual void Flush() = 0;
	};

	/**
	 * A factory for creating an IInstallerAnalytics instance.
	 */
	class FInstallerAnalyticsFactory
	{
	public:
		/**
		 * Creates an instance of an installer analytics handler, for use by other classes which report the events.
		 * @param AnalyticsProvider The analytics provider interface.
		 * @return the new IInstallerAnalytics instance created.
		 */
		static IInstallerAnalytics* Create(IAnalyticsProvider* AnalyticsProvider);
	};
}
