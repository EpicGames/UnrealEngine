// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/InstallerError.h"
#include "Misc/ScopeLock.h"
#include "HAL/ThreadSafeCounter.h"
#include "Interfaces/IBuildInstaller.h"
#include "BuildPatchServicesPrivate.h"

namespace BuildPatchServices
{
	FText GetStandardErrorText(EBuildPatchInstallError ErrorType)
	{
		switch (ErrorType)
		{
			case EBuildPatchInstallError::NoError: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_NoError", "The operation was successful.");
			case EBuildPatchInstallError::DownloadError: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_DownloadError", "Could not download patch data. Please check your internet connection, and try again.");
			case EBuildPatchInstallError::FileConstructionFail: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_FileConstructionFail", "A file corruption has occurred. Please try again.");
			case EBuildPatchInstallError::MoveFileToInstall: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_MoveFileToInstall", "A file access error has occurred. Please check your running processes.");
			case EBuildPatchInstallError::BuildVerifyFail: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_BuildCorrupt", "The installation is corrupt. Please contact support.");
			case EBuildPatchInstallError::ApplicationClosing: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_ApplicationClosing", "The application is closing.");
			case EBuildPatchInstallError::ApplicationError: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_ApplicationError", "Patching service could not start. Please contact support.");
			case EBuildPatchInstallError::UserCanceled: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_UserCanceled", "User cancelled.");
			case EBuildPatchInstallError::PrerequisiteError: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_PrerequisiteError", "The necessary prerequisites have failed to install. Please contact support.");
			case EBuildPatchInstallError::CustomUninstallActionError: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_UninstallCustomActionError", "The application's uninstall action has failed to execute. Please contact support.");
			case EBuildPatchInstallError::InitializationError: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_InitializationError", "The installer failed to initialize. Please contact support.");
			case EBuildPatchInstallError::PathLengthExceeded: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_PathLengthExceeded", "Maximum path length exceeded. Please specify a shorter install location.");
			case EBuildPatchInstallError::OutOfDiskSpace: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_OutOfDiskSpace", "Not enough disk space available. Please free up some disk space and try again.");
			default: 
				return NSLOCTEXT("BuildPatchInstallError", "BuildPatchInstallShortError_InvalidOrMax", "An unknown error ocurred. Please contact support.");
		}
	}

	FText GetDiskSpaceMessage(const FString& Location, uint64 RequiredBytes, uint64 AvailableBytes, const FNumberFormattingOptions* FormatOptions)
	{
#if PLATFORM_DESKTOP
        const FText OutOfDiskSpace(NSLOCTEXT("BuildPatchInstallError", "InstallDirectoryDiskSpace", "There is not enough space at {Location}\n{RequiredBytes} is required.\n{AvailableBytes} is available.\nYou need an additional {SpaceAdditional} to perform the installation."));
#else
        const FText OutOfDiskSpace(NSLOCTEXT("BuildPatchInstallError", "InstallDirectoryDiskSpaceDevice", "There is not enough space on your device.\n{RequiredBytes} is required.\n{AvailableBytes} is available.\nYou need an additional {SpaceAdditional} to perform the installation."));
#endif // PLATFORM_DESKTOP
        const FNumberFormattingOptions DefaultOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(2)
			.SetMaximumFractionalDigits(2);
		FormatOptions = FormatOptions == nullptr ? &DefaultOptions : FormatOptions;
		FFormatNamedArguments Arguments;
		Arguments.Emplace(TEXT("Location"), FText::FromString(Location));
		Arguments.Emplace(TEXT("RequiredBytes"), FText::AsMemory(RequiredBytes, FormatOptions, nullptr, EMemoryUnitStandard::SI));
		Arguments.Emplace(TEXT("AvailableBytes"), FText::AsMemory(AvailableBytes, FormatOptions, nullptr, EMemoryUnitStandard::SI));
		Arguments.Emplace(TEXT("SpaceAdditional"), FText::AsMemory(RequiredBytes - AvailableBytes, FormatOptions, nullptr, EMemoryUnitStandard::SI));
		return FText::Format(OutOfDiskSpace, Arguments);
	}

	class FInstallerError
		: public IInstallerError
	{
	public:
		FInstallerError();
		~FInstallerError();

		// IInstallerError interface begin.
		virtual bool HasError() const override;
		virtual bool IsCancelled() const override;
		virtual bool CanRetry() const override;
		virtual EBuildPatchInstallError GetErrorType() const override;
		virtual FString GetErrorCode() const override;
		virtual FText GetErrorText() const override;
		virtual void SetError(EBuildPatchInstallError InErrorType, const TCHAR* InErrorSubType, uint32 InErrorCode, FText InErrorText) override;
		virtual int32 RegisterForErrors(FOnErrorDelegate Delegate) override;
		virtual void UnregisterForErrors(int32 Handle) override;
		virtual void ResetError(bool bPreserveCancelled) override;
		// IInstallerError interface end.

	private:
		mutable FCriticalSection ThreadLockCs;
		EBuildPatchInstallError ErrorType;
		FString ErrorCode;
		FText ErrorText;
		FThreadSafeCounter DelegateCounter;
		TMap<int32, FOnErrorDelegate> RegisteredDelegates;
	};

	FInstallerError::FInstallerError()
		: ThreadLockCs()
		, ErrorType(EBuildPatchInstallError::NoError)
		, ErrorCode(InstallErrorPrefixes::ErrorTypeStrings[static_cast<int32>(ErrorType)])
		, ErrorText(GetStandardErrorText(ErrorType))
		, DelegateCounter(0)
		, RegisteredDelegates()
	{
	}

	FInstallerError::~FInstallerError()
	{
	}

	bool FInstallerError::HasError() const
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		return ErrorType != EBuildPatchInstallError::NoError;
	}

	bool FInstallerError::IsCancelled() const
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		return ErrorType == EBuildPatchInstallError::UserCanceled
			|| ErrorType == EBuildPatchInstallError::ApplicationClosing;
	}

	bool FInstallerError::CanRetry() const
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		return ErrorType == EBuildPatchInstallError::FileConstructionFail
			|| ErrorType == EBuildPatchInstallError::BuildVerifyFail;
	}

	EBuildPatchInstallError FInstallerError::GetErrorType() const
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		return ErrorType;
	}

	FString FInstallerError::GetErrorCode() const
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		return ErrorCode;
	}

	FText FInstallerError::GetErrorText() const
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		return ErrorText;
	}

	void FInstallerError::SetError(EBuildPatchInstallError InErrorType, const TCHAR* InErrorSubType, uint32 InErrorCode, FText InErrorText)
	{
		// Only accept the first error
		TArray<FOnErrorDelegate> DelegatesToCall;
		ThreadLockCs.Lock();
		if (!HasError())
		{
			ErrorType = InErrorType;
			ErrorCode = InstallErrorPrefixes::ErrorTypeStrings[static_cast<int32>(InErrorType)];
			ErrorCode += InErrorSubType;
			ErrorCode += (InErrorCode > 0) ? FString::Printf(TEXT("-%u"), InErrorCode): TEXT("");
			ErrorText = InErrorText.IsEmpty() ? GetStandardErrorText(ErrorType) : MoveTemp(InErrorText);
			RegisteredDelegates.GenerateValueArray(DelegatesToCall);
			if (IsCancelled())
			{
				UE_LOGF(LogBuildPatchServices, Log, "EBuildPatchInstallError::%ls %ls", LexToString(ErrorType), *ErrorCode);
			}
			else
			{
				UE_LOGF(LogBuildPatchServices, Error, "EBuildPatchInstallError::%ls %ls", LexToString(ErrorType), *ErrorCode);
			}
		}
		ThreadLockCs.Unlock();
		for (const FOnErrorDelegate& DelegateToCall : DelegatesToCall)
		{
			DelegateToCall();
		}
	}

	int32 FInstallerError::RegisterForErrors(FOnErrorDelegate Delegate)
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		int32 Handle = DelegateCounter.Increment();
		RegisteredDelegates.Add(Handle, Delegate);
		return Handle;
	}

	void FInstallerError::UnregisterForErrors(int32 Handle)
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		RegisteredDelegates.Remove(Handle);
	}

	void FInstallerError::ResetError(bool bPreserveCancelled)
	{
		FScopeLock ScopeLock(&ThreadLockCs);
		if (bPreserveCancelled && IsCancelled())
		{
			return;
		}
		ErrorType = EBuildPatchInstallError::NoError;
		ErrorCode = InstallErrorPrefixes::ErrorTypeStrings[static_cast<int32>(ErrorType)];
		ErrorText = GetStandardErrorText(ErrorType);
	}

	IInstallerError* FInstallerErrorFactory::Create()
	{
		return new FInstallerError();
	}
}
