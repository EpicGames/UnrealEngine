// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

enum class EBuildPatchInstallError;

namespace BuildPatchServices
{
	/**
	 * A list of the error codes we will use for each case of initialization failure.
	 */
	namespace InitializationErrorCodes
	{
		static const TCHAR* MissingStageDirectory = TEXT("01");
		static const TCHAR* MissingInstallDirectory = TEXT("02");
		static const TCHAR* MissingCompleteDelegate = TEXT("03");
		static const TCHAR* InvalidInstallTags = TEXT("04");
		static const TCHAR* ChunkReferenceTracking = TEXT("05");
		static const TCHAR* MissingPrereqForPrereqOnlyInstall = TEXT("06");
		static const TCHAR* InvalidDataInManifest = TEXT("07");
		static const TCHAR* EmptyCloudDirectories = TEXT("08");
		static const TCHAR* VerifyOnlyActionMixedWithOtherActions = TEXT("09");
		static const TCHAR* MissingEncryptionKeyForNonPreload = TEXT("10");
		static const TCHAR* InvalidEncryptionKeyForNonPreload = TEXT("11");
		static const TCHAR* InvalidInstallToMemory = TEXT("11");
		static const TCHAR* InvalidMemoryConstructionConfig = TEXT("12");
	}

	/**
	 * A list of the error codes we will use for each case of running out of disk space.
	 */
	namespace DiskSpaceErrorCodes
	{
		static const TCHAR* InitialSpaceCheck = TEXT("01");
		static const TCHAR* DuringInstallation = TEXT("02");
	}

	/**
	 * A list of the error codes we will use for each case of exceeding path length.
	 */
	namespace PathLengthErrorCodes
	{
		static const TCHAR* StagingDirectory = TEXT("01");
		static const TCHAR* InstallDirectory = TEXT("02");
	}

	/**
	 * A list of the error codes we will use for each case of download failure.
	 */
	namespace DownloadErrorCodes
	{
		static const TCHAR* OutOfChunkRetries = TEXT("01");
		static const TCHAR* MissingDeltaFile = TEXT("02");
		static const TCHAR* UnserialisableDeltaFile = TEXT("03");
		static const TCHAR* RejectedDeltaFile = TEXT("04");
		static const TCHAR* FailedUriRequest = TEXT("05");
	}

	/**
	 * A list of the error codes we will use for each case of file construction failure.
	 */
	namespace ConstructionErrorCodes
	{
		static const TCHAR* UnknownFail = TEXT("01");
		static const TCHAR* FileCreateFail = TEXT("02");
		static const TCHAR* MissingChunkData = TEXT("03");
		static const TCHAR* MissingFileInfo = TEXT("04");
		static const TCHAR* OutboundCorrupt = TEXT("05");
		static const TCHAR* SerializationError = TEXT("06");
		static const TCHAR* TrackingError = TEXT("07");
		static const TCHAR* InternalConsistencyFailure = TEXT("08");
		static const TCHAR* SymlinkCreateFail = TEXT("09");
		static const TCHAR* SymlinkCreateForbidden = TEXT("10");
		static const TCHAR* CloseFileError = TEXT("11");
	}

	/**
	 * A list of the error codes we will use for each case of moving files.
	 */
	namespace MoveErrorCodes
	{
		static const TCHAR* StageToInstall = TEXT("01");
		static const TCHAR* DeletePrevFileFailed = TEXT("02");
		static const TCHAR* DeleteOldFileFailed = TEXT("03");
	}

	/**
	 * A list of the error codes we will use for each case of verification failure.
	 */
	namespace VerifyErrorCodes
	{
		static const TCHAR* UnknownFail = TEXT("01");
		static const TCHAR* FileMissing = TEXT("02");
		static const TCHAR* OpenFileFailed = TEXT("03");
		static const TCHAR* HashCheckFailed = TEXT("04");
		static const TCHAR* FileSizeFailed = TEXT("05");
		static const TCHAR* DeleteCorruptedFileFailed = TEXT("06");
	}

	/**
	 * A list of the error codes we will use for each case of cancellation.
	 */
	namespace UserCancelErrorCodes
	{
		static const TCHAR* UserRequested = TEXT("01");
	}

	/**
	 * A list of the error codes we will use for each case of Application Closing.
	 */
	namespace ApplicationClosedErrorCodes
	{
		static const TCHAR* ApplicationClosed = TEXT("01");
	}

	/**
	 * A list of the error codes we will use for each case of the uninstall action failure.
	 */
	namespace UninstallCustomActionErrorPrefixes
	{
		static const TCHAR* ExecuteCode = TEXT("E");
		static const TCHAR* ReturnCode = TEXT("R");
	}

	/**
	 * A list of the error codes we will use for each case of the prerequisites install failure.
	 */
	namespace PrerequisiteErrorPrefixes
	{
		static const TCHAR* ExecuteCode = TEXT("E");
		static const TCHAR* ReturnCode = TEXT("R");
		static const TCHAR* NotFoundCode = TEXT("01");
	}

	/**
	 * Get the standard error message for an error type.
	 * @param ErrorType     The enum value for the error.
	 * @return the displayable text for the error type.
	 */
	FText GetStandardErrorText(EBuildPatchInstallError ErrorType);

	/**
	 * Get the standard error message for for a disk space issue.
	 * @param Location          The disk path requiring more space.
	 * @param RequiredBytes     The number of bytes required for the installation.
	 * @param AvailableBytes    The number of bytes available on the drive.
	 * @param FormatOptions     Optionally override the default number formatting.
	 * @return the displayable text for the out of space error.
	 */
	FText GetDiskSpaceMessage(const FString& Location, uint64 RequiredBytes, uint64 AvailableBytes, const FNumberFormattingOptions* FormatOptions = nullptr);

	/**
	 * An interface to fatal error implementation used to report an error or get informed of other errors occurring.
	 */
	class IInstallerError
	{
	public:
		typedef TFunction<void()> FOnErrorDelegate;

	public:
		virtual ~IInstallerError() = default;

		/**
		 * Get if there has been a fatal error reported.
		 * @return true if a fatal error has been set.
		 */
		virtual bool HasError() const = 0;

		/**
		 * Get whether an error reported is a cancellation request.
		 * @return true if the error which is set is a cancellation.
		 */
		virtual bool IsCancelled() const = 0;

		/**
		 * Get whether the reported error is one which should be capable of recovering with an installation retry.
		 * @return true if the installation can retry.
		 */
		virtual bool CanRetry() const = 0;

		/**
		 * Get the enum value for the error which has been reported.
		 * @return the reported enum value.
		 */
		virtual EBuildPatchInstallError GetErrorType() const = 0;

		/**
		 * Get the error code string for the error which has been reported.
		 * @return the reported error code string.
		 */
		virtual FString GetErrorCode() const = 0;

		/**
		 * Get the default display text for the error which has been reported.
		 * @return the reported error's default display text.
		 */
		virtual FText GetErrorText() const = 0;

		/**
		 * Report a fatal error that has occurred which should cause other systems to abort.
		 * @param ErrorType     The enum value for the error.
		 * @param ErrorSubType  The error sub type string for the error.
		 * @param ErrorCode     The error code for the error. 0 means no additional information.
		 * @param ErrorText     Optional specific error display text to use instead of the standard generic one.
		 */
		virtual void SetError(EBuildPatchInstallError ErrorType, const TCHAR* ErrorSubType, uint32 ErrorCode = 0, FText ErrorText = FText::GetEmpty()) = 0;

		/**
		 * Resets the current error state without unregistering delegates added by RegisterForErrors.
		 * @param bPreserveCancelled	If true the canceled error state is preserved instead of being reset.
		 */
		virtual void ResetError(bool bPreserveCancelled) = 0;

		/**
		 * Register a delegate to be called upon an error occurring.
		 * @param Delegate      The delegate to be executed on error.
		 * @return a handle for this registration, used to unregister.
		 */
		virtual int32 RegisterForErrors(FOnErrorDelegate Delegate) = 0;

		/**
		 * Unregister a delegate from being called upon an error occurring.
		 * @param Handle        The value that was returned RegisterForErrors for the delegate to be removed.
		 */
		virtual void UnregisterForErrors(int32 Handle) = 0;
	};

	/**
	 * A factory for creating an IInstallerError instance.
	 */
	class FInstallerErrorFactory
	{
	public:

		/**
		 * Creates an instance of an error class which should be shared between all systems created for the same installation.
		 * @return the new IInstallerError instance created.
		 */
		static IInstallerError* Create();
	};
}
