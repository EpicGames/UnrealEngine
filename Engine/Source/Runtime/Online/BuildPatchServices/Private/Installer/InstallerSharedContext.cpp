// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/InstallerSharedContext.h"

#include "BuildPatchServicesPrivate.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Fork.h"

namespace BuildPatchServices
{
	ENUM_CLASS_FLAGS(FBuildInstallerThread::EMsg);

	FBuildInstallerThread::~FBuildInstallerThread()
	{
		if (Thread != nullptr)
		{
			delete Thread; // Thread destructor should wait for completion
			Thread = nullptr;
		}

		FPlatformProcess::ReturnSynchEventToPool(DoWorkEvent);
		DoWorkEvent = nullptr;
	}

	bool FBuildInstallerThread::StartThread(const TCHAR* DebugName)
	{
		// We most likely *never* want support forking while an installer is running.
		DoWorkEvent = FPlatformProcess::GetSynchEventFromPool();

		Thread = FForkProcessHelper::CreateForkableThread(this, DebugName, 0, TPri_Normal, FGenericPlatformAffinity::GetNoAffinityMask(), EThreadCreateFlags::None, true);
		check(Thread != nullptr);

		return DoWorkEvent && Thread;
	}

	void FBuildInstallerThread::RunTask(TUniqueFunction<void()> InTask)
	{
		check(InTask);

		MsgQueue.Enqueue(MoveTemp(InTask), EMsg::RunTask);

		DoWorkEvent->Trigger();
	}

	uint32 FBuildInstallerThread::Run()
	{
		bool bExit = false;
		while (!bExit)
		{
			// Wait for some work to do
			DoWorkEvent->Wait();

			FMsg Msg;
			while (MsgQueue.Dequeue(Msg))
			{
				if (EnumHasAnyFlags(Msg.Msg, EMsg::RunTask))
				{
					check(!bExit);
					Msg.Task();
				}

				if (EnumHasAnyFlags(Msg.Msg, EMsg::Exit))
				{
					bExit = true;
				}
			}
		}

		return 0;
	}

	void FBuildInstallerThread::Stop()
	{
		MsgQueue.Enqueue(nullptr, EMsg::Exit);

		DoWorkEvent->Trigger();
	}

	IBuildInstallerThread* FBuildInstallerSharedContext::CreateThreadInternal()
	{
		TStringBuilder<512> ThreadNameBuilder;
		ThreadNameBuilder.Appendf(TEXT("%s #%d"), DebugName, ThreadCount);

		UE_LOGF(LogBuildPatchServices, Display, "Creating thread %ls #%d", DebugName, ThreadCount);

		FBuildInstallerThread* Thread = new FBuildInstallerThread();
		if (Thread->StartThread(ThreadNameBuilder.ToString()))
		{
			++ThreadCount;
		}
		else
		{
			delete Thread;
			Thread = nullptr;
		}

		return Thread;
	}

	void FBuildInstallerSharedContext::PreallocateThreads(uint32 NumThreads)
	{
		if (NumThreads == 0)
		{
			return;
		}

		FScopeLock Lock(&ThreadFreeListCS);
		ThreadFreeList.Reserve(NumThreads);
		for (uint32 i = 0; i < NumThreads; ++i)
		{
			if (IBuildInstallerThread* Thread = CreateThreadInternal())
			{
				ThreadFreeList.Push(Thread);
			}
		}

		bWarnOnCreateThread = true;
	}

	uint32 FBuildInstallerSharedContext::NumThreadsPerInstaller(bool bUseChunkDBs, bool bHasInstallSource) const
	{
		const uint32 NumInstallerMainThreads = 1;
		const uint32 NumWriteThreads = 1;

		const uint32 NumChunkDBThreads = bUseChunkDBs ? 1 : 0;
		const uint32 NumInstallSourceThreads = bHasInstallSource ? 1 : 0;

		const uint32 NumExpectedThreads =
			NumInstallerMainThreads + 
			NumWriteThreads +
			NumInstallSourceThreads + 
			NumChunkDBThreads;

		return NumExpectedThreads;
	}

	FBuildInstallerSharedContext::~FBuildInstallerSharedContext()
	{
		FScopeLock Lock(&ThreadFreeListCS);

		// All threads should have been freed before deleting the context
		if (ThreadFreeList.Num() != ThreadCount)
		{
			UE_LOGF(LogBuildPatchServices, Error, "Threads still allocated: Expected %d, Actual %u", ThreadFreeList.Num(), ThreadCount);

			checkf(false, TEXT("Threads still allocated: Expected %d, Actual %u"), ThreadFreeList.Num(), ThreadCount);
		}

		for (IBuildInstallerThread* Thread : ThreadFreeList)
		{
			FBuildInstallerThread* ThreadActual = static_cast<FBuildInstallerThread*>(Thread);
			ThreadActual->Stop();
		}

		for (IBuildInstallerThread* Thread : ThreadFreeList)
		{
			FBuildInstallerThread* ThreadActual = static_cast<FBuildInstallerThread*>(Thread);
			delete ThreadActual;
		}

		ThreadFreeList.Empty();
	}

	IBuildInstallerThread* FBuildInstallerSharedContext::CreateThread()
	{
		IBuildInstallerThread* Thread = nullptr;
		{
			FScopeLock Lock(&ThreadFreeListCS);
			Thread = ThreadFreeList.IsEmpty() ? nullptr : ThreadFreeList.Pop(EAllowShrinking::No);
		}

		if (!Thread)
		{
			UE_CLOGF(bWarnOnCreateThread, LogBuildPatchServices, Warning, "Allocating installer thread, free list exhausted, check PreallocateResources()");
			FScopeLock Lock(&ThreadFreeListCS);
			Thread = CreateThreadInternal();
		}

		return Thread;
	}

	void FBuildInstallerSharedContext::ReleaseThread(IBuildInstallerThread* Thread)
	{
		if (Thread)
		{
			FScopeLock Lock(&ThreadFreeListCS);
			check(!ThreadFreeList.Contains(Thread)); // Double free is bad y'all
			ThreadFreeList.Push(Thread);
		}
	}

	IBuildInstallerSharedContextRef FBuildInstallerSharedContextFactory::Create(const TCHAR* DebugName)
	{
		return MakeShared<FBuildInstallerSharedContext>(DebugName);
	}
}
