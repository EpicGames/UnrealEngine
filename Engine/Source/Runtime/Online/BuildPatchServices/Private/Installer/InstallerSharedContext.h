// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "Containers/MpscQueue.h"
#include "Interfaces/IBuildInstallerSharedContext.h"

namespace BuildPatchServices
{
	/**
	 * The concrete implementation of IBuildInstallerThread
	 */
	class FBuildInstallerThread
		: public IBuildInstallerThread
		, public FRunnable
	{
		enum class EMsg : uint8
		{
			None = 0,
			RunTask = (1 << 0),
			Exit = (1 << 1)
		};
		FRIEND_ENUM_CLASS_FLAGS(EMsg);

		struct FMsg
		{
			FMsg() = default;
			FMsg(TUniqueFunction<void()> InTask, EMsg InMsg)
				: Task(MoveTemp(InTask))
				, Msg(InMsg)
			{}

			TUniqueFunction<void()> Task;
			EMsg Msg = EMsg::None;
		};

	public:
		~FBuildInstallerThread();

		bool StartThread(const TCHAR* DebugName);

		// IBuildInstallerThread interface begin
		virtual void RunTask(TUniqueFunction<void()> Task) override;
		// IBuildInstallerThread interface end

		// FRunnable interface begin
		virtual uint32 Run() override;
		virtual void Stop() override;
		// FRunnable interface end

	private:
		FEvent* DoWorkEvent = nullptr;
		FRunnableThread* Thread = nullptr;
		TMpscQueue<FMsg> MsgQueue;
	};

	/**
	 * The concrete implementation of IBuildInstallerSharedContext
	 */
	class FBuildInstallerSharedContext : public IBuildInstallerSharedContext
	{
	public:
		FBuildInstallerSharedContext(const TCHAR* InDebugName) : DebugName(InDebugName) {}
		~FBuildInstallerSharedContext();

		// IBuildInstallerSharedContext interface begin
		virtual IBuildInstallerThread* CreateThread() override;
		virtual void ReleaseThread(IBuildInstallerThread* Thread) override;
		virtual void PreallocateThreads(uint32 NumThreads) override;
		virtual uint32 NumThreadsPerInstaller(bool bUseChunkDBs, bool bHasInstallSource) const override;
		// IBuildInstallerSharedContext interface end

	private:
		IBuildInstallerThread* CreateThreadInternal();

	private:
		TArray<IBuildInstallerThread*> ThreadFreeList;
		FCriticalSection ThreadFreeListCS;
		const TCHAR* DebugName;
		uint32 ThreadCount = 0;
		bool bWarnOnCreateThread = false;
	};

	/**
	 * A factory for creating an IBuildInstallerSharedContext instance.
	 */
	class FBuildInstallerSharedContextFactory
	{
	public:
		/**
		 * @param DebugName          Used to tag resources allocated with the shared context.
		 * @return an instance of an IBuildInstallerSharedContex implementation.
		 */
		static IBuildInstallerSharedContextRef Create(const TCHAR* DebugName);
	};
}
