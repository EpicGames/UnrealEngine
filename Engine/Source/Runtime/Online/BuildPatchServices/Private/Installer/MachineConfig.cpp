// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/MachineConfig.h"
#include "Misc/ConfigCacheIni.h"
#include "Core/AsyncHelpers.h"

namespace MachineConfigHelpers
{
	TSet<FString> LoadInstalledPrereqIds(const FString& LocalMachineConfigFile)
	{
		check(IsInGameThread());
		TArray<FString> ConfigStrings;
		GConfig->GetArray(TEXT("Portal.BuildPatch"), TEXT("InstalledPrereqs"), ConfigStrings, LocalMachineConfigFile);
		return TSet<FString>(ConfigStrings);
	};

	void SaveInstalledPrereqIds(const FString& LocalMachineConfigFile, const TSet<FString>& InstalledPrereqIds, bool bFlush)
	{
		check(IsInGameThread());
		GConfig->SetArray(TEXT("Portal.BuildPatch"), TEXT("InstalledPrereqs"), InstalledPrereqIds.Array(), LocalMachineConfigFile);
		if (bFlush)
		{
			GConfig->Flush(false, LocalMachineConfigFile);
		}
	}
}

namespace BuildPatchServices
{
	class FMachineConfig
		: public IMachineConfig
	{
	public:
		/**
		 * Constructor
		 * @param InLocalMachineConfigFile      The name of the configuration file which will hold the per machine config data.
		 * @param bAlwaysFlushChanges           If true, then we'll always flush changes straight to disk after saving values.
		 */
		FMachineConfig(const FString& InLocalMachineConfigFile, bool bAlwaysFlushChanges);

		//~ Begin IMachineConfig Interface
		virtual TSet<FString> LoadInstalledPrereqIds() override;
		virtual void SaveInstalledPrereqIds(const TSet<FString>& InstalledPrereqIds) override;
		//~ End IMachineConfig Interface
	private:
		// The filename for the local machine config. This is used for per-machine values rather than per-user or shipped config.
		FString LocalMachineConfigFile;

		// If true, then we'll always flush changes straight to disk after saving values.
		bool bAlwaysFlushChanges;
	};

	FMachineConfig::FMachineConfig(const FString& InLocalMachineConfigFile, bool bInAlwaysFlushChanges)
		: LocalMachineConfigFile(InLocalMachineConfigFile)
		, bAlwaysFlushChanges(bInAlwaysFlushChanges)
	{
		// Explicitly add the config file to the cache so that it's not "found" later.
		// "Found" files are loaded in read-only mode ("NoSave" flag is "true"), but we may need to write new prereq IDs to this file.
		GConfig->Add(LocalMachineConfigFile, FConfigFile());
	}

	TSet<FString> FMachineConfig::LoadInstalledPrereqIds()
	{
		return AsyncHelpers::ExecuteOnGameThread<TSet<FString>>([this]()
		{
			return MachineConfigHelpers::LoadInstalledPrereqIds(LocalMachineConfigFile);
		}).Get();
	}

	void FMachineConfig::SaveInstalledPrereqIds(const TSet<FString>& InstalledPrereqIds)
	{
		AsyncHelpers::ExecuteOnGameThread<void>([this, &InstalledPrereqIds]()
		{
			MachineConfigHelpers::SaveInstalledPrereqIds(LocalMachineConfigFile, InstalledPrereqIds, bAlwaysFlushChanges);
		}).Wait();
	}

	IMachineConfig* FMachineConfigFactory::Create(const FString& LocalMachineConfigFile, bool bAlwaysFlushChanges)
	{
		return new FMachineConfig(LocalMachineConfigFile, bAlwaysFlushChanges);
	}

}
