// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace BuildPatchServices
{
	class IMachineConfig
	{
	public:
		/**
		 * Virtual destructor.
		 */
		virtual ~IMachineConfig() = default;

		/**
		 * Loads the list of installed prereq ids whose corresponding prerequisites are installed on this computer.
		 * @return         A set containing the prerequisite ids which are installed on this computer.
		 */
		virtual TSet<FString> LoadInstalledPrereqIds() = 0;

		/**
		 * Saves updated list of installed prereqs to the configuration file.
		 * @param InstalledPrereqIds       A set of all installed prerequisites to save to configuration.
		 */
		virtual void SaveInstalledPrereqIds(const TSet<FString>& InstalledPrereqIds) = 0;
	};

	/**
	 * A factory for creating an IMachineConfig instance.
	 */
	class FMachineConfigFactory
	{
	public:
		/**
		 * Creates an instance of IMachineConfig.
		 * @param LocalMachineConfigFile   The path to an ini file which should contain the per machine configuration data.
		 * @param bAlwaysFlushChanges      If true, then we'll always flush changes straight to disk after saving values.
		 * @return a pointer to the new IMachineConfig instance created.
		 */
		static IMachineConfig* Create(const FString& LocalMachineConfigFile, bool bAlwaysFlushChanges);
	};

}
