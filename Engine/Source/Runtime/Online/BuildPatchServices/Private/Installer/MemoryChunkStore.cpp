// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/MemoryChunkStore.h"
#include "Misc/ScopeLock.h"
#include "Installer/ChunkEvictionPolicy.h"
#include "Installer/ChunkReferenceTracker.h"

namespace BuildPatchServices
{
	class FMemoryChunkStore
		: public IMemoryChunkStore
	{
	public:
		FMemoryChunkStore(int32 InStoreSize, IChunkEvictionPolicy* InEvictionPolicy, IChunkStore* InOverflowStore, IMemoryChunkStoreStat* InMemoryChunkStoreStat, IChunkReferenceTracker* InChunkReferenceTracker);
		~FMemoryChunkStore();

		// IChunkStore interface begin.
		virtual void Put(const FGuid& DataId, TUniquePtr<IChunkDataAccess> ChunkData) override;
		virtual IChunkDataAccess* Get(const FGuid& DataId) override;
		virtual TUniquePtr<IChunkDataAccess> Remove(const FGuid& DataId) override;
		virtual int32 GetSize() const override;
		virtual void SetLostChunkCallback(TFunction<void(const FGuid&)> Callback) override;
		// IChunkStore interface end.

		// IMemoryChunkStore interface begin.
		virtual void DumpToOverflow() override;
		// IMemoryChunkStore interface end.

	private:
		void PutInternal(const FGuid& DataId, TUniquePtr<IChunkDataAccess> ChunkData, bool bIsNewChunk);
		void UpdateStoreUsage() const;
		void ExecLostChunkCallback(const FGuid& LostChunk);

	private:
		int32 StoreSize;
		TMap<FGuid, TUniquePtr<IChunkDataAccess>> Store;
		IChunkEvictionPolicy* EvictionPolicy;
		IChunkStore* OverflowStore;
		IMemoryChunkStoreStat* MemoryChunkStoreStat;
		IChunkReferenceTracker* OptionalChunkReferenceTracker;
		mutable FCriticalSection LostChunkCallbackCs;
		TFunction<void(const FGuid&)> LostChunkCallback;
		FGuid LastGetId;
		TUniquePtr<IChunkDataAccess> LastGetData;
		mutable FCriticalSection ThreadLockCs;
	};

	FMemoryChunkStore::FMemoryChunkStore(int32 InStoreSize, IChunkEvictionPolicy* InEvictionPolicy, IChunkStore* InOverflowStore, IMemoryChunkStoreStat* InMemoryChunkStoreStat, IChunkReferenceTracker* InOptionalChunkReferenceTracker)
		: StoreSize(InStoreSize)
		, Store()
		, EvictionPolicy(InEvictionPolicy)
		, OverflowStore(InOverflowStore)
		, MemoryChunkStoreStat(InMemoryChunkStoreStat)
		, OptionalChunkReferenceTracker(InOptionalChunkReferenceTracker)
		, LostChunkCallbackCs()
		, LostChunkCallback(nullptr)
		, LastGetId()
		, LastGetData(nullptr)
		, ThreadLockCs()
	{
		MemoryChunkStoreStat->OnStoreSizeUpdated(StoreSize);
	}

	FMemoryChunkStore::~FMemoryChunkStore()
	{
		for (const TPair<FGuid, TUniquePtr<IChunkDataAccess>>& Entry : Store)
		{
			MemoryChunkStoreStat->OnChunkReleased(Entry.Key);
		}
		if (LastGetData.IsValid())
		{
			MemoryChunkStoreStat->OnChunkReleased(LastGetId);
		}
		MemoryChunkStoreStat->OnStoreUseUpdated(0);
	}

	void FMemoryChunkStore::Put(const FGuid& DataId, TUniquePtr<IChunkDataAccess> ChunkData)
	{
		PutInternal(DataId, MoveTemp(ChunkData), true);
	}

	IChunkDataAccess* FMemoryChunkStore::Get(const FGuid& DataId)
	{
		// Thread lock to protect access to Store, LastGetId, and LastGetData.
		FScopeLock ThreadLock(&ThreadLockCs);
		if (LastGetId != DataId)
		{
            TRACE_CPUPROFILER_EVENT_SCOPE(MemoryStore_Get);
			// Put back last get.
			if (LastGetData.IsValid() && LastGetId.IsValid())
			{
				if (Store.Contains(LastGetId) == false)
				{
					// If we aren't used anymore then don't put us back.
					// Since we were the previously used chunk, we are the one that could have
					// gotten released - no need to go through PutInternal and enumerate the world.
					if (OptionalChunkReferenceTracker &&
						OptionalChunkReferenceTracker->GetReferenceCount(LastGetId) == 0)
					{
						// Drop us on the ground and move on.
						MemoryChunkStoreStat->OnChunkReleased(LastGetId);
					}
					else
					{
						PutInternal(LastGetId, MoveTemp(LastGetData), false);
					}
				}
			}
			// Invalidate last get.
			LastGetId.Invalidate();
			LastGetData.Reset();
			// Retrieve requested data.
			if (Store.Contains(DataId))
			{
				LastGetData = MoveTemp(Store[DataId]);
				Store.Remove(DataId);
			}
			else if (OverflowStore != nullptr)
			{
				LastGetData = OverflowStore->Remove(DataId);
				if (LastGetData.IsValid())
				{
					MemoryChunkStoreStat->OnChunkStored(DataId);
					UpdateStoreUsage();
				}
			}
			// Save ID if successful.
			if (LastGetData.IsValid())
			{
				LastGetId = DataId;
			}
		}
		return LastGetData.Get();
	}

	TUniquePtr<IChunkDataAccess> FMemoryChunkStore::Remove(const FGuid& DataId)
	{
		// Thread lock to protect access to Store, LastGetId, and LastGetData.
		FScopeLock ThreadLock(&ThreadLockCs);
		TUniquePtr<IChunkDataAccess> Rtn = nullptr;
		if (LastGetId == DataId)
		{
			LastGetId.Invalidate();
			Rtn = MoveTemp(LastGetData);
		}
		if (Store.Contains(DataId))
		{
			Rtn = MoveTemp(Store[DataId]);
			Store.Remove(DataId);
		}
		UpdateStoreUsage();
		return MoveTemp(Rtn);
	}

	int32 FMemoryChunkStore::GetSize() const
	{
		return StoreSize;
	}

	void FMemoryChunkStore::SetLostChunkCallback(TFunction<void(const FGuid&)> Callback)
	{
		// Thread lock to protect access to LostChunkCallback.
		FScopeLock ThreadLock(&LostChunkCallbackCs);
		LostChunkCallback = MoveTemp(Callback);
	}

	void FMemoryChunkStore::DumpToOverflow()
	{
		// Thread lock to protect access to Store.
		FScopeLock ThreadLock(&ThreadLockCs);
		if (OverflowStore != nullptr)
		{
			for (TPair<FGuid, TUniquePtr<IChunkDataAccess>>& Entry : Store)
			{
				OverflowStore->Put(Entry.Key, MoveTemp(Entry.Value));
			}
			if (LastGetData.IsValid())
			{
				OverflowStore->Put(LastGetId, MoveTemp(LastGetData));
			}
		}
		Store.Empty();
		LastGetData.Reset();
		LastGetId.Invalidate();
		UpdateStoreUsage();
	}

	void FMemoryChunkStore::PutInternal(const FGuid& DataId, TUniquePtr<IChunkDataAccess> ChunkData, bool bIsNewChunk)
	{
		// Thread lock to protect access to Store, LastGetId, and LastGetData.
		FScopeLock ThreadLock(&ThreadLockCs);
		// Add this new chunk.
		Store.Add(DataId, MoveTemp(ChunkData));
		if (bIsNewChunk)
		{
			MemoryChunkStoreStat->OnChunkStored(DataId);
			UpdateStoreUsage();
		}

		if (!bIsNewChunk)
		{
			// We are putting back the last used chunk - since it's not new data we know we aren't increasing
			// any limits so we don't have to scan for evictions, and we know we don't have to scan for cleans
			// because the only thing that could get cleaned is us, which we checked before we got here.
			return;
		}

		// Clean out our store.
		TSet<FGuid> Cleanable;
		TSet<FGuid> Bootable;
		EvictionPolicy->Query(Store, StoreSize, Cleanable, Bootable);
		// Perform clean.
		for (const FGuid& CleanId : Cleanable)
		{
			Store.Remove(CleanId);
			MemoryChunkStoreStat->OnChunkReleased(CleanId);
		}
		// Perform boot.
		for (const FGuid& BootId : Bootable)
		{
			if (OverflowStore != nullptr)
			{
				OverflowStore->Put(BootId, MoveTemp(Store[BootId]));
			}
			else
			{
				ExecLostChunkCallback(BootId);
			}
			Store.Remove(BootId);
			MemoryChunkStoreStat->OnChunkBooted(BootId);
		}
	}

	void FMemoryChunkStore::UpdateStoreUsage() const
	{
		const int32 LastGetCount = LastGetId.IsValid() && !Store.Contains(LastGetId);
		const int32 StoreNum = Store.Num();
		MemoryChunkStoreStat->OnStoreUseUpdated(StoreNum + LastGetCount);
	}

	void FMemoryChunkStore::ExecLostChunkCallback(const FGuid& LostChunk)
	{
		// Thread lock to protect access to LostChunkCallback.
		FScopeLock ThreadLock(&LostChunkCallbackCs);
		if (LostChunkCallback)
		{
			LostChunkCallback(LostChunk);
		}
	}

	IMemoryChunkStore* FMemoryChunkStoreFactory::Create(int32 StoreSize, IChunkEvictionPolicy* EvictionPolicy, IChunkStore* OverflowStore, IMemoryChunkStoreStat* MemoryChunkStoreStat, IChunkReferenceTracker* InOptionalChunkReferenceTracker)
	{
		check(EvictionPolicy != nullptr);
		check(MemoryChunkStoreStat != nullptr);
		return new FMemoryChunkStore(StoreSize, EvictionPolicy, OverflowStore, MemoryChunkStoreStat, InOptionalChunkReferenceTracker);
	}
}
