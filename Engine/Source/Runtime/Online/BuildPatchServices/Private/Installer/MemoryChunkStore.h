// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/ChunkStore.h"

namespace BuildPatchServices
{
	class IChunkEvictionPolicy;
	class IMemoryChunkStoreStat;
	class IChunkReferenceTracker;

	/**
	 * An interface providing access to chunk data instances which are stored in memory.
	 */
	class IMemoryChunkStore : public IChunkStore
	{
	public:
		virtual ~IMemoryChunkStore() = default;

		/**
		 * Dumps all chunks contained in this store into the overflow provided at construction, removing all chunks from
		 * this store in the process.
		 * This function will also invalidate data ptr previously returned by Get().
		 */
		virtual void DumpToOverflow() = 0;
	};

	/**
	 * A factory for creating an IMemoryChunkStore instance.
	 */
	class FMemoryChunkStoreFactory
	{
	public:
		/**
		 * Creates an instance of a chunk store class that stores chunks in memory.
		 * When Put() is called and the store has >= ChunkCount entries, the EvictionPolicy will be used to select chunks which should
		 * be Put() into the OverflowStore instance, and removed from this store.
		 * If the EvictionPolicy allows it, the store will grow larger than ChunkCount.
		 * When Get() is called on a chunk that is not in this store, Remove() will be attempted on the OverflowStore instance, and the
		 * chunk will enter the memory store if it was successful.
		 * @param ChunkCount                The size of the store, at which point chunks will start being evicted based on the policy provided.
		 * @param EvictionPolicy            Required ptr to an eviction policy that will be used to clean out the store, and evict data when full.
		 * @param OverflowStore             Required ptr to an IChunkStore which will be used as an overflow for evicted chunks.
		 * @param MemoryChunkStoreStat      Required ptr to the statistics class.
		 * @return the new IMemoryChunkStore instance created.
		 */
		static IMemoryChunkStore* Create(int32 ChunkCount, IChunkEvictionPolicy* EvictionPolicy, IChunkStore* OverflowStore, IMemoryChunkStoreStat* MemoryChunkStoreStat, IChunkReferenceTracker* InOptionalChunkReferenceTracker);
	};

	/**
	 * This interface defines the statistics class required by the memory chunk store. It should be implemented in order to collect
	 * desired information which is being broadcast by the system.
	 */
	class IMemoryChunkStoreStat
	{
	public:
		virtual ~IMemoryChunkStoreStat() = default;

		/**
		 * Called whenever a new chunk has been put into the store.
		 * @param ChunkId           The id of the chunk.
		 */
		virtual void OnChunkStored(const FGuid& ChunkId) = 0;

		/**
		 * Called whenever a chunk has been release from the store as it was no longer referenced.
		 * @param ChunkId           The id of the chunk.
		 */
		virtual void OnChunkReleased(const FGuid& ChunkId) = 0;

		/**
		 * Called whenever a chunk has been booted from the store because a new one was added that is required sooner.
		 * @param ChunkId           The id of the chunk.
		 */
		virtual void OnChunkBooted(const FGuid& ChunkId) = 0;

		/**
		 * Called whenever the number of chunks in the store is updated.
		 * @param ChunkCount        The number of chunks now contained in the store.
		 */
		virtual void OnStoreUseUpdated(int32 ChunkCount) = 0;

		/**
		 * Called whenever the total size allotted for the store is updated.
		 * @param Size              The total number of chunks that can be stored.
		 */
		virtual void OnStoreSizeUpdated(int32 Size) = 0;
	};
}
