// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/MessagePump.h"
#include "Containers/Union.h"
#include "Containers/Queue.h"
#include <atomic>

namespace BuildPatchServices
{
	// Union of all possible message types.
	typedef TUnion<FChunkSourceEvent, FInstallationFileAction, FGenericMessage> FMessageUnion;

	// Queue type for messages.
	typedef TQueue<FMessageUnion, EQueueMode::Mpsc> FMessageQueue;

	// Tuple of the request and response for finding a chunk uri location
	typedef TTuple<FChunkUriRequest, TFunction<void(FChunkUriResponse)>> FChunkUriRequestResponse;

	// Queue of request responses of chunk we are looking for
	typedef TQueue<FChunkUriRequestResponse, EQueueMode::Mpsc> FChunkUriQueue;

	// Tuple of the request and response for finding a delta uri location
	typedef TTuple<FDeltaUriRequest, TFunction<void(FExpiratedUrisResponse)>> FDeltaUriRequestResponse;

	// Queue of request responses of delta file we are looking for
	typedef TQueue<FDeltaUriRequestResponse, EQueueMode::Mpsc> FDeltaUriQueue;

}

namespace MessagePumpHelpers
{
	template<typename Type>
	bool TryPump(const TArray<BuildPatchServices::FMessageHandler*>& Handlers, const BuildPatchServices::FMessageUnion& MessageUnion)
	{
		if (MessageUnion.HasSubtype<Type>())
		{
			for (BuildPatchServices::FMessageHandler* Handler : Handlers)
			{
				Handler->HandleMessage(MessageUnion.GetSubtype<Type>());
			}
			return true;
		}
		return false;
	}
}

namespace BuildPatchServices
{
	class FMessagePump
		: public IMessagePump
	{
	public:
		// IMessagePump interface begin.
		virtual void SendMessage(FChunkSourceEvent Message) override;
		virtual void SendMessage(FInstallationFileAction Message) override;
		virtual void SendMessage(FGenericMessage Message) override;
		virtual bool SendRequest(FChunkUriRequest Request, TFunction<void(FChunkUriResponse)> OnResponse) override;
		virtual void SendRequest(FDeltaUriRequest Request, TFunction<void(FExpiratedUrisResponse)> OnResponse) override;
		virtual void PumpMessages(const TArray<FMessageHandler*>& Handlers) override;
		virtual void PumpMessages() override;
		virtual void RegisterMessageHandler(FMessageHandler* MessageHandler) override;
		virtual void UnregisterMessageHandler(FMessageHandler* MessageHandler) override;
		// IMessagePump interface end.

	private:
		template<typename FResponse, typename FRequest>
		void PumpMessages(const TArray<FMessageHandler*>& InHandlers, FRequest& RequestQueue);

		FMessageQueue MessageQueue;
		FChunkUriQueue ChunkUriRequestQueue;
		FDeltaUriQueue DeltaUriRequestQueue;
		FDefaultMessageHandler DefaultMessageHandler;
		TArray<FMessageHandler*> Handlers;
		// std::atomic will not expose integral operations with enum types.
		std::atomic<uint32> MessageRequests{0};
	};

	void FMessagePump::SendMessage(FGenericMessage Message)
	{
		MessageQueue.Enqueue(FMessageUnion(MoveTemp(Message)));
	}

	void FMessagePump::SendMessage(FChunkSourceEvent Message)
	{
		MessageQueue.Enqueue(FMessageUnion(MoveTemp(Message)));
	}

	void FMessagePump::SendMessage(FInstallationFileAction Message)
	{
		MessageQueue.Enqueue(FMessageUnion(MoveTemp(Message)));
	}

	bool FMessagePump::SendRequest(FChunkUriRequest Request, TFunction<void(FChunkUriResponse)> OnResponse)
	{
		if (EnumHasAllFlags(static_cast<EMessageRequests>(MessageRequests.load()), EMessageRequests::ChunkUriRequest))
		{
			ChunkUriRequestQueue.Enqueue(FChunkUriRequestResponse(MoveTemp(Request), MoveTemp(OnResponse)));
			return true;
		}
		return false;
	}

	template<typename FResponse, typename FRequest>
	void FMessagePump::PumpMessages(const TArray<FMessageHandler*>& InHandlers, FRequest& RequestQueue)
	{
		FResponse ChunkUriRequestResponse;
		while (RequestQueue.Dequeue(ChunkUriRequestResponse))
		{
			bool bHandled = false;
			for (BuildPatchServices::FMessageHandler* Handler : InHandlers)
			{
				if (Handler->HandleRequest(ChunkUriRequestResponse.template Get<0>(), ChunkUriRequestResponse.template Get<1>()))
				{
					bHandled = true;
					break;
				}
			}
			if (!bHandled)
			{
				DefaultMessageHandler.HandleRequest(ChunkUriRequestResponse.template Get<0>(), ChunkUriRequestResponse.template Get<1>());
			} 
		}
	}

	void FMessagePump::PumpMessages()
	{
		PumpMessages(Handlers);
	}

	void FMessagePump::PumpMessages(const TArray<FMessageHandler*>& ExtHandlers)
	{
		FMessageUnion MessageUnion;
		while (MessageQueue.Dequeue(MessageUnion))
		{
			if (MessagePumpHelpers::TryPump<FChunkSourceEvent>(ExtHandlers, MessageUnion))
			{
				continue;
			}
			else if (MessagePumpHelpers::TryPump<FInstallationFileAction>(ExtHandlers, MessageUnion))
			{
				continue;
			}
			else if (MessagePumpHelpers::TryPump<FGenericMessage>(ExtHandlers, MessageUnion))
			{
				continue;
			}
		}

		// Handle get chunk URI request
		PumpMessages<FChunkUriRequestResponse>(ExtHandlers, ChunkUriRequestQueue);

		// Handle get delta file URI request
		PumpMessages<FDeltaUriRequestResponse>(ExtHandlers, DeltaUriRequestQueue);
	}

	void FMessagePump::SendRequest(FDeltaUriRequest Request, TFunction<void(FExpiratedUrisResponse)> OnResponse)
	{
		DeltaUriRequestQueue.Enqueue(FDeltaUriRequestResponse(MoveTemp(Request), MoveTemp(OnResponse)));
	}

	void FMessagePump::RegisterMessageHandler(FMessageHandler* InHandler)
	{
		MessageRequests |= static_cast<uint32>(InHandler->GetMessageRequests());
		Handlers.Add(InHandler);
	}

	void FMessagePump::UnregisterMessageHandler(FMessageHandler* InHandler)
	{
		MessageRequests = 0;
		Handlers.Remove(InHandler);
		for (FMessageHandler* Handler : Handlers)
		{
			MessageRequests |= static_cast<uint32>(Handler->GetMessageRequests());
		}
	}

	IMessagePump* FMessagePumpFactory::Create()
	{
		return new FMessagePump();
	}
}
