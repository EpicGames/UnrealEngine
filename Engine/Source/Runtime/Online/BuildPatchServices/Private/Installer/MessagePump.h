// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BuildPatchMessage.h"

namespace BuildPatchServices
{
	/**
	 * Interface for a message pump which allows systems to bubble up event information to the Installer's public API.
	 */
	class IMessagePump
	{
	public:
		/**
		 * Virtual destructor.
		 */
		virtual ~IMessagePump() = default;

		/**
		 * Sends a chunk source event message.
		 * @param Message   The message to be sent.
		 */
		virtual void SendMessage(FChunkSourceEvent Message) = 0;

		// See FGenericMessage for types of messages.
		virtual void SendMessage(FGenericMessage Message) = 0;

		/**
		 * Sends an installation file action message.
		 * @param Message   The message to be sent.
		 */
		virtual void SendMessage(FInstallationFileAction Message) = 0;

		/**
		 * Sends out a request to resolve the uri to the chunk location
		 * @param Request   Request for the chunk location
		 * @param OnResponse   A delegate to call with the chunk location and any potential headers to add to the http request.
		 *
		 * Note that while this function supports asynchronous processing, the calling code does not support
		 * cancelation of the requests under abort scenarios, potentially leading to crashes. As as result,
		 * incomplete shutdown of the installation requires waiting for all outstanding requests to return before
		 * shutdown can complete. For user initiated cancellations it is recommended that you abort any async
		 * uri request handling and call OnResponse with FChunkUriResponse::bFailed to true after calling CancelInstall().
		 * This will prevent the default URL concatenation from occuring and allow the shutdown logic to complete in a timely
		 * fashion.
		 *
		 * For internally initiated cancellations (due to errors) there's nothing to be done; shutdown just waits for
		 * the requests to all complete.
		 *
		 * As a result, it's highly advisable to make this as immediate as possible, e.g. caching auth tokens up front
		 * before installation launch.
		 */
		virtual bool SendRequest(FChunkUriRequest Request, TFunction<void(FChunkUriResponse)> OnResponse) = 0;

		/**
		 * Sends out a request to resolve the uri to the delta file location
		 * @param Request   Request for the delta file location
		 * @param OnResponse   A delegate to call with a response to delta location
		 */
		virtual void SendRequest(FDeltaUriRequest Request, TFunction<void(FExpiratedUrisResponse)> OnResponse) = 0;

		/**
		 * Dequeues received messages, pushing them to the provided handlers.
		 * NOTE: PumpMessages, RegisterMessageHandler, and UnregisterMessageHandler MUST all be called from the same thread.
		 * @param Handlers      The array of handlers.
		 */
		virtual void PumpMessages(const TArray<FMessageHandler*>& Handlers) = 0;
		virtual void PumpMessages() = 0;

		/**
		 * Registers a message handler.
		 * @param MessageHandler    Ptr to the message handler to add. Must not be null.
		 * NOTE: PumpMessages, RegisterMessageHandler, and UnregisterMessageHandler MUST all be called from the same thread.
		 */
		virtual void RegisterMessageHandler(FMessageHandler* MessageHandler) = 0;

		/**
		 * Unregisters a message handler.
		 * @param MessageHandler    Ptr to the message handler to remove.
		 * NOTE: PumpMessages, RegisterMessageHandler, and UnregisterMessageHandler MUST all be called from the same thread.
		 */
		virtual void UnregisterMessageHandler(FMessageHandler* MessageHandler) = 0;
	};

	/**
	 * A factory for creating an IMessagePump instance.
	 */
	class FMessagePumpFactory
	{
	public:
		/**
		 * Creates an instance of IMessagePump.
		 * @return a pointer to the new IMessagePump instance created.
		 */
		static IMessagePump* Create();
	};

}
