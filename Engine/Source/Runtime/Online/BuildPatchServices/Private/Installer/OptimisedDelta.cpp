// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/OptimisedDelta.h"

#include "Async/Future.h"
#include "Tasks/Task.h"
#include "Misc/ConfigCacheIni.h"

#include "Installer/DownloadService.h"
#include "Installer/InstallerError.h"
#include "BuildPatchUtil.h"
#include "BuildPatchMergeManifests.h"

#include <atomic>

DECLARE_LOG_CATEGORY_EXTERN(LogOptimisedDelta, Log, All);
DEFINE_LOG_CATEGORY(LogOptimisedDelta);

namespace ConfigHelpers
{
	int32 LoadRetries(const int32 Min, const FString& PreferenceName)
	{
		int32 DeltaRetries = 5;
		GConfig->GetInt(TEXT("Portal.BuildPatch"), *PreferenceName, DeltaRetries, GEngineIni);
		DeltaRetries = FMath::Clamp<int32>(DeltaRetries, Min, 1000);
		return DeltaRetries;
	}

	int32 LoadDeltaRetries(const int32 Min)
	{
		return LoadRetries(Min, TEXT("DeltaRetries"));
	}

	int32 LoadDeltaUriRetries(const int32 Min)
	{
		return LoadRetries(Min, TEXT("DeltaUriRetries"));
	}
}

namespace BuildPatchServices
{
	template<typename ResultType>
	class TAbortablePromise
	{
	public:
		TAbortablePromise() = default;
		~TAbortablePromise()
		{
			Abort();
		}

		void Abort()
		{
			if (!bSetValue)
			{
				SetValue(MakeShared<ResultType, ESPMode::ThreadSafe>(MakeError(UserCancelErrorCodes::UserRequested)));
			}
		}

		TFuture<TSharedPtr<ResultType, ESPMode::ThreadSafe>> GetFuture()
		{
			return Promise.GetFuture();
		}

		void SetValue(TSharedPtr<ResultType, ESPMode::ThreadSafe> Result)
		{
			bSetValue = true;
			Promise.SetValue(MoveTemp(Result));
		}

	private:
		std::atomic<bool> bSetValue{ false };
		TPromise<TSharedPtr<ResultType, ESPMode::ThreadSafe>> Promise;
	};

	class FOptimisedDelta :
		public IOptimisedDelta,
		public TSharedFromThis<FOptimisedDelta>
	{
	public:
		FOptimisedDelta(const FOptimisedDeltaConfiguration& Configuration, FOptimisedDeltaDependencies Dependencies);

		// IOptimisedDelta interface begin.
		virtual const IOptimisedDelta::FResultValueOrError& GetResult() const override;
		virtual int32 GetMetaDownloadSize() const override;
		virtual bool IsReady() const override;
		int32 GetDownloadResponseCode() const override;
		// IOptimisedDelta interface end.

		void Initialize();

	private:
		bool RequiresChunkDownload();
		void OnDownloadComplete(int32 RequestId, const FDownloadRef& Download);
		bool ShouldRetry(const FDownloadRef& Download);
		void RequestDeltaFileUri();
		void DeltaUriCompletionDelegate(bool bSuccessful, const FString& DeltaUri, bool bNeedRetry);
		void SetResult(const FBuildPatchAppManifestPtr& ResultManifest);
		void SetFailedDownload();

	private:
		const FOptimisedDeltaConfiguration Configuration;
		const FOptimisedDeltaDependencies Dependencies;
		const FString RelativeDeltaFilePath;
		const int32 DeltaRetries;
		const int32 DeltaUriRetries;
		EDeltaPolicy DeltaPolicy = Configuration.DeltaPolicy;
		int32 CloudDirIdx = 0;
		int32 RetryCount = 0;
		int32 DeltaUriRetryCount = 0;
		FDownloadProgressDelegate ChunkDeltaProgress;
		FDownloadCompleteDelegate ChunkDeltaComplete;
		// We use TAbortablePromise in the case when we destroy the object before actual delta is received
		// TPromise and TFuture are restricted to types that are copyable, assignable, and default constructible. Value should not be exposed.
		TAbortablePromise<IOptimisedDelta::FResultValueOrError> ChunkDeltaPromise;
		TFuture<TSharedPtr<FResultValueOrError, ESPMode::ThreadSafe>> ChunkDeltaFuture{ ChunkDeltaPromise.GetFuture() };
		FThreadSafeCounter DownloadedBytes = 0;
		const TCHAR* ErrorCode = nullptr;
		int32 DownloadResponseCode = 0;
	};

	FOptimisedDelta::FOptimisedDelta(const FOptimisedDeltaConfiguration& InConfiguration, FOptimisedDeltaDependencies InDependencies)
		: Configuration(InConfiguration)
		, Dependencies(MoveTemp(InDependencies))
		, RelativeDeltaFilePath(Configuration.SourceManifest.IsValid() ? FBuildPatchUtils::GetChunkDeltaFilename(*Configuration.SourceManifest.Get(), Configuration.DestinationManifest.Get(), Configuration.DeltaFilenameTrailer) : TEXT(""))
		, DeltaRetries(ConfigHelpers::LoadDeltaRetries(Configuration.RetriesNumber))
		, DeltaUriRetries(ConfigHelpers::LoadDeltaUriRetries(Configuration.RetriesNumber))
	{}

	void FOptimisedDelta::Initialize()
	{
		ChunkDeltaComplete = FDownloadCompleteDelegate::CreateSP(this, &FOptimisedDelta::OnDownloadComplete);

		// There are some conditions in which we do not use a delta.
		const bool bNoSourceManifest = Configuration.SourceManifest.IsValid() == false;
		const bool bNotPatching = Configuration.InstallMode == EInstallMode::PrereqOnly;
		const bool bSameBuild = bNoSourceManifest ? false : Configuration.SourceManifest->GetBuildId() == Configuration.DestinationManifest->GetBuildId();
		const bool bNoDownload = RequiresChunkDownload() == false;
		if (bNoSourceManifest || bNotPatching || bSameBuild || bNoDownload)
		{
			DeltaPolicy = EDeltaPolicy::Skip;
		}
		// Kick off the request if we should be.
		if (DeltaPolicy != EDeltaPolicy::Skip)
		{
			UE_LOGF(LogOptimisedDelta, Log, "Requesting optimised delta file %ls", *RelativeDeltaFilePath);
			RequestDeltaFileUri();
		}
		// Otherwise we provide the standard destination manifest.
		else
		{
			SetResult(Configuration.DestinationManifest);
		}
	}

	const IOptimisedDelta::FResultValueOrError& FOptimisedDelta::GetResult() const
	{
		TSharedPtr<FResultValueOrError, ESPMode::ThreadSafe> Result = ChunkDeltaFuture.Get();
		// Result should never be set nullptr.
		check(Result.IsValid());
		return *Result;
	}

	int32 FOptimisedDelta::GetMetaDownloadSize() const
	{
		ChunkDeltaFuture.Wait();
		return DownloadedBytes.GetValue();
	}

	bool FOptimisedDelta::IsReady() const
	{
		return ChunkDeltaFuture.IsReady();
	}

	bool FOptimisedDelta::RequiresChunkDownload()
	{
		if (Configuration.SourceManifest.IsValid())
		{
			for (const FString& DestinationFile : Configuration.DestinationManifest->GetBuildFileList())
			{
				// Get file manifests
				const FFileManifest* OldFile = Configuration.SourceManifest->GetFileManifest(DestinationFile);
				const FFileManifest* NewFile = Configuration.DestinationManifest->GetFileManifest(DestinationFile);
				// Different hash means different file, missing OldFile means added file.
				if (!OldFile || OldFile->SHA1Hash != NewFile->SHA1Hash)
				{
					return true;
				}
			}
			// No new or changed files found.
			return false;
		}
		// No source manifest means full download.
		return true;
	}

	void FOptimisedDelta::OnDownloadComplete(int32 RequestId, const FDownloadRef& Download)
	{
		DownloadResponseCode = Download->GetResponseCode();		// Store for analytics.
		if (Download->ResponseSuccessful())
		{
			// Perform a merge with current manifest so that the delta can support missing out unnecessary information.
			FBuildPatchAppManifestPtr NewManifest;
			FBuildPatchAppManifestRef DeltaManifest = MakeShareable(new FBuildPatchAppManifest());
			if (DeltaManifest->DeserializeFromData(Download->GetData()))
			{
				// Verify each file received matches what we expect.
				TSet<FString> DeltaFilenameList;
				DeltaManifest->GetFileList(DeltaFilenameList);
				bool bDeltaAccepted = true;
				for (const FString& DeltaFilename : DeltaFilenameList)
				{
					const FFileManifest* OriginalFileManifest = Configuration.DestinationManifest->GetFileManifest(DeltaFilename);
					const FFileManifest* DeltaFileManifest = DeltaManifest->GetFileManifest(DeltaFilename);
					if (OriginalFileManifest != nullptr && DeltaFileManifest != nullptr)
					{
						FSHAHash OriginalFileHash;
						FSHAHash DeltaFileHash;
						Configuration.DestinationManifest->GetFileHash(DeltaFilename, OriginalFileHash);
						DeltaManifest->GetFileHash(DeltaFilename, DeltaFileHash);
						if (OriginalFileHash != DeltaFileHash)
						{
							UE_LOGF(LogOptimisedDelta, Log, "Rejected optimised delta due to file SHA1 mismatch.");
							ErrorCode = DownloadErrorCodes::RejectedDeltaFile;
							bDeltaAccepted = false;
							break;
						}
					}
					else
					{
						UE_LOGF(LogOptimisedDelta, Log, "Rejected optimised delta due to file list mismatch.");
						ErrorCode = DownloadErrorCodes::RejectedDeltaFile;
						bDeltaAccepted = false;
						break;
					}
				}
				// Only use if accepted so far.
				if (bDeltaAccepted)
				{
					NewManifest = FBuildMergeManifests::MergeDeltaManifest(Configuration.DestinationManifest.Get(), DeltaManifest.Get());
					if (!NewManifest.IsValid())
					{
						UE_LOGF(LogOptimisedDelta, Log, "Rejected optimised delta due to merge failure.");
						ErrorCode = DownloadErrorCodes::RejectedDeltaFile;
						bDeltaAccepted = false;
					}
				}
			}
			else
			{
				ErrorCode = DownloadErrorCodes::UnserialisableDeltaFile;
			}

			if (NewManifest.IsValid())
			{
				UE_LOGF(LogOptimisedDelta, Log, "Received optimised delta file successfully %ls", *RelativeDeltaFilePath);
				DownloadedBytes.Set(Download->GetData().Num());
				SetResult(NewManifest);
			}
			else if (ShouldRetry(Download))
			{
				++RetryCount;
				UE_LOGF(LogOptimisedDelta, Log, "Invalid manifest (response code=%d) %ls, retrying %i/%i", Download->GetResponseCode(), *RelativeDeltaFilePath, RetryCount, DeltaRetries);
				RequestDeltaFileUri();
			}
			else
			{
				UE_LOGF(LogOptimisedDelta, Log, "Invalid manifest (response code=%d) %ls, ShouldRetry = false", Download->GetResponseCode(), *RelativeDeltaFilePath);
				SetFailedDownload();
			}
		}
		else if (ShouldRetry(Download))
		{
			++RetryCount;
			UE_LOGF(LogOptimisedDelta, Log, "Failed to download (response code=%d) %ls, retrying %i/%i", Download->GetResponseCode(), *RelativeDeltaFilePath, RetryCount, DeltaRetries);
			RequestDeltaFileUri();
		}
		else
		{
			UE_LOGF(LogOptimisedDelta, Log, "Failed to download (response code=%d) %ls, ShouldRetry = false", Download->GetResponseCode(), *RelativeDeltaFilePath);
			ErrorCode = DownloadErrorCodes::MissingDeltaFile;
			SetFailedDownload();
		}
	}

	void FOptimisedDelta::RequestDeltaFileUri()
	{
		FUriProviderCompleteDelegate DeltaUriCompletionDelegate = FUriProviderCompleteDelegate::CreateRaw(this, &FOptimisedDelta::DeltaUriCompletionDelegate);
		Dependencies.UriProvider->RequestUri(RelativeDeltaFilePath, Configuration.DestinationManifest->GetBuildId(), DeltaUriCompletionDelegate);
	}

	void FOptimisedDelta::DeltaUriCompletionDelegate(bool bSuccessful, const FString& DeltaUri, bool bNeedRetry)
	{
		if (bSuccessful)
		{
			UE_LOGF(LogOptimisedDelta, Log, "Delta file URI response received: %ls", *DeltaUri);
			Dependencies.DownloadService->RequestFile(DeltaUri, ChunkDeltaComplete, ChunkDeltaProgress);
		}
		else if (bNeedRetry && DeltaUriRetryCount < DeltaUriRetries)
		{
			++DeltaUriRetryCount;
			UE_LOGF(LogOptimisedDelta, Log, "Retrying while trying to get delta URI: %d", DeltaUriRetryCount);
			RequestDeltaFileUri();
		}
		else
		{
			UE_LOGF(LogOptimisedDelta, Log, "No delta file available");
			ErrorCode = DownloadErrorCodes::MissingDeltaFile;
			SetFailedDownload();
		}
	}

	bool FOptimisedDelta::ShouldRetry(const FDownloadRef& Download)
	{
		// If the response code was in the 'client error' range - interpreted as we asked for something invalid, then we accept that as the
		// 'no delta' response. Any other failure reason is a server or network issue which we should retry.
		const int32 ResponseCode = Download->GetResponseCode();
		const bool bCanRetry = ResponseCode < 400 || ResponseCode >= 500;
		// In case of URI signature expiration (permission issue) we should retry as well
		const bool bPermissionError = (ResponseCode == 403);
		// 404 may occur while working with network shared folders (e.g. when they are not mounted properly).
		const bool bNotFound = (ResponseCode == 404);
		const bool bMayRetry = RetryCount < DeltaRetries;
		return (bCanRetry || bPermissionError || bNotFound) && bMayRetry;
	}

	void FOptimisedDelta::SetResult(const FBuildPatchAppManifestPtr& ResultManifest)
	{
		TSharedPtr<FResultValueOrError> Result = MakeShared<FResultValueOrError, ESPMode::ThreadSafe>(MakeValue(ResultManifest));
		ChunkDeltaPromise.SetValue(Result);
		if (Dependencies.OnComplete)
		{
			Dependencies.OnComplete(*Result);
		}
	}

	void FOptimisedDelta::SetFailedDownload()
	{
		if (DeltaPolicy == EDeltaPolicy::TryFetchContinueWithout)
		{
			UE_LOGF(LogOptimisedDelta, Log, "Skipping optimised delta file.");
			SetResult(Configuration.DestinationManifest);
		}
		else
		{
			UE_LOGF(LogOptimisedDelta, Log, "Failed optimised delta file fetch %ls", *RelativeDeltaFilePath);

			TSharedPtr<FResultValueOrError> Result = MakeShared<FResultValueOrError, ESPMode::ThreadSafe>(MakeError(ErrorCode));
			ChunkDeltaPromise.SetValue(Result);
			if (Dependencies.OnComplete)
			{
				Dependencies.OnComplete(*Result);
			}
		}
	}

	int32 FOptimisedDelta::GetDownloadResponseCode() const
	{
		return DownloadResponseCode;
	}

	FOptimisedDeltaConfiguration::FOptimisedDeltaConfiguration(FBuildPatchAppManifestRef InDestinationManifest)
		: DestinationManifest(MoveTemp(InDestinationManifest))
		, DeltaPolicy(EDeltaPolicy::TryFetchContinueWithout)
		, InstallMode(EInstallMode::NonDestructiveInstall)
	{
	}

	FOptimisedDeltaDependencies::FOptimisedDeltaDependencies(TSharedRef<IUriProvider, ESPMode::ThreadSafe> InUriProvider)
		: UriProvider(MoveTemp(InUriProvider))
		, DownloadService(nullptr)
		, OnComplete([](const IOptimisedDelta::FResultValueOrError&) {})
	{
	}

	TSharedRef<IOptimisedDelta> FOptimisedDeltaFactory::Create(const FOptimisedDeltaConfiguration& Configuration, FOptimisedDeltaDependencies Dependencies)
	{
		check(Dependencies.DownloadService != nullptr);
		TSharedRef<FOptimisedDelta> Obj = MakeShared<FOptimisedDelta>(Configuration, MoveTemp(Dependencies));
		Obj->Initialize();
		return Obj;
	}
}
