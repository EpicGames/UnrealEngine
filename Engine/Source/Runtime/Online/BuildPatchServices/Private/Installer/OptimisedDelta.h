// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/ValueOrError.h"
#include "BuildPatchManifest.h"
#include "BuildPatchSettings.h"
#include "UriProvider.h"

namespace BuildPatchServices
{
	class IDownloadService;

	/**
	 * An interface providing access for retrieving the optimised delta manifest used to patch from a specific source to a specific destination.
	 */
	class IOptimisedDelta
	{
	public:
		virtual ~IOptimisedDelta() = default;
		typedef TValueOrError<FBuildPatchAppManifestPtr, FString> FResultValueOrError;

		/**
		 * Gets the manifest that should be used as the destination manifest, or the error instead if there was one.
		 * If DeltaPolicy configuration is EDeltaPolicy::TryFetchContinueWithout then the result is guarenteed to always contain a valid manifest, which
		 * may equal the original DestinationManifest provided.
		 * @return the destination manifest.
		 */
		virtual const IOptimisedDelta::FResultValueOrError& GetResult() const = 0;

		/**
		 * Gets the size of the metadata downloaded to create the optimised manifest.
		 * @return the downloaded size in bytes.
		 */
		virtual int32 GetMetaDownloadSize() const = 0;

		/**
		 * Checks if manifest returned by GetResult() is ready to be provided immediately.
		 * @return true if result can be obtained.
		 */
		virtual bool IsReady() const = 0;

		/**
		 * Gets the response code from the download of the optimised delta file.
		 * @return the response code, or 0 if no download was attempted.
		 */
		virtual int32 GetDownloadResponseCode() const = 0;
	};

	/**
	 * Defines a list of configuration details required for the IOptimisedDelta construction.
	 */
	struct FOptimisedDeltaConfiguration
	{
	public:
		/**
		 * Construct with destination manifest, this is a required param.
		 */
		FOptimisedDeltaConfiguration(FBuildPatchAppManifestRef DestinationManifest);

	public:
		// The installation provided source manifest.
		FBuildPatchAppManifestPtr SourceManifest;
		// The installation provided destination manifest.
		FBuildPatchAppManifestRef DestinationManifest;
		// The policy to follow for requesting an optimised delta.
		EDeltaPolicy DeltaPolicy;
		// The mode for installation.
		EInstallMode InstallMode;
		// A minimal number of retries for delta file download.
		int32_t RetriesNumber = 0;

		// This is an identifier that we append to the optimized delta filename in order to
		// facilitate multiple optimized deltas for the same build pairing.
		FString DeltaFilenameTrailer;
	};

	/**
	 * Defines a list of dependencies required for the IOptimisedDelta construction.
	 */
	struct FOptimisedDeltaDependencies
	{
	public:
		/**
		 * Constructor setting up default values.
		 */
		FOptimisedDeltaDependencies(TSharedRef<IUriProvider, ESPMode::ThreadSafe> InUriProvider);
	
	public:
		// A URI provider instance.
		TSharedRef<IUriProvider, ESPMode::ThreadSafe> UriProvider;
		// A download service instance.
		IDownloadService* DownloadService = nullptr;
		// Function to call once the destination manifest has been selected. Receives manifest or error.
		TUniqueFunction<void(const IOptimisedDelta::FResultValueOrError&)> OnComplete = [](const IOptimisedDelta::FResultValueOrError&) {};
	};

	class FOptimisedDeltaFactory
	{
	public:
		static TSharedRef<IOptimisedDelta> Create(const FOptimisedDeltaConfiguration& Configuration, FOptimisedDeltaDependencies Dependencies);
	};
}
