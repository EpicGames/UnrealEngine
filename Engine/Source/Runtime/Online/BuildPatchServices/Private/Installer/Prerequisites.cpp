// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Prerequisites.h"
#include "Misc/Paths.h"
#include "BuildPatchManifest.h"
#include "BuildPatchProgress.h"
#include "BuildPatchSettings.h"
#include "BuildPatchState.h"
#include "CoreMinimal.h"
#include "Interfaces/IBuildInstaller.h"
#include "Common/FileSystem.h"
#include "Core/Platform.h"
#include "Installer/InstallerAnalytics.h"
#include "Installer/InstallerError.h"
#include "Installer/MachineConfig.h"
#include "IBuildManifestSet.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPrerequisites, Log, All);
DEFINE_LOG_CATEGORY(LogPrerequisites);

namespace BuildPatchServices
{
	class FPrerequisites
		: public IPrerequisites
	{
	public:
		/**
		 * Constructor with configuration values.
		 * @param MachineConfig          A class responsible for loading and saving per machine configuration data.
		 * @param InstallerAnalytics     A pointer to the analytics object used to record analytics events about the installation.
		 * @param InstallerError         A pointer to the class used to record errors during installation.
		 * @param FileSystem             A pointer to a filesystem abstraction used to interact with the filesystem.
		 * @param Platform               A pointer to a platform abstraction used to perform platform operations.
		 */
		FPrerequisites(IMachineConfig* MachineConfig, IInstallerAnalytics* InstallerAnalytics, IInstallerError* InstallerError, IFileSystem* FileSystem, IPlatform* Platform);

		// IPrerequisites interface begin.
		bool RunPrereqs(const IBuildManifestSet* ManifestSet, const FBuildInstallerConfiguration& Configuration, const FString& InstallStagingDir, FBuildPatchProgress& BuildProgress) override;
		// IPrerequisites interface end.

	private:
		// A pointer to a helper class which manages loading and saving data about which prerequisites are installed.
		IMachineConfig* MachineConfig;

		// A pointer to the instance of installer analytics used to record analytics events.
		IInstallerAnalytics* InstallerAnalytics;

		// A pointer to the instance of IInstallerError which records errors during the installation process.
		IInstallerError* InstallerError;

		// A pointer to an abstraction representing the filesystem.
		IFileSystem* FileSystem;

		// A pointer to an abstraction representing the platform.
		IPlatform* Platform;
	};

	FPrerequisites::FPrerequisites(IMachineConfig* InMachineConfig, IInstallerAnalytics* InInstallerAnalytics, IInstallerError* InInstallerError, IFileSystem* InFileSystem, IPlatform* InPlatform)
		: MachineConfig(InMachineConfig)
		, InstallerAnalytics(InInstallerAnalytics)
		, InstallerError(InInstallerError)
		, FileSystem(InFileSystem)
		, Platform(InPlatform)
	{
	}

	bool FPrerequisites::RunPrereqs(const IBuildManifestSet* ManifestSet, const FBuildInstallerConfiguration& Configuration, const FString& InstallStagingDir, FBuildPatchProgress& BuildProgress)
	{
		BuildProgress.SetStateProgress(EBuildPatchState::PrerequisitesInstall, 0.0f);
		bool bInstallSuccessful = true;
		TArray<FPreReqInfo> PreReqInfo;
		ManifestSet->GetPreReqInfo(PreReqInfo);
		for (const FPreReqInfo& PreReq : PreReqInfo)
		{
			if (PreReq.Path.IsEmpty())
			{
				UE_LOGF(LogPrerequisites, Log, "Skipping prerequisites as manifest does not have prerequisites specified.");
				continue;
			}

			FString PrereqPath = PreReq.Path;

			// The prereq fields support some known variables.
			static const TCHAR* RootDirectoryVariable = TEXT("$[RootDirectory]");
			static const TCHAR* LogDirectoryVariable = TEXT("$[LogDirectory]");
			static const TCHAR* QuoteVariable = TEXT("$[Quote]");
			static const TCHAR* Quote = TEXT("\"");
			const FString InstallDirWithSlash = Configuration.InstallDirectory / TEXT("");
			const FString StageDirWithSlash = InstallStagingDir / TEXT("");
			const FString LogDirWithSlash = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir() / TEXT(""));

			// Load the collection of prerequisites we've already installed on this machine.
			TSet<FString> InstalledPrereqs = MachineConfig->LoadInstalledPrereqIds();

			// Check to see if we stored a successful run of this prerequisite already, and can therefore skip it.
			// We skip if the prerequisites installer already run and we are not attempting a repair.
			if (Configuration.bSkipPrereqIfAlreadyRan && !PreReq.bIsRepair && PreReq.IdSet.Num() > 0)
			{
				const TSet<FString> MissingPrereqs = PreReq.IdSet.Difference(InstalledPrereqs);
				if (MissingPrereqs.Num() == 0)
				{
					UE_LOGF(LogPrerequisites, Log, "Skipping already installed prerequisites installer");
					continue;
				}
			}

			bool bUsingInstallRoot = false;
			bool bUsingStageRoot = false;
			if (Configuration.InstallMode == EInstallMode::StageFiles)
			{
				PrereqPath = StageDirWithSlash + PreReq.Path;
				int64 FileSize;
				if (FileSystem->GetFileSize(*PrereqPath, FileSize))
				{
					bUsingStageRoot = true;
				}
			}
			if (!bUsingStageRoot)
			{
				PrereqPath = InstallDirWithSlash + PreReq.Path;
				int64 FileSize;
				if (FileSystem->GetFileSize(*PrereqPath, FileSize))
				{
					bUsingInstallRoot = true;
				}
			}
			// If we found no prerequisite above, then we have nothing to run and this is an error in the shipped build.
			if (!bUsingInstallRoot && !bUsingStageRoot)
			{
				UE_LOGF(LogPrerequisites, Error, "Could not find prerequisites file %ls on disk.", *PrereqPath);
				InstallerError->SetError(EBuildPatchInstallError::PrerequisiteError, PrerequisiteErrorPrefixes::NotFoundCode);
				bInstallSuccessful = false;
				continue;
			}

			const FString PrereqCommandline = PreReq.Args
				.Replace(RootDirectoryVariable, *InstallDirWithSlash)
				.Replace(LogDirectoryVariable, *LogDirWithSlash)
				.Replace(QuoteVariable, Quote);

			UE_LOGF(LogPrerequisites, Log, "Running prerequisites installer %ls %ls", *PrereqPath, *PrereqCommandline);

			// Prerequisites have to be ran elevated otherwise a background run of the prereq which asks for elevation itself
			// on some OSs will result in a minimised or un-focused request.
			int32 ReturnCode = INDEX_NONE;
			bool bPrereqInstallSuccessful = Platform->ExecElevatedProcess(*PrereqPath, *PrereqCommandline, &ReturnCode);
			if (!bPrereqInstallSuccessful)
			{
				ReturnCode = Platform->GetLastError();
				UE_LOGF(LogPrerequisites, Error, "Failed to start the prerequisites install process %u", ReturnCode);
				InstallerAnalytics->RecordPrereqInstallationError(PreReq.AppName, PreReq.VersionString, PrereqPath, PrereqCommandline, ReturnCode, TEXT("Failed to start installer"));
				InstallerError->SetError(EBuildPatchInstallError::PrerequisiteError, *FString::Printf(TEXT("%s%u"), PrerequisiteErrorPrefixes::ExecuteCode, ReturnCode));
			}
			else
			{
				if (ReturnCode != 0)
				{
					bool bSuspectedSuccess = false;
					if (!PreReq.bIsRepair)
					{
						// Here we try to catch any non-zero error codes that may represent a successful install.  For example, Microsoft installers like VCRedist
						//  installers that are passed the /norestart commandline arg will not show restart UX but will still return the 3010 error code.  Given this
						//  info, we can assume a successful run if we see /norestart AND a returned 3010 error code.  Another example would perhaps be if we saw a the
						//  well known Microsoft 1638 error which for things like VCRedist installer imply a newer version of VCRedist is already installed.  Since the
						//  newer VCRedist should be backwards compatible with the older version, we can assume this prereq is installed already.
						if (ReturnCode == 1638)
						{
							bSuspectedSuccess = true;
						}
						else if (ReturnCode == 3010 && PrereqCommandline.Contains(TEXT("/norestart")))
						{
							bSuspectedSuccess = true;
						}
					}

					if (bSuspectedSuccess)
					{
						UE_LOGF(LogPrerequisites, Log, "Prerequisites executable returned with code %u. This was interpreted as a successful run.", ReturnCode);
					}
					else
					{
						UE_LOGF(LogPrerequisites, Error, "Prerequisites executable failed with code %u", ReturnCode);
						InstallerAnalytics->RecordPrereqInstallationError(PreReq.AppName, PreReq.VersionString, PrereqPath, PrereqCommandline, ReturnCode, TEXT("Failed to install"));
						bPrereqInstallSuccessful = false;
						InstallerError->SetError(EBuildPatchInstallError::PrerequisiteError, *FString::Printf(TEXT("%s%u"), PrerequisiteErrorPrefixes::ReturnCode, ReturnCode));
					}
				}
			}

			if (bPrereqInstallSuccessful)
			{
				UE_LOGF(LogPrerequisites, Log, "Prerequisite installation successful");
				InstalledPrereqs.Append(PreReq.IdSet);
				MachineConfig->SaveInstalledPrereqIds(InstalledPrereqs);
			}
			bInstallSuccessful = bInstallSuccessful && bPrereqInstallSuccessful;
		}
		if (bInstallSuccessful)
		{
			BuildProgress.SetStateProgress(EBuildPatchState::PrerequisitesInstall, 1.0f);
		}
		return bInstallSuccessful;
	}

	IPrerequisites* FPrerequisitesFactory::Create(IMachineConfig* MachineConfig, IInstallerAnalytics* InstallerAnalytics, IInstallerError* InstallerError, IFileSystem* FileSystem, IPlatform* Platform)
	{
		return new FPrerequisites(MachineConfig, InstallerAnalytics, InstallerError, FileSystem, Platform);
	}
};
