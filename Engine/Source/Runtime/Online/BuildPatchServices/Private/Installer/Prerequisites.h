// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BuildPatchManifest.h"


namespace BuildPatchServices
{
	class IMachineConfig;
	class IInstallerAnalytics;
	class IInstallerError;
	class IFileSystem;
	class IPlatform;
	struct FBuildInstallerConfiguration;
	struct FBuildPatchProgress;
	class IBuildManifestSet;

	class IPrerequisites
	{
	public:
		/**
		 * Virtual destructor.
		 */
		virtual ~IPrerequisites() = default;

		/**
		 * Runs any prerequisites associated with the installation.
		 * @param BuildManifest       The manifest containing details of the prerequisite installer.
		 * @param Configuration       The installer configuration structure.
		 * @param InstallStagingDir   The directory within staging to construct install files to.
		 * @param BuildProgress       Used to keep track of install progress.
		 * @return                    Returns true if the prerequisites installer succeeded, false otherwise.
		 */
		virtual bool RunPrereqs(const IBuildManifestSet* ManifestSet, const FBuildInstallerConfiguration& Configuration, const FString& InstallStagingDir, FBuildPatchProgress& BuildProgress) = 0;
	};

	/**
	 * A factory for creating an IPrerequisites instance.
	 */
	class FPrerequisitesFactory
	{
	public:
		/**
		 * Creates an instance of IPrerequisites.
		 * @param MachineConfig           A class responsible for loading and saving per machine configuration data.
		 * @param InstallerAnalytics      The analytics implementation for reporting the installer events.
		 * @param InstallerError          The error handling implementation which any installation errors will be reported to.
		 * @param FileSystem              An abstraction representing the filesystem.
		 * @param Platform                An abstraction providing access to platform operations.
		 * @return the new IPrerequisites instance created.
		 */
		static IPrerequisites* Create(IMachineConfig* MachineConfig, IInstallerAnalytics* InstallerAnalytics, IInstallerError* InstallerError, IFileSystem* FileSystem, IPlatform* Platform);
	};
}
