// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/ChunkDbChunkSourceStatistics.h"
#include "CoreMinimal.h"
#include "Core/AsyncHelpers.h"
#include "Common/SpeedRecorder.h"
#include "Installer/Statistics/FileOperationTracker.h"

namespace BuildPatchServices
{
	class FChunkDbChunkSourceStatistics
		: public IChunkDbChunkSourceStatistics
	{
	public:
		FChunkDbChunkSourceStatistics(ISpeedRecorder* SpeedRecorder, IFileOperationTracker* FileOperationTracker);
		~FChunkDbChunkSourceStatistics();

	public:
		// IChunkDbChunkSourceStat interface begin.
		virtual void OnLoadStarted(const FGuid& ChunkId) override;
		virtual void OnLoadComplete(const FGuid& ChunkId, ELoadResult Result) override;
		virtual void OnReadComplete(const ISpeedRecorder::FRecord& Record) override;
		// IChunkDbChunkSourceStat interface end.

		// IChunkDbChunkSourceStatistics interface begin.
		virtual int32 GetNumSuccessfulLoads() const override;
		virtual int32 GetNumFailedLoads() const override;
		virtual bool IsCurrentlyReading() const override { return bIsReading.load(std::memory_order_relaxed); }
		// IChunkDbChunkSourceStatistics interface end.

	private:
		ISpeedRecorder* SpeedRecorder;
		IFileOperationTracker* FileOperationTracker;
		FThreadSafeInt32 NumSuccessfulLoads;
		FThreadSafeInt32 NumFailedLoads;
		std::atomic_bool bIsReading;
	};

	FChunkDbChunkSourceStatistics::FChunkDbChunkSourceStatistics(ISpeedRecorder* InSpeedRecorder, IFileOperationTracker* InFileOperationTracker)
		: SpeedRecorder(InSpeedRecorder)
		, FileOperationTracker(InFileOperationTracker)
		, NumSuccessfulLoads(0)
		, NumFailedLoads(0)
		, bIsReading(false)
	{
	}

	FChunkDbChunkSourceStatistics::~FChunkDbChunkSourceStatistics()
	{
	}

	void FChunkDbChunkSourceStatistics::OnLoadStarted(const FGuid& ChunkId)
	{
		FileOperationTracker->OnDataStateUpdate(ChunkId, EFileOperationState::RetrievingLocalChunkDbData);
		bIsReading.store(true, std::memory_order_relaxed);
	}

	void FChunkDbChunkSourceStatistics::OnReadComplete(const ISpeedRecorder::FRecord& Record)
	{
		SpeedRecorder->AddRecord(Record);
		bIsReading.store(false, std::memory_order_relaxed);
	}

	void FChunkDbChunkSourceStatistics::OnLoadComplete(const FGuid& ChunkId, ELoadResult Result)
	{
		if (Result == ELoadResult::Success)
		{
			NumSuccessfulLoads.Increment();
		}
		else
		{
			NumFailedLoads.Increment();
		}
		
	}

	int32 FChunkDbChunkSourceStatistics::GetNumSuccessfulLoads() const
	{
		return NumSuccessfulLoads.GetValue();
	}

	int32 FChunkDbChunkSourceStatistics::GetNumFailedLoads() const
	{
		return NumFailedLoads.GetValue();
	}

	IChunkDbChunkSourceStatistics* FChunkDbChunkSourceStatisticsFactory::Create(ISpeedRecorder* SpeedRecorder, IFileOperationTracker* FileOperationTracker)
	{
		check(SpeedRecorder != nullptr);
		check(FileOperationTracker != nullptr);
		return new FChunkDbChunkSourceStatistics(SpeedRecorder, FileOperationTracker);
	}
};