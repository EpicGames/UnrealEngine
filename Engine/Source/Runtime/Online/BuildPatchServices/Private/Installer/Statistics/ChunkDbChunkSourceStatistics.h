// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/ChunkDbChunkSource.h"

namespace BuildPatchServices
{
	class ISpeedRecorder;
	class IFileOperationTracker;

	/**
	 * Interface to the statistics class which provides access to tracked values from a chunkdb chunk source stat.
	 */
	class IChunkDbChunkSourceStatistics
		: public IChunkDbChunkSourceStat
	{
	public:
		/**
		 * @return the number of chunks successfully read from chunkdbs.
		 */
		virtual int32 GetNumSuccessfulLoads() const = 0;

		/**
		 * @return the number of chunks which failed to load from provided chunkdbs.
		 */
		virtual int32 GetNumFailedLoads() const = 0;

		virtual bool IsCurrentlyReading() const = 0;
	};

	/**
	 * A factory for creating an IChunkDbChunkSourceStatistics instance.
	 */
	class FChunkDbChunkSourceStatisticsFactory
	{
	public:
		/**
		 * Creates the chunkdb chunk source's dependency interface and exposes additional information.
		 * @param SpeedRecorder         The speed recorder instance that we send activity records to.
		 * @param FileOperationTracker  The file operation tracker which will be used to update data states.
		 * @return the new IChunkDbChunkSourceStatistics instance created.
		 */
		static IChunkDbChunkSourceStatistics* Create(ISpeedRecorder* SpeedRecorder, IFileOperationTracker* FileOperationTracker);
	};
}