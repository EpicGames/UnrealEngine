// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/CloudChunkSourceStat.h"

#include "Common/StatsCollector.h"

namespace BuildPatchServices
{
	class FCloudChunkSourceStat
		: public ICloudChunkSourceStat
	{
	public:
		FCloudChunkSourceStat(FStatsCollector* InStatsCollector)
			: StatDownloadHealth(InStatsCollector->CreateStat(TEXT("CloudSource: Download Health"), EStatFormat::Value))
			, StatDownloadSuccessRate(InStatsCollector->CreateStat(TEXT("CloudSource: Success rate"), EStatFormat::Percentage))
			, StatActiveRequestCount(InStatsCollector->CreateStat(TEXT("CloudSource: Active requests"), EStatFormat::Value))
			, StatActiveRequestCountAvg(InStatsCollector->CreateStat(TEXT("CloudSource: Active requests average"), EStatFormat::Value))
			, StatDownloadRequestCount(InStatsCollector->CreateStat(TEXT("CloudSource: Total requests"), EStatFormat::Value))
			, StatDownloadSuccessCount(InStatsCollector->CreateStat(TEXT("CloudSource: Success count"), EStatFormat::Value))
			, StatDownloadFailedCount(InStatsCollector->CreateStat(TEXT("CloudSource: Fail count"), EStatFormat::Value))
			, StatDownloadCorruptCount(InStatsCollector->CreateStat(TEXT("CloudSource: Corrupt count"), EStatFormat::Value))
			, StatDownloadAbortedCount(InStatsCollector->CreateStat(TEXT("CloudSource: Abort count"), EStatFormat::Value))
			, StatTotalDecryptCycles(InStatsCollector->CreateStat(TEXT("CloudSource: Total Decrypt Time"), EStatFormat::Timer))
		{
		}

		~FCloudChunkSourceStat() { }

		// ICloudChunkSourceStat interface begin.
		virtual void OnDownloadRequested(const FGuid& ChunkId) override
		{
			FStatsCollector::Accumulate(StatDownloadRequestCount, 1);
		}

		virtual void OnDownloadSuccess(const FGuid& ChunkId) override
		{
			FStatsCollector::Accumulate(StatDownloadSuccessCount, 1);
		}

		virtual void OnDownloadFailed(const FGuid& ChunkId, const FString& Url) override
		{
			FStatsCollector::Accumulate(StatDownloadFailedCount, 1);
		}

		virtual void OnDownloadCorrupt(const FGuid& ChunkId, const FString& Url, EChunkLoadResult LoadResult) override
		{
			FStatsCollector::Accumulate(StatDownloadCorruptCount, 1);
		}

		virtual void OnDownloadAborted(const FGuid& ChunkId, const FString& Url, double DownloadTimeMean, double DownloadTimeStd, double DownloadTime, double BreakingPoint) override
		{
			FStatsCollector::Accumulate(StatDownloadAbortedCount, 1);
		}

		virtual void OnReceivedDataUpdated(int64 TotalBytes) override { }

		virtual void OnRequiredDataUpdated(int64 TotalBytes) override { }

		virtual void OnDownloadHealthUpdated(EBuildPatchDownloadHealth DownloadHealth) override
		{
			FStatsCollector::Set(StatDownloadHealth, static_cast<int64>(DownloadHealth));
		}

		virtual void OnSuccessRateUpdated(float SuccessRate) override
		{
			FStatsCollector::SetAsPercentage(StatDownloadSuccessRate, SuccessRate);
		}

		virtual void OnActiveRequestCountUpdated(uint32 RequestCount) override
		{
			FStatsCollector::Set(StatActiveRequestCount, RequestCount);
		}

		virtual void OnActiveRequestCountAvgUpdated(int32 RequestCountAvg) override
		{
			FStatsCollector::Set(StatActiveRequestCountAvg, RequestCountAvg);
		}

		virtual void OnAcceptedNewRequirements(const TSet<FGuid>& ChunkIds) override { }

		virtual void OnChunkDecrypted(uint64 DecryptionCycles) override
		{
			FStatsCollector::Accumulate(StatTotalDecryptCycles, DecryptionCycles);
		}

		virtual void OnCloudDirectoryDataReceived(const FString& CloudDirectory, uint64 ReceivedData) override { }

		virtual void OnCloudDirectoryError(const FString& CloudDirectory) override { }
		// ICloudChunkSourceStat interface end.

	private:
		volatile FStatsCollector::FAtomicValue* StatDownloadHealth;
		volatile FStatsCollector::FAtomicValue* StatDownloadSuccessRate;
		volatile FStatsCollector::FAtomicValue* StatActiveRequestCount;
		volatile FStatsCollector::FAtomicValue* StatActiveRequestCountAvg;
		volatile FStatsCollector::FAtomicValue* StatDownloadRequestCount;
		volatile FStatsCollector::FAtomicValue* StatDownloadSuccessCount;
		volatile FStatsCollector::FAtomicValue* StatDownloadFailedCount;
		volatile FStatsCollector::FAtomicValue* StatDownloadCorruptCount;
		volatile FStatsCollector::FAtomicValue* StatDownloadAbortedCount;
		volatile FStatsCollector::FAtomicValue* StatTotalDecryptCycles;
	};

	ICloudChunkSourceStat* FCloudChunkSourceStatFactory::Create(FStatsCollector* InStatsCollector)
	{
		check(InStatsCollector != nullptr);
		return new FCloudChunkSourceStat(InStatsCollector);
	}
};