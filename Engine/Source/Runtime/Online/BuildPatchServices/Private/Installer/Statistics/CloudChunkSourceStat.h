// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/CloudChunkSource.h"

namespace BuildPatchServices
{
	class FStatsCollector;
	/**
	 * A factory for creating an ICloudChunkSourceStat instance.
	 */
	class FCloudChunkSourceStatFactory
	{
	public:
		/**
		 * Creates the cloud source stat dependency interface and exposes wraps stats collector.
		 * @param InStatsCollector    The stats collector which will be used to collect statistics on cloud source events.
		 * @return the new ICloudChunkSourceStat instance created.
		 */
		static ICloudChunkSourceStat* Create(FStatsCollector* InStatsCollector);
	};
}