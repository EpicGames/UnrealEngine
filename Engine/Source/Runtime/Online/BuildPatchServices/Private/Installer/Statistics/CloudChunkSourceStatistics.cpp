// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/CloudChunkSourceStatistics.h"
#include "Misc/ScopeLock.h"
#include "Algo/Find.h"
#include "Core/AsyncHelpers.h"
#include "Common/StatsCollector.h"
#include "Installer/InstallerAnalytics.h"
#include "Installer/Statistics/FileOperationTracker.h"
#include "BuildPatchProgress.h"

namespace BuildPatchServices
{
	class FCloudChunkSourceStatistics
		: public ICloudChunkSourceStatistics
	{
		static const int32 SuccessRateMultiplier = 10000;
	public:
		FCloudChunkSourceStatistics(IInstallerAnalytics* InstallerAnalytics, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker);
		~FCloudChunkSourceStatistics();

		// ICloudChunkSourceStat interface begin.
		virtual void OnDownloadRequested(const FGuid& ChunkId) override;
		virtual void OnDownloadSuccess(const FGuid& ChunkId) override;
		virtual void OnDownloadFailed(const FGuid& ChunkId, const FString& Url) override;
		virtual void OnDownloadCorrupt(const FGuid& ChunkId, const FString& Url, EChunkLoadResult LoadResult) override;
		virtual void OnDownloadAborted(const FGuid& ChunkId, const FString& Url, double DownloadTimeMean, double DownloadTimeStd, double DownloadTime, double BreakingPoint) override;
		virtual void OnReceivedDataUpdated(int64 TotalBytes) override;
		virtual void OnRequiredDataUpdated(int64 TotalBytes) override;
		virtual void OnDownloadHealthUpdated(EBuildPatchDownloadHealth DownloadHealth) override;
		virtual void OnSuccessRateUpdated(float SuccessRate) override;
		virtual void OnActiveRequestCountUpdated(uint32 RequestCount) override;
		virtual void OnActiveRequestCountAvgUpdated(int32 AverageRequestCount) override;
		virtual void OnAcceptedNewRequirements(const TSet<FGuid>& ChunkIds) override;
		virtual void OnChunkDecrypted(uint64 DecryptionCycles) override;
		virtual void OnCloudDirectoryDataReceived(const FString& CloudDirectory, uint64 BytesDownloaded) override;
		virtual void OnCloudDirectoryError(const FString& CloudDirectory) override;
		// ICloudChunkSourceStat interface end.

		// ICloudChunkSourceStatistics interface begin.
		virtual uint64 GetRequiredDownloadSize() const override;
		virtual uint64 GetDownloadedBytes() const override;
		virtual uint64 GetNumCorruptChunkDownloads() const override;
		virtual uint64 GetNumAbortedChunkDownloads() const override;
		virtual float GetDownloadSuccessRate() const override;
		virtual EBuildPatchDownloadHealth GetDownloadHealth() const override;
		virtual TArray<float> GetDownloadHealthTimers() const override;
		virtual uint32 GetCurrentRequestCount() const override;
		virtual uint32 GetPeakRequestCount() const override;
		virtual uint32 GetAvgRequestCount() const override;
		virtual const TCHAR* GetExperimentVariant() const override;
		virtual double GetTotalDecryptTime() const override;
		virtual const TArray<FCloudDirectoryStats>& GetCloudDirectoryStats() const override;
		// ICloudChunkSourceStatistics interface end.

	private:
		IInstallerAnalytics* InstallerAnalytics;
		FBuildPatchProgress* BuildProgress;
		IFileOperationTracker* FileOperationTracker;
		FThreadSafeInt64 TotalBytesReceived;
		FThreadSafeInt64 TotalBytesRequired;
		FThreadSafeInt32 NumDownloadsCorrupt;
		FThreadSafeInt32 NumDownloadsAborted;
		FThreadSafeInt32 ChunkSuccessRate;
		FThreadSafeInt32 CurrentRequestCount;
		volatile uint32 PeakRequestCount;
		FThreadSafeInt32 AvgRequestCount;
		mutable FCriticalSection ThreadLockCs;
		EBuildPatchDownloadHealth CurrentHealth;
		int64 CyclesAtLastHealthState;
		TArray<float> HealthStateTimes;
		FThreadSafeInt64 TotalDecryptCycles;
		TArray<FCloudDirectoryStats> CloudDirectoryStats;
	};

	FCloudChunkSourceStatistics::FCloudChunkSourceStatistics(IInstallerAnalytics* InInstallerAnalytics, FBuildPatchProgress* InBuildProgress, IFileOperationTracker* InFileOperationTracker)
		: InstallerAnalytics(InInstallerAnalytics)
		, BuildProgress(InBuildProgress)
		, FileOperationTracker(InFileOperationTracker)
		, TotalBytesReceived(0)
		, TotalBytesRequired(0)
		, NumDownloadsCorrupt(0)
		, NumDownloadsAborted(0)
		, ChunkSuccessRate(0)
		, CurrentRequestCount(0)
		, PeakRequestCount(0)
		, AvgRequestCount(0)
		, ThreadLockCs()
		, CurrentHealth(EBuildPatchDownloadHealth::Excellent)
		, CyclesAtLastHealthState(0)
		, TotalDecryptCycles(0)
		, CloudDirectoryStats()
	{
		// Initialize health states to zero time.
		HealthStateTimes.AddZeroed((int32)EBuildPatchDownloadHealth::NUM_Values);
	}

	FCloudChunkSourceStatistics::~FCloudChunkSourceStatistics()
	{
	}

	void FCloudChunkSourceStatistics::OnDownloadRequested(const FGuid& ChunkId)
	{
		FileOperationTracker->OnDataStateUpdate(ChunkId, EFileOperationState::RetrievingRemoteCloudData);
	}

	void FCloudChunkSourceStatistics::OnDownloadSuccess(const FGuid& ChunkId)
	{
	}

	void FCloudChunkSourceStatistics::OnDownloadFailed(const FGuid& ChunkId, const FString& Url)
	{
	}

	void FCloudChunkSourceStatistics::OnDownloadCorrupt(const FGuid& ChunkId, const FString& Url, EChunkLoadResult LoadResult)
	{
		InstallerAnalytics->RecordChunkDownloadError(Url, INDEX_NONE, ToString(LoadResult));
		NumDownloadsCorrupt.Increment();
	}

	void FCloudChunkSourceStatistics::OnDownloadAborted(const FGuid& ChunkId, const FString& Url, double DownloadTimeMean, double DownloadTimeStd, double DownloadTime, double BreakingPoint)
	{
		InstallerAnalytics->RecordChunkDownloadAborted(Url, DownloadTime, DownloadTimeMean, DownloadTimeStd, BreakingPoint);
		NumDownloadsAborted.Increment();
	}

	void FCloudChunkSourceStatistics::OnReceivedDataUpdated(int64 TotalBytes)
	{
		TotalBytesReceived.Set(TotalBytes);
		int64 Required = TotalBytesRequired.GetValue();
		if (Required > 0)
		{
			BuildProgress->SetStateProgress(EBuildPatchState::Downloading, (double)TotalBytes / (double)Required);
		}
	}

	void FCloudChunkSourceStatistics::OnRequiredDataUpdated(int64 TotalBytes)
	{
		TotalBytesRequired.Set(TotalBytes);
		int64 Received = TotalBytesReceived.GetValue();
		if (TotalBytes > 0)
		{
			BuildProgress->SetStateProgress(EBuildPatchState::Downloading, (double)Received / (double)TotalBytes);
		}
	}

	void FCloudChunkSourceStatistics::OnDownloadHealthUpdated(EBuildPatchDownloadHealth DownloadHealth)
	{
		FScopeLock Lock(&ThreadLockCs);
		// Update time in state.
		uint64 CyclesNow = FStatsCollector::GetCycles();
		if (CyclesAtLastHealthState > 0)
		{
			HealthStateTimes[(int32)CurrentHealth] += FStatsCollector::CyclesToSeconds(CyclesNow - CyclesAtLastHealthState);
		}
		CurrentHealth = DownloadHealth;
		FPlatformAtomics::InterlockedExchange(&CyclesAtLastHealthState, CyclesNow);
	}

	void FCloudChunkSourceStatistics::OnSuccessRateUpdated(float SuccessRate)
	{
		// The success rate comes as a 0-1 value. We can multiply it up and use atomics still.
		ChunkSuccessRate.Set(SuccessRate * SuccessRateMultiplier);
	}

	void FCloudChunkSourceStatistics::OnActiveRequestCountUpdated(uint32 RequestCount)
	{
		BuildProgress->SetIsDownloading(RequestCount > 0);
		CurrentRequestCount.Set(RequestCount);
		// Sorry for the casting. Required to get it to build.
		AsyncHelpers::LockFreePeak<int32>((volatile int32*)&PeakRequestCount, (int32)RequestCount);
	}

	void FCloudChunkSourceStatistics::OnActiveRequestCountAvgUpdated(int32 InAverageRequestCount)
	{
		AvgRequestCount.Set(InAverageRequestCount);
	}

	void FCloudChunkSourceStatistics::OnAcceptedNewRequirements(const TSet<FGuid>& ChunkIds)
	{
		FileOperationTracker->OnDataStateUpdate(ChunkIds, EFileOperationState::PendingRemoteCloudData);
	}

	void FCloudChunkSourceStatistics::OnChunkDecrypted(uint64 DecryptionCycles)
	{
		TotalDecryptCycles.Add(DecryptionCycles);
	}

	void FCloudChunkSourceStatistics::OnCloudDirectoryDataReceived(const FString& CloudDirectory, uint64 ReceivedData)
	{
		FCloudDirectoryStats* Stats = Algo::FindBy(CloudDirectoryStats, CloudDirectory, &FCloudDirectoryStats::CloudDirectory);
		if (Stats != nullptr)
		{
			Stats->TotalReceivedData += ReceivedData;
		}
		else
		{
			FCloudDirectoryStats NewStats;
			NewStats.CloudDirectory = CloudDirectory;
			NewStats.TotalReceivedData = ReceivedData;
			CloudDirectoryStats.Add(NewStats);
		}
	}

	void FCloudChunkSourceStatistics::OnCloudDirectoryError(const FString& CloudDirectory)
	{
		FCloudDirectoryStats* Stats = Algo::FindBy(CloudDirectoryStats, CloudDirectory, &FCloudDirectoryStats::CloudDirectory);
		if (Stats != nullptr)
		{
			Stats->ErrorCount += 1;
		}
		else
		{
			FCloudDirectoryStats NewStats;
			NewStats.CloudDirectory = CloudDirectory;
			NewStats.ErrorCount = 1;
			CloudDirectoryStats.Add(NewStats);
		}
	}

	uint64 FCloudChunkSourceStatistics::GetRequiredDownloadSize() const
	{
		return TotalBytesRequired.GetValue();
	}

	uint64 FCloudChunkSourceStatistics::GetDownloadedBytes() const
	{
		return TotalBytesReceived.GetValue();
	}

	uint64 FCloudChunkSourceStatistics::GetNumCorruptChunkDownloads() const
	{
		return NumDownloadsCorrupt.GetValue();
	}

	uint64 FCloudChunkSourceStatistics::GetNumAbortedChunkDownloads() const
	{
		return NumDownloadsAborted.GetValue();
	}

	float FCloudChunkSourceStatistics::GetDownloadSuccessRate() const
	{
		return (float)ChunkSuccessRate.GetValue() / (float)SuccessRateMultiplier;
	}

	EBuildPatchDownloadHealth FCloudChunkSourceStatistics::GetDownloadHealth() const
	{
		FScopeLock Lock(&ThreadLockCs);
		return CurrentHealth;
	}

	TArray<float> FCloudChunkSourceStatistics::GetDownloadHealthTimers() const
	{
		FScopeLock Lock(&ThreadLockCs);
		return HealthStateTimes;
	}

	uint32 FCloudChunkSourceStatistics::GetCurrentRequestCount() const
	{
		return (uint32)CurrentRequestCount.GetValue();
	}

	uint32 FCloudChunkSourceStatistics::GetPeakRequestCount() const
	{
		return PeakRequestCount;
	}

	uint32 FCloudChunkSourceStatistics::GetAvgRequestCount() const
	{
		return AvgRequestCount.GetValue();
	}

	const TCHAR* FCloudChunkSourceStatistics::GetExperimentVariant() const
	{
		return nullptr;
	}

	double FCloudChunkSourceStatistics::GetTotalDecryptTime() const
	{
		return FStatsCollector::CyclesToSeconds(TotalDecryptCycles.GetValue());
	}

	const TArray<FCloudDirectoryStats>& FCloudChunkSourceStatistics::GetCloudDirectoryStats() const
	{
		return CloudDirectoryStats;
	}

	ICloudChunkSourceStatistics* FCloudChunkSourceStatisticsFactory::Create(IInstallerAnalytics* InstallerAnalytics, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker)
	{
		check(InstallerAnalytics != nullptr);
		check(BuildProgress != nullptr);
		check(FileOperationTracker != nullptr);
		return new FCloudChunkSourceStatistics(InstallerAnalytics, BuildProgress, FileOperationTracker);
	}
};