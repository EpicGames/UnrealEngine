// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/CloudChunkSource.h"

namespace BuildPatchServices
{
	class IInstallerAnalytics;
	struct FBuildPatchProgress;
	class IFileOperationTracker;

	/**
	 * Interface to the statistics class which provides access to tracked values from a cloud chunk source stat.
	 */
	class ICloudChunkSourceStatistics
		: public ICloudChunkSourceStat
	{
	public:
		/**
		 * @return the number of bytes that the installation required from cloud sources.
		 */
		virtual uint64 GetRequiredDownloadSize() const = 0;

		/**
		 * @return the number of bytes received so far.
		 */
		virtual uint64 GetDownloadedBytes() const = 0;

		/**
		 * @return the number of successful chunk downloads which had invalid data.
		 */
		virtual uint64 GetNumCorruptChunkDownloads() const = 0;

		/**
		 * @return the number of chunk downloads which were aborted, having been determined as lagging.
		 */
		virtual uint64 GetNumAbortedChunkDownloads() const = 0;

		/**
		 * @return the rate of success for chunks download requests, 1.0 being 100%.
		 */
		virtual float GetDownloadSuccessRate() const = 0;

		/**
		 * @return the EBuildPatchDownloadHealth value which the success rate applies to according to the configured ranges.
		 */
		virtual EBuildPatchDownloadHealth GetDownloadHealth() const = 0;

		/**
		 * @return an array of seconds spent in each download health range, indexable by EBuildPatchDownloadHealth.
		 */
		virtual TArray<float> GetDownloadHealthTimers() const = 0;

		/**
		 * @return the current number of download requests.
		 */
		virtual uint32 GetCurrentRequestCount() const = 0;

		/**
		 * @return the peak number of download requests.
		 */
		virtual uint32 GetPeakRequestCount() const = 0;

		/**
		 * @return the avg number of download requests.
		 */
		virtual uint32 GetAvgRequestCount() const = 0;

		/**
		 *  @return the A/B Experiment if applicable
		 */
		virtual const TCHAR* GetExperimentVariant() const = 0;

		/**
		 * @return the total time in seconds spent on chunks decryption.
		 */
		virtual double GetTotalDecryptTime() const = 0;

		/**
		 * @return the stats collected for each cloud directory.
		 */
		virtual const TArray<FCloudDirectoryStats>& GetCloudDirectoryStats() const = 0;
	};

	/**
	 * A factory for creating an ICloudChunkSourceStatistics instance.
	 */
	class FCloudChunkSourceStatisticsFactory
	{
	public:
		/**
		 * Creates the cloud chunk source's dependency interface and exposes additional information.
		 * @param InstallerAnalytics    The analytics implementation for reporting the cloud source events.
		 * @param BuildProgress         The legacy progress implementation to bridge the system stats to.
		 * @param FileOperationTracker  The file operation tracker which will be used to update data states.
		 * @return the new ICloudChunkSourceStatistics instance created.
		 */
		static ICloudChunkSourceStatistics* Create(IInstallerAnalytics* InstallerAnalytics, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker);
	};
}