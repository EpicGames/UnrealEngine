// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/DownloadServiceStatistics.h"
#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"
#include "Interfaces/IHttpResponse.h"

#include "Core/AsyncHelpers.h"
#include "Common/StatsCollector.h"
#include "Installer/InstallerAnalytics.h"
#include "BuildPatchProgress.h"
#include "BuildPatchManifest.h"
#include "BuildPatchUtil.h"
#include "Containers/Queue.h"

namespace BuildPatchServices
{
	class FDownloadServiceStatistics
		: public IDownloadServiceStatistics
	{
	public:
		FDownloadServiceStatistics(const TSharedPtr<ISpeedRecorder>& SpeedRecorder, const TSharedPtr<IDataSizeProvider>& DataSizeProvider, const TSharedPtr<IInstallerAnalytics>& InstallerAnalytics);

		// IDownloadServiceStat interface begin.
		virtual void OnDownloadStarted(int32 RequestId, const FString& Uri) override;
		virtual void OnDownloadProgress(int32 RequestId, uint64 BytesReceived) override;
		virtual void OnDownloadComplete(const FDownloadRecord& DownloadRecord) override;
		// IDownloadServiceStat interface end.

		// IDownloadServiceStatistics interface begin.
		virtual uint64 GetBytesDownloaded() const override;
		virtual int32 GetNumSuccessfulChunkDownloads() const override;
		virtual int32 GetNumFailedChunkDownloads() const override;
		virtual int32 GetNumCurrentDownloads() const override;
		virtual TArray<FDownload> GetCurrentDownloads() const override;
		virtual TPair<double, uint32> GetImmediateAverageSpeedPerRequest(uint32 MinCount) override;
		virtual void Reset() override;
		// IDownloadServiceStatistics interface end.

	private:
		TWeakPtr<ISpeedRecorder> SpeedRecorder; // AddRecord is safe, but SpeedRecorder is ticked
		TWeakPtr<IDataSizeProvider> DataSizeProvider;
		TWeakPtr<IInstallerAnalytics> InstallerAnalytics;
		
		std::atomic<uint64> TotalBytesReceived;
		std::atomic<int32> NumSuccessfulDownloads;
		std::atomic<int32> NumFailedDownloads;

		typedef TTuple<FString, uint64> FDownloadTuple;
		TMap<int32, FDownloadTuple> Downloads;
		mutable FCriticalSection DownloadsCriticalSection;

		double AccumulatedRequestSpeed;
		uint32 AverageSpeedSampleCount;
		mutable FCriticalSection AverageSpeedCriticalSection;
	};

	FDownloadServiceStatistics::FDownloadServiceStatistics(const TSharedPtr<ISpeedRecorder>& InSpeedRecorder, const TSharedPtr<IDataSizeProvider>& InDataSizeProvider, const TSharedPtr<IInstallerAnalytics>& InInstallerAnalytics)
		: SpeedRecorder(InSpeedRecorder)
		, DataSizeProvider(InDataSizeProvider)
		, InstallerAnalytics(InInstallerAnalytics)
		, TotalBytesReceived(0)
		, NumSuccessfulDownloads(0)
		, NumFailedDownloads(0)
		, AccumulatedRequestSpeed(0.0)
		, AverageSpeedSampleCount(0U)
	{
	}

	void FDownloadServiceStatistics::OnDownloadStarted(int32 RequestId, const FString& Uri)
	{
		FScopeLock Lock(&DownloadsCriticalSection);
		FDownloadTuple& DownloadTuple = Downloads.FindOrAdd(RequestId);
		DownloadTuple.Get<0>() = Uri;
		DownloadTuple.Get<1>() = 0;
	}

	void FDownloadServiceStatistics::OnDownloadProgress(int32 RequestId, uint64 BytesReceived)
	{
		FScopeLock Lock(&DownloadsCriticalSection);
		FDownloadTuple& DownloadTuple = Downloads.FindOrAdd(RequestId);
		DownloadTuple.Get<1>() = BytesReceived;
	}

	void FDownloadServiceStatistics::OnDownloadComplete(const FDownloadRecord& DownloadRecord)
	{
		{
			FScopeLock Lock(&DownloadsCriticalSection);
			Downloads.Remove(DownloadRecord.RequestId);
		}

		if (DownloadRecord.bSuccess && EHttpResponseCodes::IsOk(DownloadRecord.ResponseCode))
		{
			TotalBytesReceived += DownloadRecord.SpeedRecord.Size;
			NumSuccessfulDownloads++;
			if (const TSharedPtr<ISpeedRecorder> SpeedRecorderPtr = SpeedRecorder.Pin())
			{
				SpeedRecorderPtr->AddRecord(DownloadRecord.SpeedRecord);
			}

			// update average speed
			const uint64 Cycles = DownloadRecord.SpeedRecord.CyclesEnd - DownloadRecord.SpeedRecord.CyclesStart;
			double Speed = Cycles > 0U ? DownloadRecord.SpeedRecord.Size / FStatsCollector::CyclesToSeconds(Cycles) : 0.0;
			FScopeLock Lock(&AverageSpeedCriticalSection);
			AccumulatedRequestSpeed += Speed;
			AverageSpeedSampleCount += 1;
		}
		else if (DownloadRecord.FailureReason != EHttpFailureReason::Cancelled)
		{
			// Ignore deltas as they might not exist. OptimisedDelta enforces the DeltaPolicy and handles errors if deltas are required
			FString UriFileExtension = FPaths::GetExtension(DownloadRecord.Uri);
			if (UriFileExtension.Equals(TEXT("delta"), ESearchCase::IgnoreCase) && (DownloadRecord.ResponseCode == EHttpResponseCodes::Forbidden || DownloadRecord.ResponseCode == EHttpResponseCodes::NotFound))
			{
				// Parse installed/installing version strings for analytics
				FString PatchVersion = FPaths::GetBaseFilename(DownloadRecord.Uri);
				FString BaseVersion = FPaths::GetCleanFilename(FPaths::GetPath(DownloadRecord.Uri));

				if (const TSharedPtr<IInstallerAnalytics> InstallerAnalyticsPtr = InstallerAnalytics.Pin())
				{
					InstallerAnalyticsPtr->RecordMissingDelta(DownloadRecord.Uri, BaseVersion, PatchVersion);
				}
				return;
			}

			NumFailedDownloads++;
			if (const TSharedPtr<IInstallerAnalytics> InstallerAnalyticsPtr = InstallerAnalytics.Pin())
			{
				InstallerAnalyticsPtr->RecordChunkDownloadError(DownloadRecord.Uri, DownloadRecord.ResponseCode, TEXT("DownloadFail"));
			}
		}
	}

	uint64 FDownloadServiceStatistics::GetBytesDownloaded() const
	{
		return TotalBytesReceived;
	}

	int32 FDownloadServiceStatistics::GetNumSuccessfulChunkDownloads() const
	{
		return NumSuccessfulDownloads;
	}

	int32 FDownloadServiceStatistics::GetNumFailedChunkDownloads() const
	{
		return NumFailedDownloads;
	}

	int32 FDownloadServiceStatistics::GetNumCurrentDownloads() const
	{
		FScopeLock Lock(&DownloadsCriticalSection);
		return Downloads.Num();
	}

	TArray<FDownload> FDownloadServiceStatistics::GetCurrentDownloads() const
	{
		TArray<FString> DownloadData;
		TArray<uint64> DownloadSizes;
		TArray<FDownload> Result;

		{
			FScopeLock Lock(&DownloadsCriticalSection);
			DownloadData.Empty(Downloads.Num());
			Result.Empty(Downloads.Num());
			for (const TPair<int32, FDownloadTuple>& Download : Downloads)
			{
				DownloadData.Emplace(Download.Value.Get<0>());

				Result.AddDefaulted();
				FDownload& Element = Result.Last();
				Element.Received = Download.Value.Get<1>();
			}
		}

		if (const TSharedPtr<IDataSizeProvider> DataSizeProviderPtr = DataSizeProvider.Pin())
		{
			DataSizeProviderPtr->GetDownloadSize(DownloadData, DownloadSizes);
			check(DownloadSizes.Num() == Result.Num());
		}

		for (int32 Index = 0; FDownload& Element : Result)
		{
			Element.Data = MoveTemp(DownloadData[Index]);
			Element.Size = DownloadSizes[Index];

			++Index;
		}
		return Result;
	}

	TPair<double, uint32> FDownloadServiceStatistics::GetImmediateAverageSpeedPerRequest(uint32 MinCount)
	{
		uint32 Count = 0U;
		double Result = 0.0L;
		FScopeLock Lock(&AverageSpeedCriticalSection);
		if (AverageSpeedSampleCount >= MinCount)
		{
			Result = AccumulatedRequestSpeed / AverageSpeedSampleCount;
			Count = AverageSpeedSampleCount;
			AccumulatedRequestSpeed = 0.0L;
			AverageSpeedSampleCount = 0;
		}
		return TPair<double, uint32>(Result, Count);
	}

	void FDownloadServiceStatistics::Reset()
	{
		checkSlow(IsInGameThread());

		TotalBytesReceived = 0;
		NumSuccessfulDownloads = 0;
		NumFailedDownloads = 0;

		{
			FScopeLock Lock(&DownloadsCriticalSection);
			Downloads.Empty();
		}

		{
			FScopeLock Lock(&AverageSpeedCriticalSection);
			AccumulatedRequestSpeed = 0;
			AverageSpeedSampleCount = 0;
		}
	}

	IDownloadServiceStatistics* FDownloadServiceStatisticsFactory::Create(const TSharedPtr<ISpeedRecorder>& SpeedRecorder, const TSharedPtr<IDataSizeProvider>& DataSizeProvider, const TSharedPtr<IInstallerAnalytics>& InstallerAnalytics)
	{
		check(SpeedRecorder != nullptr);
		check(DataSizeProvider != nullptr);
		check(InstallerAnalytics != nullptr);
		return new FDownloadServiceStatistics(SpeedRecorder, DataSizeProvider, InstallerAnalytics);
	}
};