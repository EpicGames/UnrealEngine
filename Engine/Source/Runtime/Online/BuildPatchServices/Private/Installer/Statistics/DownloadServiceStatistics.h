// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/DownloadService.h"
#include "Interfaces/IBuildStatistics.h"
#include "Common/DataSizeProvider.h"

class FBuildPatchAppManifest;

namespace BuildPatchServices
{
	class IInstallerAnalytics;
	class ISpeedRecorder;
	class IDataSizeProvider;

	/**
	 * Interface to the statistics class which provides access to tracked values from a download service stat.
	 */
	class IDownloadServiceStatistics
		: public IDownloadServiceStat
	{
	public:
		/**
		 * @return the total number of bytes downloaded.
		 */
		virtual uint64 GetBytesDownloaded() const = 0;

		/**
		 * @return the number of successfully downloaded chunks.
		 */
		virtual int32 GetNumSuccessfulChunkDownloads() const = 0;

		/**
		 * @return the number of chunk requests that failed.
		 */
		virtual int32 GetNumFailedChunkDownloads() const = 0;

		/**
		 * @return the number of current requests.
		 */
		virtual int32 GetNumCurrentDownloads() const = 0;

		/**
		 * @return an array of current request info.
		 */
		virtual TArray<FDownload> GetCurrentDownloads() const = 0;

		/**
		 * Calculates the average speed per request since the last time this function was called. NOT A CUMULATIVE AVERAGE
		 * @param MinCount -- The smallest number of samples that will be used for calculating an average;
		 *                    if the minimum count isn't met, the same value as the previous call is returned, and the samples will continue to accumulate.
		 * @return A pair containing an average of the per-request download speed SINCE THIS FUNCTION WAS LAST CALLED
		 *         and the count of requests completed since the last call.
		 */
		virtual TPair<double, uint32> GetImmediateAverageSpeedPerRequest(uint32 MinCount) = 0;

		/**
		 * Resets all internal statistics.
		 */
		virtual void Reset() = 0;
	};

	/**
	 * A factory for creating an IDownloadServiceStatistics instance.
	 */
	class FDownloadServiceStatisticsFactory
	{
	public:
		/**
		 * Creates the download service's dependency interface and exposes additional information.
		 * @param SpeedRecorder         The speed recorder instance that we send activity records to.
		 * @param DataSizeProvider      The data size provider for looking up the file size of each downloadable chunk.
		 * @param InstallerAnalytics    The analytics implementation for reporting the download service events.
		 * @return the new IDownloadServiceStatistics instance created.
		 */
		static IDownloadServiceStatistics* Create(const TSharedPtr<ISpeedRecorder>& SpeedRecorder, const TSharedPtr<IDataSizeProvider>& DataSizeProvider, const TSharedPtr<IInstallerAnalytics>& InstallerAnalytics);
	};
}