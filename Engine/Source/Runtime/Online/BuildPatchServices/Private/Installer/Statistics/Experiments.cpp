// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Experiments.cpp: Implements an A/B bucketing system to allow us to test features on X% of users
	An Experiment can contain any number of Variants with a % rollout on each Variant
	The EpicUserID gets hashed to determine if they are in a Variant or if they are a Control
=============================================================================*/

#include "Experiments.h"
#include "Hash/CityHash.h"
#include "Misc/ConfigCacheIni.h"

namespace BuildPatchServices 
{
	const Experiments::Experiment& Experiments::DownloadSpeed()
	{
		// Build and cache Experiment on first call to allow GConfig to initialize
		static Experiment CachedExperiment = []()
			{
				Experiment DownloadExperimentDefinition;
				DownloadExperimentDefinition.Id = EExperiment::DownloadSpeed;
				DownloadExperimentDefinition.ExperimentName = LexToString(EExperiment::DownloadSpeed);

				unsigned ConnectionFloorRollout = 1u;

				if (GConfig->IsReadyForUse())
				{
					GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("ExperimentConnectionFloor16Rollout"), (int32&)ConnectionFloorRollout, GEngineIni);
					ConnectionFloorRollout = FMath::Clamp<uint32>(ConnectionFloorRollout, 0.0f, 100.0f);
				}

				const unsigned ControlPercent = 100u - ConnectionFloorRollout;

				DownloadExperimentDefinition.Variants.Reserve(2);
				DownloadExperimentDefinition.Variants.Emplace(TEXT("Connection_Floor_16"), ConnectionFloorRollout);
				DownloadExperimentDefinition.Variants.Emplace(TEXT("Control"), ControlPercent);

				return DownloadExperimentDefinition;
			}();

		return CachedExperiment;
	}

	Experiments::Experiments() {}

	const FString Experiments::GetVariant(const Experiment& Experiment, const FString& UserId)
	{
		FString ExperimentUser = UserId + Experiment.ExperimentName;
		uint32 Hash = CityHash32(TCHAR_TO_ANSI(*ExperimentUser), sizeof(ExperimentUser));

		// Iterate through all variations of an experiment, adding their rollout% and either assigning users to a bucket or marking as control
		const uint64_t Bucket = Hash % 100ull;
		unsigned Total = 0u; 
		for (const Variant& Entry : Experiment.Variants)
		{
			float RolloutPercent = (float)Entry.RolloutPercent;
			RolloutPercent = FMath::Clamp(RolloutPercent, 0.0f, 100.0f);
			if (Total >= 100u) break;
			const unsigned Next = (Total + RolloutPercent > 100u) ? 100u : Total + RolloutPercent;

			if (Bucket < static_cast<std::uint64_t>(Next)) {
				if (Bucket >= static_cast<std::uint64_t>(Total)) {
					return Entry.VariantId;
				}
			}
			Total = Next;
		}

		return FString();
	}
}
