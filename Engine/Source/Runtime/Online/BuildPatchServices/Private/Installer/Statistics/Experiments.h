// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace BuildPatchServices
{
	class Experiments
	{
	public:
		enum class EExperiment : uint32
		{
			DownloadSpeed = 1u,
		};
		
		struct Variant
		{
			FString VariantId;
			unsigned RolloutPercent;
			Variant() : VariantId(), RolloutPercent(0u) {}
			Variant(const FString& Id, unsigned Percent) : VariantId(Id), RolloutPercent(Percent) {}
		};

		struct Experiment
		{
			EExperiment Id;
			FString ExperimentName;
			TArray<Variant> Variants;
		};

		// If a user is in an experiment, choose the variant of the experiment they are in.
		// returns the variant, or if they are not in the experiment returns nullptr
		static const FString GetVariant(const Experiment& Experiment, const FString& UserId);

		// Download Speed Experiment
		// This experiment will be used to test various methods for improving the download speed for users in the launcher
		static const Experiment& DownloadSpeed();

	private:
		Experiments();
	};
}

inline const TCHAR* LexToString(BuildPatchServices::Experiments::EExperiment Value)
{
	switch (Value)
	{
	case BuildPatchServices::Experiments::EExperiment::DownloadSpeed:
		return TEXT("DownloadSpeed_20250827");
	default:
		return TEXT("");
	}
}
