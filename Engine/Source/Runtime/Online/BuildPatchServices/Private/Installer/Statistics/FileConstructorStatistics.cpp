// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/FileConstructorStatistics.h"
#include "Misc/ScopeLock.h"
#include "Core/AsyncHelpers.h"
#include "Common/SpeedRecorder.h"
#include "Common/StatsCollector.h"
#include "Installer/Statistics/FileOperationTracker.h"

namespace BuildPatchServices
{
	class FFileConstructorStatistics
		: public IFileConstructorStatistics
	{
	public:
		FFileConstructorStatistics(ISpeedRecorder* ReadSpeedRecorder, ISpeedRecorder* WriteSpeedRecorder, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker);
		~FFileConstructorStatistics();

		// IFileConstructorStat interface begin.
		virtual void OnResumeStarted() override;
		virtual void OnResumeCompleted() override;
		virtual void OnChunkGet(const FGuid& ChunkId) override;
		virtual void OnFileStarted(const FString& Filename, int64 FileSize) override;
		virtual void OnFileProgress(const FString& Filename, int64 TotalBytes) override;
		virtual void OnFileCompleted(const FString& Filename, bool bSuccess) override;
		virtual void OnConstructionCompleted() override;
		virtual void OnProcessedDataUpdated(int64 TotalBytes) override;
		virtual void OnTotalRequiredUpdated(int64 TotalBytes) override;
		virtual void OnBeforeAdminister() override;
		virtual void OnAfterAdminister(const ISpeedRecorder::FRecord& Record) override;
		virtual void OnBeforeRead() override;
		virtual void OnAfterRead(const ISpeedRecorder::FRecord& Record) override;
		virtual void OnBeforeWrite() override;
		virtual void OnAfterWrite(const ISpeedRecorder::FRecord& Record) override;
		// IFileConstructorStat interface end.

		// IFileConstructorStatistics interface begin.
		virtual int64 GetRequiredConstructSize() const override;
		virtual uint64 GetBytesConstructed() const override;
		virtual uint64 GetFilesConstructed() const override;
		virtual FGuid GetCurrentChunk() const override;
		virtual FString GetCurrentFile() const override;
		virtual float GetCurrentFileProgress() const override;
		virtual bool IsCurrentlyWriting() const override;
		virtual bool IsCurrentlyReading() const override;
		virtual bool IsCurrentlyAdministering() const override;
		// IFileConstructorStatistics interface end.

	private:
		ISpeedRecorder* ReadSpeedRecorder;
		ISpeedRecorder* WriteSpeedRecorder;
		FBuildPatchProgress* BuildProgress;
		IFileOperationTracker* FileOperationTracker;
		FThreadSafeInt64 TotalBytesProcessed;
		FThreadSafeInt64 TotalBytesRequired;
		FThreadSafeInt64 TotalBytesConstructed;
		FThreadSafeBool bResumeCompleted;
		FThreadSafeInt32 TotalFilesConstructed;
		FThreadSafeBool bIsAdministering;
		FThreadSafeInt64 LastAdministerCycles;
		FThreadSafeBool bIsReading;
		FThreadSafeInt64 LastReadCycles;
		FThreadSafeBool bIsWriting;
		FThreadSafeInt64 LastWriteCycles;

		mutable FCriticalSection ThreadLockCs;
		FGuid CurrentChunk;
		FString CurrentFile;
		int64 CurrentFileSize;
		int64 CurrentFileProgress;
		uint64 LastByteRangeStateUpdate;
	};

	FFileConstructorStatistics::FFileConstructorStatistics(ISpeedRecorder* InReadSpeedRecorder, ISpeedRecorder* InWriteSpeedRecorder, FBuildPatchProgress* InBuildProgress, IFileOperationTracker* InFileOperationTracker)
		: ReadSpeedRecorder(InReadSpeedRecorder)
		, WriteSpeedRecorder(InWriteSpeedRecorder)
		, BuildProgress(InBuildProgress)
		, FileOperationTracker(InFileOperationTracker)
		, TotalBytesProcessed(0)
		, TotalBytesRequired(0)
		, bResumeCompleted(false)
		, TotalFilesConstructed(0)
		, bIsAdministering(false)
		, LastAdministerCycles(0)
		, bIsReading(false)
		, LastReadCycles(0)
		, bIsWriting(false)
		, LastWriteCycles(0)
		, CurrentFileSize(0)
		, CurrentFileProgress(0)
		, LastByteRangeStateUpdate(0)
	{
	}

	FFileConstructorStatistics::~FFileConstructorStatistics()
	{
	}

	void FFileConstructorStatistics::OnResumeStarted()
	{
		BuildProgress->SetStateProgress(EBuildPatchState::Resuming, 0.0f);
		bResumeCompleted = false;
	}

	void FFileConstructorStatistics::OnResumeCompleted()
	{
		BuildProgress->SetStateProgress(EBuildPatchState::Resuming, 1.0f);
		bResumeCompleted = true;
	}

	void FFileConstructorStatistics::OnChunkGet(const FGuid& ChunkId)
	{
		FScopeLock Lock(&ThreadLockCs);
		CurrentChunk = ChunkId;
	}

	void FFileConstructorStatistics::OnFileStarted(const FString& Filename, int64 FileSize)
	{
		LastByteRangeStateUpdate = 0;

		FScopeLock Lock(&ThreadLockCs);
		CurrentFile = Filename;
		CurrentFileSize = FileSize;
	}

	void FFileConstructorStatistics::OnFileProgress(const FString& Filename, int64 TotalBytes)
	{
		checkSlow(TotalBytes >= 0);
		// Currently we only stage.
		FileOperationTracker->OnFileByteRangeStateUpdate(Filename, FByteRange(LastByteRangeStateUpdate, TotalBytes), EFileOperationState::Staged);
		LastByteRangeStateUpdate = TotalBytes;

		FScopeLock Lock(&ThreadLockCs);
		CurrentFile = Filename;
		CurrentFileProgress = TotalBytes;
	}

	void FFileConstructorStatistics::OnFileCompleted(const FString& Filename, bool bSuccess)
	{
		if (bResumeCompleted)
		{
			TotalFilesConstructed.Increment();
		}

		// Currently we only stage.
		FileOperationTracker->OnFileStateUpdate(Filename, EFileOperationState::Staged);

		FScopeLock Lock(&ThreadLockCs);
		CurrentFile.Empty();
		CurrentFileSize = 0;
	}

	void FFileConstructorStatistics::OnConstructionCompleted()
	{
		FScopeLock Lock(&ThreadLockCs);
		CurrentChunk.Invalidate();
	}

	void FFileConstructorStatistics::OnProcessedDataUpdated(int64 TotalBytes)
	{
		TotalBytesProcessed.Set(TotalBytes);
		int64 Required = TotalBytesRequired.GetValue();
		if (Required > 0)
		{
			BuildProgress->SetStateProgress(EBuildPatchState::Installing, (double)TotalBytes / (double)Required);
		}
	}

	void FFileConstructorStatistics::OnTotalRequiredUpdated(int64 TotalBytes)
	{
		TotalBytesRequired.Set(TotalBytes);
		int64 Processed = TotalBytesProcessed.GetValue();
		if (TotalBytes > 0)
		{
			BuildProgress->SetStateProgress(EBuildPatchState::Installing, (double)Processed / (double)TotalBytes);
		}
	}

	void FFileConstructorStatistics::OnBeforeAdminister()
	{
		bIsAdministering = true;
	}

	void FFileConstructorStatistics::OnAfterAdminister(const ISpeedRecorder::FRecord& Record)
	{
		LastAdministerCycles = FStatsCollector::GetCycles();
		bIsAdministering = false;
		WriteSpeedRecorder->AddRecord(Record);
	}

	void FFileConstructorStatistics::OnBeforeRead()
	{
		bIsReading = true;
	}

	void FFileConstructorStatistics::OnAfterRead(const ISpeedRecorder::FRecord& Record)
	{
		LastReadCycles = FStatsCollector::GetCycles();
		bIsReading = false;
		ReadSpeedRecorder->AddRecord(Record);
	}

	void FFileConstructorStatistics::OnBeforeWrite()
	{
		bIsWriting = true;
	}

	void FFileConstructorStatistics::OnAfterWrite(const ISpeedRecorder::FRecord& Record)
	{
		LastWriteCycles = FStatsCollector::GetCycles();
		bIsWriting = false;
		WriteSpeedRecorder->AddRecord(Record);
		TotalBytesConstructed.Add(Record.Size);
	}

	int64 FFileConstructorStatistics::GetRequiredConstructSize() const
	{
		return TotalBytesRequired.GetValue();
	}

	uint64 FFileConstructorStatistics::GetBytesConstructed() const
	{
		return TotalBytesConstructed.GetValue();
	}

	uint64 FFileConstructorStatistics::GetFilesConstructed() const
	{
		return TotalFilesConstructed.GetValue();
	}

	FGuid FFileConstructorStatistics::GetCurrentChunk() const
	{
		FScopeLock Lock(&ThreadLockCs);
		return CurrentChunk;
	}

	FString FFileConstructorStatistics::GetCurrentFile() const
	{
		FScopeLock Lock(&ThreadLockCs);
		return CurrentFile;
	}

	float FFileConstructorStatistics::GetCurrentFileProgress() const
	{
		FScopeLock Lock(&ThreadLockCs);
		return CurrentFileSize > 0 ? (float)CurrentFileProgress / (float)CurrentFileSize : 0.0f;
	}

	bool FFileConstructorStatistics::IsCurrentlyWriting() const
	{
		const int64 CyclesSinceWrite = FStatsCollector::GetCycles() - LastWriteCycles.GetValue();
		return bIsWriting || FStatsCollector::CyclesToSeconds(CyclesSinceWrite) < 0.05;
	}

	bool FFileConstructorStatistics::IsCurrentlyReading() const
	{
		const int64 CyclesSinceRead = FStatsCollector::GetCycles() - LastReadCycles.GetValue();
		return bIsReading || FStatsCollector::CyclesToSeconds(CyclesSinceRead) < 0.05;
	}

	bool FFileConstructorStatistics::IsCurrentlyAdministering() const
	{
		const int64 CyclesSinceAdminister = FStatsCollector::GetCycles() - LastAdministerCycles.GetValue();
		return bIsAdministering || FStatsCollector::CyclesToSeconds(CyclesSinceAdminister) < 0.05;
	}

	IFileConstructorStatistics* FFileConstructorStatisticsFactory::Create(ISpeedRecorder* ReadSpeedRecorder, ISpeedRecorder* WriteSpeedRecorder, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker)
	{
		check(ReadSpeedRecorder != nullptr);
		check(WriteSpeedRecorder != nullptr);
		check(BuildProgress != nullptr);
		check(FileOperationTracker != nullptr);
		return new FFileConstructorStatistics(ReadSpeedRecorder, WriteSpeedRecorder, BuildProgress, FileOperationTracker);
	}
};