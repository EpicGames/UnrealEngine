// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BuildPatchFileConstructor.h"

namespace BuildPatchServices
{
	class ISpeedRecorder;
	struct FBuildPatchProgress;
	class IFileOperationTracker;

	/**
	 * Interface to the statistics class which provides access to tracked values from a file constructor stat.
	 */
	class IFileConstructorStatistics
		: public IFileConstructorStat
	{
	public:
		/**
		 * @return the number of bytes to construct to complete the installation.
		 */
		virtual int64 GetRequiredConstructSize() const = 0;

		/**
		 * @return the total number of bytes constructed.
		 */
		virtual uint64 GetBytesConstructed() const = 0;

		/**
		 * @return the total number of files constructed.
		 */
		virtual uint64 GetFilesConstructed() const = 0;

		/**
		 * @return the current chunk data is being serialized from.
		 */
		virtual FGuid GetCurrentChunk() const = 0;

		/**
		 * @return the current file data is being serialized to.
		 */
		virtual FString GetCurrentFile() const = 0;

		/**
		 * @return the current file progress.
		 */
		virtual float GetCurrentFileProgress() const = 0;

		/**
		 * @return true if currently writing disk data.
		 */
		virtual bool IsCurrentlyWriting() const = 0;

		/**
		 * @return true if currently reading disk data.
		 */
		virtual bool IsCurrentlyReading() const = 0;

		/**
		 * @return true if currently administering files.
		 */
		virtual bool IsCurrentlyAdministering() const = 0;
	};

	/**
	 * A factory for creating an IFileConstructorStatistics instance.
	 */
	class FFileConstructorStatisticsFactory
	{
	public:
		/**
		 * Creates the file constructor's dependency interface and exposes additional information.
		 * @param ReadSpeedRecorder     The read speed recorder instance that we send activity records to.
		 * @param WriteSpeedRecorder    The write speed recorder instance that we send activity records to.
		 * @param BuildProgress         The legacy progress implementation to bridge the system stats to.
		 * @param FileOperationTracker  The file operation tracker which will be used to update data states.
		 * @return the new IFileConstructorStatistics instance created.
		 */
		static IFileConstructorStatistics* Create(ISpeedRecorder* ReadSpeedRecorder, ISpeedRecorder* WriteSpeedRecorder, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker);
	};
}