// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/FileOperationTracker.h"
#include "Templates/Tuple.h"
#include "Containers/Union.h"
#include "Containers/Queue.h"
#include "HAL/ThreadSafeBool.h"
#include "Core/AsyncHelpers.h"
#include "Common/StatsCollector.h"
#include "BuildPatchManifest.h"
#include "IBuildManifestSet.h"
#include "Stats/Stats.h"
#include "Containers/Ticker.h"

DECLARE_LOG_CATEGORY_EXTERN(LogFileOperationTracker, Warning, All);
DEFINE_LOG_CATEGORY(LogFileOperationTracker);

namespace BuildPatchServices
{
	typedef TTuple<TArray<FFileOperation>, TArray<FFileOperation>> FOperationInitialiser;
	typedef TTuple<FGuid, EFileOperationState> FDataState;
	typedef TTuple<FString, EFileOperationState> FFileState;
	typedef TTuple<FString, FByteRange, EFileOperationState> FFileByteRangeState;
	typedef TUnion<FOperationInitialiser, FDataState, FFileState, FFileByteRangeState> FUpdateMessage;

	class FNullFileOperationTracker 
		: public IFileOperationTracker
	{
	public:
		// IFileOperationTracker interface begin.
		virtual const TArray<FFileOperation>& GetStates() const override { return FileOperationStates; }
		virtual void OnManifestSelection(IBuildManifestSet* ManifestSet) override {}
		virtual void OnDataStateUpdate(const FGuid& DataId, EFileOperationState State) override {}
		virtual void OnDataStateUpdate(const TSet<FGuid>& DataIds, EFileOperationState State) override {}
		virtual void OnDataStateUpdate(const TArray<FGuid>& DataIds, EFileOperationState State) override {}
		virtual void OnFileStateUpdate(const FString& Filename, EFileOperationState State) override {}
		virtual void OnFileStateUpdate(const TSet<FString>& Filenames, EFileOperationState State) override {}
		virtual void OnFileStateUpdate(const TArray<FString>& Filenames, EFileOperationState State) override {}
		virtual void OnFileByteRangeStateUpdate(const FString& Filename, FByteRange ByteRange, EFileOperationState State) override {}
		// IFileOperationTracker interface end.

	private:
		TArray<FFileOperation> FileOperationStates;
	};

	class FFileOperationTracker
		: public IFileOperationTracker
	{
	public:
		FFileOperationTracker(FTSTicker& Ticker);
		~FFileOperationTracker();

	public:
		// IFileOperationTracker interface begin.
		virtual const TArray<FFileOperation>& GetStates() const override;
		virtual void OnManifestSelection(IBuildManifestSet* ManifestSet) override;
		virtual void OnDataStateUpdate(const FGuid& DataId, EFileOperationState State) override;
		virtual void OnDataStateUpdate(const TSet<FGuid>& DataIds, EFileOperationState State) override;
		virtual void OnDataStateUpdate(const TArray<FGuid>& DataIds, EFileOperationState State) override;
		virtual void OnFileStateUpdate(const FString& Filename, EFileOperationState State) override;
		virtual void OnFileStateUpdate(const TSet<FString>& Filenames, EFileOperationState State) override;
		virtual void OnFileStateUpdate(const TArray<FString>& Filenames, EFileOperationState State) override;
		virtual void OnFileByteRangeStateUpdate(const FString& Filename, FByteRange ByteRange, EFileOperationState State) override;
		// IFileOperationTracker interface end.

	private:
		bool Tick(float Delta);
		void ProcessMessage(FOperationInitialiser& Message);
		void ProcessMessage(FDataState& Message);
		void ProcessMessage(FFileState& Message);
		void ProcessMessage(FFileByteRangeState& Message);
		static void LogOperationState(const FFileOperation* FileOperation);
		static void LogOperationState(const FFileOperation* FileOperation, const FByteRange& ByteRange);

	private:
		static FOperationInitialiser BuildOperationInitialiser(IBuildManifestSet* ManifestSet);

	private:
		FTSTicker& Ticker;
		FTSTicker::FDelegateHandle TickerHandle;
		TArray<FFileOperation> FileOperationStates;
		TArray<FFileOperation> DummyOperationStates;
		TMap<FGuid, TArray<FFileOperation*>> FileOperationStatesDataIdLookup;
		TMap<FString, TArray<FFileOperation*>> FileOperationStatesFilenameLookup;
		TQueue<FUpdateMessage, EQueueMode::Mpsc> UpdateMessages;
		IBuildManifestSet* LastUsedManifestSet;
	};

	FFileOperationTracker::FFileOperationTracker(FTSTicker& InTicker)
		: Ticker(InTicker)
		, LastUsedManifestSet(nullptr)
	{
		check(IsInGameThread());

		// Need ticker to process incoming updates.
		TickerHandle = Ticker.AddTicker(FTickerDelegate::CreateRaw(this, &FFileOperationTracker::Tick));
	}

	FFileOperationTracker::~FFileOperationTracker()
	{
		check(IsInGameThread());
		// Remove ticker.
		Ticker.RemoveTicker(TickerHandle);
	}

	const TArray<FFileOperation>& FFileOperationTracker::GetStates() const
	{
		check(IsInGameThread());
		return FileOperationStates;
	}

	void FFileOperationTracker::OnManifestSelection(IBuildManifestSet* ManifestSet)
	{
		if (LastUsedManifestSet != ManifestSet)
		{
			LastUsedManifestSet = ManifestSet;
			UpdateMessages.Enqueue(FUpdateMessage(BuildOperationInitialiser(ManifestSet)));
		}
	}

	void FFileOperationTracker::OnDataStateUpdate(const FGuid& DataId, EFileOperationState State)
	{
		UpdateMessages.Enqueue(FUpdateMessage(FDataState(DataId, State)));
	}

	void FFileOperationTracker::OnDataStateUpdate(const TSet<FGuid>& DataIds, EFileOperationState State)
	{
		for (const FGuid& DataId : DataIds)
		{
			UpdateMessages.Enqueue(FUpdateMessage(FDataState(DataId, State)));
		}
	}

	void FFileOperationTracker::OnDataStateUpdate(const TArray<FGuid>& DataIds, EFileOperationState State)
	{
		for (const FGuid& DataId : DataIds)
		{
			UpdateMessages.Enqueue(FUpdateMessage(FDataState(DataId, State)));
		}
	}

	void FFileOperationTracker::OnFileStateUpdate(const FString& Filename, EFileOperationState State)
	{
		UpdateMessages.Enqueue(FUpdateMessage(FFileState(Filename, State)));
	}

	void FFileOperationTracker::OnFileStateUpdate(const TSet<FString>& Filenames, EFileOperationState State)
	{
		for (const FString& Filename : Filenames)
		{
			UpdateMessages.Enqueue(FUpdateMessage(FFileState(Filename, State)));
		}
	}

	void FFileOperationTracker::OnFileStateUpdate(const TArray<FString>& Filenames, EFileOperationState State)
	{
		for (const FString& Filename : Filenames)
		{
			UpdateMessages.Enqueue(FUpdateMessage(FFileState(Filename, State)));
		}
	}

	void FFileOperationTracker::OnFileByteRangeStateUpdate(const FString& Filename, FByteRange ByteRange, EFileOperationState State)
	{
		UpdateMessages.Enqueue(FUpdateMessage(FFileByteRangeState(Filename, ByteRange, State)));
	}

	bool FFileOperationTracker::Tick(float Delta)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FFileOperationTracker_Tick);

		const double TimeLimitSeconds = 1.0 / 120.0;
		// Use a time limit as setting huge file state can take a while. We will catch up easily over a handful of ticks.
		uint64 TimeLimitCycles = FStatsCollector::GetCycles() + FStatsCollector::SecondsToCycles(TimeLimitSeconds);
		FUpdateMessage UpdateMessage;
		while (FStatsCollector::GetCycles() < TimeLimitCycles && UpdateMessages.Dequeue(UpdateMessage))
		{
			if (UpdateMessage.HasSubtype<FOperationInitialiser>())
			{
				ProcessMessage(UpdateMessage.GetSubtype<FOperationInitialiser>());
			}
			else if (UpdateMessage.HasSubtype<FDataState>())
			{
				ProcessMessage(UpdateMessage.GetSubtype<FDataState>());
			}
			else if (UpdateMessage.HasSubtype<FFileState>())
			{
				ProcessMessage(UpdateMessage.GetSubtype<FFileState>());
			}
			else if (UpdateMessage.HasSubtype<FFileByteRangeState>())
			{
				ProcessMessage(UpdateMessage.GetSubtype<FFileByteRangeState>());
			}
		}
		return true;
	}

	void FFileOperationTracker::LogOperationState(const FFileOperation* FileOperation)
	{
		UE_LOGF(LogFileOperationTracker, Log, "LogFileOperationTracker: file %ls state %ls.", *FileOperation->Filename, LexToString(FileOperation->CurrentState));
	}

	void FFileOperationTracker::LogOperationState(const FFileOperation* FileOperation, const FByteRange& ByteRange)
	{
		UE_LOGF(LogFileOperationTracker, Log, "LogFileOperationTracker: file %ls state %ls start %llu end %llu offset + size %llu.", *FileOperation->Filename, LexToString(FileOperation->CurrentState), ByteRange.Key, ByteRange.Value, FileOperation->Offest + FileOperation->Size);
	}

	void FFileOperationTracker::ProcessMessage(FOperationInitialiser& Message)
	{
		FileOperationStates = MoveTemp(Message.Get<0>());
		DummyOperationStates = MoveTemp(Message.Get<1>());
		FileOperationStatesDataIdLookup.Reset();
		FileOperationStatesFilenameLookup.Reset();
		for (FFileOperation& FileOperationState : FileOperationStates)
		{
			FileOperationStatesDataIdLookup.FindOrAdd(FileOperationState.DataId).Add(&FileOperationState);
			FileOperationStatesFilenameLookup.FindOrAdd(FileOperationState.Filename).Add(&FileOperationState);
		}
		for (FFileOperation& DummyOperationState : DummyOperationStates)
		{
			FileOperationStatesFilenameLookup.FindOrAdd(DummyOperationState.Filename).Add(&DummyOperationState);
		}
	}

	void FFileOperationTracker::ProcessMessage(FDataState& Message)
	{
		if (FileOperationStatesDataIdLookup.Contains(Message.Get<0>()))
		{
			for (FFileOperation* FileOp : FileOperationStatesDataIdLookup[Message.Get<0>()])
			{
				if (FileOp->CurrentState <= EFileOperationState::DataInMemoryStore)
				{
					FileOp->CurrentState = Message.Get<1>();
					LogOperationState(FileOp);
				}
			}
		}
	}

	void FFileOperationTracker::ProcessMessage(FFileState& Message)
	{
		if (FileOperationStatesFilenameLookup.Contains(Message.Get<0>()))
		{
			for (FFileOperation* FileOp : FileOperationStatesFilenameLookup[Message.Get<0>()])
			{
				FileOp->CurrentState = Message.Get<1>();
				LogOperationState(FileOp);
			}
		}
	}

	void FFileOperationTracker::ProcessMessage(FFileByteRangeState& Message)
	{
		if (FileOperationStatesFilenameLookup.Contains(Message.Get<0>()))
		{
			const uint64 FileByteStart = Message.Get<1>().Get<0>();
			const uint64 FileByteEnd = Message.Get<1>().Get<1>();
			for (FFileOperation* FileOp : FileOperationStatesFilenameLookup[Message.Get<0>()])
			{
				const uint64 FileByteFirst = FileOp->Offest;
				const uint64 FileByteLast = FileByteFirst + FileOp->Size;
				if (FileByteFirst < FileByteEnd && FileByteLast > FileByteStart)
				{
					FileOp->CurrentState = Message.Get<2>();
					LogOperationState(FileOp, Message.Get<1>());
				}
				if (FileByteFirst >= FileByteEnd)
				{
					break;
				}
			}
		}
	}

	FOperationInitialiser FFileOperationTracker::BuildOperationInitialiser(IBuildManifestSet* ManifestSet)
	{
		FOperationInitialiser Result;
		TArray<FFileOperation>& FileOperationStates = Result.Get<0>();
		TArray<FFileOperation>& DummyOperationStates = Result.Get<1>();
		// Get the list of files in the build.
		TSet<FString> Filenames;
		ManifestSet->GetExpectedFiles(Filenames);

		// Initialise all file operations to Unknown, use dummy operations for empty files.
		for (const FString& Filename : Filenames)
		{
			if (const FFileManifest* FileManifest = ManifestSet->GetNewFileManifest(Filename))
			{
				uint64 FileOffset = 0;
				for (const FChunkPart& FileChunkPart : FileManifest->ChunkParts)
				{
					FileOperationStates.Emplace(Filename, FileChunkPart.Guid, FileOffset, FileChunkPart.Size, EFileOperationState::Unknown);
					FileOffset += FileChunkPart.Size;
				}
				if (FileManifest->ChunkParts.Num() == 0)
				{
					DummyOperationStates.Emplace(Filename, FGuid(), 0, 0, EFileOperationState::Unknown);
				}
			}
			else
			{
				UE_LOGF(LogFileOperationTracker, Error, "LogFileOperationTracker: Failed to get manifest for file %ls", *Filename);
				DummyOperationStates.Emplace(Filename, FGuid(), 0, 0, EFileOperationState::Unknown);
			}
		}
		return Result;
	}

	IFileOperationTracker* FFileOperationTrackerFactory::Create(FTSTicker& Ticker)
	{
		return new FFileOperationTracker(Ticker);
	}

	IFileOperationTracker* FFileOperationTrackerFactory::CreateNull()
	{
		return new FNullFileOperationTracker();
	}
}