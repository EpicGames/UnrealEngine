// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "Interfaces/IBuildStatistics.h"

class FBuildPatchAppManifest;

namespace BuildPatchServices
{
	class IBuildManifestSet;
	typedef TTuple<uint64, uint64> FByteRange;

	/**
	 * An interface for tracking and querying states of granular file operations.
	 */
	class IFileOperationTracker
	{
	public:
		virtual ~IFileOperationTracker() = default;

		/**
		 * @return the array of states for each file operation performed by the installation.
		 */
		virtual const TArray<FFileOperation>& GetStates() const = 0;

		/**
		 * Called when manifest selection occurs. This will clear out internal states resetting them built from the provided manifest.
		 * All subsequent state updates must refer to data from within the provided manifest.
		 * @param Manifest  The manifest to be used to build up internal operation states.
		 */
		virtual void OnManifestSelection(IBuildManifestSet* ManifestSet) = 0;

		/**
		 * Called when state is updated for chunk data.
		 * @param DataId    Chunk to update state of.
		 * @param State     New state of the data.
		 */
		virtual void OnDataStateUpdate(const FGuid& DataId, EFileOperationState State) = 0;

		/**
		 * Called when state is updated for chunk data.
		 * @param DataIds   Chunk set to update state of.
		 * @param State     New state of the data.
		 */
		virtual void OnDataStateUpdate(const TSet<FGuid>& DataIds, EFileOperationState State) = 0;

		/**
		 * Called when state is updated for chunk data.
		 * @param DataId    Chunk array to update state of.
		 * @param State     New state of the data.
		 */
		virtual void OnDataStateUpdate(const TArray<FGuid>& DataIds, EFileOperationState State) = 0;

		/**
		 * Called when state is updated for file data.
		 * @param Filename  File to update state of.
		 * @param State     New state of the data.
		 */
		virtual void OnFileStateUpdate(const FString& Filename, EFileOperationState State) = 0;

		/**
		 * Called when state is updated for file data.
		 * @param Filename  File set to update state of.
		 * @param State     New state of the data.
		 */
		virtual void OnFileStateUpdate(const TSet<FString>& Filenames, EFileOperationState State) = 0;

		/**
		 * Called when state is updated for file data.
		 * @param Filename  File array to update state of.
		 * @param State     New state of the data.
		 */
		virtual void OnFileStateUpdate(const TArray<FString>& Filenames, EFileOperationState State) = 0;

		/**
		 * Called when state is updated for file data byte range.
		 * @param Filename  File to update state of.
		 * @param ByteRange The byte range of the file to include.
		 * @param State     New state of the data.
		 */
		virtual void OnFileByteRangeStateUpdate(const FString& Filename, FByteRange ByteRange, EFileOperationState State) = 0;
	};

	/**
	 * A factory for creating an IFileOperationTracker instance.
	 */
	class FFileOperationTrackerFactory
	{
	public:
		/**
		 * Creates the implementation of a file operation tracker which serves as the dependency for for systems providing these updates.
		 * @param   Ticker      The ticker to register for main thread ticks with.
		 * @param   Manifest    The manifest for the build being installed.
		 * @return the new IFileOperationTracker instance created.
		 */
		static IFileOperationTracker* Create(FTSTicker& Ticker);

		/**
		 * Creates a null implementation of a file operation tracker which serves as the dependency for for systems providing these updates.
		 * @param   Ticker      The ticker to register for main thread ticks with.
		 * @param   Manifest    The manifest for the build being installed.
		 * @return the new IFileOperationTracker instance created.
		 */
		static IFileOperationTracker* CreateNull();
	};
}
