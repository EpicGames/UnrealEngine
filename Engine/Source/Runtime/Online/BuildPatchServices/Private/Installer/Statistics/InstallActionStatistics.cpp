// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallActionStatistics.h"
#include "IBuildManifestSet.h"
#include "Misc/ScopeLock.h"
#include "Algo/Transform.h"

namespace BuildPatchServices
{
	class FInstallActionStatistics
		: public IInstallActionStatistics
	{
		using TActionsArray = TArray<const FBuildPatchInstallerAction*>;

		public:
		FInstallActionStatistics(const TArray<FBuildPatchInstallerAction>& InInstallerActions)
			: InstallerActions(InInstallerActions)
		{
			InitCachedData();	// Pre - compute data for efficiency.
		}

		TArray<FInstallActionStats> GetAllStats() const
		{
			TArray<FInstallActionStats> Stats;
			Stats.Reserve(InstallerActions.Num());
			// Return array order should match installer action order in configuration.
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				Stats.Add(GetActionStats(InstallerAction));
			}
			return Stats;
		}

		virtual FInstallActionStats GetActionStats(const FBuildPatchInstallerAction& InstallerAction) const override
		{
			const FInstallActionStats* Stats = ActionStatsMap.Find(&InstallerAction);
			if (Stats != nullptr)
			{
				return *Stats;
			}
			return {};
		}

		virtual void OnUsedOptimizedDelta(const FBuildPatchInstallerAction& InstallerAction, bool bInUsedOptimizedDelta, uint64 InHTTPResponseCode) override
		{
			FInstallActionStats& Stats = ActionStatsMap.FindOrAdd(&InstallerAction);
			
			Stats.UsedOptimizedDelta = bInUsedOptimizedDelta;
			Stats.OptimizedDeltaHTTPResponseCode = InHTTPResponseCode;
		}

		virtual void OnChunkRecycleFailure(const FGuid& ChunkId) override
		{
			// Find which action(s) chunk belongs to, then increment failure counter for each.
			TActionsArray* Actions = GuidActionMap.Find(ChunkId);
			if (Actions != nullptr)
			{
				for (const FBuildPatchInstallerAction* InstallerAction: *Actions)
				{
					ActionStatsMap.FindOrAdd(InstallerAction).NumChunksRecycleFailures++;
				}
			}
		}

		virtual void OnNumLocalChunksExpected(const FBuildPatchInstallerAction& InstallerAction, uint64 InNumLocalChunksExpected) override
		{
			ActionStatsMap.FindOrAdd(&InstallerAction).NumLocalChunksExpected = InNumLocalChunksExpected;
		}

		virtual void OnNumLocalChunksAvailable(const FBuildPatchInstallerAction& InstallerAction, uint64 InNumLocalChunksAvailable) override
		{
			ActionStatsMap.FindOrAdd(&InstallerAction).NumLocalChunksAvailable = InNumLocalChunksAvailable;
		}

		virtual void CalculateChunkStatistics(const TSet<FGuid>& InstallChunksAvailable) override
		{
			for (const FBuildPatchInstallerAction& Action : InstallerActions)
			{
				// Calculate expected and available local chunks (recyclable from existing installation).
				// Fresh installs don't have anything to recycle - process updates and repairs only.
				if (Action.IsUpdate() || Action.IsRepair())
				{
					const FBuildPatchAppManifest& CurrentManifest = Action.GetCurrentManifest();
					const FBuildPatchAppManifest& DestinationManifest = Action.GetInstallManifest();

					TSet<FGuid> CurrentChunks;
					// Get chunks from current installation (what we should have for all files in current manifest).
					TSet<FString> CurrentFilesSet(CurrentManifest.GetBuildFileList(Action.GetInstallTags()));

					CurrentManifest.GetChunksRequiredForFiles(CurrentFilesSet, CurrentChunks);

					// Get chunks from destination (what we need) from cache.
					TSet<FGuid>& DestinationChunks = ChunkCacheMap.FindOrAdd(&Action);

					// Chunks we should be able recycle.
					OnNumLocalChunksExpected(Action, DestinationChunks.Intersect(CurrentChunks).Num());

					// Chunks that are actually available for recycling.
					OnNumLocalChunksAvailable(Action, DestinationChunks.Intersect(InstallChunksAvailable).Num());
				}
			}
		}

	private:
		const TArray<FBuildPatchInstallerAction>& InstallerActions;
		TMap<const FBuildPatchInstallerAction*, FInstallActionStats> ActionStatsMap;
		TMap<FGuid, TActionsArray> GuidActionMap;
		TMap<const FBuildPatchInstallerAction*, TSet<FGuid>> ChunkCacheMap;

		void InitCachedData()
		{
			for (const FBuildPatchInstallerAction& InstallerAction : InstallerActions)
			{
				const FBuildPatchAppManifestPtr InstallManifest = InstallerAction.TryGetSharedInstallManifest();
				if (!InstallManifest.IsValid())
				{
					continue;
				}

				TSet<FGuid> Chunks;
				TSet<FString> CurrentFilesSet(InstallManifest->GetBuildFileList(InstallerAction.GetInstallTags()));

				InstallManifest->GetChunksRequiredForFiles(CurrentFilesSet, Chunks);

				// Store chunks in cache - used in CalculateChunkStatistics.
				ChunkCacheMap.Add(&InstallerAction, Chunks);

				// Build ChunkId-Actions map - used in OnChunkRecycleFailure
				// to find which actions failed chunk belongs to.
				for (const FGuid& ChunkId : Chunks)
				{
					TActionsArray& Actions = GuidActionMap.FindOrAdd(ChunkId);
					Actions.AddUnique(&InstallerAction);
				}
			}
		}
	};

	IInstallActionStatistics* FInstallActionStatisticsFactory::Create(const TArray<FBuildPatchInstallerAction>& InstallerActions)
	{
		return new FInstallActionStatistics(InstallerActions);
	}
}