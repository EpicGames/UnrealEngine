// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IBuildInstaller.h"

namespace BuildPatchServices
{
	struct FBuildPatchInstallerAction;

	class IInstallActionStatistics
	{
	public:
		virtual ~IInstallActionStatistics() = default;

		/**
		 * Get all installer actions stats. Order matches the order of installer actions in configuration.
		 * @return The list of installer actions stats.
		 */
		virtual TArray<FInstallActionStats> GetAllStats() const = 0;

		/**
		 * Return statistics for one action.
		 * @param InstallerAction The installer action to query.
		 * @return The statistics for queried action.
		 */
		virtual FInstallActionStats GetActionStats(const FBuildPatchInstallerAction& InstallerAction) const = 0;

		/**
		* Calculate expected and available statistics for all actions,
		* based on action manifests and chunks available from chunk source.
		* @param InstallChunksAvailable The list of chunks available.
		*/
		virtual void CalculateChunkStatistics(const TSet<FGuid>& InstallChunksAvailable) = 0;

		/**
		 * Set whether the action successfully downloaded and used delta optimization file.
		 * @param InstallerAction The installer action to set.
		 * @param bInUsedOptimizedDelta The optimized delta flag.
		 */
		virtual void OnUsedOptimizedDelta(const FBuildPatchInstallerAction& InstallerAction, bool bInUsedOptimizedDelta, uint64 InHTTPResponseCode) = 0;

		/**
		 * Set the number of local chunks expected for the action.
		 * @param InstallerAction The installer action to set.
		 * @param InNumLocalChunksExpected The number of local chunks expected.
		 */
		virtual void OnNumLocalChunksExpected(const FBuildPatchInstallerAction& InstallerAction, uint64 InNumLocalChunksExpected) = 0;

		/**
		 * Set the number of local chunks available for the action.
		 * @param InstallerAction The installer action to set.
		 * @param InNumLocalChunksAvailable The number of local chunks available.
		 */
		virtual void OnNumLocalChunksAvailable(const FBuildPatchInstallerAction& InstallerAction, uint64 InNumLocalChunksAvailable) = 0;

		/**
		 * Increment the number of chunks that failed to recycle.
		 * @param ChunkId The id of a chunk that was not recycled.
		 */
		virtual void OnChunkRecycleFailure(const FGuid& ChunkId) = 0;
	};

	class FInstallActionStatisticsFactory
	{
		public:
		/**
		 * Create an install action statistics instance.
		 * @return The created instance.
		 */
		static IInstallActionStatistics* Create(const TArray<FBuildPatchInstallerAction> &InstallerActions);
	};
}