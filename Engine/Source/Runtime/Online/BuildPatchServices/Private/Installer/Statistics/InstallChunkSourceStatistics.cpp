// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/InstallChunkSourceStatistics.h"
#include "HAL/ThreadSafeBool.h"
#include "Core/AsyncHelpers.h"
#include "Common/SpeedRecorder.h"
#include "Common/StatsCollector.h"
#include "Installer/InstallerAnalytics.h"
#include "InstallActionStatistics.h"
#include "Installer/Statistics/FileOperationTracker.h"

namespace BuildPatchServices
{
	class FInstallChunkSourceStatistics
		: public IInstallChunkSourceStatistics
	{
	public:
		FInstallChunkSourceStatistics(ISpeedRecorder* SpeedRecorder, IInstallerAnalytics* InstallerAnalytics, IInstallActionStatistics* InInstallerActionStatistics, IFileOperationTracker* FileOperationTracker);
		~FInstallChunkSourceStatistics();

		// IInstallChunkSourceStat interface begin.
		virtual void OnLoadStarted(const FGuid& ChunkId) override;
		virtual void OnLoadComplete(const FGuid& ChunkId, const ELoadResult& Result, const ISpeedRecorder::FRecord& Record) override;
		// IInstallChunkSourceStat interface end.

		// IInstallChunkSourceStatistics interface begin.
		virtual uint64 GetBytesRead() const;
		virtual int32 GetNumSuccessfulChunkRecycles() const override;
		virtual int32 GetNumFailedChunkRecycles() const override;
		virtual bool IsCurrentlyReading() const override;
		// IInstallChunkSourceStatistics interface end.

	private:
		ISpeedRecorder* SpeedRecorder;
		IInstallerAnalytics* InstallerAnalytics;
		IInstallActionStatistics* InstallerActionStatistics;
		IFileOperationTracker* FileOperationTracker;
		FThreadSafeInt32 NumSuccessfulLoads;
		FThreadSafeInt32 NumFailedLoads;
		FThreadSafeInt64 TotalBytesRead;
		FThreadSafeBool bIsReading;
		FThreadSafeInt64 LastReadCycles;
	};

	FInstallChunkSourceStatistics::FInstallChunkSourceStatistics(ISpeedRecorder* InSpeedRecorder, IInstallerAnalytics* InInstallerAnalytics, IInstallActionStatistics* InInstallerActionStatistics, IFileOperationTracker* InFileOperationTracker)
		: SpeedRecorder(InSpeedRecorder)
		, InstallerAnalytics(InInstallerAnalytics)
		, InstallerActionStatistics(InInstallerActionStatistics)
		, FileOperationTracker(InFileOperationTracker)
		, NumSuccessfulLoads(0)
		, NumFailedLoads(0)
		, TotalBytesRead(0)
		, bIsReading(false)
		, LastReadCycles(0)
	{
	}

	FInstallChunkSourceStatistics::~FInstallChunkSourceStatistics()
	{
	}
	
	void FInstallChunkSourceStatistics::OnLoadStarted(const FGuid& ChunkId)
	{
		bIsReading = true;
		FileOperationTracker->OnDataStateUpdate(ChunkId, EFileOperationState::RetrievingLocalInstallData);
	}

	void FInstallChunkSourceStatistics::OnLoadComplete(const FGuid& ChunkId, const ELoadResult& Result, const ISpeedRecorder::FRecord& Record)
	{
		if (Result == ELoadResult::Success)
		{
			NumSuccessfulLoads.Increment();
		}
		else
		{
			FileOperationTracker->OnDataStateUpdate(ChunkId, EFileOperationState::PendingRemoteCloudData);
			InstallerAnalytics->RecordChunkCacheError(ChunkId, TEXT(""), FPlatformMisc::GetLastError(), TEXT("InstallChunkSourceLoad"), ToString(Result));
			InstallerActionStatistics->OnChunkRecycleFailure(ChunkId);
			NumFailedLoads.Increment();
		}
		SpeedRecorder->AddRecord(Record);
		TotalBytesRead.Add(Record.Size);
		bIsReading = false;
		LastReadCycles = Record.CyclesEnd;
	}

	uint64 FInstallChunkSourceStatistics::GetBytesRead() const
	{
		return TotalBytesRead.GetValue();
	}

	int32 FInstallChunkSourceStatistics::GetNumSuccessfulChunkRecycles() const
	{
		return NumSuccessfulLoads.GetValue();
	}

	int32 FInstallChunkSourceStatistics::GetNumFailedChunkRecycles() const
	{
		return NumFailedLoads.GetValue();
	}

	bool FInstallChunkSourceStatistics::IsCurrentlyReading() const
	{
		const int64 CyclesSinceRead = FStatsCollector::GetCycles() - LastReadCycles.GetValue();
		return bIsReading || FStatsCollector::CyclesToSeconds(CyclesSinceRead) < 0.05;
	}

	IInstallChunkSourceStatistics* FInstallChunkSourceStatisticsFactory::Create(ISpeedRecorder* SpeedRecorder, IInstallerAnalytics* InstallerAnalytics, IInstallActionStatistics* InInstallerActionStatistics, IFileOperationTracker* FileOperationTracker)
	{
		check(SpeedRecorder != nullptr);
		check(InstallerAnalytics != nullptr);
		check(FileOperationTracker != nullptr);
		return new FInstallChunkSourceStatistics(SpeedRecorder, InstallerAnalytics, InInstallerActionStatistics, FileOperationTracker);
	}
};