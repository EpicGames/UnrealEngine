// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/InstallChunkSource.h"

namespace BuildPatchServices
{
	class ISpeedRecorder;
	class IInstallerAnalytics;
	class IInstallActionStatistics;
	class IFileOperationTracker;

	/**
	 * Interface to the statistics class which provides access to tracked values from an install chunk source stat.
	 */
	class IInstallChunkSourceStatistics
		: public IInstallChunkSourceStat
	{
	public:
		/**
		 * @return the total number of bytes read.
		 */
		virtual uint64 GetBytesRead() const = 0;

		/**
		 * @return the number of chunks which were successfully loaded from local installations.
		 */
		virtual int32 GetNumSuccessfulChunkRecycles() const = 0;

		/**
		 * @return the number of chunks which failed to load from local installations.
		 */
		virtual int32 GetNumFailedChunkRecycles() const = 0;

		/**
		 * @return true if currently reading disk data.
		 */
		virtual bool IsCurrentlyReading() const = 0;
	};

	/**
	 * A factory for creating an IInstallChunkSourceStatistics instance.
	 */
	class FInstallChunkSourceStatisticsFactory
	{
	public:
		/**
		 * Creates the install chunk source's dependency interface and exposes additional information.
		 * @param SpeedRecorder         The speed recorder instance that we send activity records to.
		 * @param InstallerAnalytics    The analytics implementation for reporting the cloud source events.
		 * @param FileOperationTracker  The file operation tracker which will be used to update data states.
		 * @return the new IInstallChunkSourceStatistics instance created.
		 */
		static IInstallChunkSourceStatistics* Create(ISpeedRecorder* SpeedRecorder, IInstallerAnalytics* InstallerAnalytics, IInstallActionStatistics* InInstallerActionStatistics, IFileOperationTracker* FileOperationTracker);
	};
}