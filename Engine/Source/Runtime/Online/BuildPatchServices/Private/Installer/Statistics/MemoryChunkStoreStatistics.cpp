// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/MemoryChunkStoreStatistics.h"

#include "Misc/Guid.h"
#include "Misc/ScopeLock.h"

#include "Core/AsyncHelpers.h"
#include "Installer/Statistics/FileOperationTracker.h"

namespace BuildPatchServices
{
	class FMeanUseTracker
	{
	public:
		FMeanUseTracker();

		void Increment();
		void Decrement();
		void CountSame();
		float GetMean() const;
		int32 GetPeak() const;
		int32 GetCurrent() const;

	private:
		FThreadSafeInt32 UsesCount;
		FThreadSafeInt64 SumCurrentUses;
		FThreadSafeInt32 CurrentUse;
		volatile int32 PeakUse;
	};

	FMeanUseTracker::FMeanUseTracker()
		: UsesCount(0)
		, SumCurrentUses(0)
		, CurrentUse(0)
		, PeakUse(0)
	{
	}

	void FMeanUseTracker::Increment()
	{
		int32 NewValue = CurrentUse.Increment();
		SumCurrentUses.Add(NewValue);
		UsesCount.Increment();
		AsyncHelpers::LockFreePeak(&PeakUse, NewValue);
	}

	void FMeanUseTracker::Decrement()
	{
		int32 NewValue = CurrentUse.Decrement();
		SumCurrentUses.Add(NewValue);
		UsesCount.Increment();
	}

	void FMeanUseTracker::CountSame()
	{
		SumCurrentUses.Add(CurrentUse.GetValue());
		UsesCount.Increment();
	}

	float FMeanUseTracker::GetMean() const
	{
		const double SumCurrentUsesFloat = SumCurrentUses.GetValue();
		const double UseChangeCountFloat = UsesCount.GetValue();
		return UseChangeCountFloat > 0.f ? SumCurrentUsesFloat / UseChangeCountFloat : 0.f;
	}

	int32 FMeanUseTracker::GetPeak() const
	{
		return PeakUse;
	}

	int32 FMeanUseTracker::GetCurrent() const
	{
		return CurrentUse.GetValue();
	}

	class FMemoryChunkStoreStatistics
		: public IMemoryChunkStoreStatistics
	{
	public:
		FMemoryChunkStoreStatistics(IFileOperationTracker* FileOperationTracker);
		~FMemoryChunkStoreStatistics();

		// IMemoryChunkStoreStat interface begin.
		virtual void OnChunkStored(const FGuid& ChunkId) override;
		virtual void OnChunkReleased(const FGuid& ChunkId) override;
		virtual void OnChunkBooted(const FGuid& ChunkId) override;
		virtual void OnStoreUseUpdated(int32 ChunkCount) override;
		virtual void OnStoreSizeUpdated(int32 Size) override;
		// IMemoryChunkStoreStat interface end.

		// IMemoryChunkStoreStatistics interface begin.
		virtual int32 GetStoreUse() const override;
		virtual int32 GetStoreRetained() const override;
		virtual int32 GetNumBooted() const override;
		virtual int32 GetStoreSize() const override;
		virtual float GetAverageStoreUse() const override;
		virtual int32 GetPeakStoreUse() const override;
		virtual float GetAverageStoreRetained() const override;
		virtual int32 GetPeakStoreRetained() const override;
		virtual void SetMultipleReferencedChunk(TSet<FGuid> MultipleReferencedChunks) override;
		// IMemoryChunkStoreStatistics interface end.

	private:
		bool IsMultipleReferenced(const FGuid& ChunkId);

	private:
		IFileOperationTracker* const FileOperationTracker;
		FMeanUseTracker StoreUseTracker;
		FMeanUseTracker StoreRetainTracker;
		FThreadSafeInt32 NumChunksBooted;
		FThreadSafeInt32 StoreSize;
		FCriticalSection MultipleReferencedChunksCS;
		TSet<FGuid> MultipleReferencedChunks;
	};

	FMemoryChunkStoreStatistics::FMemoryChunkStoreStatistics(IFileOperationTracker* InFileOperationTracker)
		: FileOperationTracker(InFileOperationTracker)
	{
	}

	FMemoryChunkStoreStatistics::~FMemoryChunkStoreStatistics()
	{
	}

	void FMemoryChunkStoreStatistics::OnChunkStored(const FGuid& ChunkId)
	{
		StoreUseTracker.Increment();
		FileOperationTracker->OnDataStateUpdate(ChunkId, EFileOperationState::DataInMemoryStore);
		if (IsMultipleReferenced(ChunkId))
		{
			StoreRetainTracker.Increment();
		}
		else
		{
			StoreRetainTracker.CountSame();
		}
	}

	void FMemoryChunkStoreStatistics::OnChunkReleased(const FGuid& ChunkId)
	{
		StoreUseTracker.Decrement();
		if (IsMultipleReferenced(ChunkId))
		{
			StoreRetainTracker.Decrement();
		}
		else
		{
			StoreRetainTracker.CountSame();
		}
	}

	void FMemoryChunkStoreStatistics::OnChunkBooted(const FGuid& ChunkId)
	{
		NumChunksBooted.Increment();
		// We desire a release count too, as it tracks just numbers in the store.
		OnChunkReleased(ChunkId);
	}

	void FMemoryChunkStoreStatistics::OnStoreUseUpdated(int32 ChunkCount)
	{
	}

	void FMemoryChunkStoreStatistics::OnStoreSizeUpdated(int32 Size)
	{
		StoreSize.Set(Size);
	}

	int32 FMemoryChunkStoreStatistics::GetStoreUse() const
	{
		return StoreUseTracker.GetCurrent();
	}

	int32 FMemoryChunkStoreStatistics::GetStoreRetained() const
	{
		return StoreRetainTracker.GetCurrent();
	}

	int32 FMemoryChunkStoreStatistics::GetNumBooted() const
	{
		return NumChunksBooted.GetValue();
	}

	int32 FMemoryChunkStoreStatistics::GetStoreSize() const
	{
		return StoreSize.GetValue();
	}

	float FMemoryChunkStoreStatistics::GetAverageStoreUse() const
	{
		return StoreUseTracker.GetMean();
	}

	int32 FMemoryChunkStoreStatistics::GetPeakStoreUse() const
	{
		return StoreUseTracker.GetPeak();
	}

	float FMemoryChunkStoreStatistics::GetAverageStoreRetained() const
	{
		return StoreRetainTracker.GetMean();
	}

	int32 FMemoryChunkStoreStatistics::GetPeakStoreRetained() const
	{
		return StoreRetainTracker.GetPeak();
	}

	void FMemoryChunkStoreStatistics::SetMultipleReferencedChunk(TSet<FGuid> InMultipleReferencedChunks)
	{
		FScopeLock ScopeLock(&MultipleReferencedChunksCS);
		MultipleReferencedChunks = MoveTemp(InMultipleReferencedChunks);
	}

	bool FMemoryChunkStoreStatistics::IsMultipleReferenced(const FGuid& ChunkId)
	{
		FScopeLock ScopeLock(&MultipleReferencedChunksCS);
		return MultipleReferencedChunks.Contains(ChunkId);
	}

	IMemoryChunkStoreStatistics* FMemoryChunkStoreStatisticsFactory::Create(IFileOperationTracker* FileOperationTracker)
	{
		check(FileOperationTracker != nullptr);
		return new FMemoryChunkStoreStatistics(FileOperationTracker);
	}
};