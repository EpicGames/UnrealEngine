// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/MemoryChunkStore.h"

namespace BuildPatchServices
{
	class IMemoryChunkStoreStat;
	class IFileOperationTracker;

	/**
	 * Interface to the statistics class which provides access to tracked values from a memory chunk store stat.
	 */
	class IMemoryChunkStoreStatistics
		: public IMemoryChunkStoreStat
	{
	public:
		/**
		 * @return the number of chunks held.
		 */
		virtual int32 GetStoreUse() const = 0;

		/**
		 * @return the number of chunks held which are retained due to multiple references.
		 */
		virtual int32 GetStoreRetained() const = 0;

		/**
		 * @return the number of chunks which have been booted.
		 */
		virtual int32 GetNumBooted() const = 0;

		/**
		 * @return the maximum number of chunks which can be held.
		 */
		virtual int32 GetStoreSize() const = 0;

		/**
		 * @return the average number of chunks held.
		 */
		virtual float GetAverageStoreUse() const = 0;

		/**
		 * @return the peak number of chunks held.
		 */
		virtual int32 GetPeakStoreUse() const = 0;

		/**
		 * @return the average number of chunks held which are retained due to multiple references.
		 */
		virtual float GetAverageStoreRetained() const = 0;

		/**
		 * @return the peak number of chunks held which are retained due to multiple references.
		 */
		virtual int32 GetPeakStoreRetained() const = 0;

		/**
		 * Sets the chunks that are referenced multiple times in order to track retained.
		 * @param MultipleReferencedChunks For retained stats, the set of chunks which have multiple references.
		 */
		virtual void SetMultipleReferencedChunk(TSet<FGuid> MultipleReferencedChunks) = 0;
	};

	/**
	 * A factory for creating an IMemoryChunkStoreStatistics instance.
	 */
	class FMemoryChunkStoreStatisticsFactory
	{
	public:
		/**
		 * Creates a statistics interface for getting access to store usage stats, and also forwards information to update the file operation tracker.
		 * @param FileOperationTracker          The file operation tracker which will be used to update data states.
		 * @return the new IMemoryChunkStoreStatistics instance created.
		 */
		static IMemoryChunkStoreStatistics* Create(IFileOperationTracker* FileOperationTracker);
	};
}