// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Statistics/VerifierStatistics.h"
#include "HAL/ThreadSafeBool.h"
#include "Core/AsyncHelpers.h"
#include "Common/SpeedRecorder.h"
#include "Common/StatsCollector.h"
#include "BuildPatchProgress.h"
#include "Installer/Statistics/FileOperationTracker.h"
#include "BuildPatchVerify.h"

#include <atomic>

namespace VerifierStatisticsHelpers
{
	bool TryConvert(BuildPatchServices::EVerifyResult InResult, BuildPatchServices::EVerifyError& OutError)
	{
		using namespace BuildPatchServices;
		switch (InResult)
		{
			case EVerifyResult::FileMissing: OutError = EVerifyError::FileMissing; return true;
			case EVerifyResult::OpenFileFailed: OutError = EVerifyError::OpenFileFailed; return true;
			case EVerifyResult::HashCheckFailed: OutError = EVerifyError::HashCheckFailed; return true;
			case EVerifyResult::FileSizeFailed: OutError = EVerifyError::FileSizeFailed; return true;
		}

		return false;
	}
}

namespace BuildPatchServices
{
	class FVerifierStatistics
		: public IVerifierStatistics
	{
	public:
		FVerifierStatistics(ISpeedRecorder* SpeedRecorder, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker);
		~FVerifierStatistics();

		// IVerifierStat interface start.
		virtual void OnFileStarted(const FString& Filename, int64 FileSize) override;
		virtual void OnFileProgress(const FString& Filename, int64 TotalBytes) override;
		virtual void OnFileCompleted(const FString& Filename, EVerifyResult VerifyResult) override;
		virtual void OnFileRead(const ISpeedRecorder::FRecord& Record) override;
		virtual void OnProcessedDataUpdated(int64 TotalBytes) override;
		virtual void OnTotalRequiredUpdated(int64 TotalBytes) override;
		// IVerifierStat interface end.

		// IVerifierStatistics interface start.
		virtual uint64 GetBytesVerified() const override;
		virtual int32 GetNumSuccessfulFilesVerified() const override;
		virtual int32 GetNumFailedFilesVerified() const override;
		virtual TMap<EVerifyError, int32> GetVerifyErrorCounts() const override;
		virtual bool IsCurrentlyReading() const override;
		// IVerifierStatistics interface end.

	private:
		ISpeedRecorder* SpeedRecorder;
		FBuildPatchProgress* BuildProgress;
		IFileOperationTracker* FileOperationTracker;
		std::atomic_int64_t TotalBytesProcessed;
		std::atomic_int64_t TotalBytesRequired;
		std::atomic_int64_t TotalBytesRead;
		std::atomic_int64_t NumSuccessfulFilesVerified;
		std::atomic_int64_t NumFailedFilesVerified;
		TMap<EVerifyError, FThreadSafeInt64> NumFileVerifyErrors;
		std::atomic_bool bIsReading;
		std::atomic_int64_t LastReadCycles;
		std::atomic_uint64_t LastByteRangeStateUpdate;
	};

	FVerifierStatistics::FVerifierStatistics(ISpeedRecorder* InSpeedRecorder, FBuildPatchProgress* InBuildProgress, IFileOperationTracker* InFileOperationTracker)
		: SpeedRecorder(InSpeedRecorder)
		, BuildProgress(InBuildProgress)
		, FileOperationTracker(InFileOperationTracker)
		, TotalBytesProcessed(0)
		, TotalBytesRequired(0)
		, TotalBytesRead(0)
		, NumSuccessfulFilesVerified(0)
		, NumFailedFilesVerified(0)
		, bIsReading(false)
		, LastReadCycles(0)
		, LastByteRangeStateUpdate(0)
	{
		for (EVerifyError Error : TEnumRange<EVerifyError>())
		{
			NumFileVerifyErrors.Add(Error, 0);
		}
	}

	FVerifierStatistics::~FVerifierStatistics()
	{
	}

	void FVerifierStatistics::OnFileStarted(const FString& Filename, int64 FileSize)
	{
		bIsReading = true;
		LastByteRangeStateUpdate = 0;
		FileOperationTracker->OnFileByteRangeStateUpdate(Filename, FByteRange(LastByteRangeStateUpdate, LastByteRangeStateUpdate), EFileOperationState::Verifying);
	}

	void FVerifierStatistics::OnFileProgress(const FString& Filename, int64 TotalBytes)
	{
		FileOperationTracker->OnFileByteRangeStateUpdate(Filename, FByteRange(LastByteRangeStateUpdate, TotalBytes), EFileOperationState::Verifying);
		LastByteRangeStateUpdate = TotalBytes;
	}

	void FVerifierStatistics::OnFileCompleted(const FString& Filename, EVerifyResult VerifyResult)
	{
		const bool bSuccess = VerifyResult == EVerifyResult::Success;
		if (bSuccess)
		{
			NumSuccessfulFilesVerified++;
		}
		else
		{
			EVerifyError Error;
			if (VerifierStatisticsHelpers::TryConvert(VerifyResult, Error))
			{
				NumFailedFilesVerified++;
				NumFileVerifyErrors[Error].Increment();
			}
		}
		FileOperationTracker->OnFileStateUpdate(Filename, bSuccess ? EFileOperationState::VerifiedSuccess : EFileOperationState::VerifiedFail);
		bIsReading = false;
		LastReadCycles = FStatsCollector::GetCycles();
	}

	void FVerifierStatistics::OnFileRead(const ISpeedRecorder::FRecord& Record)
	{
		SpeedRecorder->AddRecord(Record);
		TotalBytesRead.fetch_add(Record.Size);
	}

	void FVerifierStatistics::OnProcessedDataUpdated(int64 TotalBytes)
	{
		TotalBytesProcessed.store(TotalBytes);
		int64 Required = TotalBytesRequired.load();
		if (Required > 0)
		{
			BuildProgress->SetStateProgress(EBuildPatchState::BuildVerification, (double)TotalBytes / (double)Required);
		}
	}

	void FVerifierStatistics::OnTotalRequiredUpdated(int64 TotalBytes)
	{
		TotalBytesRequired.store(TotalBytes);
		int64 Processed = TotalBytesProcessed.load();
		if (TotalBytes > 0)
		{
			BuildProgress->SetStateProgress(EBuildPatchState::BuildVerification, (double)Processed / (double)TotalBytes);
		}
	}

	uint64 FVerifierStatistics::GetBytesVerified() const
	{
		return TotalBytesRead.load();
	}

	int32 FVerifierStatistics::GetNumSuccessfulFilesVerified() const
	{
		return NumSuccessfulFilesVerified.load();
	}

	int32 FVerifierStatistics::GetNumFailedFilesVerified() const
	{
		return NumFailedFilesVerified.load();
	}

	TMap<EVerifyError, int32> FVerifierStatistics::GetVerifyErrorCounts() const
	{
		TMap<EVerifyError, int32> ErrorResults;
		for (const TPair<EVerifyError, FThreadSafeInt64>& ResultPair : NumFileVerifyErrors)
		{
			if (ResultPair.Value.GetValue() > 0)
			{
				ErrorResults.Add(ResultPair.Key, ResultPair.Value.GetValue());
			}
		}
		return ErrorResults;
	}

	bool FVerifierStatistics::IsCurrentlyReading() const
	{
		const int64 CyclesSinceRead = FStatsCollector::GetCycles() - LastReadCycles.load();
		return bIsReading || FStatsCollector::CyclesToSeconds(CyclesSinceRead) < 0.05;
	}

	IVerifierStatistics* FVerifierStatisticsFactory::Create(ISpeedRecorder* SpeedRecorder, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker)
	{
		check(SpeedRecorder != nullptr);
		check(BuildProgress != nullptr);
		check(FileOperationTracker != nullptr);
		return new FVerifierStatistics(SpeedRecorder, BuildProgress, FileOperationTracker);
	}
};