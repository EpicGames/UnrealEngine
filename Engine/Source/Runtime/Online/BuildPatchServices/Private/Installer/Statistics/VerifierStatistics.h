// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/Verifier.h"

namespace BuildPatchServices
{
	class ISpeedRecorder;
	struct FBuildPatchProgress;
	class IFileOperationTracker;
	enum class EVerifyError : uint32;

	/**
	 * Interface to the statistics class which provides access to tracked values from a verifier stat.
	 */
	class IVerifierStatistics
		: public IVerifierStat
	{
	public:
		/**
		 * @return the total number of bytes verified.
		 */
		virtual uint64 GetBytesVerified() const = 0;

		/**
		 * @return the number of file which were successfully verified.
		 */
		virtual int32 GetNumSuccessfulFilesVerified() const = 0;

		/**
		 * @return the number of files which failed verification.
		 */
		virtual int32 GetNumFailedFilesVerified() const = 0;

		/**
		 * @return the map of error result to count.
		 */
		virtual TMap<EVerifyError, int32> GetVerifyErrorCounts() const = 0;

		/**
		 * @return true if currently reading disk data.
		 */
		virtual bool IsCurrentlyReading() const = 0;
	};

	/**
	 * A factory for creating an IVerifierStatistics instance.
	 */
	class FVerifierStatisticsFactory
	{
	public:
		/**
		 * Creates the verifier's dependency interface and exposes additional information.
		 * @param SpeedRecorder         The speed recorder instance that we send activity records to.
		 * @param BuildProgress         The legacy progress implementation to bridge the system stats to.
		 * @param FileOperationTracker  The file operation tracker which will be used to update data states.
		 * @return the new IVerifierStatistics instance created.
		 */
		static IVerifierStatistics* Create(ISpeedRecorder* SpeedRecorder, FBuildPatchProgress* BuildProgress, IFileOperationTracker* FileOperationTracker);
	};
}