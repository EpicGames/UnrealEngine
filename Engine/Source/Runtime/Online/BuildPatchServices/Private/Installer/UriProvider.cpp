// Copyright Epic Games, Inc. All Rights Reserved.

#include "UriProvider.h"

#include "BuildPatchMessage.h"
#include "MessagePump.h"


using namespace BuildPatchServices;

DECLARE_LOG_CATEGORY_CLASS(LogUriProvider, Log, All);

class UriProvider
	: public IUriProvider
	, public TSharedFromThis<UriProvider, ESPMode::ThreadSafe>
{
public:

	UriProvider(TArray<FString> InCloudDirectories, IMessagePump* InMessagePump)
		: CloudDirectories(MoveTemp(InCloudDirectories))
		, MessagePump(InMessagePump)
	{
	}
		
	void RequestUri(const FString& DeltaId, const FString& BuildId, FUriProviderCompleteDelegate& OnCompleteDelegate)
	{
		if (bRequestInProgress)
		{
			OnCompleteDelegate.ExecuteIfBound(false, FString(), true);
			return;
		}

		// Check whether we can retry with an existing list of URIs
		if (ReceivedUris.ExpiratedUri.Num() > 0)
		{
			EndpointIndex = (EndpointIndex + 1) % ReceivedUris.ExpiratedUri.Num();
			const FExpiratedUriResponse& CurrentEndpoint = ReceivedUris.ExpiratedUri[EndpointIndex];
			if (!CurrentEndpoint.SignatureExpiration.IsEmpty())
			{
				if (!IsDeltaUriExpired(CurrentEndpoint.SignatureExpiration))
				{
					OnCompleteDelegate.ExecuteIfBound(true, CurrentEndpoint.Uri, true);
					return;
				}
				else
				{
					UE_LOGF(LogUriProvider, Warning, "Delta file URI signature was expired or missed");
				}
			}

			// Empty expiration time means this URI doesn't expire
		}

		FDeltaUriRequest DeltaUriRequest;
		DeltaUriRequest.DeltaPath = DeltaId;
		DeltaUriRequest.UniqueBuildId = BuildId;
		DeltaUriRequest.CloudDirectory = CloudDirectories[CloudDirectoryIndex];
		CloudDirectoryIndex = (CloudDirectoryIndex + 1) % CloudDirectories.Num();
		EndpointIndex = 0;

		MessagePump->SendRequest(DeltaUriRequest, [WeakThisPtr = TWeakPtr<UriProvider, ESPMode::ThreadSafe>(SharedThis(this)), OnCompleteDelegate = MoveTemp(OnCompleteDelegate)](FExpiratedUrisResponse InReceivedUris)
		{
			if (TSharedPtr<UriProvider, ESPMode::ThreadSafe> PinnedSharedPtr = WeakThisPtr.Pin())
			{
				PinnedSharedPtr->bRequestInProgress = false;
				PinnedSharedPtr->ReceivedUris = MoveTemp(InReceivedUris);

				FString Uri;
				if (PinnedSharedPtr->ReceivedUris.ExpiratedUri.Num() > 0)
				{
					Uri = PinnedSharedPtr->ReceivedUris.ExpiratedUri[0].Uri;
				}
				OnCompleteDelegate.ExecuteIfBound(!Uri.IsEmpty(), MoveTemp(Uri), PinnedSharedPtr->ReceivedUris.bNeedRetry);
			}
			else
			{
				UE_LOGF(LogUriProvider, Warning, "Skip Delta Uri response, UriProvider already destroyed.");
			}

		});

		bRequestInProgress = true;
	}

private:

	bool IsDeltaUriExpired(const FString& ExpirationTime)
	{
		FDateTime SignedDeltaUriExpirationTime;
		if (!FDateTime::ParseIso8601(*ExpirationTime, SignedDeltaUriExpirationTime))
		{
			UE_LOGF(LogUriProvider, Warning, "Fail to parse expiration time for signed delta URL from string '%ls' (using ParseIso8601)", *ExpirationTime);
			// Considering parse failure as delta URL expiration
			return true;
		}

		FTimespan LeftTime = SignedDeltaUriExpirationTime - FDateTime::UtcNow();
		const uint32_t SecondsBeforeExpiration = 5;
		FTimespan TicketExpirationTime(0, 0, SecondsBeforeExpiration);
		return LeftTime <= TicketExpirationTime;
	}

private:

	FExpiratedUrisResponse ReceivedUris;
	int32_t EndpointIndex = 0;
	int32_t CloudDirectoryIndex = 0;
	TArray<FString> CloudDirectories;

	bool bRequestInProgress = false;

	IMessagePump* MessagePump = nullptr;
};

TSharedRef<IUriProvider, ESPMode::ThreadSafe> FUriProviderFactory::Create(IMessagePump* MessagePump, TArray<FString> CloudDirectories)
{
	check(MessagePump != nullptr);
	check(CloudDirectories.Num() > 0);
	return MakeShared<UriProvider, ESPMode::ThreadSafe>(MoveTemp(CloudDirectories), MessagePump);
}