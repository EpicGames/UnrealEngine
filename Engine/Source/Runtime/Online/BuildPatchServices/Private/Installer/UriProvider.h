// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"

namespace BuildPatchServices
{
	class IMessagePump;

	/**
	 * Delegate called when URI is ready.
	 * @param bSuccessful     Status of the request, true means success.
	 * @param Uri             The URI itself that can be used for downloading the file.
	 * @param bNeedRerty      Flag that allows a retry to be used if the URI cannot be obtained.
	 */
	DECLARE_DELEGATE_ThreeParams(FUriProviderCompleteDelegate, bool /*bSuccessful*/, const FString& /*Uri*/, bool /*bNeedRerty*/);

	/**
	 * An interface providing the ability to retrieve a URI for files downloading.
	 * Comparing to SignedUriProvider this interface is more general and hides MessagePump-related implementation inside.
	 * Methods of this class should be called from the same thread and this class doesn't support two concurrent requests.
	 */
	class IUriProvider
	{
	public:

		virtual ~IUriProvider() {}

		/**
		 * Starts a new request to obtain a URI for delta file.
		 * If there is another request in flight, this method will call OnCompleteDelegate delegate with false result immediately.
		 * @param DeltaId               The ID of the delta file for which it is needed to obtain a URI.
		 * @param BuildId               Build ID for the corresponding delta file.
		 * @param OnCompleteDelegate    The delegate that will be called when the URI will be ready.
		 */
		virtual void RequestUri(const FString& DeltaId, const FString& BuildId, FUriProviderCompleteDelegate& OnCompleteDelegate) = 0;
	};

	/**
	 * A factory for creating the default implementation of IUriProvider.
	 */
	class FUriProviderFactory
	{
	public:
		/**
		 * Instantiates an instance of an IUriProvider, using the MessagePump.
		 * @param MessagePump      The message pump object that will be used for URIs obtaining.
		 * @param CloudDirectores  A list of cloud directories that can be used for generating a URI.
		 * @return the new IUriProvider instance created.
		 */
		static TSharedRef<IUriProvider, ESPMode::ThreadSafe> Create(IMessagePump* MessagePump, TArray<FString> CloudDirectories);
	};
}
