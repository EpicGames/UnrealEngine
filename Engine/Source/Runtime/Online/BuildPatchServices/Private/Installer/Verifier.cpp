// Copyright Epic Games, Inc. All Rights Reserved.

#include "Installer/Verifier.h"
#include "HAL/ThreadSafeBool.h"
#include "Common/StatsCollector.h"
#include "Common/FileSystem.h"
#include "Core/AsyncHelpers.h"
#include "BuildPatchVerify.h"
#include "BuildPatchUtil.h"
#include "HAL/UESemaphore.h"
#include "IBuildManifestSet.h"
#include "Misc/ConfigCacheIni.h"

#include <thread>

#if PLATFORM_MAC || PLATFORM_LINUX
#include <sys/stat.h>
#include <unistd.h>
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogVerifier, Warning, All);
DEFINE_LOG_CATEGORY(LogVerifier);

namespace BuildPatchServices
{
	bool TryConvertToVerifyResult(EVerifyError InVerifyError, EVerifyResult& OutVerifyResult)
	{
		switch (InVerifyError)
		{
			case EVerifyError::FileMissing: OutVerifyResult = EVerifyResult::FileMissing; return true;
			case EVerifyError::OpenFileFailed: OutVerifyResult = EVerifyResult::OpenFileFailed; return true;
			case EVerifyError::HashCheckFailed: OutVerifyResult = EVerifyResult::HashCheckFailed; return true;
			case EVerifyError::FileSizeFailed: OutVerifyResult = EVerifyResult::FileSizeFailed; return true;
		}
		return false;
	}

	bool TryConvertToVerifyError(EVerifyResult InVerifyResult, EVerifyError& OutVerifyError)
	{
		switch (InVerifyResult)
		{
			case EVerifyResult::FileMissing: OutVerifyError = EVerifyError::FileMissing; return true;
			case EVerifyResult::OpenFileFailed: OutVerifyError = EVerifyError::OpenFileFailed; return true;
			case EVerifyResult::HashCheckFailed: OutVerifyError = EVerifyError::HashCheckFailed; return true;
			case EVerifyResult::FileSizeFailed: OutVerifyError = EVerifyError::FileSizeFailed; return true;
		}
		return false;
	}

	class FVerifier : public IVerifier
	{
	public:
		FVerifier(IFileSystem* FileSystem, IVerifierStat* InVerificationStat, EVerifyMode InVerifyMode,
			IBuildInstallerSharedContextPtr InSharedContext, IBuildManifestSet* InManifestSet, 
			FString InVerifyDirectory, FString InStagedFileDirectory, bool bUseSHA1StageFilenames, 
			const TMap<FString, FString>& InPerFileSubdirectories);
		~FVerifier() {}

		// IControllable interface begin.
		virtual void SetPaused(bool bInIsPaused) override;
		virtual void Abort() override;
		virtual void Reset() override;
		// IControllable interface end.

		// IVerifier interface begin.
		virtual EVerifyResult Verify(TArray<FString>& CorruptFiles) override;
		virtual void AddTouchedFiles(const TSet<FString>& TouchedFiles) override;
		// IVerifier interface end.

	private:
		FString SelectFullFilePath(const FString& BuildFile);
		EVerifyResult VerifyFileSha(TArray<uint8>& ReadBuffer, const FString& BuildFile, const FFileManifest& BuildFileManifest);
		EVerifyResult VerifyFileSize(const FString& BuildFile, const FFileManifest& BuildFileManifest);
		EVerifyResult VerifySymlink(const FString& BuildFile, const FFileManifest& BuildFileManifest);

	private:
		const FString VerifyDirectory;
		const FString StagedFileDirectory;
		IFileSystem* const FileSystem;
		IVerifierStat* const VerifierStat;
		IBuildManifestSet* const ManifestSet;
		const bool bUseSHA1StageFilenames;
		IBuildInstallerSharedContextPtr SharedContext;
		const TMap<FString, FString>& PerFileSubdirectories;

		EVerifyMode VerifyMode;
		TSet<FString> FilesToVerify;
		TSet<FString> FilesPassedVerify;
		FThreadSafeBool bIsPaused;
		FThreadSafeBool bShouldAbort;

		struct FThreadVerifyJob
		{
			const FFileManifest* BuildFileManifest;
			bool bVerifySha;
			FString FileName;
		};

		std::atomic_int64_t ThreadProcessedBytes;
		TArray<EVerifyResult> ThreadJobResults;

		UE::FMutex ThreadJobListLock;
		int32 NextThreadJobListIndex = 0;		
		TArray<FThreadVerifyJob> ThreadJobList;

		TArray<IBuildInstallerThread*> SecondaryProcessThreads;

		void ProcessVerifyJobs(FSemaphore* ThreadDoneSem);
	};

	FVerifier::FVerifier(IFileSystem* InFileSystem, IVerifierStat* InVerificationStat, EVerifyMode InVerifyMode, 
		IBuildInstallerSharedContextPtr InSharedContext, IBuildManifestSet* InManifestSet, FString InVerifyDirectory, FString InStagedFileDirectory,
		bool bUseSHA1StageFilenames, const TMap<FString, FString>& InPerFileSubdirectories)
		: VerifyDirectory(MoveTemp(InVerifyDirectory))
		, StagedFileDirectory(MoveTemp(InStagedFileDirectory))
		, FileSystem(InFileSystem)
		, VerifierStat(InVerificationStat)
		, ManifestSet(InManifestSet)
		, bUseSHA1StageFilenames(bUseSHA1StageFilenames)
		, SharedContext(InSharedContext)
		, PerFileSubdirectories(InPerFileSubdirectories)
		, VerifyMode(InVerifyMode)
		, bIsPaused(false)
		, bShouldAbort(false)
	{
		ManifestSet->GetFilesTaggedForRepair(FilesToVerify);
	}

	void FVerifier::ProcessVerifyJobs(FSemaphore* ThreadDoneSem)
	{
		TArray<uint8> FileReadBuffer;

		for (;;)
		{
			if (bShouldAbort)
			{
				break;
			}

			// Get a thread index.
			int32 OurJobIndex = -1;
			{
				UE::TUniqueLock _(ThreadJobListLock);
				if (NextThreadJobListIndex < ThreadJobList.Num())
				{
					OurJobIndex = NextThreadJobListIndex;
					NextThreadJobListIndex++;
				}
			}

			if (OurJobIndex == -1)
			{
				// No work to do.
				break;
			}

			// Only allocate our buffer when we get a job
			FileReadBuffer.SetNumUninitialized(4 << 20); // 4MB chunked reads.

			FThreadVerifyJob& OurJob = ThreadJobList[OurJobIndex];
			if (OurJob.BuildFileManifest == nullptr)
			{
				UE_LOGF(LogVerifier, Error, "FVerifier: Unable to find a file manifest for: %ls", *OurJob.FileName);
				ThreadJobResults[OurJobIndex] = EVerifyResult::FileMissing;
				continue;
			}

			const bool bIsSymlink = !OurJob.BuildFileManifest->SymlinkTarget.IsEmpty();
			const bool bVerifySha = OurJob.bVerifySha;

			VerifierStat->OnFileStarted(OurJob.FileName, OurJob.BuildFileManifest->FileSize);

			EVerifyResult FileVerifyResult;
			if (bIsSymlink)
			{
				FileVerifyResult = VerifySymlink(OurJob.FileName, *OurJob.BuildFileManifest);
			}
			else if (bVerifySha)
			{
				FileVerifyResult = VerifyFileSha(FileReadBuffer, OurJob.FileName, *OurJob.BuildFileManifest);
			}
			else
			{
				FileVerifyResult = VerifyFileSize(OurJob.FileName, *OurJob.BuildFileManifest);
			}

			VerifierStat->OnFileCompleted(OurJob.FileName, FileVerifyResult);

			ThreadJobResults[OurJobIndex] = FileVerifyResult;
		}

		if (ThreadDoneSem)
		{
			ThreadDoneSem->Release(1);
		}
		
		// We can't touch _anything_ after here because the outer class holding the lambda could
		// be destroyed.
	}

	void FVerifier::SetPaused(bool bInIsPaused)
	{
		bIsPaused = bInIsPaused;
	}

	void FVerifier::Abort()
	{
		bShouldAbort = true;
	}

	void FVerifier::Reset()
	{
		bShouldAbort = false;
	}

	int32 ConfigMaxFileVerifyingTasks(int32 DefaultValue)
	{
		int32 ConfigMaxFileVerifyingTasks = DefaultValue;
		GConfig->GetInt(TEXT("Portal.BuildPatch"), TEXT("MaxFileVerifyingTasks"), ConfigMaxFileVerifyingTasks, GEngineIni);
		return FMath::Clamp(ConfigMaxFileVerifyingTasks, 1, DefaultValue);
	}

	EVerifyResult FVerifier::Verify(TArray<FString>& CorruptFiles)
	{
		bShouldAbort = false;
		TRACE_CPUPROFILER_EVENT_SCOPE(Verify);
		bShouldAbort = false;
		EVerifyResult VerifyResult = EVerifyResult::Success;
		CorruptFiles.Empty();

		// If we check all files, grab them all now.
		if (VerifyMode == EVerifyMode::FileSizeCheckAllFiles || VerifyMode == EVerifyMode::ShaVerifyAllFiles)
		{
			ManifestSet->GetExpectedFiles(FilesToVerify);
		}

		// Setup progress tracking.
		TSet<FString> VerifyList = FilesToVerify.Difference(FilesPassedVerify);
		VerifierStat->OnProcessedDataUpdated(0);
		VerifierStat->OnTotalRequiredUpdated(ManifestSet->GetTotalNewFileSize(VerifyList));

		// Select verify function.
		const bool bVerifyShaMode = VerifyMode == EVerifyMode::ShaVerifyAllFiles || VerifyMode == EVerifyMode::ShaVerifyTouchedFiles;
		
		UE_LOGF(LogVerifier, Verbose, "FVerifier: Setup progress tracking: files %d, mode %ls.", VerifyList.Num(), LexToString(VerifyMode));

		ThreadJobList.Reset();
		ThreadJobResults.Reset();
		ThreadProcessedBytes = 0;
		NextThreadJobListIndex = 0;

		for (const FString& BuildFile : VerifyList)
		{
			FThreadVerifyJob Job;
			Job.FileName = BuildFile;
			Job.BuildFileManifest = ManifestSet->GetNewFileManifest(BuildFile);
			Job.bVerifySha = bVerifyShaMode || ManifestSet->IsFileRepairAction(BuildFile);

			ThreadJobList.Add(MoveTemp(Job));
			ThreadJobResults.Add(EVerifyResult::Aborted);
		}

		if (SharedContext)
		{
			// 3 cores should saturate any modern drive, and on anything that isn't we just sit in the wait
			// state anyway. 
			int32 ThreadsToCreate = 1;
			GConfig->GetInt(TEXT("BuildPatchTool"), TEXT("VerificationThreadCount"), ThreadsToCreate, GEngineIni);

			int32 ThreadsRemaining = ThreadsToCreate;

			FSemaphore ThreadDoneSem(0, ThreadsToCreate);
			
			// We wait to create our threads so that the shared threads from file construction are returned.
			for (int i = 0; i < ThreadsToCreate; i++)
			{
				SecondaryProcessThreads.Add(SharedContext->CreateThread());
				SecondaryProcessThreads[i]->RunTask([this, &ThreadDoneSem]() { ProcessVerifyJobs(&ThreadDoneSem); });
			}

			for (;;)
			{
				if (ThreadDoneSem.TryAcquire(100))
				{
					ThreadsRemaining--;
					if (ThreadsRemaining == 0)
					{
						// We acquired all the counts so we're done.
						break;
					}
				}

				// Update our overall progress tracker.
				VerifierStat->OnProcessedDataUpdated(ThreadProcessedBytes.load(std::memory_order_acquire));
			}

			for (int i = 0; i < ThreadsToCreate; i++)
			{
				SharedContext->ReleaseThread(SecondaryProcessThreads[i]);
			}
			SecondaryProcessThreads.Empty();
		}
		else
		{
			// Can't create threads, just do the work here.
			ProcessVerifyJobs(nullptr);
		}

		// move results over.
		for (int32 i=0; i<ThreadJobList.Num(); i++)
		{
			EVerifyResult FileVerifyResult = ThreadJobResults[i];

			if (FileVerifyResult != EVerifyResult::Success)
			{
				CorruptFiles.Add(ThreadJobList[i].FileName);
				UE_LOGF(LogVerifier, Warning, "File verification failed on: %ls (cause = %d)", *ThreadJobList[i].FileName, EnumToUnderlyingType(FileVerifyResult));
				if (VerifyResult == EVerifyResult::Success)
				{
					VerifyResult = FileVerifyResult;
				}
			}
			// If success, and it was an SHA verify, cache the result so we don't repeat an SHA verify.
			else if (ThreadJobList[i].bVerifySha)
			{
				FilesPassedVerify.Add(ThreadJobList[i].FileName);
			}
		}

		return VerifyResult;
	}

	void FVerifier::AddTouchedFiles(const TSet<FString>& TouchedFiles)
	{
		FilesToVerify.Append(TouchedFiles);
		FilesPassedVerify = FilesPassedVerify.Difference(TouchedFiles);
	}

	FString FVerifier::SelectFullFilePath(const FString& BuildFile)
	{
		FString FullFilePath;
		if (StagedFileDirectory.IsEmpty() == false)
		{
			FString BuildFilenameToUse = BuildFile;
			if (bUseSHA1StageFilenames)
			{
				const FFileManifest* FileManifest = ManifestSet->GetNewFileManifest(BuildFile);
				if (FileManifest != nullptr)
				{
					FBuildPatchUtils::SHAToBase32(FileManifest->SHA1Hash, BuildFilenameToUse);
				}
				else
				{
					UE_LOGF(LogVerifier, Error, "FVerifier: Failed to get manifest for file %ls", *BuildFile);
				}
			}
			FullFilePath = StagedFileDirectory / BuildFilenameToUse;
			int64 FileSize;
			if (FileSystem->GetFileSize(*FullFilePath, FileSize))
			{
				return FullFilePath;
			}
		}

		return FBuildPatchUtils::ResolveInstallationFileName(VerifyDirectory, PerFileSubdirectories, BuildFile);
	}

	EVerifyResult FVerifier::VerifyFileSha(TArray<uint8>& ReadBuffer, const FString& BuildFile, const FFileManifest& BuildFileManifest)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VerifyFileSha);
		ISpeedRecorder::FRecord ActivityRecord;
		EVerifyResult VerifyResult;
		const FString FileToVerify = SelectFullFilePath(BuildFile);
		TUniquePtr<FArchive> FileReader = FileSystem->CreateFileReader(*FileToVerify);
		int64 FileReaderPrevTell = 0;
		VerifierStat->OnFileProgress(BuildFile, 0);
		if (FileReader.IsValid())
		{
			FileReaderPrevTell = FileReader->Tell();
			const int64 FileSize = FileReader->TotalSize();
			if (FileSize != BuildFileManifest.FileSize)
			{
				VerifyResult = EVerifyResult::FileSizeFailed;
			}
			else
			{
				FSHA1 HashState;
				FSHAHash HashValue;
				while (!FileReader->AtEnd() && !bShouldAbort)
				{
					// Pause if necessary
					while (bIsPaused && !bShouldAbort)
					{
						FPlatformProcess::Sleep(0.1f);
					}
					ActivityRecord.CyclesStart = FStatsCollector::GetCycles();
					// Read file and update hash state
					const int64 SizeLeft = FileSize - FileReader->Tell();
					ActivityRecord.Size = FMath::Min<int64>(ReadBuffer.Num(), SizeLeft);

					{
						TRACE_CPUPROFILER_EVENT_SCOPE(VerifyFileSha_Serialize);
						FileReader->Serialize(ReadBuffer.GetData(), ActivityRecord.Size);
					}
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(VerifyFileSha_Hash);
						
						HashState.Update(ReadBuffer.GetData(), ActivityRecord.Size);
					}

					ActivityRecord.CyclesEnd = FStatsCollector::GetCycles();
					VerifierStat->OnFileRead(ActivityRecord);
					VerifierStat->OnFileProgress(BuildFile, FileReader->Tell());

					ThreadProcessedBytes.fetch_add(ActivityRecord.Size, std::memory_order_release);
				}
				HashState.Final();
				HashState.GetHash(HashValue.Hash);
				if (HashValue == BuildFileManifest.SHA1Hash)
				{
					VerifyResult = EVerifyResult::Success;
				}
				else if (!bShouldAbort)
				{
					UE_LOGF(LogVerifier, Error, "FVerifier: Hash check failed for %ls. Calculated SHA1: %ls, Manifest SHA1: %ls", *BuildFile, *HashValue.ToString(), *BuildFileManifest.SHA1Hash.ToString());
					VerifyResult = EVerifyResult::HashCheckFailed;
				}
				else
				{
					VerifyResult = EVerifyResult::Aborted;
				}
			}
			FileReader->Close();
		}
		else if (FileSystem->FileExists(*FileToVerify))
		{
			VerifyResult = EVerifyResult::OpenFileFailed;
		}
		else
		{
			VerifyResult = EVerifyResult::FileMissing;
		}

		if (VerifyResult != EVerifyResult::Success)
		{
			VerifierStat->OnFileProgress(BuildFile, BuildFileManifest.FileSize);
		}

		return VerifyResult;
	}

	EVerifyResult FVerifier::VerifyFileSize(const FString& BuildFile, const FFileManifest& BuildFileManifest)
	{
		// Pause if necessary.
		while (bIsPaused && !bShouldAbort)
		{
			FPlatformProcess::Sleep(0.1f);
		}

		VerifierStat->OnFileProgress(BuildFile, 0);
		int64 FileSize = 0;
		EVerifyResult VerifyResult = EVerifyResult::FileSizeFailed;
		if (FileSystem->GetFileSize(*SelectFullFilePath(BuildFile), FileSize))
		{
			if (FileSize == BuildFileManifest.FileSize)
			{
				VerifyResult = EVerifyResult::Success;
			}
			else
			{
				VerifyResult = EVerifyResult::FileSizeFailed;
			}
		}
		else
		{
			VerifyResult = EVerifyResult::FileMissing;
		}
		VerifierStat->OnFileProgress(BuildFile, BuildFileManifest.FileSize);

		ThreadProcessedBytes.fetch_add(BuildFileManifest.FileSize, std::memory_order_release);
		return VerifyResult;
	}

	EVerifyResult FVerifier::VerifySymlink(const FString& BuildFile, const FFileManifest& BuildFileManifest)
	{
		VerifierStat->OnFileProgress(BuildFile, 0);
		EVerifyResult VerifyResult = EVerifyResult::HashCheckFailed;
#if PLATFORM_MAC || PLATFORM_LINUX
		// Load the symlink
		TArray<ANSICHAR> SymlinkTargetAnsi;
		SymlinkTargetAnsi.AddZeroed(FPlatformMisc::GetMaxPathLength() + 1);
		if (readlink(TCHAR_TO_UTF8(*SelectFullFilePath(BuildFile)), SymlinkTargetAnsi.GetData(), SymlinkTargetAnsi.Num() - 1) != -1)
		{
			// Check the path is correct
			const FString SymlinkTarget = UTF8_TO_TCHAR(SymlinkTargetAnsi.GetData());
			if (SymlinkTarget == BuildFileManifest.SymlinkTarget)
			{
				VerifyResult = EVerifyResult::Success;
			}
			else
			{
				VerifyResult = EVerifyResult::HashCheckFailed;
			}
		}
		// If the symlink does not exist
		else if (errno == ENOENT)
		{
			VerifyResult = EVerifyResult::FileMissing;
		}
		// Otherwise we treat as not being able to open the link (e.g. permission error or not a link)
		else
		{
			VerifyResult = EVerifyResult::OpenFileFailed;
		}
#else
		// Cannot verify on unsupported platforms
		VerifyResult = EVerifyResult::Success;
#endif
		VerifierStat->OnFileProgress(BuildFile, BuildFileManifest.FileSize);

		ThreadProcessedBytes.fetch_add(BuildFileManifest.FileSize, std::memory_order_release);
		return VerifyResult;
	}

	IVerifier* FVerifierFactory::Create(IFileSystem* FileSystem, IVerifierStat* VerifierStat, EVerifyMode VerifyMode, 
		IBuildInstallerSharedContextPtr SharedContext, IBuildManifestSet* ManifestSet, FString VerifyDirectory, FString StagedFileDirectory,
		bool bUseSHA1StageFilenames, const TMap<FString, FString>& PerFileSubdirectories)
	{
		check(FileSystem != nullptr);
		check(VerifierStat != nullptr);
		check(ManifestSet != nullptr);
		return new FVerifier(FileSystem, VerifierStat, VerifyMode, SharedContext, ManifestSet, MoveTemp(VerifyDirectory), MoveTemp(StagedFileDirectory), bUseSHA1StageFilenames, PerFileSubdirectories);
	}
}
