// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/Controllable.h"
#include "Common/SpeedRecorder.h"
#include "BuildPatchManifest.h"
#include "Interfaces/IBuildInstallerSharedContext.h"

namespace BuildPatchServices
{
	class IFileSystem;
	class IVerifierStat;
	class IBuildManifestSet;
	enum class EVerifyMode : uint32;
	enum class EVerifyError : uint32;
	/**
	 * An enum defining the result of a verification process.
	 */
	enum class EVerifyResult : uint32
	{
		Success = 0,

		// If the process was ended due to an external cancel.
		Aborted,

		// The verify failed due to a missing file.
		FileMissing,

		// The verify failed due to a file failing to open.
		OpenFileFailed,

		// The expected data hash for a file did not match.
		HashCheckFailed,

		// A file did not match the expected size.
		FileSizeFailed,
	};

	/**
	 * Used to convert a EVerifyError to a EVerifyResult.
	 * @param InVerifyError   - Error to convert.
	 * @param OutVerifyResult - Reference to completed.
	 * @return true if successful.
	 */
	bool TryConvertToVerifyResult(EVerifyError InVerifyError, EVerifyResult& OutVerifyResult);

	/**
	 * Used to convert a EVerifyResult to a EVerifyError.
	 * @param InVerifyResult - Result to convert.
	 * @param OutVerifyError - Reference to complete.
	 * @return true if successful.
	 */
	bool TryConvertToVerifyError(EVerifyResult InVerifyResult, EVerifyError& OutVerifyError);

	/**
	 * An interface providing the functionality to verify a local installation.
	 */
	class IVerifier
		: public IControllable
	{
	public:
		/**
		 * Verifies a local directory structure against a given manifest.
		 * NOTE: This function is blocking and will not return until finished. Don't run on main thread.
		 * @param CorruptFiles     OUT  The array of files that do not match or are locally missing.
		 * @return    EVerifiyResult::Success if no file errors occurred AND the verification was successful.
		 *            Otherwise it will return the first error encountered during verification.
		 */
		virtual EVerifyResult Verify(TArray<FString>& CorruptFiles) = 0;

		/**
		 * Register any files that have been built so that they are verified in addition to any marked for repair when Verify is ran.
		 * @param TouchedFiles          The array of files that have been created or edited.
		 */
		virtual void AddTouchedFiles(const TSet<FString>& TouchedFiles) = 0;
	};

	class FVerifierFactory
	{
	public:
		/**
		 * Creates a verifier class that will verify a local directory structure against a given manifest, optionally
		 * taking account of a staging directory where alternative files are used.
		 * @param StagedFileDirectory    A stage directory for updated files, ignored if empty string. If a file exists here, it will be checked instead of the one in VerifyDirectory.
		 * @param bUseSHA1StageFilenames A boolean value to indicate that we use hashes instead of real file names.
		 * @param PerFileSubdirectories  A map of path adjustments to use keyed off the manifest filename, leave empty if unneeded.
		 * @return The verifier object. Delete when complete. See IVerifier class for usage.
		 */
		static IVerifier* Create(IFileSystem* FileSystem, IVerifierStat* VerifierStat, EVerifyMode VerifyMode, 
			IBuildInstallerSharedContextPtr SharedContext, IBuildManifestSet* ManifestSet, FString VerifyDirectory, FString StagedFileDirectory,
			bool bUseSHA1StageFilenames, const TMap<FString, FString>& PerFileSubdirectories);
	};

	/**
	 * This interface defines the statistics class required by the verifier system. It should be implemented in order to collect
	 * desired information which is being broadcast by the system.
	 */
	class IVerifierStat
	{
	public:
		virtual ~IVerifierStat() = default;

		/**
		 * Called each time a file is going to be verified.
		 * @param Filename      The filename of the file.
		 * @param FileSize      The size of the file being verified.
		 */
		virtual void OnFileStarted(const FString& Filename, int64 FileSize) = 0;

		/**
		 * Called during a file verification with the current progress.
		 * @param Filename      The filename of the file.
		 * @param TotalBytes    The number of bytes processed so far.
		 */
		virtual void OnFileProgress(const FString& Filename, int64 TotalBytes) = 0;

		/**
		 * Called each time a file has finished being verified.
		 * @param Filename      The filename of the file.
		 * @param VerifyResult  The result of the file's verify test.
		 */
		virtual void OnFileCompleted(const FString& Filename, EVerifyResult VerifyResult) = 0;

		/**
		 * Called each time a read operation is made.
		 * @param Record        The details for the operation.
		 */
		virtual void OnFileRead(const ISpeedRecorder::FRecord& Record) = 0;

		/**
		 * Called to update the total amount of bytes which have been processed.
		 * @param TotalBytes    The number of bytes processed so far.
		 */
		virtual void OnProcessedDataUpdated(int64 TotalBytes) = 0;

		/**
		 * Called to update the total number of bytes to be processed.
		 * @param TotalBytes    The total number of bytes to be processed.
		 */
		virtual void OnTotalRequiredUpdated(int64 TotalBytes) = 0;
	};
}
