// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Data/ChunkData.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FFakeChunkDataAccess
		: public IChunkDataAccess
	{
	public:
		FFakeChunkDataAccess()
			:ChunkData(nullptr)
		{
			ChunkHeader.DataSizeCompressed = 0;
			ChunkHeader.DataSizeUncompressed = 0;
		}

		virtual ~FFakeChunkDataAccess()
		{
			if (OnDeleted)
			{
				OnDeleted();
			}
		}

		virtual void GetDataLock(const uint8** OutChunkData, const FChunkHeader** OutChunkHeader) const override
		{
			(*OutChunkData) = ChunkData;
			(*OutChunkHeader) = &ChunkHeader;
		}

		virtual void GetDataLock(uint8** OutChunkData, FChunkHeader** OutChunkHeader) override
		{
			(*OutChunkData) = ChunkData;
			(*OutChunkHeader) = &ChunkHeader;
		}

		virtual void ReleaseDataLock() const override
		{
		}

		const FGuid& GetGuid() const
		{
			return ChunkHeader.Guid;
		}

	public:
		FChunkHeader ChunkHeader;
		uint8* ChunkData;
		TFunction<void()> OnDeleted;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
