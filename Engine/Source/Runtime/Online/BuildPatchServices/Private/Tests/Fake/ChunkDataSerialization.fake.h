// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Tests/Mock/ChunkDataSerialization.mock.h"
#include "Tests/Fake/ChunkDataAccess.fake.h"
#include "Serialization/MemoryReader.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FFakeChunkDataSerialization
		: public FMockChunkDataSerialization
	{
	public:
		virtual IChunkDataAccess* LoadFromMemory(const TArray<uint8>& Memory, EChunkLoadResult& OutLoadResult) const override
		{
			if (TxLoadFromMemory.Num())
			{
				return FMockChunkDataSerialization::LoadFromMemory(Memory, OutLoadResult);
			}
			FFakeChunkDataAccess* ChunkDataAccess = new FFakeChunkDataAccess();
			FMemoryReader Ar(Memory);
			Ar << ChunkDataAccess->ChunkHeader;
			Ar.Close();
			OutLoadResult = EChunkLoadResult::Success;
			RxLoadFromMemory.Emplace(Memory, OutLoadResult);
			return ChunkDataAccess;
		}
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
