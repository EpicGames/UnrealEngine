// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/ChunkReferenceTracker.h"
#include "Templates/Greater.h"
#include "Algo/Sort.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FFakeChunkReferenceTracker
		: public IChunkReferenceTracker
	{
	public:
		virtual TSet<FGuid> GetReferencedChunks() const override
		{
			return ReferencedChunks;
		}

		virtual int32 GetReferenceCount(const FGuid& ChunkId) const override
		{
			return ReferenceCounts.FindRef(ChunkId);
		}

		virtual void SortByUseOrder(TArray<FGuid>& ChunkList, ESortDirection Direction) const override
		{
			switch (Direction)
			{
				case ESortDirection::Ascending:
					Algo::SortBy(ChunkList, [this](const FGuid& Element) { return NextReferences.IndexOfByKey(Element); }, TLess<int32>());
					break;
				case ESortDirection::Descending:
					Algo::SortBy(ChunkList, [this](const FGuid& Element) { return NextReferences.IndexOfByKey(Element); }, TGreater<int32>());
					break;
			}
		}

		virtual TArray<FGuid> GetNextReferences(int32 Count, const TFunction<bool(const FGuid&)>& SelectPredicate) const override
		{
			return NextReferences.FilterByPredicate([&Count, &SelectPredicate](const FGuid& Element)
			{
				bool bSelected = (Count > 0) && SelectPredicate(Element);
				if (bSelected)
				{
					--Count;
				}
				return bSelected;
			});
		}

		virtual TArray<FGuid> SelectFromNextReferences(int32 Count, const TFunction<bool(const FGuid&)>& SelectPredicate) const override
		{
			return NextReferences.FilterByPredicate([&Count, &SelectPredicate](const FGuid& Element)
			{
				bool bSelected = (Count > 0) && SelectPredicate(Element);
				--Count;
				return bSelected;
			});
		}

		virtual bool PopReference(const FGuid& ChunkId) override
		{
			if (NextReferences.Num() && NextReferences[0] == ChunkId)
			{
				PoppedCount++;
				NextReferences.RemoveAt(0);
				return true;
			}
			return false;
		}
		virtual int32 GetRemainingChunkCount() const override { return NextReferences.Num(); };

		virtual void CopyOutOrderedUseList(TArray<FGuid>& OutUseList) const
		{
			OutUseList = NextReferences;
		}

		int32 GetNextUsageForChunk(const FGuid& ChunkId, int32& OutLastUsageIndex) const
		{
			for (int32 i=0; i < NextReferences.Num(); i++)
			{
				if (NextReferences[i] == ChunkId)
				{
					return i + PoppedCount;
				}
			}
			return -1;
		}

		virtual int32 GetCurrentUsageIndex() const
		{
			return PoppedCount;
		}

	public:
		int32 PoppedCount = 0;
		TSet<FGuid> ReferencedChunks;
		TMap<FGuid, int32> ReferenceCounts;
		TArray<FGuid> NextReferences;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
