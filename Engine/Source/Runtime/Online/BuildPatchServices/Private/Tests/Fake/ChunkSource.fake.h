// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "Containers/Map.h"
#include "Data/ChunkData.h"
#include "Installer/ChunkSource.h"

namespace BuildPatchServices
{
	class FFakeChunkSource
		: public IChunkSource
	{
	public:
		virtual IChunkDataAccess* Get(const FGuid& DataId) override
		{
			TUniquePtr<IChunkDataAccess>* FindResult = ChunkDatas.Find(DataId);
			return FindResult ? FindResult->Get() : nullptr;
		}

		virtual TSet<FGuid> AddRuntimeRequirements(TSet<FGuid> NewRequirements) override
		{
			return MoveTemp(NewRequirements);
		}

		bool AddRepeatRequirement(const FGuid& RepeatRequirement)
		{
			return false;
		}

		virtual void SetUnavailableChunksCallback(TFunction<void(TSet<FGuid>)> Callback) override
		{
		}

	public:
		TMap<FGuid, TUniquePtr<IChunkDataAccess>> ChunkDatas;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
