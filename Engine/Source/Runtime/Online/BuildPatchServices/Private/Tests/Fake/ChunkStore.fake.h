// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Tests/Mock/ChunkStore.mock.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FFakeChunkStore
		: public FMockChunkStore
	{
	public:
		virtual void Put(const FGuid& DataId, TUniquePtr<IChunkDataAccess> ChunkData) override
		{
			FScopeLock ScopeLock(&ThreadLock);
			Store.Add(DataId, MoveTemp(ChunkData));
			RxPut.Emplace(FStatsCollector::GetSeconds(), DataId);
		}

		virtual IChunkDataAccess* Get(const FGuid& DataId) override
		{
			FScopeLock ScopeLock(&ThreadLock);
			IChunkDataAccess* Result = nullptr;
			if (Store.Contains(DataId))
			{
				Result = Store[DataId].Get();
			}
			RxGet.Emplace(FStatsCollector::GetSeconds(), Result, DataId);
			return Result;
		}

		virtual TUniquePtr<IChunkDataAccess> Remove(const FGuid& DataId) override
		{
			FScopeLock ScopeLock(&ThreadLock);
			TUniquePtr<IChunkDataAccess> ReturnValue;
			if (Store.Contains(DataId))
			{
				ReturnValue = MoveTemp(Store[DataId]);
				Store.Remove(DataId);
			}
			RxRemove.Emplace(FStatsCollector::GetSeconds(), DataId);
			return ReturnValue;
		}

		virtual int32 GetSize() const override
		{
			FScopeLock ScopeLock(&ThreadLock);
			int32 Result = StoreMax - Store.Num();
			RxGetSize.Emplace(FStatsCollector::GetSeconds(), Result);
			return Result;
		}

	public:
		mutable FCriticalSection ThreadLock;
		TMap<FGuid, TUniquePtr<IChunkDataAccess>> Store;
		int32 StoreMax;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
