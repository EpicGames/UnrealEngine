// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Tests/Mock/Download.mock.h"
#include "Tests/Mock/DownloadService.mock.h"
#include "Interfaces/IHttpResponse.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "Serialization/MemoryWriter.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Data/ChunkData.h"

namespace BuildPatchServices
{
	class FFakeDownloadService
		: public FMockDownloadService
	{
	public:
		typedef TTuple<double, bool, EHttpResponseCodes::Type, FChunkHeader> FTxRequestFile;

	public:
		FFakeDownloadService()
			: bRunDownloadThread(false)
		{
		}

		~FFakeDownloadService()
		{
			StopService();
		}

	public:
		void StartService()
		{
			ThreadLock.Lock();
			bRunDownloadThread = true;
			ThreadLock.Unlock();
			TFunction<void()> Task = [this]()
			{
				bool bKeepRunning = true;
				TArray<FRequestFile> RxToBeProcessed;
				int32 IndicesTaken = 0;
				while (bKeepRunning)
				{
					ThreadLock.Lock();
					for (int32 Idx = IndicesTaken; Idx < RxRequestFile.Num(); ++Idx)
					{
						RxToBeProcessed.Push(RxRequestFile[Idx]);
						if (TxRequestFile.Num() > 0)
						{
							Responses.Add(RxToBeProcessed.Last().Get<1>(), TxRequestFile[0]);
							TxRequestFile.RemoveAt(0);
						}
						else
						{
							Responses.Add(RxToBeProcessed.Last().Get<1>(), FTxRequestFile(0.0f, true, EHttpResponseCodes::Ok, DefaultChunkHeader));
						}
						IndicesTaken = Idx + 1;
					}
					ThreadLock.Unlock();
					TArray<FRequestFile> RxToConsider = MoveTemp(RxToBeProcessed);
					const double TimeNow = FStatsCollector::GetSeconds();
					for (int32 Idx = 0; Idx < RxToConsider.Num(); ++Idx)
					{
						FRequestFile& RequestFile = RxToConsider[Idx];
						FTxRequestFile& ResponseFile = Responses[RequestFile.Get<1>()];
						const double Elapsed = TimeNow - RequestFile.Get<0>();
						const double Desired = ResponseFile.Get<0>();
						if (Elapsed >= Desired)
						{
							FMockDownload* MockDownload = new FMockDownload();
							MockDownload->bSuccess = ResponseFile.Get<1>();
							if (MockDownload->bSuccess)
							{
								MockDownload->ResponseCode = ResponseFile.Get<2>();
								FMemoryWriter Ar(MockDownload->Data);
								Ar << ResponseFile.Get<3>();
								Ar.Close();
								MockDownload->Data.AddUninitialized(ResponseFile.Get<3>().DataSizeCompressed);
								RequestFile.Get<4>().ExecuteIfBound(RequestFile.Get<1>(), MockDownload->Data.Num() / 3);
								RequestFile.Get<4>().ExecuteIfBound(RequestFile.Get<1>(), MockDownload->Data.Num() / 2);
								RequestFile.Get<4>().ExecuteIfBound(RequestFile.Get<1>(), MockDownload->Data.Num());
							}
							else
							{
								MockDownload->ResponseCode = EHttpResponseCodes::Unknown;
							}
							RequestFile.Get<3>().ExecuteIfBound(RequestFile.Get<1>(), MakeShareable(MockDownload));
						}
						else
						{
							RxToBeProcessed.Push(MoveTemp(RequestFile));
						}
					}
					FPlatformProcess::Sleep(0.0f);
					ThreadLock.Lock();
					bKeepRunning = bRunDownloadThread;
					ThreadLock.Unlock();
				}
			};
			Future = Async(EAsyncExecution::Thread, MoveTemp(Task));
		}

		void StopService()
		{
			ThreadLock.Lock();
			bRunDownloadThread = false;
			ThreadLock.Unlock();
			if (Future.IsValid())
			{
				Future.Wait();
			}
		}

	public:
		bool bRunDownloadThread;
		TFuture<void> Future;
		FChunkHeader DefaultChunkHeader;
		TArray<FTxRequestFile> TxRequestFile;
		TMap<int32, FTxRequestFile> Responses;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
