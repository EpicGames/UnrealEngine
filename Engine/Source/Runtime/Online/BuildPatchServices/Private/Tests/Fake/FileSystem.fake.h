// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Tests/Mock/FileSystem.mock.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	/**
	 * Using a fake file reader allows us to simulate Unreal Engine's file reader behavior, where if the file is written to
	 * after the handle is opened, you will cause an assert if you try to read the new data at the end as the total size is cached.
	 */
	class FFakeFileReader
		: public FMemoryArchive
	{
	public:
		FFakeFileReader(const TArray<uint8>& InBytes)
			: Bytes(InBytes)
			, FakeTotalSize(Bytes.Num())
		{
			this->SetIsLoading(true);
		}

		virtual FString GetArchiveName() const
		{
			return TEXT("FFakeFileReader");
		}

		virtual int64 TotalSize() override
		{
			return FakeTotalSize;
		}

		void Serialize(void* Data, int64 Num)
		{
			if (Num && !IsError())
			{
				if (Offset + Num <= TotalSize())
				{
					FMemory::Memcpy(Data, &Bytes[Offset], Num);
					Offset += Num;
				}
				else
				{
					SetError();
				}
			}
		}

	public:
		const TArray<uint8>& Bytes;
		int64 FakeTotalSize;
	};

	class FFakeFileSystem
		: public FMockFileSystem
	{
	public:
		virtual TUniquePtr<FArchive> CreateFileReader(const TCHAR* Filename, EReadFlags ReadFlags = EReadFlags::None) const override
		{
			TUniquePtr<FArchive> Reader;
			if (CreateFileReaderFunc)
			{
				Reader = CreateFileReaderFunc(Filename, ReadFlags);
			}
			else
			{
				FString NormalizedFilename = Filename;
				FPaths::NormalizeFilename(NormalizedFilename);
				NormalizedFilename = FPaths::ConvertRelativePathToFull(TEXT(""), MoveTemp(NormalizedFilename));
				FScopeLock ScopeLock(&ThreadLock);
				if (DiskData.Contains(NormalizedFilename) && !DiskDataOpenFailure.Contains(NormalizedFilename))
				{
					Reader.Reset(new FFakeFileReader(DiskData[NormalizedFilename]));
				}
			}
			RxCreateFileReader.Emplace(FStatsCollector::GetSeconds(), Reader.Get(), Filename, ReadFlags);
			return Reader;
		}

		virtual TUniquePtr<FArchive> CreateFileReaderEx(const TCHAR* Filename, EReadFlags ReadFlags, IFileHandle** OutHandle) const override
		{
			check(OutHandle == nullptr); // The output parameter is not supported for fake implementation yet.
			return CreateFileReader(Filename, ReadFlags);
		}

		virtual TUniquePtr<FArchive> CreateFileWriter(const TCHAR* Filename, EWriteFlags WriteFlags = EWriteFlags::None) const override
		{
			TUniquePtr<FArchive> Writer;
			if (CreateFileWriterFunc)
			{
				Writer = CreateFileWriterFunc(Filename, WriteFlags);
			}
			else
			{
				FString NormalizedFilename = Filename;
				FPaths::NormalizeFilename(NormalizedFilename);
				NormalizedFilename = FPaths::ConvertRelativePathToFull(TEXT(""), MoveTemp(NormalizedFilename));
				FScopeLock ScopeLock(&ThreadLock);
				Writer.Reset(new FMemoryWriter(DiskData.FindOrAdd(NormalizedFilename)));
			}
			RxCreateFileWriter.Emplace(FStatsCollector::GetSeconds(), Writer.Get(), Filename, WriteFlags);
			return Writer;
		}

		bool DeleteFile(const TCHAR* Filename) const
		{
			FString NormalizedFilename = Filename;
			FPaths::NormalizeFilename(NormalizedFilename);
			NormalizedFilename = FPaths::ConvertRelativePathToFull(TEXT(""), MoveTemp(NormalizedFilename));
			FScopeLock ScopeLock(&ThreadLock);
			DiskData.Remove(NormalizedFilename);
			return true;
		}

		virtual bool GetFileSize(const TCHAR* Filename, int64& OutFileSize) const override
		{
			FScopeLock ScopeLock(&ThreadLock);
			OutFileSize = -1;
			if (DiskData.Contains(Filename))
			{
				OutFileSize = DiskData[Filename].Num();
			}
			RxGetFileSize.Emplace(FStatsCollector::GetSeconds(), Filename, OutFileSize);
			return OutFileSize >= 0;
		}

		virtual bool FileExists(const TCHAR* Filename) const override
		{
			return DiskData.Contains(Filename) || DiskDataOpenFailure.Contains(Filename);
		}

	public:
		mutable TMap<FString, TArray<uint8>> DiskData;
		mutable TArray<FString> DiskDataOpenFailure;

		TFunction<TUniquePtr<FArchive>(const TCHAR*, EReadFlags)> CreateFileReaderFunc;
		TFunction<TUniquePtr<FArchive>(const TCHAR*, EWriteFlags)> CreateFileWriterFunc;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
