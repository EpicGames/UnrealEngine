// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Ticker.h"
#include "Tests/Mock/HttpManager.mock.h"
#include "Tests/Mock/HttpRequest.mock.h"
#include "Tests/Mock/HttpResponse.mock.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FFakeHttpRequest
		: public FMockHttpRequest
	{
	public:
		virtual bool ProcessRequest() override;

	public:
		class FFakeHttpManager* FakeHttpModule;
	};

	class FFakeHttpResponse
		: public FMockHttpResponse
	{
	public:
		virtual int32 GetResponseCode() const override;
		virtual const TArray<uint8>& GetContent() const override;
		virtual FString GetContentAsString() const override;
		virtual uint64 GetContentLength() const override;

	public:
		TArray<uint8> Data;
		EHttpResponseCodes::Type Code;
	};

	class FFakeHttpManager
		: public FMockHttpManager
	{
	public:
		FFakeHttpManager(FTSTicker& Ticker);
		virtual TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateRequest() override;
		virtual bool Tick(float Delta);
		virtual bool OnProcessRequest(FFakeHttpRequest* FakeHttpRequest);

	public:
		TArray<TSharedRef<FFakeHttpRequest, ESPMode::ThreadSafe>> NewRequests;
		TArray<TSharedRef<FFakeHttpRequest, ESPMode::ThreadSafe>> RunningRequests;
		TArray<TSharedRef<FFakeHttpRequest, ESPMode::ThreadSafe>> ProgressedRequests;
		TMap<FString, TArray<uint8>> DataServed;
	};

	inline bool FFakeHttpRequest::ProcessRequest()
	{
		++RxProcessRequest;
		return FakeHttpModule->OnProcessRequest(this);
	}

	inline int32 FFakeHttpResponse::GetResponseCode() const
	{
		return static_cast<int32>(Code);
	}

	inline const TArray<uint8>& FFakeHttpResponse::GetContent() const
	{
		return Data;
	}

	inline FString FFakeHttpResponse::GetContentAsString() const
	{
		TArray<uint8> ZeroTerminatedPayload(GetContent());
		ZeroTerminatedPayload.Add(0);
		return UTF8_TO_TCHAR(ZeroTerminatedPayload.GetData());
	}

	inline uint64 FFakeHttpResponse::GetContentLength() const
	{
		return Data.Num();
	}

	inline FFakeHttpManager::FFakeHttpManager(FTSTicker& Ticker)
	{
		Ticker.AddTicker(FTickerDelegate::CreateRaw(this, &FFakeHttpManager::Tick));
	}

	inline TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FFakeHttpManager::CreateRequest()
	{
		++RxCreateRequest;
		NewRequests.Emplace(new FFakeHttpRequest());
		NewRequests.Last()->FakeHttpModule = this;
		return NewRequests.Last();
	}

	inline bool FFakeHttpManager::Tick(float Delta)
	{
		// Complete progressed.
		for (const TSharedRef<FFakeHttpRequest, ESPMode::ThreadSafe>& ProgressedRequest : ProgressedRequests)
		{
			FFakeHttpResponse* FakeHttpResponse = new FFakeHttpResponse();
			FakeHttpResponse->Code = EHttpResponseCodes::Ok;
			const FString& Url = ProgressedRequest->RxSetURL.Last().Get<0>();
			if (DataServed.Contains(Url))
			{
				FakeHttpResponse->Data = DataServed[Url];
			}
			ProgressedRequest->HttpRequestCompleteDelegate.ExecuteIfBound(
				ProgressedRequest,
				MakeShareable(FakeHttpResponse),
				true);
		}

		// Progress started.
		ProgressedRequests = MoveTemp(RunningRequests);
		for (const TSharedRef<FFakeHttpRequest, ESPMode::ThreadSafe>& ProgressedRequest : ProgressedRequests)
		{
			const FString& Url = ProgressedRequest->RxSetURL.Last().Get<0>();
			int32 Progress = 0;
			if (DataServed.Contains(Url))
			{
				Progress = DataServed[Url].Num() / 2;
			}
			ProgressedRequest->HttpRequestProgressDelegate.ExecuteIfBound(
				ProgressedRequest,
				0,
				Progress);
		}

		return true;
	}

	inline bool FFakeHttpManager::OnProcessRequest(FFakeHttpRequest* FakeHttpRequest)
	{
		int32 Index = NewRequests.IndexOfByPredicate([FakeHttpRequest](const TSharedRef<FFakeHttpRequest, ESPMode::ThreadSafe>& Element)
		{
			return &Element.Get() == FakeHttpRequest;
		});
		bool bValidRequest = Index >= 0;
		if (bValidRequest)
		{
			const TSharedRef<FFakeHttpRequest, ESPMode::ThreadSafe>& Request = NewRequests[Index];
			bValidRequest = Request->RxSetURL.Num() > 0;
			if (bValidRequest)
			{
				RunningRequests.Add(Request);
				NewRequests.RemoveAtSwap(Index);
			}
		}
		return bValidRequest;
	}
}

#endif //WITH_DEV_AUTOMATION_TESTS
