// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/InstallerError.h"
#include "Interfaces/IBuildInstaller.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FFakeInstallerError
		: public IInstallerError
	{
	public:
		virtual bool HasError() const override
		{
			return bHasError;
		}

		virtual bool IsCancelled() const override
		{
			return bIsCancelled;
		}

		virtual bool CanRetry() const override
		{
			return bCanRetry;
		}

		virtual EBuildPatchInstallError GetErrorType() const override
		{
			return ErrorType;
		}

		virtual FString GetErrorCode() const override
		{
			return ErrorCode;
		}

		virtual FText GetErrorText() const override
		{
			return ErrorText;
		}

		virtual void SetError(EBuildPatchInstallError InErrorType, const TCHAR* InErrorSubType, uint32 InErrorCode, FText InErrorText) override
		{
			ErrorType = InErrorType;
			ErrorCode = InErrorSubType + ((InErrorCode > 0) ? FString::Printf(TEXT("-%u"), InErrorCode) : TEXT(""));
			ErrorText = MoveTemp(InErrorText);
		}

		virtual int32 RegisterForErrors(FOnErrorDelegate Delegate)
		{
			int32 Handle = HandleCount++;
			Delegates.Add(Handle, Delegate);
			return Handle;
		}

		virtual void UnregisterForErrors(int32 Handle) override
		{
			Delegates.Remove(Handle);
		}

		virtual void ResetError(bool bPreserveCancelled) override
		{
			if (bPreserveCancelled && bCanRetry)
			{
				return;
			}
			bHasError = false;
			bIsCancelled = false;
			bCanRetry = true;
			ErrorType = EBuildPatchInstallError::NoError;
			ErrorCode = TEXT("");
			ErrorText = FText::GetEmpty();
			// To mimic real imp, delegates are kept.
		}

	public:
		bool bHasError;
		bool bIsCancelled;
		bool bCanRetry;
		EBuildPatchInstallError ErrorType;
		FString ErrorCode;
		FText ErrorText;
		TMap<int32, FOnErrorDelegate> Delegates;
		int32 HandleCount;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
