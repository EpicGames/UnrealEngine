// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BuildPatchProgress.h"
#include "Tests/TestHelpers.h"
#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockBuildPatchProgress
		: public FBuildPatchProgress
	{
	public:
		typedef TTuple<double, EBuildPatchState, float> FSetStateProgress;

	public:

		virtual void SetPaused(bool bIsPaused) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::SetPaused");
		}

		virtual void Abort() override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::Abort");
		}

		virtual void SetStateProgress(const EBuildPatchState& State, const float& Value) override
		{
			if (SetStateProgressFunc)
			{
				SetStateProgressFunc(State, Value);
			}
			RxSetStateProgress.Emplace(FStatsCollector::GetSeconds(), State, Value);
		}

		virtual void SetStateWeight(const EBuildPatchState& State, const float& Value) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::SetStateWeight");
		}

		virtual EBuildPatchState GetState() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::GetState");
			return EBuildPatchState();
		}

		virtual float GetProgress() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::GetProgress");
			return float();
		}

		virtual float GetProgressNoMarquee() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::GetProgressNoMarquee");
			return float();
		}

		virtual float GetStateProgress(const EBuildPatchState& State) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::GetStateProgress");
			return float();
		}

		virtual float GetStateWeight(const EBuildPatchState& State) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::GetStateWeight");
			return float();
		}

		virtual bool TogglePauseState() override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::TogglePauseState");
			return bool();
		}

		virtual double WaitWhilePaused() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::WaitWhilePaused");
			return double();
		}

		virtual bool GetPauseState() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::GetPauseState");
			return bool();
		}

		virtual void SetIsDownloading(bool bInIsDownloading) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockBuildPatchProgress::SetIsDownloading");
		}

	public:
		TArray<FSetStateProgress> RxSetStateProgress;
		TFunction<void(const EBuildPatchState&, const float&)> SetStateProgressFunc;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
