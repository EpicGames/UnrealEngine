// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Data/ChunkData.h"
#include "Tests/TestHelpers.h"
#include "Common/StatsCollector.h"
#include "Memory/MemoryView.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockChunkDataSerialization
		: public IChunkDataSerialization
	{
	public:
		typedef TTuple<FString, EChunkLoadResult> FRxLoadFromFile;
		typedef TTuple<FString, const IChunkDataAccess*> FRxSaveToFile;
		typedef TTuple<TArray<uint8>, EChunkLoadResult> FRxLoadFromMemory;
		typedef TTuple<EChunkLoadResult> FRxLoadFromArchive;
		typedef TTuple<const IChunkDataAccess*> FRxSaveToArchive;
		typedef TTuple<TArray<uint8>, FSHAHash> FRxInjectShaToChunkData;

		typedef TTuple<IChunkDataAccess*, EChunkLoadResult> FTxLoadFromFile;
		typedef TTuple<IChunkDataAccess*, EChunkLoadResult> FTxLoadFromMemory;
		typedef TTuple<IChunkDataAccess*, EChunkLoadResult> FTxLoadFromArchive;

	public:
		virtual IChunkDataAccess* LoadFromFile(const FString& Filename, EChunkLoadResult& OutLoadResult) const override
		{
			IChunkDataAccess* Result = nullptr;
			if (TxLoadFromFile.Num())
			{
				FTxLoadFromFile LoadFromFileResult = TxLoadFromFile.Pop();
				Result = LoadFromFileResult.Get<0>();
				OutLoadResult = LoadFromFileResult.Get<1>();
			}
			RxLoadFromFile.Emplace(Filename, OutLoadResult);
			return Result;
		}

		virtual EChunkSaveResult SaveToFile(const FString& Filename, IChunkDataAccess* ChunkDataAccess) const override
		{
			RxSaveToFile.Emplace(Filename, ChunkDataAccess);
			return EChunkSaveResult::Success;
		}

		virtual IChunkDataAccess* LoadFromMemory(const TArray<uint8>& Memory, EChunkLoadResult& OutLoadResult) const override
		{
			IChunkDataAccess* Result = nullptr;
			if (TxLoadFromMemory.Num())
			{
				FTxLoadFromMemory LoadFromMemoryResult = TxLoadFromMemory.Pop();
				Result = LoadFromMemoryResult.Get<0>();
				OutLoadResult = LoadFromMemoryResult.Get<1>();
			}
			RxLoadFromMemory.Emplace(Memory, OutLoadResult);
			return Result;
		}

		virtual EChunkSaveResult SaveToMemory(TArray<uint8>& Memory, IChunkDataAccess* ChunkDataAccess) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockChunkDataSerialization::SaveToMemory");
			return EChunkSaveResult::SerializationError;
		}

		virtual IChunkDataAccess* LoadFromArchive(FArchive& Archive, EChunkLoadResult& OutLoadResult) const override
		{
			IChunkDataAccess* Result = nullptr;
			if (TxLoadFromArchive.Num())
			{
				FTxLoadFromFile LoadFromArchiveResult = TxLoadFromArchive.Pop();
				Result = LoadFromArchiveResult.Get<0>();
				OutLoadResult = LoadFromArchiveResult.Get<1>();
			}
			RxLoadFromArchive.Emplace(OutLoadResult);
			return Result;
		}

		virtual EChunkSaveResult SaveToArchive(FArchive& Archive, IChunkDataAccess* ChunkDataAccess) const override
		{
			EChunkSaveResult Result = EChunkSaveResult::Success;
			if (SaveToArchiveFunc)
			{
				Result = SaveToArchiveFunc(Archive, ChunkDataAccess);
			}
			RxSaveToArchive.Emplace(ChunkDataAccess);
			return Result;
		}

		virtual bool InjectShaToChunkData(TArray<uint8>& Memory, const FSHAHash& ShaHashData) const override
		{
			RxInjectShaToChunkData.Emplace(Memory, ShaHashData);
			return true;
		}

		virtual bool ValidateAndRead(FArchive& InArchive, FMutableMemoryView InDestinationBuffer, FChunkHeader& OutHeader, FUniqueBuffer& OutCompressedBuffer) const
		{
			return false;
		}
		virtual bool DecompressAndDecryptValidatedRead(const FChunkHeader& InHeader, FMutableMemoryView InDestionationBuffer, const FUniqueBuffer& InCompressedBuffer) const
		{
			return false;
		}


	public:
		mutable TArray<FRxLoadFromFile> RxLoadFromFile;
		mutable TArray<FRxSaveToFile> RxSaveToFile;
		mutable TArray<FRxLoadFromMemory> RxLoadFromMemory;
		mutable TArray<FRxLoadFromArchive> RxLoadFromArchive;
		mutable TArray<FRxSaveToArchive> RxSaveToArchive;
		mutable TArray<FRxInjectShaToChunkData> RxInjectShaToChunkData;

		mutable TArray<FTxLoadFromFile> TxLoadFromFile;
		mutable TArray<FTxLoadFromMemory> TxLoadFromMemory;
		mutable TArray<FTxLoadFromArchive> TxLoadFromArchive;

		TFunction<EChunkSaveResult(FArchive&, const IChunkDataAccess*)> SaveToArchiveFunc;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
