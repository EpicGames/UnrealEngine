// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/ChunkEvictionPolicy.h"
#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockChunkEvictionPolicy
		: public IChunkEvictionPolicy
	{
	public:
		typedef TTuple<double, TSet<FGuid>, int32, TSet<FGuid>, TSet<FGuid>> FQuery;

	public:
		virtual void Query(const TMap<FGuid, TUniquePtr<IChunkDataAccess>>& CurrentMap, int32 DesiredMax, TSet<FGuid>& OutCleanable, TSet<FGuid>& OutBootable) const override
		{
			OutCleanable = Cleanable;
			OutBootable = Bootable;
			RxQuery.Emplace(FStatsCollector::GetSeconds(), CurrentMapToKeySet(CurrentMap), DesiredMax, OutCleanable, OutBootable);
		}

	public:
		TSet<FGuid> CurrentMapToKeySet(const TMap<FGuid, TUniquePtr<IChunkDataAccess>>& CurrentMap) const
		{
			TArray<FGuid> Result;
			CurrentMap.GetKeys(Result);
			return TSet<FGuid>(MoveTemp(Result));
		}

	public:
		TSet<FGuid> Cleanable;
		TSet<FGuid> Bootable;
		mutable TArray<FQuery> RxQuery;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
