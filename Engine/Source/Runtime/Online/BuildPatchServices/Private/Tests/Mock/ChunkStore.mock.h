// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Data/ChunkData.h"
#include "Installer/ChunkStore.h"
#include "Common/StatsCollector.h"
#include "Tests/TestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockChunkStore
		: public IChunkStore
	{
	public:
		typedef TTuple<double, FGuid> FPut;
		typedef TTuple<double, IChunkDataAccess*, FGuid> FGet;
		typedef TTuple<double, FGuid> FRemove;
		typedef TTuple<double, int32> FGetSize;

	public:
		virtual void Put(const FGuid& DataId, TUniquePtr<IChunkDataAccess> ChunkData) override
		{
			RxPut.Emplace(FStatsCollector::GetSeconds(), DataId);
		}

		virtual IChunkDataAccess* Get(const FGuid& DataId) override
		{
			RxGet.Emplace(FStatsCollector::GetSeconds(), nullptr, DataId);
			return nullptr;
		}

		virtual TUniquePtr<IChunkDataAccess> Remove(const FGuid& DataId) override
		{
			RxRemove.Emplace(FStatsCollector::GetSeconds(), DataId);
			return TUniquePtr<IChunkDataAccess>();
		}

		virtual int32 GetSize() const override
		{
			RxGetSize.Emplace(FStatsCollector::GetSeconds(), INDEX_NONE);
			return INDEX_NONE;
		}

		virtual void SetLostChunkCallback(TFunction<void(const FGuid&)> Callback) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockChunkStore::SetLostChunkCallback");
		}

	public:
		mutable TArray<FPut> RxPut;
		mutable TArray<FGet> RxGet;
		mutable TArray<FRemove> RxRemove;
		mutable TArray<FGetSize> RxGetSize;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
