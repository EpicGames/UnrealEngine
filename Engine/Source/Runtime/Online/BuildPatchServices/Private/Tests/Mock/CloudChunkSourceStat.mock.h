// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/CloudChunkSource.h"
#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockCloudChunkSourceStat
		: public ICloudChunkSourceStat
	{
	public:
		typedef TTuple<double, FGuid> FDownloadRequested;
		typedef TTuple<double, FGuid> FDownloadSuccess;
		typedef TTuple<double, FGuid, FString> FDownloadFailed;
		typedef TTuple<double, FGuid, FString, EChunkLoadResult> FDownloadCorrupt;
		typedef TTuple<double, FGuid, FString, double, double, double, double> FDownloadAborted;
		typedef TTuple<double, int64> FReceivedDataUpdated;
		typedef TTuple<double, int64> FRequiredDataUpdated;
		typedef TTuple<double, EBuildPatchDownloadHealth> FDownloadHealthUpdated;
		typedef TTuple<double, float> FSuccessRateUpdated;
		typedef TTuple<double, int32> FActiveRequestCountUpdated;
		typedef TTuple<double, int32> FActiveRequestCountAverageUpdated;
		typedef TTuple<double, FString, uint64> FCloudDirectoryDataReceived;
		typedef TTuple<double, FString> FCloudDirectoryError;

	public:
		virtual void OnDownloadRequested(const FGuid& ChunkId) override
		{
			RxDownloadRequested.Emplace(FStatsCollector::GetSeconds(), ChunkId);
		}

		virtual void OnDownloadSuccess(const FGuid& ChunkId) override
		{
			RxDownloadSuccess.Emplace(FStatsCollector::GetSeconds(), ChunkId);
		}

		virtual void OnDownloadFailed(const FGuid& ChunkId, const FString& Url) override
		{
			RxDownloadFailed.Emplace(FStatsCollector::GetSeconds(), ChunkId, Url);
		}

		virtual void OnDownloadCorrupt(const FGuid& ChunkId, const FString& Url, EChunkLoadResult LoadResult) override
		{
			RxDownloadCorrupt.Emplace(FStatsCollector::GetSeconds(), ChunkId, Url, LoadResult);
		}

		virtual void OnDownloadAborted(const FGuid& ChunkId, const FString& Url, double DownloadTimeMean, double DownloadTimeStd, double DownloadTime, double BreakingPoint) override
		{
			RxDownloadAborted.Emplace(FStatsCollector::GetSeconds(), ChunkId, Url, DownloadTimeMean, DownloadTimeStd, DownloadTime, BreakingPoint);
		}

		virtual void OnReceivedDataUpdated(int64 TotalBytes) override
		{
			RxReceivedDataUpdated.Emplace(FStatsCollector::GetSeconds(), TotalBytes);
		}

		virtual void OnRequiredDataUpdated(int64 TotalBytes) override
		{
			RxRequiredDataUpdated.Emplace(FStatsCollector::GetSeconds(), TotalBytes);
		}

		virtual void OnDownloadHealthUpdated(EBuildPatchDownloadHealth DownloadHealth) override
		{
			RxDownloadHealthUpdated.Emplace(FStatsCollector::GetSeconds(), DownloadHealth);
		}

		virtual void OnSuccessRateUpdated(float SuccessRate) override
		{
			RxSuccessRateUpdated.Emplace(FStatsCollector::GetSeconds(), SuccessRate);
		}

		virtual void OnActiveRequestCountUpdated(uint32 RequestCount) override
		{
			RxActiveRequestCountUpdated.Emplace(FStatsCollector::GetSeconds(), RequestCount);
		}

		virtual void OnActiveRequestCountAvgUpdated(int32 AverageRequestCount) override
		{
			RxActiveRequestCountAverageUpdated.Emplace(FStatsCollector::GetSeconds(), AverageRequestCount);
		}

		virtual void OnAcceptedNewRequirements(const TSet<FGuid>& ChunkIds) override
		{
		}

		virtual void OnChunkDecrypted(uint64 DecryptionCycles) override
		{
		}

		virtual void OnCloudDirectoryDataReceived(const FString& CloudDirectory, uint64 ReceivedData) override
		{
			RxCloudDirectoryDataReceived.Emplace(FStatsCollector::GetSeconds(), CloudDirectory, ReceivedData);
		}

		virtual void OnCloudDirectoryError(const FString& CloudDirectory) override
		{
			RxCloudDirectoryError.Emplace(FStatsCollector::GetSeconds(), CloudDirectory);
		}

	public:
		TArray<FDownloadRequested> RxDownloadRequested;
		TArray<FDownloadSuccess> RxDownloadSuccess;
		TArray<FDownloadFailed> RxDownloadFailed;
		TArray<FDownloadCorrupt> RxDownloadCorrupt;
		TArray<FDownloadAborted> RxDownloadAborted;
		TArray<FReceivedDataUpdated> RxReceivedDataUpdated;
		TArray<FRequiredDataUpdated> RxRequiredDataUpdated;
		TArray<FDownloadHealthUpdated> RxDownloadHealthUpdated;
		TArray<FSuccessRateUpdated> RxSuccessRateUpdated;
		TArray<FActiveRequestCountUpdated> RxActiveRequestCountUpdated;
		TArray<FActiveRequestCountUpdated> RxActiveRequestCountAverageUpdated;
		TArray<FCloudDirectoryDataReceived> RxCloudDirectoryDataReceived;
		TArray<FCloudDirectoryError> RxCloudDirectoryError;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
