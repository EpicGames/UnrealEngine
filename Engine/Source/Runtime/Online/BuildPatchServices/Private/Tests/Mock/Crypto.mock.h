// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Common/Crypto.h"
#include "Tests/TestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockCrypto
		: public ICrypto
	{
	public:
		virtual bool IsAvailable() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockCrypto::IsAvailable");
			return false;
		}

		virtual bool IsValidKey_AES_256_GCM(const TArrayView<const uint8> Key) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockCrypto::IsValidKey_AES_256_GCM");
			return false;
		}

		virtual TArray<uint8> Encrypt_AES_256_GCM(const TArrayView<const uint8> Plaintext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, TArray<uint8>& OutAuthTag, bool& OutResult) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockCrypto::Encrypt_AES_256_GCM");
			return {};
		}

		virtual TArray<uint8> Decrypt_AES_256_GCM(const TArrayView<const uint8> Ciphertext, const TArrayView<const uint8> Key, const TArrayView<const uint8> InitializationVector, const TArrayView<const uint8> AuthTag, bool& OutResult) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockCrypto::Decrypt_AES_256_GCM");
			return {};
		}

		virtual bool CreateRandomBytes(const TArrayView<uint8> OutData) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockCrypto::CreateRandomBytes");
			return false;
		}
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
