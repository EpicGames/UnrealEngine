// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockCyclesProvider
	{
	public:
		typedef TTuple<double, uint64> FGetCycles;
		typedef TTuple<double, double, uint64> FCyclesToSeconds;

	public:
		static uint64 GetCycles()
		{
			RxGetCycles.Emplace(FStatsCollector::GetSeconds(), CurrentCycles);
			return CurrentCycles;
		}

		static double CyclesToSeconds(uint64 Cycles)
		{
			double Seconds = double(Cycles) * SecondsPerCycle;
			RxCyclesToSeconds.Emplace(FStatsCollector::GetSeconds(), Seconds, Cycles);
			return Seconds;
		}

		static void Reset()
		{
			RxGetCycles.Reset();
			RxCyclesToSeconds.Reset();
			CurrentCycles = 0;
			SecondsPerCycle = 1.0 / 100.0;
		}

	public:
		static TArray<FGetCycles> RxGetCycles;
		static TArray<FCyclesToSeconds> RxCyclesToSeconds;
		static uint64 CurrentCycles;
		static double SecondsPerCycle;
	};
	TArray<FMockCyclesProvider::FGetCycles> FMockCyclesProvider::RxGetCycles;
	TArray<FMockCyclesProvider::FCyclesToSeconds> FMockCyclesProvider::RxCyclesToSeconds;
	uint64 FMockCyclesProvider::CurrentCycles;
	double FMockCyclesProvider::SecondsPerCycle;
}

#endif //WITH_DEV_AUTOMATION_TESTS
