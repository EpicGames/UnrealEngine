// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/DownloadService.h"
#include "Interfaces/IHttpResponse.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockDownload
		: public IDownload
	{
	public:
		virtual bool RequestSuccessful() const override
		{
			return bSuccess;
		}

		virtual bool ResponseSuccessful() const override
		{
			return bSuccess && EHttpResponseCodes::IsOk(ResponseCode);
		}

		virtual int32 GetResponseCode() const override
		{
			return ResponseCode;
		}

		virtual const TArray<uint8>& GetData() const override
		{
			return Data;
		}

	public:
		bool bSuccess;
		int32 ResponseCode;
		TArray<uint8> Data;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
