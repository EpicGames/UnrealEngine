// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/DownloadConnectionCount.h"

enum class EBuildPatchDownloadHealth;

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockDownloadConnectionCount : public IDownloadConnectionCount
	{
	public:
		FMockDownloadConnectionCount(uint32 InCount)
			: Count(InCount)
		{

		}
		FMockDownloadConnectionCount()
			:Count(8)
		{

		}
		uint32 GetAdjustedCount(uint32 NumProcessing, EBuildPatchDownloadHealth CurrentHealth) { return Count; }
		void SetConnectionScaling(bool bInEnableScaling) { }
	private:
		const uint32 Count;
	};
}

#endif  //WITH_DEV_AUTOMATION_TESTS