// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/DownloadService.h"
#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockDownloadService
		: public IDownloadService
	{
	public:
		typedef TTuple<double, int32, FString, FDownloadCompleteDelegate, FDownloadProgressDelegate> FRequestFile;
		typedef TTuple<double, int32> FRequestCancel;

	public:
		FMockDownloadService()
			: Count(0)
		{
		}

		virtual bool RequestFile(const FString& FileUri, const FDownloadCompleteDelegate& OnCompleteDelegate, const FDownloadProgressDelegate& OnProgressDelegate, int32* OutRequestId = nullptr) override
		{
			FScopeLock ScopeLock(&ThreadLock);
			bool bSuccess = false;
			if (RequestFileFunc)
			{
				bSuccess = RequestFileFunc(FileUri, OnCompleteDelegate, OnProgressDelegate, OutRequestId);
			}
			RxRequestFile.Emplace(FStatsCollector::GetSeconds(), OutRequestId ? *OutRequestId : -1, FileUri, OnCompleteDelegate, OnProgressDelegate);
			return bSuccess;
		}

		virtual bool RequestFileWithHeaders(const FString& FileUri, const TMap<FString, FString>& Headers, const FDownloadCompleteDelegate& OnCompleteDelegate, const FDownloadProgressDelegate& OnProgressDelegate, int32* OutRequestId = nullptr) override
		{
			return RequestFile(FileUri, OnCompleteDelegate, OnProgressDelegate, OutRequestId);
		}

		virtual void RequestCancel(int32 RequestId) override
		{
			FScopeLock ScopeLock(&ThreadLock);
			if (RequestCancelFunc)
			{
				RequestCancelFunc(RequestId);
			}
			RxRequestCancel.Emplace(FStatsCollector::GetSeconds(), RequestId);
		}

		virtual void RequestAbandon(int32 RequestId) override
		{
			FScopeLock ScopeLock(&ThreadLock);
			if (RequestCancelFunc)
			{
				RequestCancelFunc(RequestId);
			}
			RxRequestCancel.Emplace(FStatsCollector::GetSeconds(), RequestId);
		}

	public:
		FCriticalSection ThreadLock;
		int32 Count;
		TArray<FRequestFile> RxRequestFile;
		TArray<FRequestCancel> RxRequestCancel;
		TFunction<bool(const FString&, const FDownloadCompleteDelegate&, const FDownloadProgressDelegate&, int32*)> RequestFileFunc;
		TFunction<void(int32)> RequestCancelFunc;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
