// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Common/StatsCollector.h"
#include "Installer/DownloadService.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockDownloadServiceStat
		: public IDownloadServiceStat
	{
	public:
		typedef TTuple<double, int32, FString> FDownloadStarted;
		typedef TTuple<double, int32, int32> FDownloadProgress;
		typedef TTuple<double, FDownloadRecord> FDownloadComplete;

	public:
		virtual void OnDownloadStarted(int32 RequestId, const FString& Uri) override
		{
			RxDownloadStarted.Emplace(FStatsCollector::GetSeconds(), RequestId, Uri);
		}

		virtual void OnDownloadProgress(int32 RequestId, uint64 BytesReceived) override
		{
			RxDownloadProgress.Emplace(FStatsCollector::GetSeconds(), RequestId, BytesReceived);
		}

		virtual void OnDownloadComplete(const FDownloadRecord& DownloadRecord) override
		{
			RxDownloadComplete.Emplace(FStatsCollector::GetSeconds(), DownloadRecord);
		}

	public:
		TArray<FDownloadStarted> RxDownloadStarted;
		TArray<FDownloadProgress> RxDownloadProgress;
		TArray<FDownloadComplete> RxDownloadComplete;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
