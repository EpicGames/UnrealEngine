// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BuildPatchFileConstructor.h"
#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockFileConstructorStat
		: public IFileConstructorStat
	{
	public:
		typedef TTuple<double> FResumeStarted;
		typedef TTuple<double> FResumeCompleted;
		typedef TTuple<double, FString, int64> FFileStarted;
		typedef TTuple<double, FString, int64> FFileProgress;
		typedef TTuple<double, FString, bool> FFileCompleted;
		typedef TTuple<double, int64> FProcessedDataUpdated;
		typedef TTuple<double, int64> FTotalRequiredUpdated;

	public:
		virtual void OnResumeStarted() override
		{
			RxResumeStarted.Emplace(FStatsCollector::GetSeconds());
		}

		virtual void OnResumeCompleted() override
		{
			RxResumeCompleted.Emplace(FStatsCollector::GetSeconds());
		}

		virtual void OnFileStarted(const FString& Filename, int64 FileSize) override
		{
			RxFileStarted.Emplace(FStatsCollector::GetSeconds(), Filename, FileSize);
		}

		virtual void OnFileProgress(const FString& Filename, int64 TotalBytes) override
		{
			RxFileProgress.Emplace(FStatsCollector::GetSeconds(), Filename, TotalBytes);
		}

		virtual void OnFileCompleted(const FString& Filename, bool bSuccess) override
		{
			RxFileCompleted.Emplace(FStatsCollector::GetSeconds(), Filename, bSuccess);
		}

		virtual void OnProcessedDataUpdated(int64 TotalBytes) override
		{
			RxProcessedDataUpdated.Emplace(FStatsCollector::GetSeconds(), TotalBytes);
		}

		virtual void OnTotalRequiredUpdated(int64 TotalBytes) override
		{
			RxTotalRequiredUpdated.Emplace(FStatsCollector::GetSeconds(), TotalBytes);
		}

	public:
		TArray<FResumeStarted> RxResumeStarted;
		TArray<FResumeCompleted> RxResumeCompleted;
		TArray<FFileStarted> RxFileStarted;
		TArray<FFileProgress> RxFileProgress;
		TArray<FFileCompleted> RxFileCompleted;
		TArray<FProcessedDataUpdated> RxProcessedDataUpdated;
		TArray<FTotalRequiredUpdated> RxTotalRequiredUpdated;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
