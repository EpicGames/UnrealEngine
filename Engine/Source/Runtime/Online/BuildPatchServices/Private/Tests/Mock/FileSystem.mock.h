// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Common/FileSystem.h"
#include "Tests/TestHelpers.h"
#include "Common/StatsCollector.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Misc/Paths.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockFileSystem
		: public IFileSystem
	{
	public:
		typedef TTuple<double, FArchive*, FString, EReadFlags> FCreateFileReader;
		typedef TTuple<double, FArchive*, FString, EWriteFlags> FCreateFileWriter;
		typedef TTuple<double, FString, int64> FGetFileSize;
		typedef TTuple<double, FString, EAttributeFlags> FGetAttributes;
		typedef TTuple<double, FString, bool> FSetReadOnly;
		typedef TTuple<double, FString, bool> FSetCompressed;
		typedef TTuple<double, FString, bool> FSetExecutable;

	public:
		virtual TUniquePtr<FArchive> CreateFileReader(const TCHAR* Filename, EReadFlags ReadFlags = EReadFlags::None) const override
		{
			FScopeLock ScopeLock(&ThreadLock);
			TUniquePtr<FArchive> Reader(new FMemoryReader(ReadFile));
			RxCreateFileReader.Emplace(FStatsCollector::GetSeconds(), Reader.Get(), Filename, ReadFlags);
			return Reader;
		}

		virtual TUniquePtr<FArchive> CreateFileReaderEx(const TCHAR* Filename, EReadFlags ReadFlags, IFileHandle** OutHandle) const override
		{
			check(OutHandle == nullptr); // The output parameter is not supported for fake implementation yet.
			return CreateFileReader(Filename, ReadFlags);
		}

		virtual TUniquePtr<FArchive> CreateFileWriter(const TCHAR* Filename, EWriteFlags WriteFlags = EWriteFlags::None) const override
		{
			FScopeLock ScopeLock(&ThreadLock);
			TUniquePtr<FArchive> Writer(new FMemoryWriter(WriteFile));
			RxCreateFileWriter.Emplace(FStatsCollector::GetSeconds(), Writer.Get(), Filename, WriteFlags);
			return Writer;
		}

		virtual bool LoadFileToString(const TCHAR* Filename, FString& Contents) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::LoadFileToString");
			return false;
		}

		virtual bool SaveStringToFile(const TCHAR* Filename, const FString& Contents) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::SaveStringToFile");
			return false;
		}

		virtual bool LoadFileToArray(const TCHAR* Filename, TArray<uint8>& Contents) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::LoadFileToArray");
			return false;
		}

		virtual bool SaveArrayToFile(const TCHAR* Filename, const TArray<uint8>& Contents) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::SaveArrayToFile");
			return false;
		}

		virtual bool DirectoryExists(const TCHAR* DirectoryPath) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::DirectoryExists");
			return false;
		}

		virtual bool MakeDirectory(const TCHAR* DirectoryPath) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::MakeDirectory");
			return false;
		}

		virtual bool DeleteDirectoryRecursively(const TCHAR* DirectoryPath) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::DeleteDirectoryRecursively");
			return false;
		}

		virtual bool DeleteFile(const TCHAR* Filename) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::DeleteFile");
			return false;
		}

		virtual bool MoveFile(const TCHAR* FileDest, const TCHAR* FileSource, bool bDoNotRetryOrError) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::MoveFile");
			return false;
		}

		virtual bool CopyFile(const TCHAR* FileDest, const TCHAR* FileSource) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::CopyFile");
			return false;
		}

		virtual bool GetFileSize(const TCHAR* Filename, int64& OutFileSize) const override
		{
			OutFileSize = -1;
			if (FileSizes.Contains(Filename))
			{
				OutFileSize = FileSizes[Filename];
			}
			RxGetFileSize.Emplace(FStatsCollector::GetSeconds(), Filename, OutFileSize);
			return OutFileSize >= 0;
		}

		virtual bool GetAttributes(const TCHAR* Filename, EAttributeFlags& OutFileAttributes) const override
		{
			OutFileAttributes = EAttributeFlags::None;
			if (FileAttributes.Contains(Filename))
			{
				OutFileAttributes = FileAttributes[Filename];
			}
			RxGetAttributes.Emplace(FStatsCollector::GetSeconds(), Filename, OutFileAttributes);
			return true;
		}

		virtual bool GetTimeStamp(const TCHAR* Path, FDateTime& TimeStamp) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::GetTimeStamp");
			return false;
		}

		virtual bool SetReadOnly(const TCHAR* Filename, bool bIsReadOnly) const override
		{
			RxSetReadOnly.Emplace(FStatsCollector::GetSeconds(), Filename, bIsReadOnly);
			return true;
		}

		virtual bool SetCompressed(const TCHAR* Filename, bool bIsCompressed) const override
		{
			RxSetCompressed.Emplace(FStatsCollector::GetSeconds(), Filename, bIsCompressed);
			return true;
		}

		virtual bool SetExecutable(const TCHAR* Filename, bool bIsExecutable) const override
		{
			RxSetExecutable.Emplace(FStatsCollector::GetSeconds(), Filename, bIsExecutable);
			return true;
		}

		virtual bool SetFileSize(const TCHAR* Filename, int64 FileSize) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::SetFileSize");
			return false;
		}

		virtual bool FileExists(const TCHAR* Filename) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::FileExists");
			return true;
		}

		virtual void FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::FindFiles");
			return;
		}

		virtual void FindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension, bool bFollowSymlinks = true) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::FindFilesRecursively");
			return;
		}

		virtual void ParallelFindFilesRecursively(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr, EAsyncExecution AsyncExecution = EAsyncExecution::ThreadPool, bool bFollowSymlinks = true) const override
		{
			FindFilesRecursively(FoundFiles, Directory, FileExtension, bFollowSymlinks);
		}

		virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) const override
		{
			if (!FileExists(FilenameOrDirectory))
			{
				return {};
			}
			FFileStatData Result;
			Result.bIsDirectory = false;
			Result.bIsValid = true;

			Result.bIsValid &= GetFileSize(FilenameOrDirectory, Result.FileSize);
			Result.bIsValid &= GetTimeStamp(FilenameOrDirectory, Result.ModificationTime);
			Result.AccessTime = Result.ModificationTime;
			Result.CreationTime = Result.ModificationTime;

			EAttributeFlags Attributes;
			Result.bIsValid &= GetAttributes(FilenameOrDirectory, Attributes);
			Result.bIsReadOnly = EnumHasAllFlags(Attributes, EAttributeFlags::ReadOnly);
			return Result;
		}

		virtual int64 GetAllowedBytesToWriteThrottledStorage(const TCHAR* DestinationPath = nullptr) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockFileSystem::GetAllowedBytesToWriteThrottledStorage");
			return INT64_MAX;
		}

	public:
		mutable FCriticalSection ThreadLock;
		mutable TArray<FCreateFileReader> RxCreateFileReader;
		mutable TArray<FCreateFileWriter> RxCreateFileWriter;
		mutable TArray<FGetAttributes> RxGetAttributes;
		mutable TArray<FGetFileSize> RxGetFileSize;
		mutable TArray<FSetReadOnly> RxSetReadOnly;
		mutable TArray<FSetCompressed> RxSetCompressed;
		mutable TArray<FSetExecutable> RxSetExecutable;
		mutable TArray<uint8> ReadFile;
		mutable TArray<uint8> WriteFile;
		mutable TMap<FString, int64> FileSizes;
		mutable TMap<FString, EAttributeFlags> FileAttributes;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
