// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Common/HttpManager.h"
#include "Tests/TestHelpers.h"
#include "Tests/Mock/HttpRequest.mock.h"
#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockHttpManager
		: public IHttpManager
	{
	public:
		FMockHttpManager()
			: RxCreateRequest(0)
		{
		}

		virtual TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateRequest() override
		{
			++RxCreateRequest;
			return TSharedRef<IHttpRequest, ESPMode::ThreadSafe>(new FMockHttpRequest());
		}

	public:
		int32 RxCreateRequest;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
