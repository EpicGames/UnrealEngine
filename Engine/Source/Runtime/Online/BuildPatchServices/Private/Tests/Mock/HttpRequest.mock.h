// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/IHttpRequest.h"
#include "Tests/TestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockHttpRequest
		: public IHttpRequest
	{
	public:
		typedef TTuple<FString> FRxSetVerb;
		typedef TTuple<FString> FRxSetURL;

	public:
		virtual const FString& GetURL() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetURL");
			return URL;
		}

		virtual FHttpRequestWillRetryDelegate& OnRequestWillRetry() override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::OnRequestWillRetry");
			return HttpRequestWillRetryDelegate;
		}

		virtual FString GetURLParameter(const FString& ParameterName) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetURLParameter");
			return FString();
		}

		virtual FString GetHeader(const FString& HeaderName) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetHeader");
			return FString();
		}

		virtual TArray<FString> GetAllHeaders() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetAllHeaders");
			return TArray<FString>();
		}

		virtual FString GetContentType() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetContentType");
			return FString();
		}

		virtual uint64 GetContentLength() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetContentLength");
			return uint64();
		}

		virtual const TArray<uint8>& GetContent() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetContent");
			static TArray<uint8> None;
			return None;
		}

		virtual FString GetVerb() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetVerb");
			return FString();
		}

		virtual FString GetOption(const FName Option) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetOption");
			return FString();
		}

		virtual void SetVerb(const FString& Verb) override
		{
			RxSetVerb.Emplace(Verb);
		}

		virtual void SetURL(const FString& InURL) override
		{
			RxSetURL.Emplace(InURL);
		}

		virtual void SetOption(const FName Option, const FString& OptionValue) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetOption");
		}

		virtual void SetContent(const TArray<uint8>& ContentPayload) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetContent");
		}

		virtual void SetContent(TArray<uint8>&& ContentPayload) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetContent");
		}

		virtual void SetContentAsString(const FString& ContentString) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetContentAsString");
		}
        
        virtual bool SetContentAsStreamedFile(const FString& Filename) override
        {
            MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetContentAsStreamedFile");
			return false;
        }

		virtual bool SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetContentFromStream");
			return false;
		}

		virtual bool SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetResponseBodyReceiveStream");
			return false;
		}

		virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetHeader");
		}

		virtual void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::AppendToHeader");
		}

		virtual void SetTimeout(float InTimeoutSecs) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetTimeout");
		}

		virtual void ClearTimeout() override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::ClearTimeout");
		}

		virtual void ResetTimeoutStatus() override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::ResetTimeoutStatus");
		}

		virtual TOptional<float> GetTimeout() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetTimeout");
			return TOptional<float>();
		}

		virtual void SetActivityTimeout(float InTimeoutSecs) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetActivityTimeout");
		}

		virtual void ProcessRequestUntilComplete() override
		{
		}

		virtual bool ProcessRequest() override
		{
			++RxProcessRequest;
			return true;
		}

		virtual FHttpRequestCompleteDelegate& OnProcessRequestComplete() override
		{
			return HttpRequestCompleteDelegate;
		}

		virtual FHttpRequestProgressDelegate64& OnRequestProgress64() override
		{
			return HttpRequestProgressDelegate64;
		}

		virtual FHttpRequestStatusCodeReceivedDelegate& OnStatusCodeReceived() override
		{
			return HttpStatusCodeReceivedDelegate;
		}

		virtual FHttpRequestHeaderReceivedDelegate& OnHeaderReceived() override
		{
			return HttpHeaderReceivedDelegate;
		}

		virtual void CancelRequest() override
		{
			++RxCancelRequest;
		}

		virtual EHttpRequestStatus::Type GetStatus() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetStatus");
			return EHttpRequestStatus::Type();
		}

		virtual EHttpFailureReason GetFailureReason() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetFailureReason");
			return EHttpFailureReason::None;
		}

		virtual const FHttpResponsePtr GetResponse() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetResponse");
			return FHttpResponsePtr();
		}

		virtual void Tick(float DeltaSeconds) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::Tick");
		}

		virtual float GetElapsedTime() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetElapsedTime");
			return float();
		}

		virtual void SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetDelegateThreadPolicy");
		}

		virtual EHttpRequestDelegateThreadPolicy GetDelegateThreadPolicy() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetDelegateThreadPolicy");
			return EHttpRequestDelegateThreadPolicy::CompleteOnGameThread;
		}

		virtual void SetPriority(EHttpRequestPriority InPriority) override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::SetPriority");
		}

		virtual EHttpRequestPriority GetPriority() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetPriority");
			return EHttpRequestPriority::Normal;
		}

		virtual const FString& GetEffectiveURL() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetEffectiveURL");
			return EffectiveURL;
		}

	public:
		FHttpRequestProgressDelegate HttpRequestProgressDelegate;
		FHttpRequestProgressDelegate64 HttpRequestProgressDelegate64;
		FHttpRequestCompleteDelegate HttpRequestCompleteDelegate;
		FHttpRequestStatusCodeReceivedDelegate HttpStatusCodeReceivedDelegate;
		FHttpRequestHeaderReceivedDelegate HttpHeaderReceivedDelegate;
		FHttpRequestWillRetryDelegate HttpRequestWillRetryDelegate;

		TArray<FRxSetVerb> RxSetVerb;
		TArray<FRxSetURL> RxSetURL;
		int32 RxProcessRequest;
		int32 RxCancelRequest;
		FString EffectiveURL;
		FString URL;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
