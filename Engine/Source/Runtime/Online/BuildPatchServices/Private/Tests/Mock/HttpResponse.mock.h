// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/IHttpResponse.h"
#include "Tests/TestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockHttpResponse
		: public IHttpResponse
	{
	public:
		typedef TTuple<FString> FRxSetVerb;
		typedef TTuple<FString> FRxSetURL;

	public:
		virtual int32 GetResponseCode() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetResponseCode");
			return static_cast<int32>(EHttpResponseCodes::Ok);
		}

		virtual FString GetContentAsString() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetContentAsString");
			return FString();
		}

		virtual FUtf8StringView GetContentAsUtf8StringView() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetContentAsUtf8StringView");
			return FUtf8StringView();
		}

		virtual const FString& GetURL() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetURL");
			return URL;
		}

		virtual FString GetURLParameter(const FString& ParameterName) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetURLParameter");
			return FString();
		}

		virtual FString GetHeader(const FString& HeaderName) const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetHeader");
			return FString();
		}

		virtual TArray<FString> GetAllHeaders() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetAllHeaders");
			return TArray<FString>();
		}

		virtual FString GetContentType() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetContentType");
			return FString();
		}

		virtual uint64 GetContentLength() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetContentLength");
			return uint64();
		}

		virtual const TArray<uint8>& GetContent() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetContent");
			static TArray<uint8> None;
			return None;
		}

		virtual TArray<uint8> TakeContent() override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::TakeContent");
			return {};
		}

		virtual EHttpRequestStatus::Type GetStatus() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetStatus");
			return EHttpRequestStatus::Failed;
		}

		virtual EHttpFailureReason GetFailureReason() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetFailureReason");
			return EHttpFailureReason::Other;
		}

		virtual const FString& GetEffectiveURL() const override
		{
			MOCK_FUNC_NOT_IMPLEMENTED("FMockHttpRequest::GetEffectiveURL");
			static FString None;
			return None;
		}

	private:
		FString URL;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
