// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/InstallChunkSource.h"
#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockInstallChunkSourceStat
		: public IInstallChunkSourceStat
	{
	public:
		typedef TTuple<double, FGuid> FLoadStarted;
		typedef TTuple<double, FGuid, ELoadResult, ISpeedRecorder::FRecord> FLoadComplete;

	public:
		virtual void OnLoadStarted(const FGuid& ChunkId) override
		{
			if (OnLoadStartedFunc != nullptr)
			{
				OnLoadStartedFunc(ChunkId);
			}
			RxLoadStarted.Emplace(FStatsCollector::GetSeconds(), ChunkId);
		}

		virtual void OnLoadComplete(const FGuid& ChunkId, const ELoadResult& Result, const ISpeedRecorder::FRecord& Record) override
		{
			if (OnLoadCompleteFunc != nullptr)
			{
				OnLoadCompleteFunc(ChunkId, Result, Record);
			}
			RxLoadComplete.Emplace(FStatsCollector::GetSeconds(), ChunkId, Result, Record);
		}

	public:
		TArray<FLoadStarted> RxLoadStarted;
		TArray<FLoadComplete> RxLoadComplete;
		TFunction<void(const FGuid&)> OnLoadStartedFunc;
		TFunction<void(const FGuid&, const ELoadResult&, const ISpeedRecorder::FRecord&)> OnLoadCompleteFunc;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
