// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/InstallerAnalytics.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Common/StatsCollector.h"

namespace BuildPatchServices
{
	class FMockInstallerAnalytics
		: public IInstallerAnalytics
	{
	public:
		typedef TTuple<double, FString, int32,  FString> FRecordChunkDownloadError;
		typedef TTuple<double, FString, FString, FString> FRecordMissingDelta;
		typedef TTuple<double, FString, double, double, double, double> FRecordChunkDownloadAborted;
		typedef TTuple<double, FGuid, FString, int32, FString, FString> FRecordChunkCacheError;
		typedef TTuple<double, FString, int32, FString> FRecordConstructionError;
		typedef TTuple<double, FString, FString, FString, FString, int32, FString> FRecordPrereqInstallationError;
		typedef TTuple<double, FHttpRequestPtr> FTrackRequest;

	public:
		virtual void RecordChunkDownloadError(const FString& ChunkUrl, int32 ResponseCode, const FString& ErrorString) override
		{
			RxRecordChunkDownloadError.Emplace(FStatsCollector::GetSeconds(), ChunkUrl, ResponseCode, ErrorString);
		}

		virtual void RecordMissingDelta(const FString& DeltaUrl, const FString& BaseVersion, const FString& PatchVersion) override
		{
			RxRecordMissingDelta.Emplace(FStatsCollector::GetSeconds(), DeltaUrl, BaseVersion, PatchVersion);
		}

		virtual void RecordChunkDownloadAborted(const FString& ChunkUrl, double ChunkTime, double ChunkMean, double ChunkStd, double BreakingPoint) override
		{
			RxRecordChunkDownloadAborted.Emplace(FStatsCollector::GetSeconds(), ChunkUrl, ChunkTime, ChunkMean, ChunkStd, BreakingPoint);
		}

		virtual void RecordChunkCacheError(const FGuid& ChunkGuid, const FString& Filename, int32 LastError, const FString& SystemName, const FString& ErrorString) override
		{
			RxRecordChunkCacheError.Emplace(FStatsCollector::GetSeconds(), ChunkGuid, Filename, LastError, SystemName, ErrorString);
		}

		virtual void RecordConstructionError(const FString& Filename, int32 LastError, const FString& ErrorString) override
		{
			RxRecordConstructionError.Emplace(FStatsCollector::GetSeconds(), Filename, LastError, ErrorString);
		}

		virtual void RecordPrereqInstallationError(const FString& AppName, const FString& AppVersion, const FString& Filename, const FString& CommandLine, int32 ErrorCode, const FString& ErrorString) override
		{
			RxRecordPrereqInstallationError.Emplace(FStatsCollector::GetSeconds(), AppName, AppVersion, Filename, CommandLine, ErrorCode, ErrorString);
		}

		virtual void TrackRequest(const FHttpRequestPtr& Request) override
		{
			RxTrackRequest.Emplace(FStatsCollector::GetSeconds(), Request);
		}

		virtual void Flush() override
		{
			++RxFlush;
		}

	public:
		TArray<FRecordChunkDownloadError> RxRecordChunkDownloadError;
		TArray<FRecordMissingDelta> RxRecordMissingDelta;
		TArray<FRecordChunkDownloadAborted> RxRecordChunkDownloadAborted;
		TArray<FRecordChunkCacheError> RxRecordChunkCacheError;
		TArray<FRecordConstructionError> RxRecordConstructionError;
		TArray<FRecordPrereqInstallationError> RxRecordPrereqInstallationError;
		TArray<FTrackRequest> RxTrackRequest;
		uint64 RxFlush;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
