// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/IBuildManifest.h"
#include "Algo/Copy.h"
#include "BuildPatchManifest.h"
#include "Tests/Mock/ManifestField.mock.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockManifest
		: public FBuildPatchAppManifest
	{
	public:
		virtual uint32 GetAppID() const override
		{
			return AppId;
		}

		virtual const FString& GetAppName() const override
		{
			return AppName;
		}

		virtual const FString& GetVersionString() const override
		{
			return VersionString;
		}

		virtual const FString& GetLaunchExe() const override
		{
			return LaunchExe;
		}

		virtual const FString& GetLaunchCommand() const override
		{
			return LaunchCommand;
		}

		virtual const FString& GetPrereqName() const override
		{
			return PrereqName;
		}

		virtual const FString& GetPrereqPath() const override
		{
			return PrereqPath;
		}

		virtual const FString& GetPrereqArgs() const override
		{
			return PrereqArgs;
		}

		virtual int64 GetDownloadSize() const override
		{
			return DownloadSize;
		}

		virtual int64 GetDownloadSize(const TSet<FString>& Tags) const override
		{
			return TagDownloadSize;
		}

		virtual int64 GetDeltaDownloadSize(const TSet<FString>& Tags, const IBuildManifestRef& PreviousVersion) const override
		{
			return DeltaDownloadSize;
		}

		virtual int64 GetBuildSize() const override
		{
			return BuildSize;
		}

		virtual int64 GetBuildSize(const TSet<FString>& Tags) const override
		{
			return TagBuildSize;
		}

		virtual TArray<FString> GetBuildFileList() const override
		{
			TArray<FString> Filenames;
			GetFileList(Filenames);
			return Filenames;
		}

		virtual TArray<FStringView> GetBuildFileListView() const override
		{
			TArray<FStringView> Filenames;
			GetFileList(Filenames);
			return Filenames;
		}

		virtual TArray<FString> GetBuildFileList(const TSet<FString>& Tags) const override
		{
			TArray<FString> Filenames;
			GetTaggedFileList(Tags, Filenames);
			return Filenames;
		}

		virtual TArray<FStringView> GetBuildFileListView(const TSet<FString>& Tags) const override
		{
			TArray<FStringView> Filenames;
			GetTaggedFileList(Tags, Filenames);
			return Filenames;
		}

		virtual void GetFileTagList(TSet<FString>& Tags) const override
		{
			Tags = FileTagList;
		}

		virtual void GetRemovableFiles(const IBuildManifestRef& OldManifest, TArray< FString >& OutRemovableFiles) const override
		{
			OutRemovableFiles = RemovableFiles;
		}

		virtual void GetRemovableFiles(const TCHAR* InstallPath, TArray< FString >& OutRemovableFiles) const override
		{
			OutRemovableFiles = RemovableFiles;
		}

		virtual void CopyCustomFields(const IBuildManifestRef& Other, bool bClobber) override
		{
		}

		virtual bool NeedsResaving() const override
		{
			return false;
		}

		virtual TArray<FString> GetCustomFieldNames() const override
		{
			TArray<FString> CustomFieldNames;
			CustomFields.GenerateKeyArray(CustomFieldNames);
			return CustomFieldNames;
		}

		virtual const IManifestFieldPtr GetCustomField(const FString& FieldName) const override
		{
			return MakeShareable(new FMockManifestField(CustomFields[FieldName]));
		}

		virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const FString& Value) override
		{
			CustomFields.FindOrAdd(FieldName).String = Value; return GetCustomField(FieldName);
		}

		virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const double& Value) override
		{
			CustomFields.FindOrAdd(FieldName).Double = Value; return GetCustomField(FieldName);
		}

		virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const int64& Value) override
		{
			CustomFields.FindOrAdd(FieldName).Integer = Value; return GetCustomField(FieldName);
		}

		virtual void RemoveCustomField(const FString& FieldName) override
		{
			CustomFields.Remove(FieldName);
		}

		virtual IBuildManifestRef Duplicate() const override
		{
			return MakeShareable(new FMockManifest(*this));
		}

		virtual bool LoadFromFile(const FString& Filename) override
		{
			return true;
		}

		virtual bool DeserializeFromData(const TArray<uint8>& DataInput) override
		{
			return true;
		}

		virtual bool DeserializeFromJSON(const FString& JSONInput) override
		{
			return true;
		}

		virtual bool SaveToFile(const FString& Filename, EFeatureLevel InFeatureLevel, FSHAHash* OutSHA1Hash, FMD5Hash* OutMD5Hash) override
		{
			return true;
		}

		virtual void SerializeToJSON(FString& JSONOutput) override
		{
		}

		virtual EFeatureLevel GetFeatureLevel() const override
		{
			return FeatureLevel;
		}

		virtual void GetChunksRequiredForFiles(const TSet<FString>& Filenames, TSet<FGuid>& RequiredChunks) const override
		{
			RequiredChunks = ChunksRequiredForFiles;
		}

		virtual uint32 GetNumberOfChunkReferences(const FGuid& ChunkGuid) const override
		{
			return NumberOfChunkReferences;
		}

		virtual int64 GetDataSize(const FGuid& DataGuid) const override
		{
			return DataSize;
		}

		virtual int64 GetDataSize(const TArray<FGuid>& DataGuids) const override
		{
			return DataSize;
		}

		virtual int64 GetDataSize(const TSet <FGuid>& DataGuids) const override
		{
			return DataSize;
		}

		virtual int64 GetFileSize(FStringView Filename) const override
		{
			return FileNameToFileSize.FindRef(FString(Filename));
		}

		virtual int64 GetFileSize(const TArray<FString>& Filenames) const override
		{
			int64 FileSize = 0;
			for (const FString& Filename : Filenames)
			{
				FileSize += GetFileSize(Filename);
			}
			return FileSize;
		}

		virtual int64 GetFileSize(const TSet <FString>& Filenames) const override
		{
			return GetFileSize(Filenames.Array());
		}

		virtual EFileMetaFlags GetFileMetaFlags(const FString& Filename) const override
		{
			return FileManifests[Filename].FileMetaFlags;
		}

		virtual const FMD5Hash* GetFileMD5Hash(const FString& Filename) const override
		{
			return &FileManifests[Filename].MD5Hash;
		}

		virtual const FSHAHash* GetFileSHA1Hash(const FString& Filename) const override
		{
			return &FileManifests[Filename].SHA1Hash;
		}

		virtual const FSHA256Signature* GetFileSHA256Hash(const FString& Filename) const override
		{
			return &FileManifests[Filename].SHA256Hash;
		}

		virtual const FString* GetFileMIMEType(const FString& Filename) const override
		{
			return &FileManifests[Filename].MIMEType;
		}

		virtual uint32 GetNumFiles() const override
		{
			return NumFiles;
		}

		virtual void GetFileList(TArray<FString>& Filenames) const override
		{
			Filenames = BuildFileList;
		}

		virtual void GetFileList(TArray<FStringView>& Filenames) const override
		{
			Algo::Copy(BuildFileList, Filenames);
		}

		virtual void GetFileList(TSet<FString>& Filenames) const override
		{
			Algo::Copy(BuildFileList, Filenames);
		}

		virtual void GetTaggedFileList(const TSet<FString>& Tags, TSet<FString>& TaggedFiles) const override
		{
			TaggedFiles = TaggedFileList;
		}

		virtual void GetTaggedFileList(const TSet<FString>& Tags, TArray<FString>& TaggedFiles) const override
		{
			TaggedFiles = TaggedFileList.Array();
		}

		virtual void GetTaggedFileList(const TSet<FString>& Tags, TArray<FStringView>& TaggedFiles) const override
		{
			Algo::Copy(TaggedFileList, TaggedFiles);
		}

		virtual void GetDataList(TArray<FGuid>& DataGuids) const override
		{
			DataGuids = DataList;
		}

		virtual void GetDataList(TSet <FGuid>& DataGuids) const override
		{
			DataGuids.Append(DataList);
		}

		virtual const FFileManifest* GetFileManifest(const FString& Filename) const override
		{
			return FileManifests.Find(Filename);
		}

		virtual bool IsFileDataManifest() const override
		{
			return false;
		}

		virtual bool GetChunkHash(const FGuid& ChunkGuid, uint64& OutHash) const override
		{
			if (ChunkInfos.Contains(ChunkGuid))
			{
				OutHash = ChunkInfos[ChunkGuid].Hash;
				return true;
			}
			return false;
		}

		virtual bool GetChunkShaHash(const FGuid& ChunkGuid, FSHAHash& OutHash) const override
		{
			static const uint8 Zero[FSHA1::DigestSize] = { 0 };
			if (ChunkInfos.Contains(ChunkGuid))
			{
				OutHash = ChunkInfos[ChunkGuid].ShaHash;
				return FMemory::Memcmp(OutHash.Hash, Zero, FSHA1::DigestSize) != 0;
			}
			return false;
		}

		virtual bool GetChunkEncryptionSecretId(const FGuid& ChunkGuid, FGuid& OutEncryptionSecretId) const override
		{
			if (ChunkInfos.Contains(ChunkGuid))
			{
				OutEncryptionSecretId = ChunkInfos[ChunkGuid].EncryptionSecretId;
				return OutEncryptionSecretId.IsValid();
			}
			return false;
		}

		virtual const FChunkInfo* GetChunkInfo(const FGuid& ChunkGuid) const override
		{
			if (ChunkInfos.Contains(ChunkGuid))
			{
				return &ChunkInfos[ChunkGuid];
			}
			return nullptr;
		}

		virtual bool GetFileHash(const FGuid& FileGuid, FSHAHash& OutHash) const override
		{
			if (FileIdToHashes.Contains(FileGuid))
			{
				OutHash = FileIdToHashes[FileGuid];
				return true;
			}
			return false;
		}

		virtual bool GetFileHash(const FString& Filename, FSHAHash& OutHash) const override
		{
			if (FileNameToHashes.Contains(Filename))
			{
				OutHash = FileNameToHashes[Filename];
				return true;
			}
			return false;
		}

		virtual bool GetFilePartHash(const FGuid& FilePartGuid, uint64& OutHash) const override
		{
			if (FilePartHashes.Contains(FilePartGuid))
			{
				OutHash = FilePartHashes[FilePartGuid];
				return true;
			}
			return false;
		}

		virtual void GetOutdatedFiles(const IBuildManifestRef& OldManifest, TSet<FString>& OutOutdatedFiles) const override { GetOutdatedFiles(FBuildPatchAppManifestPtr(StaticCastSharedRef<FBuildPatchAppManifest>(OldManifest)), TEXT(""), OutOutdatedFiles); }
		virtual void GetOutdatedFiles(const FBuildPatchAppManifestPtr& OldManifest, const FString& InstallDirectory, TSet<FString>& OutOutdatedFiles) const override { GetOutdatedFiles(OldManifest.Get(), InstallDirectory, OutOutdatedFiles); }
		virtual void GetOutdatedFiles(const FBuildPatchAppManifest*    OldManifest, const FString& InstallDirectory, TSet<FString>& OutOutdatedFiles) const override { GetOutdatedFiles(OldManifest, InstallDirectory, TSet<FString>(), OutOutdatedFiles); }
		virtual void GetOutdatedFiles(const FBuildPatchAppManifest*    OldManifest, const FString& InstallDirectory, const TSet<FString>& FilesToCheck, TSet<FString>& OutOutdatedFiles) const
		{
			OutOutdatedFiles = OutOutdatedFiles.Union(OutdatedFiles);
		}

		virtual bool IsFileOutdated(const FBuildPatchAppManifestRef& OldManifest, const FString& Filename) const override
		{
			return true;
		}

		virtual bool HasFileAttributes() const override
		{
			return true;
		}

		void SyncInternalManifestStructures()
		{
			// Normal manifests have some lookup structures we need to be fixed up in order for
			// things to work.
			for (const TPair<FString, FFileManifest>& FileManifest : FileManifests)
			{
				FileManifestList.FileList.Add(FileManifest.Value);
			}

			InitLookups();
		}

	public:
		uint64 AppId;
		FString AppName;
		FString VersionString;
		FString LaunchExe;
		FString LaunchCommand;
		FString PrereqName;
		FString PrereqPath;
		FString PrereqArgs;
		TArray<FString> BuildFileList;
		TSet<FString> FileTagList;
		int64 DownloadSize;
		int64 TagDownloadSize;
		int64 DeltaDownloadSize;
		int64 BuildSize;
		int64 TagBuildSize;
		TArray<FString> RemovableFiles;
		TMap<FString, FMockManifestField> CustomFields;
		EFeatureLevel FeatureLevel;
		TSet<FGuid> ChunksRequiredForFiles;
		uint32 NumberOfChunkReferences;
		int64 DataSize;
		TMap<FString, int64> FileNameToFileSize;
		uint32 NumFiles;
		TSet<FString> TaggedFileList;
		TArray<FGuid> DataList;
		TMap<FString, FFileManifest> FileManifests;
		TMap<FGuid, FChunkInfo> ChunkInfos;
		TMap<FGuid, FSHAHash> FileIdToHashes;
		TMap<FString, FSHAHash> FileNameToHashes;
		TMap<FGuid, uint64> FilePartHashes;
		TSet<FGuid> ProducibleChunks;
		TSet<FString> OutdatedFiles;
	};

	typedef TSharedPtr<BuildPatchServices::FMockManifest, ESPMode::ThreadSafe> FMockManifestPtr;
}

#endif //WITH_DEV_AUTOMATION_TESTS
