// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/IBuildManifest.h"
#include "BuildPatchManifest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockManifestField
		: public IManifestField
	{
	public:
		virtual FString AsString() const override
		{
			return String;
		}

		virtual double AsDouble() const override
		{
			return Double;
		}

		virtual int64 AsInteger() const override
		{
			return Integer;
		}

	public:
		FString String;
		double Double;
		int64 Integer;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
