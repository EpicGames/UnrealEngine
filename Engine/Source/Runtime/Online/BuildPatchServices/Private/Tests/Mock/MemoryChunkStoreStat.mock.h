// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/MemoryChunkStore.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Common/StatsCollector.h"

namespace BuildPatchServices
{
	class FMockMemoryChunkStoreStat
		: public IMemoryChunkStoreStat
	{
	public:
		typedef TTuple<double, FGuid> FChunkStored;
		typedef TTuple<double, FGuid> FChunkReleased;
		typedef TTuple<double, FGuid> FChunkBooted;
		typedef TTuple<double, int32> FStoreUseUpdated;
		typedef TTuple<double, int32> FStoreSizeUpdated;

	public:
		virtual void OnChunkStored(const FGuid& ChunkId) override
		{
			RxChunkStored.Emplace(FStatsCollector::GetSeconds(), ChunkId);
		}

		virtual void OnChunkReleased(const FGuid& ChunkId) override
		{
			RxChunkReleased.Emplace(FStatsCollector::GetSeconds(), ChunkId);
		}

		virtual void OnChunkBooted(const FGuid& ChunkId) override
		{
			RxChunkBooted.Emplace(FStatsCollector::GetSeconds(), ChunkId);
		}

		virtual void OnStoreUseUpdated(int32 ChunkCount) override
		{
			RxStoreUseUpdated.Emplace(FStatsCollector::GetSeconds(), ChunkCount);
		}

		virtual void OnStoreSizeUpdated(int32 Size) override
		{
			RxStoreSizeUpdated.Emplace(FStatsCollector::GetSeconds(), Size);
		}

	public:
		TArray<FChunkStored> RxChunkStored;
		TArray<FChunkReleased> RxChunkReleased;
		TArray<FChunkBooted> RxChunkBooted;
		TArray<FStoreUseUpdated> RxStoreUseUpdated;
		TArray<FStoreSizeUpdated> RxStoreSizeUpdated;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
