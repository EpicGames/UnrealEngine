// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/MessagePump.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockMessagePump
		: public IMessagePump
	{
	public:
		virtual void SendMessage(FChunkSourceEvent Message) override
		{
		}

		virtual void SendMessage(FInstallationFileAction Message) override
		{
		}

		virtual void SendMessage(FGenericMessage Message) override
		{
		}

		virtual void PumpMessages(const TArray<FMessageHandler*>& Handlers) override
		{
		}
		virtual void PumpMessages() override
		{
		}
		virtual void SendRequest(FDeltaUriRequest Request, TFunction<void(FExpiratedUrisResponse)> OnResponse) override
		{
		}

		virtual bool SendRequest(FChunkUriRequest Request, TFunction<void(FChunkUriResponse)> OnResponse) override
		{
			return false;
		}

		virtual void RegisterMessageHandler(FMessageHandler* MessageHandler) override
		{
		}

		virtual void UnregisterMessageHandler(FMessageHandler* MessageHandler) override
		{
		}
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
