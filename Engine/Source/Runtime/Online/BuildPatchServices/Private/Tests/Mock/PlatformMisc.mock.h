// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockPlatformMisc
	{
	public:
		static uint32 GetLastError()
		{
			return 0;
		}
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
