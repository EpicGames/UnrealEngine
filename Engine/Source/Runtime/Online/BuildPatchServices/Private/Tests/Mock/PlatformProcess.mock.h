// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "CoreTypes.h"

namespace BuildPatchServices
{
	class FMockPlatformProcess
	{
	public:
		static bool ExecElevatedProcess(const TCHAR* URL, const TCHAR* Params, int32* OutReturnCode)
		{
			*OutReturnCode = 0;
			return true;
		}

		static void SleepNoStats(float Seconds)
		{
		}
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
