// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Installer/Verifier.h"
#include "Common/StatsCollector.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace BuildPatchServices
{
	class FMockVerifierStat
		: public IVerifierStat
	{
	public:
		typedef TTuple<double, FString, int64> FOnFileStarted;
		typedef TTuple<double, FString, int64> FOnFileProgress;
		typedef TTuple<double, FString, EVerifyResult> FOnFileCompleted;
		typedef TTuple<double, ISpeedRecorder::FRecord> FOnFileRead;
		typedef TTuple<double, int64> FOnProcessedDataUpdated;
		typedef TTuple<double, int64> FOnTotalRequiredUpdated;

	public:
		virtual void OnFileStarted(const FString& Filename, int64 FileSize) override
		{
			RxOnFileStarted.Emplace(FStatsCollector::GetSeconds(), Filename, FileSize);
		}

		virtual void OnFileProgress(const FString& Filename, int64 TotalBytes) override
		{
			if (OnFileProgressFunc)
			{
				OnFileProgressFunc(Filename, TotalBytes);
			}
			RxOnFileProgress.Emplace(FStatsCollector::GetSeconds(), Filename, TotalBytes);
		}

		virtual void OnFileCompleted(const FString& Filename, EVerifyResult VerifyResult) override
		{
			if (OnFileCompletedFunc)
			{
				OnFileCompletedFunc(Filename, VerifyResult);
			}
			RxOnFileCompleted.Emplace(FStatsCollector::GetSeconds(), Filename, VerifyResult);
		}

		virtual void OnFileRead(const ISpeedRecorder::FRecord& Record) override
		{
			RxOnFileRead.Emplace(FStatsCollector::GetSeconds(), Record);
		}

		virtual void OnProcessedDataUpdated(int64 TotalBytes) override
		{
			RxOnProcessedDataUpdated.Emplace(FStatsCollector::GetSeconds(), TotalBytes);
		}

		virtual void OnTotalRequiredUpdated(int64 TotalBytes) override
		{
			RxOnTotalRequiredUpdated.Emplace(FStatsCollector::GetSeconds(), TotalBytes);
		}

	public:
		TArray<FOnFileStarted> RxOnFileStarted;
		TArray<FOnFileProgress> RxOnFileProgress;
		TArray<FOnFileCompleted> RxOnFileCompleted;
		TArray<FOnFileRead> RxOnFileRead;
		TArray<FOnProcessedDataUpdated> RxOnProcessedDataUpdated;
		TArray<FOnTotalRequiredUpdated> RxOnTotalRequiredUpdated;
		TFunction<void(const FString&, int64)> OnFileProgressFunc;
		TFunction<void(const FString&, EVerifyResult)> OnFileCompletedFunc;
	};
}

#endif //WITH_DEV_AUTOMATION_TESTS
