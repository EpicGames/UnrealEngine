// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BuildPatchServicesPrivate.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "Misc/AutomationTest.h"

namespace BuildPatchTestHelpers
{
	template<typename T1, typename T2>
	inline bool TestEqual(const FString& Description, T1 Expression, T2 Expected, FAutomationSpecBase* This)
	{
		This->TestEqual(Description, Expression, Expected);
		return Expression == Expected;
	}
	
	template<typename T1, typename T2>
	inline bool TestNotEqual(const FString& Description, T1 Expression, T2 Expected, FAutomationSpecBase* This)
	{
		This->TestNotEqual(Description, Expression, Expected);
		return Expression != Expected;
	}

	template<typename T>
	inline bool TestNull(const FString& Description, T Expression, FAutomationSpecBase* This)
	{
		This->TestNull(Description, Expression);
		return Expression == nullptr;
	}

	template<typename T>
	inline bool TestNotNull(const FString& Description, T Expression, FAutomationSpecBase* This)
	{
		This->TestNotNull(Description, Expression);
		return Expression != nullptr;
	}

	inline bool WaitUntilTrue(const TFunction<bool()>& Pred, const TFunction<bool()>& TestResult, double TimeoutSeconds)
	{
		const double TimeoutEnd = FPlatformTime::Seconds() + TimeoutSeconds;
		while (!Pred() && FPlatformTime::Seconds() < TimeoutEnd)
		{
			FPlatformProcess::Sleep(0);
		}
		return TestResult();
	}
}

#if WITH_DEV_AUTOMATION_TESTS

#define _TEST_EQUAL(text, expression, expected) \
	BuildPatchTestHelpers::TestEqual(text, expression, expected, this)

#define _TEST_NOT_EQUAL(text, expression, expected) \
	BuildPatchTestHelpers::TestNotEqual(text, expression, expected, this)

#define _TEST_NULL(text, expression) \
	BuildPatchTestHelpers::TestNull(text, expression, this)

#define _TEST_NOT_NULL(text, expression) \
	BuildPatchTestHelpers::TestNotNull(text, expression, this)

#define TEST_EQUAL(expression, expected) \
	_TEST_EQUAL(TEXT(#expression), expression, expected)

#define TEST_NOT_EQUAL(expression, expected) \
	_TEST_NOT_EQUAL(TEXT(#expression), expression, expected)

#define TEST_TRUE(expression) \
	TEST_EQUAL(expression, true)

#define TEST_FALSE(expression) \
	TEST_EQUAL(expression, false)

#define TEST_NULL(expression) \
	_TEST_NULL(TEXT(#expression), expression)

#define TEST_NOT_NULL(expression) \
	_TEST_NOT_NULL(TEXT(#expression), expression)

#define TEST_SIZE(ar, value) \
	_TEST_EQUAL(TEXT("size check"), ar.Num(), value)

#define TEST_EMPTY(ar) TEST_SIZE(ar, 0)

#define TEST_CONTAINS(expression, value) \
	_TEST_EQUAL(TEXT("contains check"), expression.Contains(value), true)

#define TEST_BECOMES_TRUE(expression, timeout) \
	BuildPatchTestHelpers::WaitUntilTrue([this](){ return expression; }, [this](){ return TEST_EQUAL(expression, true); }, timeout)

#define MOCK_FUNC_NOT_IMPLEMENTED(funcname) \
	UE_LOG(LogBuildPatchServices, Error, TEXT(funcname) TEXT(": Called but there is no implementation."))

#define ARRAY(Type, ...) TArray<Type>({ __VA_ARGS__ })
#define ARRAYU64(...) ARRAY(uint64, __VA_ARGS__)

template<typename ElementType>
bool SetsAreEqual(const TSet<ElementType>& LHS, const TSet<ElementType>& RHS)
{
	return (LHS.Num() == RHS.Num())
	    && (LHS.Difference(RHS).Num() == 0)
	    && (RHS.Difference(LHS).Num() == 0);
}

#endif //WITH_DEV_AUTOMATION_TESTS
