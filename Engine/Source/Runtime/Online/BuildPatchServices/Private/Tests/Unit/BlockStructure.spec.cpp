// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/TestHelpers.h"
#include "Core/BlockStructure.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	template<typename ElementType>
	TArray<ElementType> operator+(const TArray<ElementType>& LHS, const TArray<ElementType>& RHS)
	{
		TArray<ElementType> Result;
		Result.Empty(LHS.Num() + RHS.Num());
		Result.Append(LHS);
		Result.Append(RHS);
		return Result;
	}

	template<typename ElementType>
	TArray<ElementType> ArrayLeft(const TArray<ElementType>& Array, int32 Count)
	{
		TArray<ElementType> Result;
		Result.Empty(Count);
		Result.Append(Array.GetData(), Count);
		return Result;
	}

	template<typename ElementType>
	TArray<ElementType> ArrayRight(const TArray<ElementType>& Array, int32 Count)
	{
		TArray<ElementType> Result;
		Result.Empty(Count);
		Result.Append(Array.GetData() + (Array.Num() - Count), Count);
		return Result;
	}

	template<typename ElementType>
	TArray<ElementType> ArrayChopLeft(const TArray<ElementType>& Array, int32 Count)
	{
		TArray<ElementType> Result;
		Result.Empty(Array.Num() - Count);
		Result.Append(Array.GetData() + Count, Array.Num() - Count);
		return Result;
	}

	template<typename ElementType>
	TArray<ElementType> ArrayChopRight(const TArray<ElementType>& Array, int32 Count)
	{
		TArray<ElementType> Result;
		Result.Empty(Array.Num() - Count);
		Result.Append(Array.GetData(), Array.Num() - Count);
		return Result;
	}

	template<typename ElementType>
	TArray<ElementType> ArrayHead(const TArray<ElementType>& Array)
	{
		return ArrayLeft(Array, 2);
	}

	template<typename ElementType>
	TArray<ElementType> ArrayTail(const TArray<ElementType>& Array)
	{
		return ArrayRight(Array, 2);
	}

	template<typename ElementType>
	TArray<ElementType> ArrayClobber(const TArray<ElementType>& Array, int32 Start, const TArray<ElementType>& Elements)
	{
		TArray<ElementType> Result;
		int32 NewLength = FMath::Max(Array.Num(), Start + (Elements.Num() - 1));
		Result.Empty(NewLength);
		Result.Append(Array);
		for (int32 ElementIdx = 0; ElementIdx < Elements.Num(); ++ElementIdx)
		{
			Result[Start + ElementIdx] = Elements[ElementIdx];
		}
		return Result;
	}
}

BEGIN_DEFINE_SPEC(FBlockStructureSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
// Unit.
BuildPatchServices::FBlockStructure BlockStructure;
BuildPatchServices::FBlockStructure OtherStructure;
// Data.
TArray<FString> DirNames;
TArray<ESearchDir::Type> DirTypes;
TArray<uint64> SetupBlocks;
// Test helpers.
bool IsLooping(const BuildPatchServices::FBlockStructure& Structure);
bool IsMirrored(const BuildPatchServices::FBlockStructure& Structure);
TArray<uint64> ToArrayU64(const BuildPatchServices::FBlockStructure& Structure);
TSet<const BuildPatchServices::FBlockEntry*> EnumeratePtrs(const BuildPatchServices::FBlockStructure& Structure);
END_DEFINE_SPEC(FBlockStructureSpec)

void FBlockStructureSpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	DirNames = {TEXT("FromStart"), TEXT("FromEnd")};
	DirTypes = {ESearchDir::FromStart, ESearchDir::FromEnd};
	SetupBlocks = {10, 5, 20, 7, 30, 3, 40, 5};

	// Specs.
	BeforeEach([this]()
	{
		BlockStructure = FBlockStructure();
		OtherStructure = FBlockStructure();
	});

	Describe("BlockStructure", [this]()
	{
		Describe("Copy Constructor", [this]()
		{
			Describe("when copying an empty structure", [this]()
			{
				It("should create an empty structure.", [this]()
				{
					FBlockStructure NewBlockStructure(BlockStructure);
					TEST_NULL(NewBlockStructure.GetHead());
					TEST_NULL(NewBlockStructure.GetTail());
				});
			});

			Describe("when some blocks already exist", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should create an equal structure.", [this]()
				{
					FBlockStructure NewBlockStructure(BlockStructure);
					TEST_EQUAL(ToArrayU64(BlockStructure), ToArrayU64(NewBlockStructure));
				});

				It("should not share memory.", [this]()
				{
					FBlockStructure NewBlockStructure(BlockStructure);
					TEST_TRUE(EnumeratePtrs(NewBlockStructure).Intersect(EnumeratePtrs(BlockStructure)).Num() == 0);
				});
			});
		});

		Describe("Move Constructor", [this]()
		{
			Describe("when currently an empty structure", [this]()
			{
				It("should create an empty structure.", [this]()
				{
					FBlockStructure NewBlockStructure(MoveTemp(BlockStructure));
					TEST_NULL(NewBlockStructure.GetHead());
					TEST_NULL(NewBlockStructure.GetTail());
				});

				It("should leave source as an empty structure.", [this]()
				{
					FBlockStructure NewBlockStructure(MoveTemp(BlockStructure));
					TEST_NULL(BlockStructure.GetHead());
					TEST_NULL(BlockStructure.GetTail());
				});
			});

			Describe("when some blocks already exist", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should create a structure with same data.", [this]()
				{
					const FBlockEntry* HeadPtr = BlockStructure.GetHead();
					const FBlockEntry* TailPtr = BlockStructure.GetTail();
					FBlockStructure NewBlockStructure(MoveTemp(BlockStructure));
					TEST_EQUAL(NewBlockStructure.GetHead(), HeadPtr);
					TEST_EQUAL(NewBlockStructure.GetTail(), TailPtr);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), SetupBlocks);
				});

				It("should leave source as an empty structure.", [this]()
				{
					FBlockStructure NewBlockStructure(MoveTemp(BlockStructure));
					TEST_NULL(BlockStructure.GetHead());
					TEST_NULL(BlockStructure.GetTail());
				});
			});
		});

		Describe("Copy Assignment Operator", [this]()
		{
			BeforeEach([this]()
			{
				OtherStructure.Add(5, 7);
				OtherStructure.Add(19, 12);
				OtherStructure.Add(42, 2);
			});

			Describe("when copying an empty structure", [this]()
			{
				It("should create an empty structure.", [this]()
				{
					OtherStructure = BlockStructure;
					TEST_NULL(OtherStructure.GetHead());
					TEST_NULL(OtherStructure.GetTail());
				});
			});

			Describe("when some blocks already exist", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should create an equal structure.", [this]()
				{
					OtherStructure = BlockStructure;
					TEST_EQUAL(ToArrayU64(OtherStructure), ToArrayU64(BlockStructure));
				});

				It("should not share memory.", [this]()
				{
					OtherStructure = BlockStructure;
					TEST_TRUE(EnumeratePtrs(OtherStructure).Intersect(EnumeratePtrs(BlockStructure)).Num() == 0);
				});
			});
		});

		Describe("Move Assignment Operator", [this]()
		{
			BeforeEach([this]()
			{
				OtherStructure.Add(5, 7);
				OtherStructure.Add(19, 12);
				OtherStructure.Add(42, 2);
			});

			Describe("when moving from an empty structure", [this]()
			{
				It("should become an empty structure.", [this]()
				{
					OtherStructure = MoveTemp(BlockStructure);
					TEST_NULL(OtherStructure.GetHead());
					TEST_NULL(OtherStructure.GetTail());
				});

				It("should leave source as an empty structure.", [this]()
				{
					OtherStructure = MoveTemp(BlockStructure);
					TEST_NULL(BlockStructure.GetHead());
					TEST_NULL(BlockStructure.GetTail());
				});
			});

			Describe("when moving from a non-empty structure", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should take memory ownership.", [this]()
				{
					const FBlockEntry* HeadPtr = BlockStructure.GetHead();
					const FBlockEntry* TailPtr = BlockStructure.GetTail();
					OtherStructure = MoveTemp(BlockStructure);
					TEST_EQUAL(OtherStructure.GetHead(), HeadPtr);
					TEST_EQUAL(OtherStructure.GetTail(), TailPtr);
				});

				It("should become a structure with same data.", [this]()
				{
					OtherStructure = MoveTemp(BlockStructure);
					TEST_EQUAL(ToArrayU64(OtherStructure), SetupBlocks);
				});

				It("should make source an empty structure.", [this]()
				{
					OtherStructure = MoveTemp(BlockStructure);
					TEST_NULL(BlockStructure.GetHead());
					TEST_NULL(BlockStructure.GetTail());
				});
			});
		});

		Describe("GetHead", [this]()
		{
			Describe("when currently an empty structure", [this]()
			{
				It("should return nullptr.", [this]()
				{
					const FBlockEntry* HeadPtr = BlockStructure.GetHead();
					TEST_NULL(HeadPtr);
				});
			});

			Describe("when some blocks already exist", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should return valid ptr.", [this]()
				{
					TEST_NOT_NULL(BlockStructure.GetHead());
				});

				It("should return the ptr to the head block.", [this]()
				{
					const FBlockEntry* HeadPtr = BlockStructure.GetHead();
					if (HeadPtr != nullptr)
					{
						TEST_EQUAL(ARRAYU64(HeadPtr->GetOffset(), HeadPtr->GetSize()), ArrayHead(SetupBlocks));
					}
				});
			});
		});

		Describe("GetTail", [this]()
		{
			Describe("when currently an empty structure", [this]()
			{
				It("should return nullptr.", [this]()
				{
					const FBlockEntry* TailPtr = BlockStructure.GetTail();
					TEST_NULL(TailPtr);
				});
			});

			Describe("when some blocks already exist", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should return valid ptr.", [this]()
				{
					TEST_NOT_NULL(BlockStructure.GetTail());
				});

				It("should return the ptr to the head block.", [this]()
				{
					const FBlockEntry* TailPtr = BlockStructure.GetTail();
					if (TailPtr != nullptr)
					{
						TEST_EQUAL(ARRAYU64(TailPtr->GetOffset(), TailPtr->GetSize()), ArrayTail(SetupBlocks));
					}
				});
			});
		});

		Describe("Empty", [this]()
		{
			Describe("when currently an empty structure", [this]()
			{
				It("should stay as an empty structure.", [this]()
				{
					BlockStructure.Empty();
					TEST_NULL(BlockStructure.GetHead());
					TEST_NULL(BlockStructure.GetTail());
				});
			});

			Describe("when some blocks already exist", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should become an empty structure.", [this]()
				{
					BlockStructure.Empty();
					TEST_NULL(BlockStructure.GetHead());
					TEST_NULL(BlockStructure.GetTail());
				});
			});
		});

		Describe("Add", [this]()
		{
			for (int32 DirIdx = 0; DirIdx < DirTypes.Num(); ++DirIdx)
			{
				ESearchDir::Type Dir = DirTypes[DirIdx];
				Describe(FString::Printf(TEXT("when called with %s"), *DirNames[DirIdx]), [this, Dir]()
				{
					Describe("when currently an empty structure", [this, Dir]()
					{
						Describe("when adding a single block", [this, Dir]()
						{
							It("should result in containing the single block.", [this, Dir]()
							{
								BlockStructure.Add(70, 10, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(70, 10));
							});

							It("should stay empty when adding zero.", [this, Dir]()
							{
								BlockStructure.Add(70, 0, Dir);
								TEST_NULL(BlockStructure.GetHead());
								TEST_NULL(BlockStructure.GetTail());
							});
						});

						Describe("when adding multiple blocks", [this, Dir]()
						{
							It("should result in containing each of the blocks.", [this, Dir]()
							{
								for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
								{
									BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1], Dir);
								}
								TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
							});

							It("should combine right adjacent blocks.", [this, Dir]()
							{
								BlockStructure.Add(10, 7, Dir);
								BlockStructure.Add(17, 5, Dir);
								BlockStructure.Add(22, 9, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(10, 21));
							});

							It("should combine left adjacent blocks.", [this, Dir]()
							{
								BlockStructure.Add(22, 9, Dir);
								BlockStructure.Add(17, 5, Dir);
								BlockStructure.Add(10, 7, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(10, 21));
							});
						});

						Describe("when adding another structure", [this, Dir]()
						{
							BeforeEach([this, Dir]()
							{
								for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
								{
									OtherStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1], Dir);
								}
							});

							It("should result in the same structure.", [this, Dir]()
							{
								BlockStructure.Add(OtherStructure, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), ToArrayU64(OtherStructure));
							});
						});
					});

					Describe("when some blocks already exist", [this, Dir]()
					{
						BeforeEach([this, Dir]()
						{
							for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
							{
								BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1], Dir);
							}
						});

						It("should ignore zero size block left of head.", [this, Dir]()
						{
							BlockStructure.Add(0, 0, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should ignore zero size block right of tail.", [this, Dir]()
						{
							BlockStructure.Add(100, 0, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should ignore zero size block in empty space.", [this, Dir]()
						{
							BlockStructure.Add(28, 0, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should insert a new head.", [this, Dir]()
						{
							BlockStructure.Add(0, 6, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(0, 6) + SetupBlocks);
						});

						It("should grow left an existing head with adjacent block.", [this, Dir]()
						{
							BlockStructure.Add(8, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(8, 7)));
						});

						It("should grow left an existing head with partial overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(8, 4, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(8, 7)));
						});

						It("should grow left an existing head with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(8, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(8, 7)));
						});

						It("should grow right an existing head with adjacent block.", [this, Dir]()
						{
							BlockStructure.Add(15, 4, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(10, 9)));
						});

						It("should grow right an existing head with partial overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(12, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(10, 9)));
						});

						It("should grow right an existing head with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(10, 9, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(10, 9)));
						});

						It("should grow outwards an existing head with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(8, 9, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(8, 9)));
						});

						It("should combine an existing head with second block when exactly filling the gap.", [this, Dir]()
						{
							BlockStructure.Add(15, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(10, 17) + ArrayChopLeft(SetupBlocks, 4));
						});

						It("should combine an existing head with second block when overlapping the gap.", [this, Dir]()
						{
							BlockStructure.Add(13, 9, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(10, 17) + ArrayChopLeft(SetupBlocks, 4));
						});

						It("should swallow a block enclosed in existing head from left edge to inside.", [this, Dir]()
						{
							BlockStructure.Add(10, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block enclosed in existing head from inside to right edge.", [this, Dir]()
						{
							BlockStructure.Add(12, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block fully enclosed in existing head.", [this, Dir]()
						{
							BlockStructure.Add(11, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block perfectly matching an existing head.", [this, Dir]()
						{
							BlockStructure.Add(10, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should insert a new tail.", [this, Dir]()
						{
							BlockStructure.Add(50, 6, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks + ARRAYU64(50, 6));
						});

						It("should grow left an existing tail with adjacent block.", [this, Dir]()
						{
							BlockStructure.Add(38, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(38, 7)));
						});

						It("should grow left an existing tail with partial overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(38, 4, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(38, 7)));
						});

						It("should grow left an existing tail with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(38, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(38, 7)));
						});

						It("should grow right an existing tail with adjacent block.", [this, Dir]()
						{
							BlockStructure.Add(45, 4, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(40, 9)));
						});

						It("should grow right an existing tail with partial overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(42, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(40, 9)));
						});

						It("should grow right an existing tail with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(40, 9, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(40, 9)));
						});

						It("should grow outwards an existing tail with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(38, 9, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(38, 9)));
						});

						It("should combine an existing tail with second last block when exactly filling the gap.", [this, Dir]()
						{
							BlockStructure.Add(33, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayChopRight(SetupBlocks, 4) + ARRAYU64(30, 15));
						});

						It("should combine an existing tail with second last block when overlapping the gap.", [this, Dir]()
						{
							BlockStructure.Add(31, 11, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayChopRight(SetupBlocks, 4) + ARRAYU64(30, 15));
						});

						It("should swallow a block enclosed in existing tail from left edge to inside.", [this, Dir]()
						{
							BlockStructure.Add(40, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block enclosed in existing tail from inside to right edge.", [this, Dir]()
						{
							BlockStructure.Add(42, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block fully enclosed in existing tail.", [this, Dir]()
						{
							BlockStructure.Add(41, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block perfectly matching an existing tail.", [this, Dir]()
						{
							BlockStructure.Add(40, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should insert a new middle block.", [this, Dir]()
						{
							BlockStructure.Add(16, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayHead(SetupBlocks) + ARRAYU64(16, 2) + ArrayChopLeft(SetupBlocks, 2));
						});

						It("should grow left an existing middle block with adjacent block.", [this, Dir]()
						{
							BlockStructure.Add(18, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(18, 9)));
						});

						It("should grow left an existing middle block with partial overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(18, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(18, 9)));
						});

						It("should grow left an existing middle block with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(18, 9, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(18, 9)));
						});

						It("should grow right an existing middle block with adjacent block.", [this, Dir]()
						{
							BlockStructure.Add(27, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(20, 9)));
						});

						It("should grow right an existing middle block with partial overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(26, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(20, 9)));
						});

						It("should grow right an existing middle block with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(20, 9, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(20, 9)));
						});

						It("should grow outwards an existing middle block with fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Add(18, 11, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(18, 11)));
						});

						It("should combine existing blocks when exactly filling the gap.", [this, Dir]()
						{
							BlockStructure.Add(27, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayHead(SetupBlocks) + ARRAYU64(20, 13) + ArrayTail(SetupBlocks));
						});

						It("should combine existing blocks when overlapping the gap.", [this, Dir]()
						{
							BlockStructure.Add(26, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayHead(SetupBlocks) + ARRAYU64(20, 13) + ArrayTail(SetupBlocks));
						});

						It("should swallow a block enclosed in existing middle block from left edge to inside.", [this, Dir]()
						{
							BlockStructure.Add(20, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block enclosed in existing middle block from inside to right edge.", [this, Dir]()
						{
							BlockStructure.Add(24, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block fully enclosed in existing middle block.", [this, Dir]()
						{
							BlockStructure.Add(21, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should swallow a block perfectly matching an existing middle block.", [this, Dir]()
						{
							BlockStructure.Add(20, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should combine blocks when left of head up to edge of middle block.", [this, Dir]()
						{
							BlockStructure.Add(5, 25, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(5, 28) + ArrayTail(SetupBlocks));
						});

						It("should combine blocks when start of head up to edge of middle block.", [this, Dir]()
						{
							BlockStructure.Add(10, 20, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(10, 23) + ArrayTail(SetupBlocks));
						});

						It("should combine blocks when inside of head up to edge of middle block.", [this, Dir]()
						{
							BlockStructure.Add(12, 18, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(10, 23) + ArrayTail(SetupBlocks));
						});

						It("should combine blocks when left of second block up to right of second last block.", [this, Dir]()
						{
							BlockStructure.Add(16, 23, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayHead(SetupBlocks) + ARRAYU64(16, 23) + ArrayTail(SetupBlocks));
						});

						It("should combine blocks when start of second block up to end of second last block.", [this, Dir]()
						{
							BlockStructure.Add(20, 13, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayHead(SetupBlocks) + ARRAYU64(20, 13) + ArrayTail(SetupBlocks));
						});

						It("should combine blocks when inside of second block up to inside of second last block.", [this, Dir]()
						{
							BlockStructure.Add(22, 10, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayHead(SetupBlocks) + ARRAYU64(20, 13) + ArrayTail(SetupBlocks));
						});

						It("should combine blocks when end of second block up to start of second last block.", [this, Dir]()
						{
							BlockStructure.Add(27, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayHead(SetupBlocks) + ARRAYU64(20, 13) + ArrayTail(SetupBlocks));
						});

						It("should combine blocks when overlapping whole structure.", [this, Dir]()
						{
							BlockStructure.Add(5, 45, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(5, 45));
						});

						Describe("when adding another structure the same", [this, Dir]()
						{
							BeforeEach([this, Dir]()
							{
								for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
								{
									OtherStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1], Dir);
								}
							});

							It("should result in the same structure.", [this, Dir]()
							{
								BlockStructure.Add(OtherStructure, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
							});
						});

						Describe("when adding a different structure with no overlap", [this, Dir]()
						{
							BeforeEach([this, Dir]()
							{
								OtherStructure.Add(5, 3, Dir);
								OtherStructure.Add(16, 3, Dir);
								OtherStructure.Add(28, 1, Dir);
								OtherStructure.Add(35, 4, Dir);
								OtherStructure.Add(50, 6, Dir);
							});

							It("should result in the combined structure.", [this, Dir]()
							{
								BlockStructure.Add(OtherStructure, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(5, 3, 10, 5, 16, 3, 20, 7, 28, 1, 30, 3, 35, 4, 40, 5, 50, 6));
							});
						});

						Describe("when adding a different structure with some overlap", [this, Dir]()
						{
							BeforeEach([this, Dir]()
							{
								OtherStructure.Add(5, 5, Dir);
								OtherStructure.Add(19, 9, Dir);
								OtherStructure.Add(33, 7, Dir);
							});

							It("should result in the combined structure.", [this, Dir]()
							{
								BlockStructure.Add(OtherStructure, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(5, 10, 19, 9, 30, 15));
							});
						});
					});
				});
			}
		});

		Describe("Remove", [this]()
		{
			for (int32 DirIdx = 0; DirIdx < DirTypes.Num(); ++DirIdx)
			{
				ESearchDir::Type Dir = DirTypes[DirIdx];
				Describe(FString::Printf(TEXT("when called with %s"), *DirNames[DirIdx]), [this, Dir]()
				{
					Describe("when currently an empty structure", [this, Dir]()
					{
						Describe("when removing a block", [this, Dir]()
						{
							It("should result in empty structure.", [this, Dir]()
							{
								BlockStructure.Remove(70, 0, Dir);
								TEST_NULL(BlockStructure.GetHead());
								TEST_NULL(BlockStructure.GetTail());
							});
						});

						Describe("when removing another structure", [this, Dir]()
						{
							BeforeEach([this, Dir]()
							{
								for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
								{
									OtherStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1], Dir);
								}
							});

							It("should result in empty structure.", [this, Dir]()
							{
								BlockStructure.Remove(OtherStructure, Dir);
								TEST_NULL(BlockStructure.GetHead());
								TEST_NULL(BlockStructure.GetTail());
							});
						});
					});

					Describe("when some blocks already exist", [this, Dir]()
					{
						BeforeEach([this, Dir]()
						{
							for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
							{
								BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1], Dir);
							}
						});

						It("should ignore zero size block on head.", [this, Dir]()
						{
							BlockStructure.Remove(11, 0, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should ignore zero size block on tail.", [this, Dir]()
						{
							BlockStructure.Remove(41, 0, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should ignore zero size block on middle block.", [this, Dir]()
						{
							BlockStructure.Remove(21, 0, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should ignore block before head.", [this, Dir]()
						{
							BlockStructure.Remove(5, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should ignore block after tail.", [this, Dir]()
						{
							BlockStructure.Remove(45, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should ignore block inside gap.", [this, Dir]()
						{
							BlockStructure.Remove(27, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
						});

						It("should remove head when exact match.", [this, Dir]()
						{
							BlockStructure.Remove(10, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayChopLeft(SetupBlocks, 2));
						});

						It("should remove head when overlapping.", [this, Dir]()
						{
							BlockStructure.Remove(9, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayChopLeft(SetupBlocks, 2));
						});

						It("should shrink start of head.", [this, Dir]()
						{
							BlockStructure.Remove(10, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(12, 3)));
						});

						It("should shrink end of head.", [this, Dir]()
						{
							BlockStructure.Remove(13, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 0, ARRAYU64(10, 3)));
						});

						It("should split head.", [this, Dir]()
						{
							BlockStructure.Remove(12, 1, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(10, 2) + ArrayClobber(SetupBlocks, 0, ARRAYU64(13, 2)));
						});

						It("should remove tail when exact match.", [this, Dir]()
						{
							BlockStructure.Remove(40, 5, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayChopRight(SetupBlocks, 2));
						});

						It("should remove tail when overlapping.", [this, Dir]()
						{
							BlockStructure.Remove(39, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayChopRight(SetupBlocks, 2));
						});

						It("should shrink start of tail.", [this, Dir]()
						{
							BlockStructure.Remove(40, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(42, 3)));
						});

						It("should shrink end of tail.", [this, Dir]()
						{
							BlockStructure.Remove(43, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(40, 3)));
						});

						It("should split tail.", [this, Dir]()
						{
							BlockStructure.Remove(42, 1, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, SetupBlocks.Num() - 2, ARRAYU64(40, 2)) + ARRAYU64(43, 2));
						});

						It("should remove block when exact match.", [this, Dir]()
						{
							BlockStructure.Remove(20, 7, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayLeft(SetupBlocks, 2) + ArrayRight(SetupBlocks, 4));
						});

						It("should remove block when overlapping.", [this, Dir]()
						{
							BlockStructure.Remove(19, 9, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayLeft(SetupBlocks, 2) + ArrayRight(SetupBlocks, 4));
						});

						It("should shrink start of block.", [this, Dir]()
						{
							BlockStructure.Remove(20, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(22, 5)));
						});

						It("should shrink end of block.", [this, Dir]()
						{
							BlockStructure.Remove(25, 2, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayClobber(SetupBlocks, 2, ARRAYU64(20, 5)));
						});

						It("should split block.", [this, Dir]()
						{
							BlockStructure.Remove(22, 3, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayLeft(SetupBlocks, 2) + ARRAYU64(20, 2, 25, 2) + ArrayRight(SetupBlocks, 4));
						});

						It("should remove all blocks with exact overlap.", [this, Dir]()
						{
							BlockStructure.Remove(10, 35, Dir);
							TEST_NULL(BlockStructure.GetHead());
							TEST_NULL(BlockStructure.GetTail());
						});

						It("should remove all blocks with extra overlap.", [this, Dir]()
						{
							BlockStructure.Remove(0, 100, Dir);
							TEST_NULL(BlockStructure.GetHead());
							TEST_NULL(BlockStructure.GetTail());
						});

						It("should shrink semi overlapped head and tail, removing fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Remove(12, 31, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(10, 2, 43, 2));
						});

						It("should shrink semi overlapped blocks, removing fully overlapped block.", [this, Dir]()
						{
							BlockStructure.Remove(21, 11, Dir);
							TEST_EQUAL(ToArrayU64(BlockStructure), ArrayHead(SetupBlocks) + ARRAYU64(20, 1, 32, 1) + ArrayTail(SetupBlocks));
						});

						Describe("when removing another structure the same", [this, Dir]()
						{
							BeforeEach([this, Dir]()
							{
								for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
								{
									OtherStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1], Dir);
								}
							});

							It("should result in an empty structure.", [this, Dir]()
							{
								BlockStructure.Remove(OtherStructure, Dir);
								TEST_NULL(BlockStructure.GetHead());
								TEST_NULL(BlockStructure.GetTail());
							});
						});

						Describe("when removing a different structure with no overlap", [this, Dir]()
						{
							BeforeEach([this, Dir]()
							{
								OtherStructure.Add(5, 3, Dir);
								OtherStructure.Add(16, 3, Dir);
								OtherStructure.Add(28, 1, Dir);
								OtherStructure.Add(35, 4, Dir);
								OtherStructure.Add(50, 6, Dir);
							});

							It("should result in the original structure.", [this, Dir]()
							{
								BlockStructure.Remove(OtherStructure, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), SetupBlocks);
							});
						});

						Describe("when removing a different structure with some overlap", [this, Dir]()
						{
							BeforeEach([this, Dir]()
							{
								OtherStructure.Add(5, 7, Dir);
								OtherStructure.Add(19, 12, Dir);
								OtherStructure.Add(42, 2, Dir);
							});

							It("should result in the chopped structure.", [this, Dir]()
							{
								BlockStructure.Remove(OtherStructure, Dir);
								TEST_EQUAL(ToArrayU64(BlockStructure), ARRAYU64(12, 3, 31, 2, 40, 2, 44, 1));
							});
						});
					});
				});
			}
		});

		Describe("SelectSerialBytes", [this]()
		{
			Describe("when the structure is empty", [this]()
			{
				It("should return 0.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(0, 100, OtherStructure), 0);
				});

				It("should not effect OutputStructure.", [this]()
				{
					BlockStructure.SelectSerialBytes(0, 100, OtherStructure);
					TEST_NULL(OtherStructure.GetHead());
					TEST_NULL(OtherStructure.GetTail());
				});
			});

			Describe("when the structure has enough bytes", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should supply exactly selected entire structure.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(0, 20, OtherStructure), 20);
					TEST_EQUAL(ToArrayU64(OtherStructure), SetupBlocks);
				});

				It("should supply exactly selected serial blocks.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(5, 10, OtherStructure), 10);
					TEST_EQUAL(ToArrayU64(OtherStructure), ARRAYU64(20, 7, 30, 3));
				});

				It("should supply partially selected serial blocks, shrinking head and tail.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(2, 15, OtherStructure), 15);
					TEST_EQUAL(ToArrayU64(OtherStructure), ARRAYU64(12, 3, 20, 7, 30, 3, 40, 2));
				});

				It("should supply partially selected serial blocks, shrinking two blocks.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(7, 6, OtherStructure), 6);
					TEST_EQUAL(ToArrayU64(OtherStructure), ARRAYU64(22, 5, 30, 1));
				});

				It("should supply single portion of one block.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(6, 5, OtherStructure), 5);
					TEST_EQUAL(ToArrayU64(OtherStructure), ARRAYU64(21, 5));
				});

				It("should supply partially selected serial blocks, skipping head by adjacent index.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(5, 15, OtherStructure), 15);
					TEST_EQUAL(ToArrayU64(OtherStructure), ARRAYU64(20, 7, 30, 3, 40, 5));
				});

				It("should supply empty structure for selecting nothing, with intersecting index.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(5, 0, OtherStructure), 0);
					TEST_NULL(OtherStructure.GetHead());
					TEST_NULL(OtherStructure.GetTail());
				});

				It("should supply empty structure for selecting nothing, with blank index.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(100, 0, OtherStructure), 0);
					TEST_NULL(OtherStructure.GetHead());
					TEST_NULL(OtherStructure.GetTail());
				});
			});

			Describe("when the structure has less bytes than requested", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should supply entire structure.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(0, 1000, OtherStructure), 20);
					TEST_EQUAL(ToArrayU64(OtherStructure), SetupBlocks);
				});

				It("should supply last part of structure, with intersecting index.", [this]()
				{
					TEST_EQUAL(BlockStructure.SelectSerialBytes(6, 1000, OtherStructure), 14);
					TEST_EQUAL(ToArrayU64(OtherStructure), ARRAYU64(21, 6, 30, 3, 40, 5));
				});
			});
		});

		Describe("Intersect", [this]()
		{
			BeforeEach([this]()
			{
				for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
				{
					BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
				}
			});

			Describe("when the given an empty structure", [this]()
			{
				It("should return an empty structure.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_NULL(NewBlockStructure.GetHead());
					TEST_NULL(NewBlockStructure.GetTail());
				});
			});

			Describe("when given the same structure", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						OtherStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should return a structure of the same blocks.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), SetupBlocks);
				});
			});

			Describe("when given an inverted structure", [this]()
			{
				BeforeEach([this]()
				{
					OtherStructure.Add(0, 50);
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						OtherStructure.Remove(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should return an empty structure.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_NULL(NewBlockStructure.GetHead());
					TEST_NULL(NewBlockStructure.GetTail());
				});
			});

			Describe("when given an fully overlapping structure", [this]()
			{
				BeforeEach([this]()
				{
					OtherStructure.Add(0, 50);
				});

				It("should return a structure of the same blocks.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), SetupBlocks);
				});
			});

			Describe("when given a structure with just the same head", [this]()
			{
				BeforeEach([this]()
				{
					OtherStructure.Add(ArrayHead(SetupBlocks)[0], ArrayHead(SetupBlocks)[1]);
				});

				It("should return a structure with just the same head.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), ArrayHead(SetupBlocks));
				});
			});

			Describe("when given a structure with just the same tail", [this]()
			{
				BeforeEach([this]()
				{
					OtherStructure.Add(ArrayTail(SetupBlocks)[0], ArrayTail(SetupBlocks)[1]);
				});

				It("should return a structure with just the same tail.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), ArrayTail(SetupBlocks));
				});
			});

			Describe("when given a structure with a single matching block", [this]()
			{
				BeforeEach([this]()
				{
					OtherStructure.Add(30, 3);
				});

				It("should return a structure with the single matching block.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), ARRAYU64(30, 3));
				});
			});

			Describe("when given a structure with every block slightly shrunk", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						OtherStructure.Add(SetupBlocks[Idx] + 1, SetupBlocks[Idx + 1] - 2);
					}
				});

				It("should return a structure with every block slightly shrunk.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), ARRAYU64(11, 3, 21, 5, 31, 1, 41, 3));
				});
			});

			Describe("when given a structure with every block slightly grown", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						OtherStructure.Add(SetupBlocks[Idx] - 1, SetupBlocks[Idx + 1] + 2);
					}
				});

				It("should return a structure with every original block.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), SetupBlocks);
				});
			});

			Describe("when given a structure with start and end overlaps for all blocks", [this]()
			{
				BeforeEach([this]()
				{
					OtherStructure.Add(0, 50);
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						OtherStructure.Remove(SetupBlocks[Idx] + 1, SetupBlocks[Idx + 1] - 2);
					}
				});

				It("should return a structure with just the start and end bytes.", [this]()
				{
					FBlockStructure NewBlockStructure = BlockStructure.Intersect(OtherStructure);
					TEST_EQUAL(ToArrayU64(NewBlockStructure), ARRAYU64(10, 1, 14, 1, 20, 1, 26, 1, 30, 1, 32, 1, 40, 1, 44, 1));
				});
			});
		});

		Describe("ToString", [this]()
		{
			Describe("when the structure is empty", [this]()
			{
				It("should return empty string.", [this]()
				{
					TEST_EQUAL(BlockStructure.ToString(), TEXT(""));
				});
			});

			Describe("when the structure has blocks", [this]()
			{
				BeforeEach([this]()
				{
					for (int32 Idx = 0; Idx < SetupBlocks.Num(); Idx += 2)
					{
						BlockStructure.Add(SetupBlocks[Idx], SetupBlocks[Idx + 1]);
					}
				});

				It("should return full string representation.", [this]()
				{
					TEST_EQUAL(BlockStructure.ToString(), TEXT("[10,5]-[20,7]-[30,3]-[40,5]."));
				});

				It("should return clamped string representation.", [this]()
				{
					TEST_EQUAL(BlockStructure.ToString(2), TEXT("[10,5]-[20,7].. 2 more."));
				});
			});
		});
	});
}

bool FBlockStructureSpec::IsLooping(const BuildPatchServices::FBlockStructure& Structure)
{
	const BuildPatchServices::FBlockEntry* Slow = Structure.GetHead();
	const BuildPatchServices::FBlockEntry* Fast = Structure.GetHead();
	while (Slow != nullptr && Fast != nullptr && Fast->GetNext() != nullptr)
	{
		Slow = Slow->GetNext();
		Fast = Fast->GetNext()->GetNext();
		bool bIsLooping = Slow == Fast;
		TEST_FALSE(bIsLooping);
		if (bIsLooping)
		{
			return true;
		}
	}
	Slow = Structure.GetTail();
	Fast = Structure.GetTail();
	while (Slow != nullptr && Fast != nullptr && Fast->GetPrevious() != nullptr)
	{
		Slow = Slow->GetPrevious();
		Fast = Fast->GetPrevious()->GetPrevious();
		bool bIsLooping = Slow == Fast;
		TEST_FALSE(bIsLooping);
		if (bIsLooping)
		{
			return true;
		}
	}
	return false;
}

bool FBlockStructureSpec::IsMirrored(const BuildPatchServices::FBlockStructure& Structure)
{
	bool bMirrored = false;
	if (!IsLooping(Structure))
	{
		TArray<const BuildPatchServices::FBlockEntry*> Forward;
		const BuildPatchServices::FBlockEntry* Block = Structure.GetHead();
		while (Block != nullptr)
		{
			Forward.Add(Block);
			Block = Block->GetNext();
		}
		TArray<const BuildPatchServices::FBlockEntry*> Backward;
		Block = Structure.GetTail();
		while (Block != nullptr)
		{
			Backward.Add(Block);
			Block = Block->GetPrevious();
		}
		Algo::Reverse(Backward);
		bMirrored = Forward == Backward;
	}
	TEST_TRUE(bMirrored);
	return bMirrored;
}

TArray<uint64> FBlockStructureSpec::ToArrayU64(const BuildPatchServices::FBlockStructure& Structure)
{
	TArray<uint64> Result;
	if (!IsLooping(Structure) && IsMirrored(Structure))
	{
		const BuildPatchServices::FBlockEntry* Block = Structure.GetHead();
		while (Block != nullptr)
		{
			Result.Add(Block->GetOffset());
			Result.Add(Block->GetSize());
			Block = Block->GetNext();
		}
	}
	return Result;
}

TSet<const BuildPatchServices::FBlockEntry*> FBlockStructureSpec::EnumeratePtrs(const BuildPatchServices::FBlockStructure& Structure)
{
	TSet<const BuildPatchServices::FBlockEntry*> Result;
	if (!IsLooping(Structure) && IsMirrored(Structure))
	{
		const BuildPatchServices::FBlockEntry* Block = Structure.GetHead();
		while (Block != nullptr)
		{
			Result.Add(Block);
			Block = Block->GetNext();
		}
	}
	return Result;
}

#endif //WITH_DEV_AUTOMATION_TESTS
