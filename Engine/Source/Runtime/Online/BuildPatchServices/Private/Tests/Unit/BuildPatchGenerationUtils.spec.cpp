// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Generation/FileAttributesParser.h"
#include "Generation/RegexFind.h"
#include "Misc/AutomationTest.h"
#include "Tests/TestHelpers.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FBuildPatchGenerationUtilsSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags::ProgramContext)
END_DEFINE_SPEC(FBuildPatchGenerationUtilsSpec)

namespace PatchGenerationHelpers
{
	void StripIgnoredFiles(const TSet<FString>& IgnoreFiles, const TSet<FString>& IgnorePatterns, TArray<FString>& AllFiles);
	void ReadIgnoreFileList(const FString& IgnoreFile, const FString& BuildRoot, TSet<FString>& IgnoreFiles, TSet<FString>& IgnorePatterns);
}

namespace
{
	struct TempFile {
		TempFile(const char** Attributes, int32 Size)
		{
			TUniquePtr<IFileHandle> Handle{ PlatformFile.OpenWrite(*Filename) };
			for (int32 i = 0; i < Size; i++)
			{
				Handle->Write((uint8*)Attributes[i], strlen(Attributes[i]));
			}
		}
		~TempFile() {
			PlatformFile.DeleteFile(*Filename);
		}
		FString Filename = FPaths::CreateTempFilename(
			FPlatformProcess::UserTempDir(), TEXT("attribute-unit-tests-"), TEXT(".txt"));
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	};
}

void FBuildPatchGenerationUtilsSpec::Define()
{
	auto CheckIfDebugging = [](bool bSucceeded) {
		if (FPlatformMisc::IsDebuggerPresent())
		{
			check(bSucceeded);
		}
	};

	using namespace BuildPatchServices;

	Describe("TMatchingType", [&]()
		{
			Describe("when using ExtractPattern", [&]()
				{
					It("should extract pattern.", [&]()
						{							
							CheckIfDebugging(TEST_EQUAL(FRegexPatternType::ExtractPattern(TEXT("R\"mypattern\"")), TEXT("mypattern")));
							CheckIfDebugging(TEST_EQUAL(FWildcardPatternType::ExtractPattern(TEXT("W\"mypattern\"")), TEXT("mypattern")));
							CheckIfDebugging(TEST_EQUAL(FRegexPatternType::ExtractPattern(TEXT("R\"^[a-z]+\\W+$\"")), TEXT("^[a-z]+\\W+$")));
							CheckIfDebugging(TEST_EQUAL(FWildcardPatternType::ExtractPattern(TEXT("W\"\\d+?.*{3,}\"")), TEXT("\\d+?.*{3,}")));
						});
				});
			Describe("when using IsPattern", [&]()
				{
					It("should find pattern.", [&]()
						{
							CheckIfDebugging(TEST_TRUE(FRegexPatternType::IsPattern(TEXT("R\"mypattern\""))));
							CheckIfDebugging(TEST_TRUE(FWildcardPatternType::IsPattern(TEXT("W\"mypattern\""))));
						});
					It("should not find pattern.", [&]()
						{
							CheckIfDebugging(TEST_FALSE(FRegexPatternType::IsPattern(TEXT("Rmypattern\""))));
							CheckIfDebugging(TEST_FALSE(FRegexPatternType::IsPattern(TEXT("R\"mypattern"))));
							CheckIfDebugging(TEST_FALSE(FRegexPatternType::IsPattern(TEXT("Rmypattern\\"))));
							CheckIfDebugging(TEST_FALSE(FRegexPatternType::IsPattern(TEXT("X\"mypattern\""))));
							CheckIfDebugging(TEST_FALSE(FWildcardPatternType::IsPattern(TEXT("Wmypattern\""))));
							CheckIfDebugging(TEST_FALSE(FWildcardPatternType::IsPattern(TEXT("W\"mypattern"))));
						});
				});
		});

	Describe("FRegexFinder", [&]()
		{
			Describe("when using ExactMatch", [&]()
				{
#define REGEX_TEST_TRUE(Pattern, Value) CheckIfDebugging(TEST_TRUE(FRegexFinder::Match(Pattern, Value)))
#define REGEX_TEST_FALSE(Pattern, Value) CheckIfDebugging(TEST_FALSE(FRegexFinder::Match(Pattern, Value)))

					It("should match.", [&]()
						{
							// regex
							REGEX_TEST_TRUE(TEXT("R\"^bus.png$\""), TEXT("bus.png"));
							REGEX_TEST_TRUE(TEXT("R\"bus.png\""), TEXT("busApng")); // . - is any symbol
							REGEX_TEST_TRUE(TEXT("R\".*\""), TEXT("bus.png"));
							REGEX_TEST_TRUE(TEXT("R\"bus.pn.\""), TEXT("bus.png"));
							REGEX_TEST_TRUE(TEXT("R\".*.png\""), TEXT("bus.png"));						
							REGEX_TEST_TRUE(TEXT("R\".*bus.png\""), TEXT("/foo/bus.png"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/.*.png\""), TEXT("/foo/bus.png"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/.*\""), TEXT("/foo/bus.png"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/[0-9].txt\""), TEXT("/foo/0.txt"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/[0-9]+.txt\""), TEXT("/foo/0222.txt"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/\\d+.txt\""), TEXT("/foo/0222.txt"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/\\w+.txt\""), TEXT("/foo/0.txt"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/\\d.txt\""), TEXT("/foo/0.txt"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/\\w+.txt\""), TEXT("/foo/absc.txt"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/a{3,}.txt\""), TEXT("/foo/aaa.txt"));
							REGEX_TEST_TRUE(TEXT("R\"bus.*.png\""), TEXT("/foo/bus.png"));
							REGEX_TEST_TRUE(TEXT("R\"/foo/bar/my.txt\""), TEXT("/foo/bar/my.txt.o"));
							REGEX_TEST_TRUE(TEXT("R\"foo/bar/my.txt\""), TEXT("/foo/bar/my.txt"));
							// wildcard
							REGEX_TEST_TRUE(TEXT("W\"*\""), TEXT("bus.png"));
							REGEX_TEST_TRUE(TEXT("W\"*.png\""), TEXT("bus.png"));
							REGEX_TEST_TRUE(TEXT("W\"bus.pn*\""), TEXT("bus.png"));
							REGEX_TEST_TRUE(TEXT("W\"*bus.png\""), TEXT("/foo/bus.png"));
							REGEX_TEST_TRUE(TEXT("W\"/*/bus.png\""), TEXT("/foo/bus.png"));
							REGEX_TEST_TRUE(TEXT("W\"/foo/*.png\""), TEXT("/foo/bus.png"));
							REGEX_TEST_TRUE(TEXT("W\"/foo/*\""), TEXT("/foo/bus.png"));
							REGEX_TEST_TRUE(TEXT("W\"/foo/fa*/*png\""), TEXT("/foo/fake/bus.png"));
							REGEX_TEST_TRUE(TEXT("W\"$*.png\""), TEXT("$my.png"));
							REGEX_TEST_TRUE(TEXT("W\"$*.png\""), TEXT("$myasdasdasdasdsa.png"));
							REGEX_TEST_TRUE(TEXT("W\"/foo/$*\""), TEXT("/foo/$myasdasdasdasdsa.png2"));
						});
					
					It("should not match.", [&]()
						{
							REGEX_TEST_FALSE(TEXT("bus.png"), TEXT("bus.png"));
							REGEX_TEST_FALSE(TEXT("BuS-?*12_?()#@&*$^@#*&^%!).png"), TEXT("bUs-?*12_?()#@&*$^@#*&^%!).png"));
							REGEX_TEST_FALSE(TEXT("$bus.png"), TEXT("$bus.png"));

							REGEX_TEST_FALSE(TEXT("R\"^bus.png"), TEXT("bus.png2"));
							REGEX_TEST_FALSE(TEXT("2bus.png"), TEXT("bus.png"));
							REGEX_TEST_FALSE(TEXT("bus2.png"), TEXT("bus.png"));
							REGEX_TEST_FALSE(TEXT("bus.png2"), TEXT("bus.png"));
							REGEX_TEST_FALSE(TEXT("bus.png"), TEXT("mybus.png"));
							REGEX_TEST_FALSE(TEXT("bus.png"), TEXT("busmy.png"));
							REGEX_TEST_FALSE(TEXT("bus.png"), TEXT("bus.pngmy"));
							REGEX_TEST_FALSE(TEXT("bus.png"), TEXT("busApngz"));
							REGEX_TEST_FALSE(TEXT("bus.pn1*"), TEXT("mybus.png"));
							REGEX_TEST_FALSE(TEXT("bus.png"), TEXT("/foo/bus.png"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/[0-9].txt\""), TEXT("/foo/01.txt"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/fake/*\""), TEXT("/foo/bus.png"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/a{3,}.txt\""), TEXT("/foo/aaabsc.txt"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/\\d+.txt\""), TEXT("/foo/a0.txt"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/[0-9a-y].txt\""), TEXT("/foo/zzzz.txt"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/bar/my.txt\""), TEXT("my.txt"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/bar/my.txt\""), TEXT("/bar/my.txt"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/bar/my.txt\""), TEXT("bar/my.txt"));
							REGEX_TEST_FALSE(TEXT("R\"$.*.png\""), TEXT("$myasdasdasdasdsa.png2"));
							REGEX_TEST_FALSE(TEXT("R\"/foo/$.*\""), TEXT("/foo/$myasdasdasdasdsa.png2"));
						});

#undef REGEX_TEST_TRUE
#undef REGEX_TEST_FALSE
				});	
		});

	Describe("PatchGenerationHelpers", [&]()
		{
			Describe("when using ReadIgnoreFileList", [&]()
				{
					It("should find patterns and files.", [&]()
						{
							static const char* IgnoreList[] = {
								"mybus.png\n"
								"R\"foo/\\w+.png\"\n",
								"W\"*foo/bar*\"\n"
							};
							TSet<FString> IgnoreFiles, IgnorePatterns;
							int32 Size = sizeof(IgnoreList) / sizeof(IgnoreList[0]);
							TempFile File(IgnoreList, Size);
							PatchGenerationHelpers::ReadIgnoreFileList(File.Filename, TEXT("/root"), IgnoreFiles, IgnorePatterns);
							CheckIfDebugging(TEST_SIZE(IgnoreFiles, 1));
							CheckIfDebugging(TEST_SIZE(IgnorePatterns, 2));
						});
				});
			Describe("when using StripIgnoredFiles", [&]()
				{
					It("should work ok with empty arrays.", [&]()
						{
							TArray<FString> AllFiles;
							PatchGenerationHelpers::StripIgnoredFiles({}, {}, AllFiles);
							CheckIfDebugging(TEST_EMPTY(AllFiles));
						});
					It("should work ok with empty ignore files.", [&]()
						{
							TArray<FString> AllFiles{ TEXT("mybus.png"), TEXT("mybus.png2") };
							PatchGenerationHelpers::StripIgnoredFiles({}, {}, AllFiles);
							CheckIfDebugging(TEST_SIZE(AllFiles, 2));
							CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("mybus.png2")));
							CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("mybus.png")));
						});
					It("should work ok with empty all files.", [&]()
						{
							TArray<FString> AllFiles;
							TSet<FString> IgnoreFiles{ TEXT("mybus.png"), TEXT("mybus.png2") };
							PatchGenerationHelpers::StripIgnoredFiles(IgnoreFiles, {}, AllFiles);
							CheckIfDebugging(TEST_SIZE(AllFiles, 0));
						});
					It("should filter ok without regexes.", [&]()
						{
							{
								TArray<FString> AllFiles{ TEXT("/root/mybus.png"), TEXT("/root/mybus.png2") };
								TSet<FString> IgnoreFiles{ TEXT("/root/mybus2.png"), TEXT("/root/1mybus.png") };
								PatchGenerationHelpers::StripIgnoredFiles(IgnoreFiles, {}, AllFiles);
								CheckIfDebugging(TEST_SIZE(AllFiles, 2));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png2")));
							}
							{
								TArray<FString> AllFiles{ TEXT("/root/foo/bar/mybus.png"), TEXT("/root/foo/bar2/mybus.png2") };
								TSet<FString> IgnoreFiles{ TEXT("/root/foo/bar/mybus.png"), TEXT("/root/foo/bar/mybus.png2") };
								PatchGenerationHelpers::StripIgnoredFiles(IgnoreFiles, {}, AllFiles);
								CheckIfDebugging(TEST_SIZE(AllFiles, 1));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo/bar2/mybus.png2")));								
							}
							{
								TArray<FString> AllFiles{
									TEXT("/root/mybus.png"),
									TEXT("/root/mybus.png2"),
									TEXT("/root/foo/mybus.png"),
									TEXT("/root/foo2/mybus.png"),
									TEXT("/root/foo/bar/mybus.png"),
									TEXT("/root/foo/bar2/mybus.png2"),
									TEXT("/root/foo/bar2/mybus.png.0")
								};
								TSet<FString> IgnoreFiles{
									TEXT("/root/mybus.png"),
									TEXT("/root/foo/bar2/mybus.png2"),
									TEXT("/root/foo/bar2/mybus.png.0")
								};
								PatchGenerationHelpers::StripIgnoredFiles(IgnoreFiles, {}, AllFiles);
								CheckIfDebugging(TEST_SIZE(AllFiles, 4));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png2")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo/mybus.png")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo2/mybus.png")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo/bar/mybus.png")));
							}
						});
					It("should filter ok with regexes.", [&]()
						{
							{
								TArray<FString> AllFiles{ TEXT("/root/mybus.png"), TEXT("/root/mybus.png2") };
								TSet<FString> IgnorePatterns{ TEXT("R\"mybus.png\""), TEXT("R\"mybus.png2\"") };
								PatchGenerationHelpers::StripIgnoredFiles({}, IgnorePatterns, AllFiles);
								CheckIfDebugging(TEST_SIZE(AllFiles, 0));
							}
							{
								TArray<FString> AllFiles{
									TEXT("/root/mybus.png"),
									TEXT("/root/mybus.png2"),
									TEXT("/root/foo/mybus.png"),
									TEXT("/root/foo2/mybus.png"),
									TEXT("/root/foo/bar/mybus.png"),
									TEXT("/root/foo/bar2/mybus.png2"),
									TEXT("/root/foo/bar2/mybus.png.0")
								};
								TSet<FString> IgnorePatterns{TEXT("R\".*mybus.png$\"")};
								PatchGenerationHelpers::StripIgnoredFiles({}, IgnorePatterns, AllFiles);
								CheckIfDebugging(TEST_SIZE(AllFiles, 3));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png2")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo/bar2/mybus.png2")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo/bar2/mybus.png.0")));
							}
							{
								TArray<FString> AllFiles{
									TEXT("/root/mybus.png"),
									TEXT("/root/mybus.png2"),
									TEXT("/root/foo/mybus.png"),
									TEXT("/root/foo2/mybus.png"),
									TEXT("/root/foo/bar/mybus.png"),
									TEXT("/root/foo/bar2/mybus.png2"),
									TEXT("/root/foo/bar2/mybus.png.0")
								};
								TSet<FString> IgnorePatterns{TEXT("R\"foo/\\w+/.*.png\"")};
								PatchGenerationHelpers::StripIgnoredFiles({}, IgnorePatterns, AllFiles);
								CheckIfDebugging(TEST_SIZE(AllFiles, 4));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png2")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo/mybus.png")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo2/mybus.png")));
							}
						});
					It("should filter ok with wildcards.", [&]()
						{
							{
								TArray<FString> AllFiles{
									TEXT("/root/mybus.png"),
									TEXT("/root/mybus.png2"),
									TEXT("/root/foo/mybus.png"),
									TEXT("/root/foo2/mybus.png"),
									TEXT("/root/foo/bar/mybus.png"),
									TEXT("/root/foo/bar2/mybus.png2"),
									TEXT("/root/foo/bar2/mybus.png.0")
								};
								TSet<FString> IgnoreFiles{TEXT("/root/foo2/mybus.png")};
								TSet<FString> IgnorePatterns{TEXT("W\"*foo/*\"")};
								PatchGenerationHelpers::StripIgnoredFiles(IgnoreFiles, IgnorePatterns, AllFiles);
								CheckIfDebugging(TEST_SIZE(AllFiles, 2));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png2")));
							}
							{
								TArray<FString> AllFiles{
									TEXT("/root/mybus.png"),
									TEXT("/root/mybus.png2"),
									TEXT("/root/foo/mybus.png"),
									TEXT("/root/foo2/mybus.png"),
									TEXT("/root/foo/bar/mybus.png"),
									TEXT("/root/foo/bar2/mybus.png2"),
									TEXT("/root/foo/bar2/mybus.png.0")
								};
								TSet<FString> IgnoreFiles{
									TEXT("/root/mybus.png"),
									TEXT("/root/foo2/mybus.png")
								};
								TSet<FString> IgnorePatterns{
									TEXT("W\"*foo/bar*\""),
								};
								PatchGenerationHelpers::StripIgnoredFiles(IgnoreFiles, IgnorePatterns, AllFiles);
								CheckIfDebugging(TEST_SIZE(AllFiles, 2));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/mybus.png2")));
								CheckIfDebugging(TEST_CONTAINS(AllFiles, TEXT("/root/foo/mybus.png")));
							}
						});
				});
		});

	static const char* AttributesList[] = {
		"\"mybus.png\" tag:foo\n",
		"R\"foo/\\w+.png\" executable tag:foo2\n",
		"W\"*foo/bar*\" tag:foo3\n",
		"W\"*foo/*\" readonly tag:foo5\n",
		"\"foomybus2.png\" tag:foo6\n",
		"\"foo/mybus2.png\" tag:foo7\n",
		"\"foo\\mybus3.png\" tag:foo8\n",
	};
	Describe("FFileAttributesParser", [&]()
		{
			Describe("when using ParseFileAttributes", [&]()
				{
					It("should parse file ok.", [&]()
						{
							int32 Size = sizeof(AttributesList) / sizeof(AttributesList[0]);
							TempFile File(AttributesList, Size);
							TMap<FString, FFileAttributes> FileAttributes;
							FFileAttributesParserRef FileAttributesParser = FFileAttributesParserFactory::Create();
							bool bSucceeded = FileAttributesParser->ParseFileAttributes(*File.Filename, FileAttributes);
							CheckIfDebugging(TEST_TRUE(bSucceeded));
							CheckIfDebugging(TEST_SIZE(FileAttributes, Size));
							CheckIfDebugging(TEST_CONTAINS(FileAttributes, TEXT("mybus.png")));
							CheckIfDebugging(TEST_CONTAINS(FileAttributes, TEXT("R\"foo/\\w+.png\"")));
							CheckIfDebugging(TEST_CONTAINS(FileAttributes, TEXT("W\"*foo/bar*\"")));
							CheckIfDebugging(TEST_CONTAINS(FileAttributes, TEXT("W\"*foo/*\"")));
							CheckIfDebugging(TEST_CONTAINS(FileAttributes, TEXT("foomybus2.png")));
							CheckIfDebugging(TEST_CONTAINS(FileAttributes, TEXT("foo/mybus2.png")));
							CheckIfDebugging(TEST_CONTAINS(FileAttributes, TEXT("foo\\mybus3.png")));
						});
				});
			Describe("when using ParseFileAttributes with FFileAttributesMap", [&]()
				{
					It("should sort patterns and filenames ok.", [&]()
						{
							int32 Size = sizeof(AttributesList) / sizeof(AttributesList[0]);
							TempFile File(AttributesList, Size);
							FFileAttributesParserRef FileAttributesParser = FFileAttributesParserFactory::Create();
							TMap<FString, FFileAttributes> FileAttributesMap;
							bool bSucceeded = FileAttributesParser->ParseFileAttributes(*File.Filename, FileAttributesMap);
							FFileAttributesMap FileAttributes(MoveTemp(FileAttributesMap));
							CheckIfDebugging(TEST_SIZE(FileAttributes.Raw(), 4));
							CheckIfDebugging(TEST_SIZE(FileAttributes.RawPatterns(), Size - FileAttributes.Raw().Num()));
							CheckIfDebugging(TEST_CONTAINS(FileAttributes.Raw(), TEXT("foo/mybus3.png")));
						});
					It("should sort find attributes ok.", [&]()
						{
							int32 Size = sizeof(AttributesList) / sizeof(AttributesList[0]);
							TempFile File(AttributesList, Size);
							FFileAttributesParserRef FileAttributesParser = FFileAttributesParserFactory::Create();
							TMap<FString, FFileAttributes> FileAttributesMap;
							bool bSucceeded = FileAttributesParser->ParseFileAttributes(*File.Filename, FileAttributesMap);
							FFileAttributesMap FileAttributes(MoveTemp(FileAttributesMap));
							{
								FFileAttributes Attributes = FileAttributes.Find("mybus2.png");
								CheckIfDebugging(TEST_FALSE(Attributes.bReadOnly));
								CheckIfDebugging(TEST_FALSE(Attributes.bCompressed));
								CheckIfDebugging(TEST_FALSE(Attributes.bUnixExecutable));
								CheckIfDebugging(TEST_EMPTY(Attributes.InstallTags));
							}
							{
								FFileAttributes Attributes = FileAttributes.Find("foo/mybus2.png");
								CheckIfDebugging(TEST_TRUE(Attributes.bReadOnly));
								CheckIfDebugging(TEST_FALSE(Attributes.bCompressed));
								CheckIfDebugging(TEST_TRUE(Attributes.bUnixExecutable));
								CheckIfDebugging(TEST_SIZE(Attributes.InstallTags, 3));
							}
						});
				});
		});
}

#endif //WITH_DEV_AUTOMATION_TESTS
