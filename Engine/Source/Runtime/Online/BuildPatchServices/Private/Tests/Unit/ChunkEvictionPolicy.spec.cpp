// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/TestHelpers.h"
#include "Tests/Fake/ChunkReferenceTracker.fake.h"
#include "Tests/Fake/ChunkDataAccess.fake.h"
#include "Installer/ChunkEvictionPolicy.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FChunkEvictionPolicySpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
const uint32 TestChunkSize = 128 * 1024;
// Unit
TUniquePtr<BuildPatchServices::IChunkEvictionPolicy> ChunkEvictionPolicy;
// Mock
TUniquePtr<BuildPatchServices::FFakeChunkReferenceTracker> MockChunkReferenceTracker;
TUniquePtr<BuildPatchServices::FFakeChunkDataAccess> MockChunkDataAccess;
// Data
TSet<FGuid> ReferencedChunks;
TMap<FGuid, int32> ReferenceCounts;
TArray<FGuid> NextReferences;
TMap<FGuid, TUniquePtr<BuildPatchServices::IChunkDataAccess>> CurrentMap;
TArray<uint8> ChunkData;
END_DEFINE_SPEC(FChunkEvictionPolicySpec)

void FChunkEvictionPolicySpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	for (int32 Idx = 0; Idx < 50; ++Idx)
	{
		NextReferences.Add(FGuid::NewGuid());
	}
	for (int32 Idx = 0; Idx < 25; ++Idx)
	{
		NextReferences.Add(FGuid(NextReferences[Idx]));
	}
	for (const FGuid& NextReference : NextReferences)
	{
		ReferencedChunks.Add(NextReference);
		++ReferenceCounts.FindOrAdd(NextReference);
	}
	ChunkData.SetNumUninitialized(TestChunkSize);
	MockChunkDataAccess.Reset(new FFakeChunkDataAccess());
	MockChunkDataAccess->ChunkData = ChunkData.GetData();

	// Specs.
	BeforeEach([this]()
	{
		for (int32 Idx = 0; Idx < 10; ++Idx)
		{
			CurrentMap.Emplace(NextReferences[Idx], new FFakeChunkDataAccess(*MockChunkDataAccess.Get()));
		}
		MockChunkReferenceTracker.Reset(new FFakeChunkReferenceTracker());
		MockChunkReferenceTracker->ReferencedChunks = ReferencedChunks;
		MockChunkReferenceTracker->ReferenceCounts = ReferenceCounts;
		MockChunkReferenceTracker->NextReferences = NextReferences;
		ChunkEvictionPolicy.Reset(FChunkEvictionPolicyFactory::Create(MockChunkReferenceTracker.Get()));
	});

	Describe("ChunkEvictionPolicy", [this]()
	{
		Describe("Query", [this]()
		{
			Describe("when chunk map has free space", [this]()
			{
				It("should provide cleanable chunks which have refcount 0.", [this]()
				{
					TSet<FGuid> ExpectedCleanable, Cleanable, Bootable;
					for (int32 Idx = 0; Idx < 5; ++Idx)
					{
						MockChunkReferenceTracker->ReferenceCounts.Remove(NextReferences[Idx]);
						ExpectedCleanable.Add(NextReferences[Idx]);
					}
					ChunkEvictionPolicy->Query(CurrentMap, CurrentMap.Num() + 10, Cleanable, Bootable);
					TEST_TRUE(SetsAreEqual(Cleanable, ExpectedCleanable));
				});

				It("should provide no chunks if all chunks are referenced.", [this]()
				{
					TSet<FGuid> Cleanable, Bootable;
					ChunkEvictionPolicy->Query(CurrentMap, CurrentMap.Num() + 10, Cleanable, Bootable);
					TEST_EQUAL(Cleanable.Num(), 0);
					TEST_EQUAL(Bootable.Num(), 0);
				});
			});

			Describe("when chunk map is exact desired size", [this]()
			{
				It("should provide cleanable chunks which have refcount 0.", [this]()
				{
					TSet<FGuid> ExpectedCleanable, Cleanable, Bootable;
					for (int32 Idx = 0; Idx < 5; ++Idx)
					{
						MockChunkReferenceTracker->ReferenceCounts.Remove(NextReferences[Idx]);
						ExpectedCleanable.Add(NextReferences[Idx]);
					}
					ChunkEvictionPolicy->Query(CurrentMap, CurrentMap.Num(), Cleanable, Bootable);
					TEST_TRUE(SetsAreEqual(Cleanable, ExpectedCleanable));
				});

				It("should provide no chunks if all chunks are referenced.", [this]()
				{
					TSet<FGuid> Cleanable, Bootable;
					ChunkEvictionPolicy->Query(CurrentMap, CurrentMap.Num(), Cleanable, Bootable);
					TEST_EQUAL(Cleanable.Num(), 0);
					TEST_EQUAL(Bootable.Num(), 0);
				});
			});

			Describe("when chunk map is full", [this]()
			{
				It("should provide cleanable chunks which have refcount 0.", [this]()
				{
					TSet<FGuid> ExpectedCleanable, Cleanable, Bootable;
					for (int32 Idx = 0; Idx < 5; ++Idx)
					{
						MockChunkReferenceTracker->ReferenceCounts.Remove(NextReferences[Idx]);
						ExpectedCleanable.Add(NextReferences[Idx]);
					}
					ChunkEvictionPolicy->Query(CurrentMap, CurrentMap.Num() - 3, Cleanable, Bootable);
					TEST_TRUE(SetsAreEqual(Cleanable, ExpectedCleanable));
				});

				It("should provide minimum number of bootable chunks.", [this]()
				{
					TSet<FGuid> Cleanable, Bootable;
					ChunkEvictionPolicy->Query(CurrentMap, CurrentMap.Num() - 3, Cleanable, Bootable);
					TEST_EQUAL(Bootable.Num(), 3);
				});

				It("should provide cleanable chunks over bootable chunks.", [this]()
				{
					TSet<FGuid> ExpectedCleanable, Cleanable, ExpectedBootable, Bootable;
					MockChunkReferenceTracker->ReferenceCounts.Remove(NextReferences[0]);
					ExpectedCleanable.Add(NextReferences[0]);
					MockChunkReferenceTracker->ReferenceCounts.Remove(NextReferences[1]);
					ExpectedCleanable.Add(NextReferences[1]);
					ExpectedBootable.Add(NextReferences[9]);
					ChunkEvictionPolicy->Query(CurrentMap, CurrentMap.Num() - 3, Cleanable, Bootable);
					TEST_TRUE(SetsAreEqual(Cleanable, ExpectedCleanable));
					TEST_TRUE(SetsAreEqual(Bootable, ExpectedBootable));
				});

				It("should provide bootable chunks which are needed the latest.", [this]()
				{
					TSet<FGuid> Cleanable, ExpectedBootable, Bootable;
					ExpectedBootable.Add(NextReferences[9]);
					ExpectedBootable.Add(NextReferences[8]);
					ExpectedBootable.Add(NextReferences[7]);
					ChunkEvictionPolicy->Query(CurrentMap, CurrentMap.Num() - 3, Cleanable, Bootable);
					TEST_TRUE(SetsAreEqual(Bootable, ExpectedBootable));
				});
			});
		});
	});

	AfterEach([this]()
	{
		ChunkEvictionPolicy.Reset();
		MockChunkReferenceTracker.Reset();
		CurrentMap.Reset();
	});
}

#endif //WITH_DEV_AUTOMATION_TESTS
