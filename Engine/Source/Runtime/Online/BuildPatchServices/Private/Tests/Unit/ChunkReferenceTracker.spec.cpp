// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "BuildPatchSettings.h"

#include "Tests/TestHelpers.h"
#include "Tests/Mock/Manifest.mock.h"
#include "Installer/ChunkReferenceTracker.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FChunkReferenceTrackerSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
const uint32 TestChunkSize = 128 * 1024;
// Unit
TUniquePtr<BuildPatchServices::IChunkReferenceTracker> ChunkReferenceTracker;
// Mock
BuildPatchServices::FMockManifestPtr MockManifest;
TUniquePtr<BuildPatchServices::IBuildManifestSet> ManifestSet;
// Data
TArray<FString> FileList;
TArray<FString> SubsetFileList;
TSet<FGuid> AllChunks;
TSet<FGuid> SubsetReferencedChunks;
TMap<FString, BuildPatchServices::FFileManifest> FileManifests;
TMap<FGuid, int32> ChunkRefCounts;
TArray<FGuid> UseOrderForwardSortedSubsetArray;
TArray<FGuid> UseOrderReverseSortedSubsetArray;
TArray<FGuid> UseOrderForwardSortedSubsetUniqueArray;
END_DEFINE_SPEC(FChunkReferenceTrackerSpec)

void FChunkReferenceTrackerSpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	for (int32 Idx = 0; Idx < 50; ++Idx)
	{
		FileList.Add(FString::Printf(TEXT("Some/Install/File%d.exe"), Idx));
		FileList.Add(FString::Printf(TEXT("Other/Install/File%d.exe"), Idx));
	}
	FileList.Sort(TLess<FString>());
	SubsetFileList = FileList.FilterByPredicate([](const FString& Element) { return Element.StartsWith(TEXT("Some")); });
	for (const FString& Filename : FileList)
	{
		FFileManifest& FileManifest = FileManifests.Add(Filename, FFileManifest());
		FileManifest.Filename = Filename;
		FChunkPart ChunkPartData;
		// New chunk.
		ChunkPartData.Guid = FGuid::NewGuid();
		AllChunks.Add(ChunkPartData.Guid);
		ChunkPartData.Offset = 0;
		ChunkPartData.Size = TestChunkSize;
		FileManifest.ChunkParts.Add(ChunkPartData);
		// New chunk.
		ChunkPartData.Guid = FGuid::NewGuid();
		AllChunks.Add(ChunkPartData.Guid);
		ChunkPartData.Offset += ChunkPartData.Size;
		ChunkPartData.Size = TestChunkSize;
		FileManifest.ChunkParts.Add(ChunkPartData);
		// Dupe chunk.
		ChunkPartData.Offset += ChunkPartData.Size;
		ChunkPartData.Size = TestChunkSize;
		FileManifest.ChunkParts.Add(ChunkPartData);
	}
	for (const FString& File : SubsetFileList)
	{
		for (const FChunkPart& FileChunkPart : FileManifests[File].ChunkParts)
		{
			SubsetReferencedChunks.Add(FileChunkPart.Guid);
			UseOrderForwardSortedSubsetArray.Add(FileChunkPart.Guid);
			UseOrderForwardSortedSubsetUniqueArray.AddUnique(FileChunkPart.Guid);
			UseOrderReverseSortedSubsetArray.Insert(FileChunkPart.Guid, 0);
		}
	}
	for (const FString& File : FileList)
	{
		for (const FChunkPart& FileChunkPart : FileManifests[File].ChunkParts)
		{
			++ChunkRefCounts.FindOrAdd(FileChunkPart.Guid);
		}
	}

	// Specs.
	Describe("ChunkReferenceTracker", [this]()
	{
		Describe("GetReferencedChunks", [this]()
		{
			Describe("when constructing all files in the manifest", [this]()
			{
				BeforeEach([this]()
				{
					MockManifest = MakeShareable(new FMockManifest());
					MockManifest->FileManifests = FileManifests;
					MockManifest->BuildFileList = FileList;
					MockManifest->TaggedFileList = TSet<FString>(FileList);
					MockManifest->SyncInternalManifestStructures();

					TSet<FString> FilesToConstruct(FileList);
					FilesToConstruct.Sort(TLess<FString>());
					ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef()) }));
					ChunkReferenceTracker.Reset(FChunkReferenceTrackerFactory::Create(ManifestSet.Get(), MoveTemp(FilesToConstruct)));
				});

				It("should return all chunks before any are popped.", [this]()
				{
					TSet<FGuid> ReferencedChunks = ChunkReferenceTracker->GetReferencedChunks();
					TEST_EQUAL(AllChunks.Difference(ReferencedChunks).Num(), 0);
					TEST_EQUAL(AllChunks.Num(), ReferencedChunks.Num());
				});

				It("should return all chunks that are still referenced.", [this]()
				{
					FGuid FirstChunk = FileManifests[FileList[0]].ChunkParts[0].Guid;
					TEST_EQUAL(ChunkRefCounts[FirstChunk], 1);
					ChunkReferenceTracker->PopReference(FirstChunk);
					TSet<FGuid> ReferencedChunks = ChunkReferenceTracker->GetReferencedChunks();
					TSet<FGuid> UnreferencedChunks = AllChunks.Difference(ReferencedChunks);
					TEST_EQUAL(UnreferencedChunks.Num(), 1);
					TEST_TRUE(UnreferencedChunks.Contains(FirstChunk));
				});

				It("should return no chunks if all references are popped.", [this]()
				{
					for (const FString& File : FileList)
					{
						for (const FChunkPart& FileChunkPart : FileManifests[File].ChunkParts)
						{
							ChunkReferenceTracker->PopReference(FileChunkPart.Guid);
						}
					}
					TSet<FGuid> ReferencedChunks = ChunkReferenceTracker->GetReferencedChunks();
					TEST_EQUAL(ReferencedChunks.Num(), 0);
				});
			});

			Describe("when constructing a subset of files in the manifest", [this]()
			{
				BeforeEach([this]()
				{
					MockManifest = MakeShareable(new FMockManifest());
					MockManifest->FileManifests = FileManifests;
					MockManifest->BuildFileList = FileList;
					MockManifest->TaggedFileList = TSet<FString>(FileList);
					MockManifest->SyncInternalManifestStructures();

					TSet<FString> FilesToConstruct(SubsetFileList);
					FilesToConstruct.Sort(TLess<FString>());
					ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef()) }));
					ChunkReferenceTracker.Reset(FChunkReferenceTrackerFactory::Create(ManifestSet.Get(), MoveTemp(FilesToConstruct)));
				});

				It("should return only chunks referenced by subset of files.", [this]()
				{
					TSet<FGuid> ReferencedChunks = ChunkReferenceTracker->GetReferencedChunks();
					TEST_EQUAL(SubsetReferencedChunks.Difference(ReferencedChunks).Num(), 0);
					TEST_EQUAL(SubsetReferencedChunks.Num(), ReferencedChunks.Num());
				});

				It("should return all chunks that are still referenced.", [this]()
				{
					FGuid FirstChunk = FileManifests[SubsetFileList[0]].ChunkParts[0].Guid;
					TEST_EQUAL(ChunkRefCounts[FirstChunk], 1);
					ChunkReferenceTracker->PopReference(FirstChunk);
					TSet<FGuid> ReferencedChunks = ChunkReferenceTracker->GetReferencedChunks();
					TSet<FGuid> UnreferencedChunks = SubsetReferencedChunks.Difference(ReferencedChunks);
					TEST_EQUAL(UnreferencedChunks.Num(), 1);
					TEST_TRUE(UnreferencedChunks.Contains(FirstChunk));
				});

				It("should return no chunks if all references are popped.", [this]()
				{
					for (const FString& File : SubsetFileList)
					{
						for (const FChunkPart& FileChunkPart : FileManifests[File].ChunkParts)
						{
							ChunkReferenceTracker->PopReference(FileChunkPart.Guid);
						}
					}
					TSet<FGuid> ReferencedChunks = ChunkReferenceTracker->GetReferencedChunks();
					TEST_EQUAL(ReferencedChunks.Num(), 0);
				});
			});
		});

		Describe("GetReferenceCount", [this]()
		{
			BeforeEach([this]()
			{
				MockManifest = MakeShareable(new FMockManifest());
				MockManifest->FileManifests = FileManifests;
				MockManifest->BuildFileList = FileList;
				MockManifest->TaggedFileList = TSet<FString>(FileList);
				MockManifest->SyncInternalManifestStructures();

				TSet<FString> FilesToConstruct(FileList);
				FilesToConstruct.Sort(TLess<FString>());
				ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef()) }));
				ChunkReferenceTracker.Reset(FChunkReferenceTrackerFactory::Create(ManifestSet.Get(), MoveTemp(FilesToConstruct)));
			});

			It("should return original counts before anything is popped.", [this]()
			{
				for (const FGuid& ChunkId : AllChunks)
				{
					TEST_EQUAL(ChunkReferenceTracker->GetReferenceCount(ChunkId), ChunkRefCounts[ChunkId]);
				}
			});

			It("should return 0 for unknown chunks.", [this]()
			{
				TEST_EQUAL(ChunkReferenceTracker->GetReferenceCount(FGuid::NewGuid()), 0);
			});

			It("should return adjusted count for popped references.", [this]()
			{
				FGuid FirstChunk = FileManifests[FileList[0]].ChunkParts[0].Guid;
				TEST_EQUAL(ChunkReferenceTracker->GetReferenceCount(FirstChunk), ChunkRefCounts[FirstChunk]);
				ChunkReferenceTracker->PopReference(FirstChunk);
				TEST_EQUAL(ChunkReferenceTracker->GetReferenceCount(FirstChunk), ChunkRefCounts[FirstChunk]-1);
			});

			It("should return 0 for all chunks once all references popped.", [this]()
			{
				for (const FString& File : FileList)
				{
					for (const FChunkPart& FileChunkPart : FileManifests[File].ChunkParts)
					{
						ChunkReferenceTracker->PopReference(FileChunkPart.Guid);
					}
				}
				for (const FGuid& ChunkId : AllChunks)
				{
					TEST_EQUAL(ChunkReferenceTracker->GetReferenceCount(ChunkId), 0);
				}
			});
		});

		Describe("SortByUseOrder", [this]()
		{
			BeforeEach([this]()
			{
				MockManifest = MakeShareable(new FMockManifest());
				MockManifest->FileManifests = FileManifests;
				MockManifest->BuildFileList = FileList;
				MockManifest->TaggedFileList = TSet<FString>(FileList);
				MockManifest->SyncInternalManifestStructures();

				TSet<FString> FilesToConstruct(FileList);
				FilesToConstruct.Sort(TLess<FString>());
				ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef()) }));
				ChunkReferenceTracker.Reset(FChunkReferenceTrackerFactory::Create(ManifestSet.Get(), MoveTemp(FilesToConstruct)));
			});

			Describe("when used with ESortDirection::Ascending", [this]()
			{
				It("should place soonest required chunks first.", [this]()
				{
					TArray<FGuid> SortedArray = UseOrderForwardSortedSubsetArray;
					TArray<FGuid> ArrayToSort = UseOrderReverseSortedSubsetArray;
					ChunkReferenceTracker->SortByUseOrder(ArrayToSort, IChunkReferenceTracker::ESortDirection::Ascending);
					TEST_EQUAL(SortedArray, ArrayToSort);
				});

				It("should place unused chunks last.", [this]()
				{
					TArray<FGuid> SortedArray = UseOrderForwardSortedSubsetArray;
					TArray<FGuid> ArrayToSort = UseOrderReverseSortedSubsetArray;
					SortedArray.Add(FGuid::NewGuid());
					ArrayToSort.Insert(SortedArray.Last(), ArrayToSort.Num() / 2);
					ChunkReferenceTracker->SortByUseOrder(ArrayToSort, IChunkReferenceTracker::ESortDirection::Ascending);
					TEST_EQUAL(SortedArray, ArrayToSort);
				});
			});

			Describe("when used with ESortDirection::Descending", [this]()
			{
				It("should place soonest required chunks last.", [this]()
				{
					TArray<FGuid> SortedArray = UseOrderReverseSortedSubsetArray;
					TArray<FGuid> ArrayToSort = UseOrderForwardSortedSubsetArray;
					ChunkReferenceTracker->SortByUseOrder(ArrayToSort, IChunkReferenceTracker::ESortDirection::Descending);
					TEST_EQUAL(SortedArray, ArrayToSort);
				});

				It("should place unused chunks first.", [this]()
				{
					TArray<FGuid> SortedArray = UseOrderReverseSortedSubsetArray;
					TArray<FGuid> ArrayToSort = UseOrderForwardSortedSubsetArray;
					SortedArray.Insert(FGuid::NewGuid(), 0);
					ArrayToSort.Insert(SortedArray[0], ArrayToSort.Num() / 2);
					ChunkReferenceTracker->SortByUseOrder(ArrayToSort, IChunkReferenceTracker::ESortDirection::Descending);
					TEST_EQUAL(SortedArray, ArrayToSort);
				});
			});

			Describe("when used with an invalid ESortDirection", [this]()
			{
				It("should leave the array unchanged.", [this]()
				{
					TArray<FGuid> ArrayToSort, ArrayToSortDupe;
					ArrayToSort = UseOrderForwardSortedSubsetArray;
					ArrayToSort.Insert(FGuid::NewGuid(), ArrayToSort.Num() / 2);
					ArrayToSortDupe = ArrayToSort;
					ChunkReferenceTracker->SortByUseOrder(ArrayToSort, static_cast<IChunkReferenceTracker::ESortDirection>(255));
					TEST_EQUAL(ArrayToSortDupe, ArrayToSort);
				});
			});
		});

		Describe("GetNextReferences", [this]()
		{
			BeforeEach([this]()
			{
				MockManifest = MakeShareable(new FMockManifest());
				MockManifest->FileManifests = FileManifests;
				MockManifest->BuildFileList = FileList;
				MockManifest->TaggedFileList = TSet<FString>(FileList);
				MockManifest->SyncInternalManifestStructures();

				TSet<FString> FilesToConstruct(FileList);
				FilesToConstruct.Sort(TLess<FString>());
				ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef()) }));
				ChunkReferenceTracker.Reset(FChunkReferenceTrackerFactory::Create(ManifestSet.Get(), MoveTemp(FilesToConstruct)));
			});

			It("should return the correct number of selected references.", [this]()
			{
				int32 NumChunks = SubsetReferencedChunks.Num() / 2;
				TArray<FGuid> SelectedChunks = ChunkReferenceTracker->GetNextReferences(NumChunks, [this](const FGuid& ChunkID) { return SubsetReferencedChunks.Contains(ChunkID); });
				TEST_EQUAL(SelectedChunks.Num(), NumChunks);
				TEST_EQUAL(TSet<FGuid>(SelectedChunks).Difference(SubsetReferencedChunks).Num(), 0);
			});

			It("should return the selected references in the correct order.", [this]()
			{
				TArray<FGuid> SortedArray = UseOrderForwardSortedSubsetUniqueArray;
				int32 NumChunks = SortedArray.Num() / 2;
				SortedArray.SetNum(NumChunks);
				TArray<FGuid> SelectedChunks = ChunkReferenceTracker->GetNextReferences(NumChunks, [this](const FGuid& ChunkID) { return SubsetReferencedChunks.Contains(ChunkID); });
				TEST_EQUAL(SelectedChunks, SortedArray);
			});

			It("should return up to the given count if there are less available.", [this]()
			{
				TArray<FGuid> SelectedChunks = ChunkReferenceTracker->GetNextReferences(MAX_int32, [this](const FGuid& ChunkID) { return SubsetReferencedChunks.Contains(ChunkID); });
				TEST_EQUAL(SelectedChunks.Num(), SubsetReferencedChunks.Num());
				TEST_EQUAL(TSet<FGuid>(SelectedChunks).Difference(SubsetReferencedChunks).Num(), 0);
			});

			It("should return no ids if none were selected.", [this]()
			{
				TArray<FGuid> SelectedChunks = ChunkReferenceTracker->GetNextReferences(MAX_int32, [this](const FGuid& ChunkID) { return false; });
				TEST_EQUAL(SelectedChunks.Num(), 0);
			});

			It("should not return duplicates.", [this]()
			{
				TArray<FGuid> SelectedChunks = ChunkReferenceTracker->GetNextReferences(MAX_int32, [this](const FGuid& ChunkID) { return true; });
				TSet<FGuid> SelectedChunksSet(SelectedChunks);
				TEST_EQUAL(SelectedChunks.Num(), SelectedChunksSet.Num());
			});
		});

		Describe("PopReference", [this]()
		{
			BeforeEach([this]()
			{
				MockManifest = MakeShareable(new FMockManifest());
				MockManifest->FileManifests = FileManifests;
				MockManifest->BuildFileList = FileList;
				MockManifest->TaggedFileList = TSet<FString>(FileList);
				MockManifest->SyncInternalManifestStructures();

				TSet<FString> FilesToConstruct(FileList);
				FilesToConstruct.Sort(TLess<FString>());
				ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef()) }));
				ChunkReferenceTracker.Reset(FChunkReferenceTrackerFactory::Create(ManifestSet.Get(), MoveTemp(FilesToConstruct)));
			});

			It("should return true when popping the top chunk", [this]()
			{
				FGuid TopChunk = FileManifests[FileList[0]].ChunkParts[0].Guid;
				TEST_TRUE(ChunkReferenceTracker->PopReference(TopChunk));
			});

			It("should return false when popping unknown chunk", [this]()
			{
				TEST_FALSE(ChunkReferenceTracker->PopReference(FGuid::NewGuid()));
			});

			It("should always return true for popping the chunks in order", [this]()
			{
				for (const FString& File : FileList)
				{
					for (const FChunkPart& FileChunkPart : FileManifests[File].ChunkParts)
					{
						TEST_TRUE(ChunkReferenceTracker->PopReference(FileChunkPart.Guid));
					}
				}
			});

			It("should return false for all except the top chunk", [this]()
			{
				FGuid FirstChunk = FileManifests[FileList[0]].ChunkParts[0].Guid;
				for (const FGuid& ChunkId : AllChunks)
				{
					if (ChunkId != FirstChunk)
					{
						TEST_FALSE(ChunkReferenceTracker->PopReference(ChunkId));
					}
				}
			});

			It("should return true for correct pop following many incorrect pops", [this]()
			{
				FGuid FirstChunk = FileManifests[FileList[0]].ChunkParts[0].Guid;
				for (const FGuid& ChunkId : AllChunks)
				{
					if (ChunkId != FirstChunk)
					{
						ChunkReferenceTracker->PopReference(ChunkId);
					}
				}
				TEST_TRUE(ChunkReferenceTracker->PopReference(FirstChunk));
			});
		});
	});

	AfterEach([this]()
	{
		ChunkReferenceTracker.Reset();
		MockManifest.Reset();
		ManifestSet.Reset();
	});
}

#endif //WITH_DEV_AUTOMATION_TESTS
