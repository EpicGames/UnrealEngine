// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/OutputDeviceRedirector.h"

#include "BuildPatchSettings.h"

#include "Tests/TestHelpers.h"
#include "Tests/Mock/PlatformProcess.mock.h"
#include "Tests/Mock/PlatformMisc.mock.h"
#include "Tests/Fake/ChunkStore.fake.h"
#include "Tests/Mock/Download.mock.h"
#include "Tests/Fake/ChunkReferenceTracker.fake.h"
#include "Tests/Fake/InstallerError.fake.h"
#include "Tests/Mock/CloudChunkSourceStat.mock.h"
#include "Tests/Mock/Manifest.mock.h"
#include "Tests/Fake/ChunkDataAccess.fake.h"
#include "Tests/Fake/DownloadService.fake.h"
#include "Tests/Fake/ChunkDataSerialization.fake.h"
#include "Tests/Mock/MessagePump.mock.h"
#include "Tests/Mock/DownloadConnectionCount.mock.h"
#include "Installer/CloudChunkSource.h"
#include "Installer/InstallerSharedContext.h"
#include "Core/Platform.h"
#include "BuildPatchHash.h"

#if WITH_DEV_AUTOMATION_TESTS
namespace BuildPatchServices
{
	typedef TPlatform<FMockPlatformProcess, FMockPlatformMisc> FMockPlatform;
}
BEGIN_DEFINE_SPEC(FCloudChunkSourceSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
// Unit.
TUniquePtr<BuildPatchServices::ICloudChunkSource> CloudChunkSource;
TUniquePtr<BuildPatchServices::FBuildInstallerSharedContext> InstallerSharedContext;
// Mock/Fake.
TUniquePtr<BuildPatchServices::FMockPlatform> MockPlatform;
TUniquePtr<BuildPatchServices::FFakeChunkStore> FakeChunkStore;
TUniquePtr<BuildPatchServices::FFakeDownloadService> FakeDownloadService;
TUniquePtr<BuildPatchServices::FFakeChunkReferenceTracker> MockChunkReferenceTracker;
TUniquePtr<BuildPatchServices::FFakeChunkDataSerialization> FakeChunkDataSerialization;
TUniquePtr<BuildPatchServices::FMockMessagePump> MockMessagePump;
TUniquePtr<BuildPatchServices::FMockDownloadConnectionCount> MockConnectionCount;
TUniquePtr<BuildPatchServices::FFakeInstallerError> MockInstallerError;
TUniquePtr<BuildPatchServices::FMockCloudChunkSourceStat> MockCloudChunkSourceStat;
BuildPatchServices::FMockManifestPtr MockManifest;
TUniquePtr<BuildPatchServices::IBuildManifestSet> ManifestSet;
// Data.
TUniquePtr<BuildPatchServices::FCloudSourceConfig> CloudSourceConfig;
TSet<FGuid> InitialDownloadSet;
TSet<FGuid> EmptyInitialDownloadSet;
TSet<FGuid> LargeInitialDownloadSet;
FSHAHash SomeShaData;
FGuid SomeChunk;
FGuid FirstChunk;
BuildPatchServices::FChunkHeader FirstHeader;
float PausePadding;
float PauseTime;
int32 CallCount;
uint32 ForceConnectionCount;
// Helper functions for specs.
void MakeUnit(TSet<FGuid> DownloadSet);
TArray<FGuid> CheckForChunkRequests(TArray<FGuid> ExpectedChunks, float SecondsLimit = 10.0f);
TArray<FGuid> GetAllChunks(TArray<FGuid> Chunks);
TFuture<void> PauseFor(float Seconds);
END_DEFINE_SPEC(FCloudChunkSourceSpec)

void FCloudChunkSourceSpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	PausePadding = 1.1f;
	FRollingHashConst::Init();
	SomeChunk = FGuid::NewGuid();
	FMemory::Memcpy(SomeShaData.Hash, TEXT("At least enough data for SHA!"), FSHA1::DigestSize);
	InitialDownloadSet.Empty();
	for (int32 Idx = 0; Idx < 25; ++Idx)
	{
		InitialDownloadSet.Add(FGuid::NewGuid());
	}
	LargeInitialDownloadSet = InitialDownloadSet;
	for (int32 Idx = 0; Idx < 75; ++Idx)
	{
		LargeInitialDownloadSet.Add(FGuid::NewGuid());
	}

	// Specs.
	BeforeEach([this]()
	{
		MockPlatform.Reset(new FMockPlatform());
		InstallerSharedContext.Reset(new FBuildInstallerSharedContext(TEXT("CloudChunkSourceSpec")));
		CloudSourceConfig.Reset(new FCloudSourceConfig({ "http://download.mydomain.com/clouddata" }));
		FakeChunkStore.Reset(new FFakeChunkStore());
		FakeDownloadService.Reset(new FFakeDownloadService());
		MockChunkReferenceTracker.Reset(new FFakeChunkReferenceTracker());
		FakeChunkDataSerialization.Reset(new FFakeChunkDataSerialization());
		MockMessagePump.Reset(new FMockMessagePump());
		MockConnectionCount.Reset(new FMockDownloadConnectionCount());
		MockInstallerError.Reset(new FFakeInstallerError());
		MockCloudChunkSourceStat.Reset(new FMockCloudChunkSourceStat());
		MockManifest = MakeShareable(new FMockManifest());
		ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef()) }));
		for (const FGuid& Guid : InitialDownloadSet)
		{
			MockChunkReferenceTracker->ReferencedChunks.Add(Guid);
			MockChunkReferenceTracker->ReferenceCounts.Add(Guid, 1);
			MockChunkReferenceTracker->NextReferences.Add(Guid);
		}
		FirstChunk = MockChunkReferenceTracker->NextReferences[0];
		FirstHeader.Guid = FirstChunk;
		FirstHeader.DataSizeCompressed = 128;
		FirstHeader.DataSizeUncompressed = 128;
		FakeDownloadService->ThreadLock.Lock();
		FakeDownloadService->DefaultChunkHeader = FirstHeader;
		FakeDownloadService->ThreadLock.Unlock();
		ForceConnectionCount = 8U;
	});
	// @TODO Antony.Carter Needs to be a latent It 
	xDescribe("CloudChunkSource", [this]()
	{
		Describe("Get", [this]()
		{
			Describe("when source is given no chunks to fetch upfront", [this]()
			{
				BeforeEach([this]()
				{
					MakeUnit(EmptyInitialDownloadSet);
				});

				Describe("when a chunk is already in the store", [this]()
				{
					BeforeEach([this]()
					{
						FGuid NewId = FGuid::NewGuid();
						FFakeChunkDataAccess* ChunkDataAccess = new FFakeChunkDataAccess();
						ChunkDataAccess->ChunkHeader.Guid = NewId;
						FakeChunkStore->Store.Emplace(NewId, ChunkDataAccess);
						ChunkDataAccess = nullptr;
					});

					It("should not call into download service.", [this]()
					{
						TArray<FGuid> StoreKeys;
						FakeChunkStore->Store.GetKeys(StoreKeys);
						CloudChunkSource->Get(StoreKeys[0]);
						TEST_EQUAL(FakeDownloadService->RxRequestFile.Num(), 0);
						TEST_EQUAL(FakeDownloadService->RxRequestCancel.Num(), 0);
					});

					It("should return the chunk.", [this]()
					{
						TArray<FGuid> StoreKeys;
						FakeChunkStore->Store.GetKeys(StoreKeys);
						TEST_EQUAL(CloudChunkSource->Get(StoreKeys[0]), FakeChunkStore->Store[StoreKeys[0]].Get());
					});
				});

				Describe("when a chunk is not already in the store", [this]()
				{
					BeforeEach([this]()
					{
						FakeDownloadService->StartService();
					});

					It("should retrieve it using the download service.", [this]()
					{
						IChunkDataAccess* ChunkDataAccess = CloudChunkSource->Get(FirstChunk);
						TEST_EQUAL(FakeDownloadService->RxRequestFile.Num(), 1);
						TEST_EQUAL(FakeDownloadService->RxRequestCancel.Num(), 0);
						TEST_NOT_NULL(ChunkDataAccess);
						if (ChunkDataAccess != nullptr)
						{
							FFakeChunkDataAccess* MockChunkDataAccess = static_cast<FFakeChunkDataAccess*>(ChunkDataAccess);
							TEST_EQUAL(MockChunkDataAccess->ChunkHeader.Guid, FirstChunk);
						}
					});

					It("should deserialize using the chunk data serialization.", [this]()
					{
						IChunkDataAccess* ChunkDataAccess = CloudChunkSource->Get(FirstChunk);
						TEST_EQUAL(FakeChunkDataSerialization->RxLoadFromMemory.Num(), 1);
						TEST_EQUAL(FakeChunkDataSerialization->RxLoadFromFile.Num(), 0);
						TEST_EQUAL(FakeChunkDataSerialization->RxSaveToFile.Num(), 0);
						TEST_EQUAL(FakeChunkDataSerialization->RxInjectShaToChunkData.Num(), 0);
						TEST_NOT_NULL(ChunkDataAccess);
						if (ChunkDataAccess != nullptr)
						{
							FFakeChunkDataAccess* MockChunkDataAccess = static_cast<FFakeChunkDataAccess*>(ChunkDataAccess);
							TEST_EQUAL(MockChunkDataAccess->ChunkHeader.Guid, FirstChunk);
						}
					});

					It("should have placed it in the chunk store.", [this]()
					{
						FGuid ChunkToGet = FGuid::NewGuid();
						CloudChunkSource->Get(ChunkToGet);
						TEST_TRUE(FakeChunkStore->Store.Contains(ChunkToGet));
					});
				});
			});

			Describe("when a chunk is always failing to download", [this]()
			{
				BeforeEach([this]()
				{
					CloudSourceConfig->MaxRetryCount = 4;
					CloudSourceConfig->RetryDelayTimes = { 0 };
					FakeDownloadService->ThreadLock.Lock();
					for (int32 Idx = 0; Idx < CloudSourceConfig->MaxRetryCount + 5; ++Idx)
					{
						FakeDownloadService->TxRequestFile.Emplace(0.0f, false, EHttpResponseCodes::Unknown, FChunkHeader());
					}
					FakeDownloadService->ThreadLock.Unlock();
					FakeDownloadService->StartService();
					MakeUnit(EmptyInitialDownloadSet);
				});

				It("should return nullptr after retrying the number of times according to the config.", [this]()
				{
					TEST_NULL(CloudChunkSource->Get(FGuid::NewGuid()));
					TEST_EQUAL(FakeDownloadService->RxRequestFile.Num(), CloudSourceConfig->MaxRetryCount + 1);
				});
			});
		});
		Describe("SetPaused", [this]()
		{
			BeforeEach([this]()
			{
				CallCount = 0;
				PauseTime = 0.5f;
				CloudSourceConfig->PreFetchMinimum = InitialDownloadSet.Num() + 1;
				CloudSourceConfig->PreFetchMaximum = InitialDownloadSet.Num() + 1;
				CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
				ForceConnectionCount = 1;
				FakeDownloadService->RequestFileFunc = [this](const FString&, const FDownloadCompleteDelegate&, const FDownloadProgressDelegate&, int32* OutRequestId) -> bool
				{
					if (++CallCount == 2)
					{
						PauseFor(PauseTime);
					}
					const int32 NewRequestId = ++FakeDownloadService->Count;
					if (OutRequestId)
					{
						*OutRequestId = NewRequestId;
					}
					return true;
				};
				MakeUnit(InitialDownloadSet);
				FakeDownloadService->StartService();
			});

			xIt("should delay the download requests.", [this]()
			{
				TArray<FGuid> Unrequested = CheckForChunkRequests(InitialDownloadSet.Array());
				TEST_EQUAL(Unrequested.Num(), 0);
				if (Unrequested.Num() == 0)
				{
					double LongestDelay = 0.0f;
					for (int32 Idx = 1; Idx < FakeDownloadService->RxRequestFile.Num(); ++Idx)
					{
						double ThisDelay = FakeDownloadService->RxRequestFile[Idx].Get<0>() - FakeDownloadService->RxRequestFile[Idx - 1].Get<0>();
						if (ThisDelay > LongestDelay)
						{
							LongestDelay = ThisDelay;
						}
					}
					TEST_TRUE(LongestDelay >= PauseTime);
				}
			});
		});

		Describe("when a chunk download is corrupt", [this]()
		{
			BeforeEach([this]()
			{
				FakeChunkDataSerialization->TxLoadFromMemory.Emplace(nullptr, EChunkLoadResult::HashCheckFailed);
				FakeDownloadService->StartService();
				MakeUnit(EmptyInitialDownloadSet);
			});

			It("should report and retry the chunk.", [this]()
			{
				TEST_NOT_NULL(CloudChunkSource->Get(FGuid::NewGuid()));
				TEST_EQUAL(FakeDownloadService->RxRequestFile.Num(), 2);
				TEST_EQUAL(MockCloudChunkSourceStat->RxDownloadCorrupt.Num(), 1);
			});
		});

		Describe("when some chunk SHA is in the manifest", [this]()
		{
			BeforeEach([this]()
			{
				FChunkInfo ChunkInfo;
				ChunkInfo.ShaHash = SomeShaData;
				MockManifest->ChunkInfos.Add(SomeChunk, ChunkInfo);
				FakeDownloadService->StartService();
				MakeUnit(EmptyInitialDownloadSet);
			});

			xIt("should inject the SHA to the downloaded chunk.", [this]()
			{
				TEST_NOT_NULL(CloudChunkSource->Get(SomeChunk));
				TEST_EQUAL(FakeChunkDataSerialization->RxInjectShaToChunkData.Num(), 1);
				if (FakeChunkDataSerialization->RxInjectShaToChunkData.Num() == 1)
				{
					TEST_EQUAL(FMemory::Memcmp(SomeShaData.Hash, FakeChunkDataSerialization->RxInjectShaToChunkData[0].Get<1>().Hash, FSHA1::DigestSize), 0);
				}
			});
		});

		Describe("when set up with initial download list", [this]()
		{
			Describe("Abort", [this]()
			{
				BeforeEach([this]()
				{
					CloudSourceConfig->PreFetchMinimum = InitialDownloadSet.Num() + 1;
					CloudSourceConfig->PreFetchMaximum = InitialDownloadSet.Num() + 1;
					FakeDownloadService->ThreadLock.Lock();
					FakeDownloadService->TxRequestFile.Emplace(10.0f, true, EHttpResponseCodes::Ok, FirstHeader);
					FakeDownloadService->TxRequestFile.Emplace(10.0f, true, EHttpResponseCodes::Ok, FirstHeader);
					FakeDownloadService->TxRequestFile.Emplace(10.0f, true, EHttpResponseCodes::Ok, FirstHeader);
					FakeDownloadService->TxRequestFile.Emplace(10.0f, true, EHttpResponseCodes::Ok, FirstHeader);
					FakeDownloadService->ThreadLock.Unlock();
					FakeDownloadService->StartService();
					MakeUnit(InitialDownloadSet);
				});

				It("should halt requests and stop processing.", [this]()
				{
					CloudChunkSource->Abort();
					TArray<FGuid> Succeeded = GetAllChunks(InitialDownloadSet.Array());
					TEST_TRUE(Succeeded.Num() < InitialDownloadSet.Num());
				});
			});

			Describe("when no download failures are occurring", [this]()
			{
				BeforeEach([this]()
				{
					const int32 OneMoreChunk = LargeInitialDownloadSet.Num() + 1;
					CloudSourceConfig->PreFetchMinimum = OneMoreChunk;
					CloudSourceConfig->PreFetchMaximum = OneMoreChunk;
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					ForceConnectionCount = OneMoreChunk;
					FakeDownloadService->StartService();
					MakeUnit(LargeInitialDownloadSet);
				});

				It("should end with Excellent download health.", [this]()
				{
					GetAllChunks(LargeInitialDownloadSet.Array());
					TEST_TRUE(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0);
					if (MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0)
					{
						TEST_EQUAL(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Last().Get<1>(), EBuildPatchDownloadHealth::Excellent);
					}
				});
			});
			Describe("when up to 1% download failures are occurring", [this]()
			{
				BeforeEach([this]()
				{
					const int32 OneMoreChunk = LargeInitialDownloadSet.Num() + 1;
					CloudSourceConfig->PreFetchMinimum = OneMoreChunk;
					CloudSourceConfig->PreFetchMaximum = OneMoreChunk;
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					ForceConnectionCount = OneMoreChunk;
					FakeDownloadService->ThreadLock.Lock();
					for (int32 Count = 0; (float(Count) / float(Count + LargeInitialDownloadSet.Num())) <= 0.0f; ++Count)
					{
						FakeDownloadService->TxRequestFile.Emplace(0.0f, false, EHttpResponseCodes::Unknown, FirstHeader);
					}
					FakeDownloadService->ThreadLock.Unlock();
					FakeDownloadService->StartService();
					MakeUnit(LargeInitialDownloadSet);
				});

				It("should end with Good download health.", [this]()
				{
					GetAllChunks(LargeInitialDownloadSet.Array());
					TEST_TRUE(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0);
					if (MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0)
					{
						TEST_EQUAL(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Last().Get<1>(), EBuildPatchDownloadHealth::Good);
					}
				});
			});

			Describe("when up to 10% download failures are occurring", [this]()
			{
				BeforeEach([this]()
				{
					const int32 OneMoreChunk = LargeInitialDownloadSet.Num() + 1;
					CloudSourceConfig->PreFetchMinimum = OneMoreChunk;
					CloudSourceConfig->PreFetchMaximum = OneMoreChunk;
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					ForceConnectionCount = OneMoreChunk;
					FakeDownloadService->ThreadLock.Lock();
					for (int32 Count = 0; (float(Count) / float(Count + LargeInitialDownloadSet.Num())) <= 0.05f; ++Count)
					{
						FakeDownloadService->TxRequestFile.Emplace(0.0f, false, EHttpResponseCodes::Unknown, FirstHeader);
					}
					FakeDownloadService->ThreadLock.Unlock();
					FakeDownloadService->StartService();
					MakeUnit(LargeInitialDownloadSet);
				});

				It("should end with OK download health.", [this]()
				{
					GetAllChunks(LargeInitialDownloadSet.Array());
					TEST_TRUE(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0);
					if (MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0)
					{
						TEST_EQUAL(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Last().Get<1>(), EBuildPatchDownloadHealth::OK);
					}
				});
			});

			Describe("when over 10% download failures are occurring", [this]()
			{
				BeforeEach([this]()
				{
					const int32 OneMoreChunk = LargeInitialDownloadSet.Num() + 1;
					CloudSourceConfig->PreFetchMinimum = OneMoreChunk;
					CloudSourceConfig->PreFetchMaximum = OneMoreChunk;
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					ForceConnectionCount = OneMoreChunk;
					FakeDownloadService->ThreadLock.Lock();
					for (int32 Count = 0; (float(Count) / float(Count + LargeInitialDownloadSet.Num())) <= 0.1f; ++Count)
					{
						FakeDownloadService->TxRequestFile.Emplace(0.0f, false, EHttpResponseCodes::Unknown, FirstHeader);
					}
					FakeDownloadService->ThreadLock.Unlock();
					FakeDownloadService->StartService();
					MakeUnit(LargeInitialDownloadSet);
				});

				It("should end with Poor download health.", [this]()
				{
					GetAllChunks(LargeInitialDownloadSet.Array());
					TEST_TRUE(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0);
					if (MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0)
					{
						TEST_EQUAL(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Last().Get<1>(), EBuildPatchDownloadHealth::Poor);
					}
				});
			});

			Describe("when all downloads are failing", [this]()
			{
				BeforeEach([this]()
				{
					const int32 OneMoreChunk = InitialDownloadSet.Num() + 1;
					CloudSourceConfig->PreFetchMinimum = OneMoreChunk;
					CloudSourceConfig->PreFetchMaximum = OneMoreChunk;
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					CloudSourceConfig->MaxRetryCount = 1;
					CloudSourceConfig->DisconnectedDelay = 0.0f;
					ForceConnectionCount = InitialDownloadSet.Num() / 3;
					FakeDownloadService->ThreadLock.Lock();
					for (int32 Count = 0; Count < OneMoreChunk; ++Count)
					{
						FakeDownloadService->TxRequestFile.Emplace(0.0f, false, EHttpResponseCodes::Unknown, FirstHeader);
					}
					FakeDownloadService->ThreadLock.Unlock();
					FakeDownloadService->StartService();
					MakeUnit(InitialDownloadSet);
				});

				It("should end with Disconnected download health.", [this]()
				{
					GetAllChunks(InitialDownloadSet.Array());
					TEST_TRUE(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0);
					if (MockCloudChunkSourceStat->RxDownloadHealthUpdated.Num() > 0)
					{
						TEST_EQUAL(MockCloudChunkSourceStat->RxDownloadHealthUpdated.Last().Get<1>(), EBuildPatchDownloadHealth::Disconnected);
					}
				});
			});

			Describe("when config says to prefetch more than initial list", [this]()
			{
				BeforeEach([this]()
				{
					CloudSourceConfig->PreFetchMinimum = InitialDownloadSet.Num() + 1;
					CloudSourceConfig->PreFetchMaximum = InitialDownloadSet.Num() + 1;
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					FakeDownloadService->StartService();
					MakeUnit(InitialDownloadSet);
				});

				xIt("should automatically request all chunks.", [this]()
				{
					TArray<FGuid> Unrequested = CheckForChunkRequests(InitialDownloadSet.Array());
					TEST_EQUAL(Unrequested.Num(), 0);
				});
			});

			Describe("when config says to prefetch equal number of chunks to initial list", [this]()
			{
				BeforeEach([this]()
				{
					CloudSourceConfig->PreFetchMinimum = InitialDownloadSet.Num();
					CloudSourceConfig->PreFetchMaximum = InitialDownloadSet.Num();
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					FakeDownloadService->StartService();
					MakeUnit(InitialDownloadSet);
				});

				xIt("should automatically request all chunks.", [this]()
				{
					TArray<FGuid> Unrequested = CheckForChunkRequests(InitialDownloadSet.Array());
					TEST_EQUAL(Unrequested.Num(), 0);
				});
			});

			Describe("when config says to prefetch 5 of the initial list", [this]()
			{
				BeforeEach([this]()
				{
					CloudSourceConfig->PreFetchMinimum = 5;
					CloudSourceConfig->PreFetchMaximum = 5;
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					FakeDownloadService->StartService();
					MakeUnit(InitialDownloadSet);
				});

				xIt("should automatically request 5 chunks.", [this]()
				{
					TArray<FGuid> Unrequested = CheckForChunkRequests(InitialDownloadSet.Array(), 1.0f);
					TEST_EQUAL(Unrequested.Num(), InitialDownloadSet.Num() - 5);
				});
			});

			Describe("when a chunk takes longer that the expected limit", [this]()
			{
				BeforeEach([this]()
				{
					CloudSourceConfig->PreFetchMinimum = 100;
					CloudSourceConfig->PreFetchMaximum = 100;
					CloudSourceConfig->TcpZeroWindowMinimumSeconds = 0.5f;
					CloudSourceConfig->bBeginDownloadsOnFirstGet = false;
					FakeDownloadService->ThreadLock.Lock();
					FakeDownloadService->TxRequestFile.Emplace(1.0f, true, EHttpResponseCodes::Ok, FirstHeader);
					FakeDownloadService->ThreadLock.Unlock();
					FakeDownloadService->StartService();
					MakeUnit(InitialDownloadSet);
				});

				xIt("should abort the chunk and retry it.", [this]()
				{
					TArray<FGuid> Succeeded = GetAllChunks(InitialDownloadSet.Array());
					TEST_EQUAL(Succeeded.Num(), InitialDownloadSet.Num());
					TEST_EQUAL(FakeDownloadService->RxRequestCancel.Num(), 1);
					TEST_EQUAL(MockCloudChunkSourceStat->RxDownloadAborted.Num(), 1);
				});
			});
		});
	});

	AfterEach([this]()
	{
		FakeDownloadService->StopService();
		CloudChunkSource.Reset();
		CloudSourceConfig.Reset();
		FakeChunkStore.Reset();
		FakeDownloadService.Reset();
		MockChunkReferenceTracker.Reset();
		FakeChunkDataSerialization.Reset();
		MockMessagePump.Reset();
		MockConnectionCount.Reset();
		MockInstallerError.Reset();
		MockCloudChunkSourceStat.Reset();
		ManifestSet.Reset();
		MockManifest.Reset();
		GLog->FlushThreadedLogs();
	});
}

void FCloudChunkSourceSpec::MakeUnit(TSet<FGuid> DownloadSet)
{
	MockConnectionCount.Reset(new BuildPatchServices::FMockDownloadConnectionCount(ForceConnectionCount));
	CloudChunkSource.Reset(BuildPatchServices::FCloudChunkSourceFactory::Create(
		*CloudSourceConfig.Get(),
		MockPlatform.Get(),
		FakeChunkStore.Get(),
		FakeDownloadService.Get(),
		MockChunkReferenceTracker.Get(),
		FakeChunkDataSerialization.Get(),
		MockMessagePump.Get(),
		MockInstallerError.Get(),
		MockConnectionCount.Get(),
		MockCloudChunkSourceStat.Get(),
		ManifestSet.Get(),
		DownloadSet));
}

TArray<FGuid> FCloudChunkSourceSpec::CheckForChunkRequests(TArray<FGuid> ExpectedChunks, float SecondsLimit)
{
	const double TimeStarted = BuildPatchServices::FStatsCollector::GetSeconds();
	bool bWaiting = true;
	while (bWaiting)
	{
		TArray<BuildPatchServices::FMockDownloadService::FRequestFile> CopyCalls;
		FakeDownloadService->ThreadLock.Lock();
		CopyCalls = FakeDownloadService->RxRequestFile;
		FakeDownloadService->ThreadLock.Unlock();
		ExpectedChunks.RemoveAll([&CopyCalls](const FGuid& Element)
		{
			for (const BuildPatchServices::FMockDownloadService::FRequestFile& RequestFile : CopyCalls)
			{
				if (RequestFile.Get<2>().Contains(Element.ToString()))
				{
					return true;
				}
			}
			return false;
		});
		const double TimeWaiting = BuildPatchServices::FStatsCollector::GetSeconds() - TimeStarted;
		bWaiting = ExpectedChunks.Num() > 0 && TimeWaiting < SecondsLimit;
		FPlatformProcess::Sleep(0);
	}
	return ExpectedChunks;
}

TArray<FGuid> FCloudChunkSourceSpec::GetAllChunks(TArray<FGuid> Chunks)
{
	TArray<FGuid> Successes;
	for (const FGuid& Chunk : Chunks)
	{
		if (CloudChunkSource->Get(Chunk) != nullptr)
		{
			Successes.Add(Chunk);
		}
	}
	return Successes;
}

TFuture<void> FCloudChunkSourceSpec::PauseFor(float Seconds)
{
	Seconds *= PausePadding;
	double PauseAt = BuildPatchServices::FStatsCollector::GetSeconds();
	CloudChunkSource->SetPaused(true);
	TFunction<void()> Task = [this, PauseAt, Seconds]()
	{
		while ((BuildPatchServices::FStatsCollector::GetSeconds() - PauseAt) < Seconds)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		CloudChunkSource->SetPaused(false);
	};
	return Async(EAsyncExecution::Thread, MoveTemp(Task));
}

#endif //WITH_DEV_AUTOMATION_TESTS
