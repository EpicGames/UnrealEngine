// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/TestHelpers.h"
#include "Tests/Fake/HttpManager.fake.h"
#include "Tests/Mock/FileSystem.mock.h"
#include "Tests/Mock/DownloadServiceStat.mock.h"
#include "Tests/Mock/InstallerAnalytics.mock.h"
#include "Installer/DownloadService.h"
#include "Containers/Ticker.h"
#include "BuildPatchHash.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FDownloadServiceSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
// Unit.
TUniquePtr<BuildPatchServices::IDownloadService> DownloadService;
// Mock/Fake.
FTSTicker Ticker;
TSharedPtr<BuildPatchServices::FFakeHttpManager> FakeHttpModule;
TSharedPtr<BuildPatchServices::FMockFileSystem> MockFileSystem;
TSharedPtr<BuildPatchServices::FMockDownloadServiceStat> MockDownloadServiceStat;
TSharedPtr<BuildPatchServices::FMockInstallerAnalytics> MockInstallerAnalytics;
// Data.
BuildPatchServices::FDownloadProgressDelegate DownloadProgress;
BuildPatchServices::FDownloadCompleteDelegate DownloadComplete;
TArray<TTuple<double, int32, int32>> RxDownloadProgress;
TArray<TTuple<double, int32, BuildPatchServices::FDownloadRef>> RxDownloadComplete;
FString HttpFileUrl;
FString HttpsFileUrl;
FString NetworkFileUrl;
int32 MadeRequestId;
// Ticker helpers.
void DoTick();
bool DoTicksUntilCreated(float Seconds = 5.0);
bool DoTicksUntilComplete(float Seconds = 5.0, int32 CompleteCount = 1);
bool DoTicksUntil(float Seconds, const TFunction<bool()>& pred);
int RxCreateFileReaderNum() const;

END_DEFINE_SPEC(FDownloadServiceSpec)

void FDownloadServiceSpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	FRollingHashConst::Init();
	DownloadProgress.BindLambda([this](int32 RequestId, int32 BytesSoFar)
	{
		RxDownloadProgress.Emplace(FStatsCollector::GetSeconds(), RequestId, BytesSoFar);
	});
	DownloadComplete.BindLambda([this](int32 RequestId, const FDownloadRef& Download)
	{
		RxDownloadComplete.Emplace(FStatsCollector::GetSeconds(), RequestId, Download);
	});
	HttpFileUrl = TEXT("http://download.tests.com/file.dat");
	HttpsFileUrl = TEXT("https://download.tests.com/file.dat");
	NetworkFileUrl = TEXT("\\\\somenetwork\\somefolder\\file.dat");

	// Specs.
	BeforeEach([this]()
	{
		Ticker.Reset();
		FakeHttpModule = MakeShared<FFakeHttpManager>(Ticker);
		MockFileSystem = MakeShared<FMockFileSystem>();
		MockDownloadServiceStat = MakeShared<FMockDownloadServiceStat>();
		MockInstallerAnalytics = MakeShared<FMockInstallerAnalytics>();
		DownloadService.Reset(FDownloadServiceFactory::Create(
			FakeHttpModule,
			MockFileSystem,
			MockDownloadServiceStat,
			MockInstallerAnalytics));
	});

	xDescribe("DownloadService", [this]()
	{
		Describe("RequestFile", [this]()
		{
			Describe("when given a FileUri starting with http", [this]()
			{
				It("should use the http module to process the request.", [this]()
				{
					DownloadService->RequestFile(HttpFileUrl, DownloadComplete, DownloadProgress);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(FakeHttpModule->RxCreateRequest, 1);
					TEST_EQUAL(MockFileSystem->RxCreateFileReader.Num(), 0);
				});
			});

			Describe("when given a FileUri starting with https", [this]()
			{
				It("should use the http module to process the request.", [this]()
				{
					DownloadService->RequestFile(HttpsFileUrl, DownloadComplete, DownloadProgress);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(FakeHttpModule->RxCreateRequest, 1);
					TEST_EQUAL(MockFileSystem->RxCreateFileReader.Num(), 0);
				});
			});

			Describe("when given a FileUri not starting with http", [this]()
			{
				xIt("should use the file manager to process the request.", [this]()
				{
					DownloadService->RequestFile(NetworkFileUrl, DownloadComplete, DownloadProgress);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(FakeHttpModule->RxCreateRequest, 0);
					TEST_EQUAL(MockFileSystem->RxCreateFileReader.Num(), 1);
				});
			});

			Describe("when the http request results in success", [this]()
			{
				BeforeEach([this]()
				{
					FakeHttpModule->DataServed.Add(HttpFileUrl, {1,2,3,4,5,6,7,8,9,10});
				});

				It("should provide an IDownload with access to success status.", [this]()
				{
					DownloadService->RequestFile(HttpFileUrl, DownloadComplete, DownloadProgress);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(RxDownloadComplete.Num(), 1);
					if (RxDownloadComplete.Num() == 1)
					{
						const FDownloadRef& Download = RxDownloadComplete[0].Get<2>();
						TEST_TRUE(Download->ResponseSuccessful());
						TEST_EQUAL(Download->GetResponseCode(), EHttpResponseCodes::Ok);
						TEST_EQUAL(Download->GetData(), FakeHttpModule->DataServed[HttpFileUrl]);
					}
				});
			});

			Describe("when the file request results in success", [this]()
			{
				BeforeEach([this]()
				{
					MockFileSystem->ReadFile = {1,2,3,4,5,6,7,8,9,10};
				});

				xIt("should provide an IDownload with access to success status.", [this]()
				{
					DownloadService->RequestFile(NetworkFileUrl, DownloadComplete, DownloadProgress);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(RxDownloadComplete.Num(), 1);
					if (RxDownloadComplete.Num() == 1)
					{
						const FDownloadRef& Download = RxDownloadComplete[0].Get<2>();
						TEST_TRUE(Download->ResponseSuccessful());
						TEST_EQUAL(Download->GetResponseCode(), EHttpResponseCodes::Ok);
						TEST_EQUAL(Download->GetData(), MockFileSystem->ReadFile);
					}
				});
			});
		});

		Describe("RequestCancel", [this]()
		{
			Describe("when a file request was made", [this]()
			{
				BeforeEach([this]()
				{
					MockFileSystem->ReadFile.AddUninitialized(1024 * 1024 * 50);
					DownloadService->RequestFile(NetworkFileUrl, DownloadComplete, DownloadProgress, &MadeRequestId);
				});

				It("should cancel if it was not started yet.", [this]()
				{
					DownloadService->RequestCancel(MadeRequestId);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(FakeHttpModule->RxCreateRequest, 0);
					TEST_EQUAL(MockFileSystem->RxCreateFileReader.Num(), 0);
					TEST_EQUAL(RxDownloadComplete.Num(), 1);
					if (RxDownloadComplete.Num() == 1)
					{
						const FDownloadRef& Download = RxDownloadComplete[0].Get<2>();
						TEST_FALSE(Download->ResponseSuccessful());
					}
				});

				xIt("should cancel if it had already started.", [this]()
				{
					TEST_TRUE(DoTicksUntilCreated());
					DownloadService->RequestCancel(MadeRequestId);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(FakeHttpModule->RxCreateRequest, 0);
					TEST_EQUAL(RxDownloadComplete.Num(), 1);
					if (RxDownloadComplete.Num() == 1)
					{
						const FDownloadRef& Download = RxDownloadComplete[0].Get<2>();
						TEST_FALSE(Download->ResponseSuccessful());
					}
				});
			});

			Describe("when a HTTP request was made", [this]()
			{
				BeforeEach([this]()
				{
					FakeHttpModule->DataServed.FindOrAdd(HttpFileUrl).AddUninitialized(1024*1024*50);
					DownloadService->RequestFile(HttpFileUrl, DownloadComplete, DownloadProgress, &MadeRequestId);
				});

				It("should cancel if it was not started yet.", [this]()
				{
					DownloadService->RequestCancel(MadeRequestId);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(FakeHttpModule->RxCreateRequest, 0);
					TEST_EQUAL(MockFileSystem->RxCreateFileReader.Num(), 0);
					TEST_EQUAL(RxDownloadComplete.Num(), 1);
					if (RxDownloadComplete.Num() == 1)
					{
						const FDownloadRef& Download = RxDownloadComplete[0].Get<2>();
						TEST_FALSE(Download->ResponseSuccessful());
					}
				});

				It("should cancel if it had already started.", [this]()
				{
					TEST_TRUE(DoTicksUntilCreated());
					DownloadService->RequestCancel(MadeRequestId);
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(FakeHttpModule->RxCreateRequest, 1);
					TEST_EQUAL(MockFileSystem->RxCreateFileReader.Num(), 0);
					TEST_EQUAL(RxDownloadComplete.Num(), 1);
					if (RxDownloadComplete.Num() == 1)
					{
						const FDownloadRef& Download = RxDownloadComplete[0].Get<2>();
						TEST_FALSE(Download->ResponseSuccessful());
					}
				});
			});
		});

		Describe("Destructor", [this]()
		{
			Describe("when there are currently active http requests", [this]()
			{
				It("should cancel the request.", [this]()
				{
					DownloadService->RequestFile(HttpFileUrl, DownloadComplete, DownloadProgress);
					TEST_TRUE(DoTicksUntilCreated());
					DownloadService.Reset();
					TEST_TRUE(DoTicksUntilComplete());
					TEST_EQUAL(FakeHttpModule->RxCreateRequest, 1);
					TEST_EQUAL(MockFileSystem->RxCreateFileReader.Num(), 0);
					TEST_EQUAL(RxDownloadComplete.Num(), 1);
					if (RxDownloadComplete.Num() == 1)
					{
						const FDownloadRef& Download = RxDownloadComplete[0].Get<2>();
						TEST_FALSE(Download->ResponseSuccessful());
					}
				});
			});
		});
	});

	AfterEach([this]()
	{
		DownloadService.Reset();
		RxDownloadProgress.Reset();
		RxDownloadComplete.Reset();
		FakeHttpModule.Reset();
		MockFileSystem.Reset();
		MockDownloadServiceStat.Reset();
		MockInstallerAnalytics.Reset();
	});
}

void FDownloadServiceSpec::DoTick()
{
	Ticker.Tick(0.1f);
}

bool FDownloadServiceSpec::DoTicksUntilComplete(float Seconds, int32 CompleteCount)
{
	return DoTicksUntil(Seconds, [this, CompleteCount]() {return RxDownloadComplete.Num() == CompleteCount; });
}

bool FDownloadServiceSpec::DoTicksUntil(float Seconds, const TFunction<bool()>& pred)
{
	FDateTime Start = FDateTime::UtcNow();
	do
	{
		if (pred())
		{
			return true;
		}
		DoTick();
		FPlatformProcess::Sleep(.005f);
	} while ((FDateTime::UtcNow() - Start) < FTimespan::FromSeconds(Seconds));

	return pred();
}

bool FDownloadServiceSpec::DoTicksUntilCreated(float Seconds)
{
	return DoTicksUntil(Seconds, [this]() { return FakeHttpModule->RxCreateRequest == 1 || RxCreateFileReaderNum() == 1; });
}

int FDownloadServiceSpec::RxCreateFileReaderNum() const
{
	FScopeLock Lock(&MockFileSystem->ThreadLock);
	return MockFileSystem->RxCreateFileReader.Num();
}

#endif //WITH_DEV_AUTOMATION_TESTS
