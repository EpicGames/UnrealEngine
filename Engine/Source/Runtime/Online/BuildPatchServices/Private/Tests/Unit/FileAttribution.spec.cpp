// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Algo/Accumulate.h"
#include "Tests/TestHelpers.h"
#include "Tests/Mock/FileSystem.mock.h"
#include "Tests/Mock/Manifest.mock.h"
#include "Tests/Mock/BuildPatchProgress.mock.h"
#include "Installer/FileAttribution.h"
#include "IBuildManifestSet.h"
#include "BuildPatchSettings.h"
//#include "BuildPatchManifestSetItem.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FFileAttributionSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
// Unit.
TUniquePtr<BuildPatchServices::IFileAttribution> FileAttribution;
// Mock.
TUniquePtr<BuildPatchServices::FMockFileSystem> MockFileSystem;
BuildPatchServices::FMockManifestPtr MockNewManifest;
BuildPatchServices::FMockManifestPtr MockOldManifest;
TUniquePtr<BuildPatchServices::FMockBuildPatchProgress> MockBuildProgress;
TUniquePtr<BuildPatchServices::IBuildManifestSet> ManifestSet;
// Data.
FString InstallDirectory;
FString StagedFileDirectory;
FString MissingFile;
TSet<FString> NewFiles;
TSet<FString> ChangedFiles;
TSet<FString> SameFiles;
TSet<FString> ExeFiles;
TSet<FString> ReadOnlyFiles;
TSet<FString> CompressedFiles;
TSet<FString> TouchedFiles;
TSet<FString> AllFiles;
float PauseTime;
bool bHasPaused;
// Test helpers.
void MakeUnit();
TFuture<void> PauseFor(float Seconds);
END_DEFINE_SPEC(FFileAttributionSpec)

void FFileAttributionSpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	InstallDirectory = TEXT("InstallDir/");
	StagedFileDirectory = TEXT("StagedFileDir/");
	ExeFiles.Add(NewFiles[NewFiles.Add(TEXT("New/Exe"))]);
	ReadOnlyFiles.Add(NewFiles[NewFiles.Add(TEXT("New/ReadOnly"))]);
	CompressedFiles.Add(NewFiles[NewFiles.Add(TEXT("New/Compressed"))]);
	ExeFiles.Add(ChangedFiles[ChangedFiles.Add(TEXT("Changed/Exe"))]);
	ReadOnlyFiles.Add(ChangedFiles[ChangedFiles.Add(TEXT("Changed/ReadOnly"))]);
	CompressedFiles.Add(ChangedFiles[ChangedFiles.Add(TEXT("Changed/Compressed"))]);
	ExeFiles.Add(SameFiles[SameFiles.Add(TEXT("Same/Exe"))]);
	ReadOnlyFiles.Add(SameFiles[SameFiles.Add(TEXT("Same/ReadOnly"))]);
	CompressedFiles.Add(SameFiles[SameFiles.Add(TEXT("Same/Compressed"))]);
	TouchedFiles = NewFiles.Union(ChangedFiles);
	AllFiles = NewFiles.Union(ChangedFiles).Union(SameFiles);
	MissingFile = TEXT("MissingFile.dat");

	// Specs.
	BeforeEach([this]()
	{
		MockFileSystem.Reset(new FMockFileSystem());
		MockBuildProgress.Reset(new FMockBuildPatchProgress());
		MakeUnit();
	});

	Describe("FileAttribution", [this]()
	{
		Describe("Construction", [this]()
		{
			It("should initialize progress to 0.", [this]()
			{
				TEST_EQUAL(MockBuildProgress->RxSetStateProgress.Num(), 1);
				if (MockBuildProgress->RxSetStateProgress.Num() == 1)
				{
					TEST_EQUAL(MockBuildProgress->RxSetStateProgress[0].Get<1>(), EBuildPatchState::SettingAttributes);
					TEST_EQUAL(MockBuildProgress->RxSetStateProgress[0].Get<2>(), 0.0f);
				}
			});
		});

		Describe("ApplyAttributes", [this]()
		{
			Describe("when some files were staged only", [this]()
			{
				BeforeEach([this]()
				{
					for (const FString& File : NewFiles)
					{
						MockFileSystem->FileSizes.Add(StagedFileDirectory / File, 64);
						MockFileSystem->FileAttributes.Add(StagedFileDirectory / File, EAttributeFlags::Exists);
					}
				});

				xIt("should select staged files instead of installed files if they exist.", [this]()
				{
					FileAttribution->ApplyAttributes();
					TSet<FString> StagedFiles;
					for (const FString& File : NewFiles)
					{
						StagedFiles.Add(StagedFileDirectory / File);
					}
					TSet<FString> AppliedFiles;
					for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetReadOnly)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					// TODO: fix this the correct way
					auto StagedFileTest = AppliedFiles.Intersect(StagedFiles);
					TEST_TRUE(StagedFileTest.Num() == 1);
					//TEST_EQUAL(StagedFileTest, StagedFiles);
				});
			});

			Describe("when a file is known missing", [this]()
			{
				BeforeEach([this]()
				{
					FFileManifest FileManifest;
					FileManifest.Filename = MissingFile;
					MockFileSystem->FileSizes.Add(StagedFileDirectory / MissingFile, INDEX_NONE);
					MockFileSystem->FileAttributes.Add(StagedFileDirectory / MissingFile, EAttributeFlags::None);
					MockFileSystem->FileSizes.Add(InstallDirectory / MissingFile, INDEX_NONE);
					MockFileSystem->FileAttributes.Add(InstallDirectory / MissingFile, EAttributeFlags::None);
					MockNewManifest->BuildFileList.Add(MissingFile);
					MockNewManifest->FileManifests.Add(MissingFile, FileManifest);
				});

				It("should skip making calls to set attributes on that file.", [this]()
				{
					FileAttribution->ApplyAttributes();
					TSet<FString> AppliedFiles;
					for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetCompressed)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetExecutable)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetReadOnly)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					TEST_FALSE(AppliedFiles.Contains(StagedFileDirectory / MissingFile));
					TEST_FALSE(AppliedFiles.Contains(InstallDirectory / MissingFile));
				});
			});

			xDescribe("when called with bForce set to false", [this]()
			{
				xIt("should apply compressed attributes to new or changed compressed files.", [this]()
				{
					FileAttribution->ApplyAttributes(/*false*/);
					TSet<FString> NewOrChangedCompressedFiles;
					for (const FString& File : NewFiles.Union(ChangedFiles).Intersect(CompressedFiles))
					{
						NewOrChangedCompressedFiles.Add(InstallDirectory / File);
					}
					TSet<FString> AppliedFiles;
					for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetCompressed)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					TEST_TRUE(SetsAreEqual(AppliedFiles, NewOrChangedCompressedFiles));
				});

				xIt("should apply readonly attributes to new or changed readonly files.", [this]()
				{
					FileAttribution->ApplyAttributes(/*false*/);
					TSet<FString> NewOrChangedReadOnlyFiles;
					for (const FString& File : NewFiles.Union(ChangedFiles).Intersect(ReadOnlyFiles))
					{
						NewOrChangedReadOnlyFiles.Add(InstallDirectory / File);
					}
					TSet<FString> AppliedFiles;
					for (const FMockFileSystem::FSetReadOnly& Call : MockFileSystem->RxSetReadOnly)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					TEST_TRUE(SetsAreEqual(AppliedFiles, NewOrChangedReadOnlyFiles));
				});

				xIt("should apply executable attributes to new or changed executable files.", [this]()
				{
					FileAttribution->ApplyAttributes(/*false*/);
					TSet<FString> NewOrChangedExecutableFiles;
					for (const FString& File : NewFiles.Union(ChangedFiles).Intersect(ExeFiles))
					{
						NewOrChangedExecutableFiles.Add(InstallDirectory / File);
					}
					TSet<FString> AppliedFiles;
					for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetExecutable)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					TEST_TRUE(SetsAreEqual(AppliedFiles, NewOrChangedExecutableFiles));
				});
			});

			xDescribe("when called with bForce set to true", [this]()
			{
				xIt("should apply compressed attribute to all files.", [this]()
				{
					FileAttribution->ApplyAttributes(/*true*/);
					TSet<FString> FullAllFiles;
					for (const FString& File : AllFiles)
					{
						FullAllFiles.Add(InstallDirectory / File);
					}
					TSet<FString> AppliedFiles;
					for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetCompressed)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					TEST_TRUE(SetsAreEqual(AppliedFiles, FullAllFiles));
				});

				xIt("should apply readonly attributes to all files.", [this]()
				{
					FileAttribution->ApplyAttributes(/*true*/);
					TSet<FString> FullAllFiles;
					for (const FString& File : AllFiles)
					{
						FullAllFiles.Add(InstallDirectory / File);
					}
					TSet<FString> AppliedFiles;
					for (const FMockFileSystem::FSetReadOnly& Call : MockFileSystem->RxSetReadOnly)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					TEST_TRUE(SetsAreEqual(AppliedFiles, FullAllFiles));
				});

				xIt("should apply executable attributes to all files.", [this]()
				{
					FileAttribution->ApplyAttributes(/*true*/);
					TSet<FString> FullAllFiles;
					for (const FString& File : AllFiles)
					{
						FullAllFiles.Add(InstallDirectory / File);
					}
					TSet<FString> AppliedFiles;
					for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetExecutable)
					{
						AppliedFiles.Add(Call.Get<1>());
					}
					TEST_TRUE(SetsAreEqual(AppliedFiles, FullAllFiles));
				});
			});
		});

		Describe("SetPaused", [this]()
		{
			BeforeEach([this]()
			{
				bHasPaused = false;
				PauseTime = 0.1f;
				MockBuildProgress->SetStateProgressFunc = [this](const EBuildPatchState& State, const float& Value)
				{
					if (!bHasPaused && Value > 0.1f)
					{
						bHasPaused = true;
						PauseFor(PauseTime);
					}
				};
			});

			xIt("should delay the attribution process.", [this]()
			{
				FileAttribution->ApplyAttributes();
				double LongestDelay = 0.0f;
				for (int32 Idx = 1; Idx < MockFileSystem->RxSetReadOnly.Num(); ++Idx)
				{
					double ThisDelay = MockFileSystem->RxSetReadOnly[Idx].Get<0>() - MockFileSystem->RxSetReadOnly[Idx - 1].Get<0>();
					if (ThisDelay > LongestDelay)
					{
						LongestDelay = ThisDelay;
					}
				}
				TEST_TRUE(LongestDelay >= PauseTime);
			});
		});

		Describe("Abort", [this]()
		{
			BeforeEach([this]()
			{
				MockBuildProgress->SetStateProgressFunc = [this](const EBuildPatchState&, const float&)
				{
					FileAttribution->Abort();
				};
			});

			It("should halt process and stop.", [this]()
			{
				FileAttribution->ApplyAttributes();
				TSet<FString> AppliedFiles;
				for (const FMockFileSystem::FSetCompressed& Call : MockFileSystem->RxSetReadOnly)
				{
					AppliedFiles.Add(Call.Get<1>());
				}
				TEST_TRUE(AppliedFiles.Num() < AllFiles.Num());
			});
		});
	});

	AfterEach([this]()
	{
		FileAttribution.Reset();
		ManifestSet.Reset();
		MockBuildProgress.Reset();
		MockOldManifest.Reset();
		MockNewManifest.Reset();
		MockFileSystem.Reset();
	});
}

void FFileAttributionSpec::MakeUnit()
{
	using namespace BuildPatchServices;

	MockNewManifest = MakeShareable(new FMockManifest());
	MockOldManifest = MakeShareable(new FMockManifest());
	for (const FString& File : AllFiles)
	{
		FChunkPart ChunkPart;
		ChunkPart.Guid = FGuid::NewGuid();
		ChunkPart.Offset = 0;
		ChunkPart.Size = 64;
		FFileManifest FileManifest;
		FileManifest.Filename = File;
		FSHA1::HashBuffer(*FGuid::NewGuid().ToString(), sizeof(TCHAR) * 32, FileManifest.SHA1Hash.Hash);
		FileManifest.ChunkParts.Add(ChunkPart);
		FileManifest.FileMetaFlags |= ReadOnlyFiles.Contains(File) ? EFileMetaFlags::ReadOnly : EFileMetaFlags::None;
		FileManifest.FileMetaFlags |= CompressedFiles.Contains(File) ? EFileMetaFlags::Compressed : EFileMetaFlags::None;
		FileManifest.FileMetaFlags |= ExeFiles.Contains(File) ? EFileMetaFlags::UnixExecutable : EFileMetaFlags::None;
		FileManifest.FileSize = Algo::Accumulate<int64>(FileManifest.ChunkParts, 0, [] (int64 Count, const FChunkPart& Element) { return Count + Element.Size; });
		MockNewManifest->BuildFileList.Add(File);
		MockNewManifest->FileManifests.Add(File, FileManifest);
		if (SameFiles.Contains(File))
		{
			MockOldManifest->BuildFileList.Add(File);
			MockOldManifest->FileManifests.Add(File, FileManifest);
		}
		else if (ChangedFiles.Contains(File))
		{
			FSHA1::HashBuffer(*FGuid::NewGuid().ToString(), sizeof(TCHAR) * 32, FileManifest.SHA1Hash.Hash);
			MockOldManifest->BuildFileList.Add(File);
			MockOldManifest->FileManifests.Add(File, FileManifest);
		}
		MockFileSystem->FileSizes.Add(InstallDirectory / File, FileManifest.FileSize);
		MockFileSystem->FileAttributes.Add(InstallDirectory / File, EAttributeFlags::Exists);
	}
	ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstallOrUpdate(MockOldManifest, MockNewManifest.ToSharedRef()) }));
	TMap<FString, FString> EmptyPerFileSubdirectories;
	FileAttribution.Reset(BuildPatchServices::FFileAttributionFactory::Create(
		MockFileSystem.Get(),
		ManifestSet.Get(),
		TouchedFiles,
		InstallDirectory,
		StagedFileDirectory,
		MockBuildProgress.Get(),
		EmptyPerFileSubdirectories));
}

TFuture<void> FFileAttributionSpec::PauseFor(float Seconds)
{
	double PauseAt = BuildPatchServices::FStatsCollector::GetSeconds();
	FileAttribution->SetPaused(true);
	TFunction<void()> Task = [this, PauseAt, Seconds]()
	{
		while ((BuildPatchServices::FStatsCollector::GetSeconds() - PauseAt) < Seconds)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		FileAttribution->SetPaused(false);
	};
	return Async(EAsyncExecution::Thread, MoveTemp(Task));
}

#endif //WITH_DEV_AUTOMATION_TESTS
