// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "BuildPatchSettings.h"

#include "Tests/TestHelpers.h"
#include "Tests/Fake/ChunkDataAccess.fake.h"
#include "Tests/Fake/FileSystem.fake.h"
#include "Tests/Fake/ChunkStore.fake.h"
#include "Tests/Fake/ChunkReferenceTracker.fake.h"
#include "Tests/Fake/InstallerError.fake.h"
#include "Tests/Mock/InstallChunkSourceStat.mock.h"
#include "Tests/Mock/Manifest.mock.h"
#include "Installer/InstallChunkSource.h"
#include "Math/RandomStream.h"
#include "BuildPatchHash.h"
#include "Misc/SecureHash.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FInstallChunkSourceSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
const uint32 TestChunkSize = 128 * 1024;
// Unit.
TUniquePtr<BuildPatchServices::IConstructorInstallChunkSource> InstallChunkSource;
// Mock.
TUniquePtr<BuildPatchServices::FFakeFileSystem> FakeFileSystem;
TUniquePtr<BuildPatchServices::FMockInstallChunkSourceStat> MockInstallChunkSourceStat;
BuildPatchServices::FMockManifestPtr MockManifest;
// Data.
TMultiMap<FString, FBuildPatchAppManifestRef> InstallationSources;
TSet<FGuid> SomeAvailableChunks;
FGuid SomeChunk;
TUniquePtr<BuildPatchServices::IBuildManifestSet> ManifestSet;
TArray<uint8> ReadDestination;
// Test helpers.
void MakeUnit();
void InventUsableChunkData();
void SomeChunkAvailable();
void SomeChunkUnavailable();
void SomeChunkCorrupted();
END_DEFINE_SPEC(FInstallChunkSourceSpec)

void FInstallChunkSourceSpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	FRollingHashConst::Init();
	SomeChunk = FGuid::NewGuid();

	// Specs.
	BeforeEach([this]()
	{
		FakeFileSystem.Reset(new FFakeFileSystem());
		MockInstallChunkSourceStat.Reset(new FMockInstallChunkSourceStat());
		MockManifest = MakeShareable(new FMockManifest());
		ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef()) }));
	});

	Describe("InstallChunkSource", [this]()
	{
		Describe("GetAvailableChunks", [this]()
		{
			Describe("when there are no chunks available", [this]()
			{
				BeforeEach([this]()
				{
					MakeUnit();
				});

				It("should return an empty set.", [this]()
				{
					TSet<FGuid> AvailableChunks = InstallChunkSource->GetAvailableChunks();
					TEST_TRUE(AvailableChunks.Num() == 0);
				});
			});

			Describe("when there are some available chunks", [this]()
			{
				BeforeEach([this]()
				{
					InventUsableChunkData();
					MakeUnit();
				});

				It("should return the available chunks.", [this]()
				{
					TSet<FGuid> AvailableChunks = InstallChunkSource->GetAvailableChunks();
					TEST_TRUE(SetsAreEqual(AvailableChunks, SomeAvailableChunks));
				});
			});
		});

		Describe("Get", [this]()
		{
			Describe("when some chunk is not available", [this]()
			{
				BeforeEach([this]()
				{
					InventUsableChunkData();
					SomeChunkUnavailable();
					MakeUnit();
				});

				Describe("when some chunk is not in the store", [this]()
				{
					It("should fail.", [this]()
					{
						FEvent* DoneEvent = FPlatformProcess::GetSynchEventFromPool(true);
						bool bFailed = false;
						TUniqueFunction<void(bool)> Request = InstallChunkSource->CreateRequest(SomeChunk, {}, 0, IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateLambda(
						[&bFailed, DoneEvent](const FGuid& DataId, bool bAborted, bool bFailedToRead, void* UserPtr)
						{
							bFailed = bFailedToRead;
							DoneEvent->Trigger();
						}
						));

						Request(false);
						DoneEvent->Wait();
						FPlatformProcess::ReturnSynchEventToPool(DoneEvent);

						TEST_TRUE(bFailed);
					});
				});
			});

			Describe("when some chunk is available", [this]()
			{
				BeforeEach([this]()
				{
					InventUsableChunkData();
					SomeChunkAvailable();
					MakeUnit();
				});

				Describe("when some chunk is not in the store", [this]()
				{
					It("should return some chunk loading from disk.", [this]()
					{
						FEvent* DoneEvent = FPlatformProcess::GetSynchEventFromPool(true);
						bool bFailed = false;
						TUniqueFunction<void(bool)> Request = InstallChunkSource->CreateRequest(SomeChunk, 
							FMutableMemoryView(ReadDestination.GetData(), ReadDestination.Num()), 
							0, 
							IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateLambda(
								[&bFailed, DoneEvent](const FGuid& DataId, bool bAborted, bool bFailedToRead, void* UserPtr)
								{
									bFailed = bFailedToRead;
									DoneEvent->Trigger();
								}
							));

						Request(false);
						DoneEvent->Wait();
						FPlatformProcess::ReturnSynchEventToPool(DoneEvent);

						TEST_FALSE(bFailed);
						TEST_EQUAL(MockInstallChunkSourceStat->RxLoadStarted.Num(), 1);
						TEST_EQUAL(MockInstallChunkSourceStat->RxLoadComplete.Num(), 1);
						TEST_TRUE(FakeFileSystem->RxCreateFileReader.Num() > 0);
					});
					
					Describe("when some chunk hashes are not known", [this]()
					{
						BeforeEach([this]()
						{
							for (TPair<FString, FBuildPatchAppManifestRef>& InstallationSourcePair : InstallationSources)
							{
								FMockManifest* MockInstallationManifest = (FMockManifest*)&InstallationSourcePair.Value.Get();
								MockInstallationManifest->ChunkInfos.Remove(SomeChunk);
							}
						});

						It("should not have attempted to load some chunk from disk.", [this]()
						{
							FEvent* DoneEvent = FPlatformProcess::GetSynchEventFromPool(true);
							bool bFailed = false;
							TUniqueFunction<void(bool)> Request = InstallChunkSource->CreateRequest(SomeChunk, 
								FMutableMemoryView(ReadDestination.GetData(), ReadDestination.Num()), 
								0, 
								IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateLambda(
									[&bFailed, DoneEvent](const FGuid& DataId, bool bAborted, bool bFailedToRead, void* UserPtr)
									{
										bFailed = bFailedToRead;
										DoneEvent->Trigger();
									}
							));

							Request(false);
							DoneEvent->Wait();
							FPlatformProcess::ReturnSynchEventToPool(DoneEvent);

							TEST_TRUE(bFailed);
							TEST_EQUAL(FakeFileSystem->RxCreateFileReader.Num(), 0);
						});
					});

					Describe("when some chunk sha is not known", [this]()
					{
						BeforeEach([this]()
						{
							for (TPair<FString, FBuildPatchAppManifestRef>& InstallationSourcePair : InstallationSources)
							{
								FMockManifest* MockInstallationManifest = (FMockManifest*)&InstallationSourcePair.Value.Get();
								if (MockInstallationManifest->ChunkInfos.Contains(SomeChunk))
								{
									FMemory::Memzero(MockInstallationManifest->ChunkInfos[SomeChunk].ShaHash.Hash, FSHA1::DigestSize);
								}
							}
						});

						It("should still succeed to load some chunk from disk.", [this]()
						{
							FEvent* DoneEvent = FPlatformProcess::GetSynchEventFromPool(true);
							bool bFailed = false;
							TUniqueFunction<void(bool)> Request = InstallChunkSource->CreateRequest(SomeChunk, 
								FMutableMemoryView(ReadDestination.GetData(), ReadDestination.Num()), 
								0, 
								IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateLambda(
									[&bFailed, DoneEvent](const FGuid& DataId, bool bAborted, bool bFailedToRead, void* UserPtr)
									{
										bFailed = bFailedToRead;
										DoneEvent->Trigger();
									}
							));

							Request(false);
							DoneEvent->Wait();
							FPlatformProcess::ReturnSynchEventToPool(DoneEvent);

							TEST_FALSE(bFailed);
							TEST_EQUAL(MockInstallChunkSourceStat->RxLoadStarted.Num(), 1);
							TEST_EQUAL(MockInstallChunkSourceStat->RxLoadComplete.Num(), 1);
							TEST_TRUE(FakeFileSystem->RxCreateFileReader.Num() > 0);
						});

						Describe("when data required for some chunk is corrupt", [this]()
						{
							BeforeEach([this]()
							{
								SomeChunkCorrupted();
							});

							It("should fail to load some chunk from disk, reporting IInstallChunkSourceStat::ELoadResult::HashCheckFailed.", [this]()
							{
								FEvent* DoneEvent = FPlatformProcess::GetSynchEventFromPool(true);
								bool bFailed = false;
								TUniqueFunction<void(bool)> Request = InstallChunkSource->CreateRequest(SomeChunk, 
									FMutableMemoryView(ReadDestination.GetData(), ReadDestination.Num()), 
									0, 
									IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateLambda(
										[&bFailed, DoneEvent](const FGuid& DataId, bool bAborted, bool bFailedToRead, void* UserPtr)
										{
											bFailed = bFailedToRead;
											DoneEvent->Trigger();
										}
								));

								Request(false);
								DoneEvent->Wait();
								FPlatformProcess::ReturnSynchEventToPool(DoneEvent);

								TEST_TRUE(bFailed);

								TEST_EQUAL(MockInstallChunkSourceStat->RxLoadComplete.Num(), 1);
								if (MockInstallChunkSourceStat->RxLoadComplete.Num() == 1)
								{
									TEST_EQUAL(MockInstallChunkSourceStat->RxLoadComplete[0].Get<2>(), IInstallChunkSourceStat::ELoadResult::HashCheckFailed);
								}
							});
						});
					});

					Describe("when data required for some chunk is corrupt", [this]()
					{
						BeforeEach([this]()
						{
							SomeChunkCorrupted();
						});

						It("should fail to load some chunk from disk, reporting IInstallChunkSourceStat::ELoadResult::HashCheckFailed.", [this]()
						{
							FEvent* DoneEvent = FPlatformProcess::GetSynchEventFromPool(true);
							bool bFailed = false;
							TUniqueFunction<void(bool)> Request = InstallChunkSource->CreateRequest(SomeChunk, 
								FMutableMemoryView(ReadDestination.GetData(), ReadDestination.Num()), 
								0, 
								IConstructorChunkSource::FChunkRequestCompleteDelegate::CreateLambda(
									[&bFailed, DoneEvent](const FGuid& DataId, bool bAborted, bool bFailedToRead, void* UserPtr)
									{
										bFailed = bFailedToRead;
										DoneEvent->Trigger();
									}
							));

							Request(false);
							DoneEvent->Wait();
							FPlatformProcess::ReturnSynchEventToPool(DoneEvent);

							TEST_TRUE(bFailed);
							TEST_EQUAL(MockInstallChunkSourceStat->RxLoadComplete.Num(), 1);
							if (MockInstallChunkSourceStat->RxLoadComplete.Num() == 1)
							{
								TEST_EQUAL(MockInstallChunkSourceStat->RxLoadComplete[0].Get<2>(), IInstallChunkSourceStat::ELoadResult::HashCheckFailed);
							}
						});
					});
				});
			});
		});

	});

	AfterEach([this]()
	{
		FakeFileSystem.Reset();
		MockInstallChunkSourceStat.Reset();
		ManifestSet.Reset();
		MockManifest.Reset();
		InstallationSources.Reset();
		SomeAvailableChunks.Reset();
	});
}

void FInstallChunkSourceSpec::MakeUnit()
{
	using namespace BuildPatchServices;
	TSet<FGuid> ChunksThatWillBeNeeded;
	ManifestSet->GetReferencedChunks(ChunksThatWillBeNeeded);
	static TMap<FString, FString> EmptyPerFileSubdirectories;
	InstallChunkSource.Reset(IConstructorInstallChunkSource::CreateInstallSource(
		FakeFileSystem.Get(),
		MockInstallChunkSourceStat.Get(),
		InstallationSources,
		ChunksThatWillBeNeeded,
		false,
		EmptyPerFileSubdirectories));
}

void FInstallChunkSourceSpec::InventUsableChunkData()
{
	ReadDestination.SetNumUninitialized(TestChunkSize);

	//
	// Make two manifests to act as installation sources. The chunks in the overall manifest
	// are used to make a bunch of files in eacxh installation.
	//
	using namespace BuildPatchServices;
	const int32 ChunkCountPerSource = 50;
	for (int32 Count = 0; Count < ChunkCountPerSource*2; ++Count)
	{
		MockManifest->DataList.Add(FGuid::NewGuid());
	}

	TArray<uint8> ChunkData;
	ChunkData.SetNumUninitialized(TestChunkSize);
	FRandomStream RandomData(0);
		
	int32 FileCounter = 0;

	FMockManifest* MockInstallationManifestA = new FMockManifest();
	FMockManifest* MockInstallationManifestB = new FMockManifest();

	InstallationSources.Add(TEXT("LocationA/"), MakeShareable(MockInstallationManifestA));
	InstallationSources.Add(TEXT("LocationB/"), MakeShareable(MockInstallationManifestB));

	for (int32 ManifestIndex = 0; ManifestIndex < 2; ManifestIndex++)
	{
		FMockManifest* ThisMockManifest = ManifestIndex ? MockInstallationManifestB : MockInstallationManifestA;
		FString InstallLocation = ManifestIndex ? TEXT("LocationB/") : TEXT("LocationA/");

		for (int32 ChunkIndex = 0; ChunkIndex < ChunkCountPerSource; ChunkIndex++)
		{
			const FGuid& TheChunk = MockManifest->DataList[ChunkIndex + ChunkCountPerSource*ManifestIndex];
			SomeAvailableChunks.Add(TheChunk);
			ThisMockManifest->ProducibleChunks.Add(TheChunk);
			ThisMockManifest->ChunksRequiredForFiles.Add(TheChunk);
			MockManifest->ChunksRequiredForFiles.Add(TheChunk);

			// make sure we can evenly divide the chunk in to 4 files without worrying about leftovers,
			// and that we can fill using unsigned ints.
			check((TestChunkSize % 4) == 0);
			
			uint32 ChunkSizeCounter = 0;
			for (int32 FileIdx = 0; FileIdx < 4; ++FileIdx)
			{
				FFileManifest FileManifest;
				FileManifest.Filename = FString::Printf(TEXT("File%d.dat"), FileCounter++);
				FChunkPart& ChunkPart = FileManifest.ChunkParts.AddDefaulted_GetRef();
				ChunkPart.Guid = TheChunk;
				ChunkPart.Offset = ChunkSizeCounter;
				ChunkPart.Size = TestChunkSize / 4;
				ChunkSizeCounter += ChunkPart.Size;

				FileManifest.FileSize = TestChunkSize / 4;

				// Put the chunk's data in our VFS.
				TArray<uint8>& FileData = FakeFileSystem->DiskData.FindOrAdd(InstallLocation / FileManifest.Filename);
				FileData.SetNumUninitialized(ChunkPart.Size);

				uint8* Data = FileData.GetData();
				for (int32 DataIdx = 0; DataIdx <= (FileData.Num()-4); DataIdx += 4)
				{
					*((uint32*)(Data + DataIdx)) = RandomData.GetUnsignedInt();
				}

				FSHA1::HashBuffer(Data, FileManifest.FileSize, FileManifest.SHA1Hash.Hash);

				// Also fill the chunk array so we can hash it later.
				FMemory::Memcpy(&ChunkData[ChunkPart.Offset], Data, ChunkPart.Size);

				ThisMockManifest->BuildFileList.Add(FileManifest.Filename);
				ThisMockManifest->FileNameToFileSize.Add(FileManifest.Filename, FileManifest.FileSize);
				ThisMockManifest->FileNameToHashes.Add(FileManifest.Filename,FileManifest.SHA1Hash);
				ThisMockManifest->FileManifests.Add(FileManifest.Filename, MoveTemp(FileManifest));
			}

			uint64 ChunkPolyHash;
			FSHAHash ChunkShaHash;
			ChunkPolyHash = FRollingHash::GetHashForDataSet(ChunkData.GetData(), ChunkData.Num());
			FSHA1::HashBuffer(ChunkData.GetData(), TestChunkSize, ChunkShaHash.Hash);

			FChunkInfo ChunkInfo;
			ChunkInfo.Guid = TheChunk;
			ChunkInfo.Hash = ChunkPolyHash;
			ChunkInfo.ShaHash = ChunkShaHash;
			ChunkInfo.GroupNumber = 0;
			ChunkInfo.DataSizeUncompressed = TestChunkSize;
			ChunkInfo.FileSize = TestChunkSize;
			ThisMockManifest->ChunkInfos.Add(TheChunk, ChunkInfo);
		}
	}
}

void FInstallChunkSourceSpec::SomeChunkAvailable()
{
	SomeChunk = *SomeAvailableChunks.CreateConstIterator();
}

void FInstallChunkSourceSpec::SomeChunkUnavailable()
{
	SomeChunk = FGuid::NewGuid();
}

void FInstallChunkSourceSpec::SomeChunkCorrupted()
{
	// Find where the data is for SomeChunk and corrupt it.
	using namespace BuildPatchServices;
	for (TPair<FString, FBuildPatchAppManifestRef>& InstallationSourcePair : InstallationSources)
	{
		FMockManifest* MockInstallationManifest = (FMockManifest*)&InstallationSourcePair.Value.Get();
		for (TPair<FString, FFileManifest>& File : MockInstallationManifest->FileManifests)
		{
			for (FChunkPart& ChunkPart : File.Value.ChunkParts)
			{
				if (ChunkPart.Guid == SomeChunk)
				{
					TArray<uint8>& FileData = FakeFileSystem->DiskData[InstallationSourcePair.Key / File.Value.Filename];
					FMemory::Memzero(FileData.GetData(), FileData.Num());
				}
			}
		}
	}
}


#endif //WITH_DEV_AUTOMATION_TESTS
