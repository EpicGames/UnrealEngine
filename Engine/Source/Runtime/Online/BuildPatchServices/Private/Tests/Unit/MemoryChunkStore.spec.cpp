// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Installer/MemoryChunkStore.h"
#include "Tests/TestHelpers.h"
#include "Tests/Mock/ChunkEvictionPolicy.mock.h"
#include "Tests/Fake/ChunkStore.fake.h"
#include "Tests/Mock/MemoryChunkStoreStat.mock.h"
#include "Tests/Fake/ChunkDataAccess.fake.h"
#include "Algo/Transform.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FMemoryChunkStoreSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
// Unit
TUniquePtr<BuildPatchServices::IMemoryChunkStore> MemoryChunkStore;
// Mock
TUniquePtr<BuildPatchServices::FMockChunkEvictionPolicy> MockChunkEvictionPolicy;
TUniquePtr<BuildPatchServices::FFakeChunkStore> FakeChunkStore;
TUniquePtr<BuildPatchServices::FMockMemoryChunkStoreStat> MockMemoryChunkStoreStat;
// Data
int32 StoreSize;
TSet<FGuid> SomeChunks;
TSet<FGuid> CleanableChunks;
TSet<FGuid> BootableChunks;
FGuid SomeChunk;
END_DEFINE_SPEC(FMemoryChunkStoreSpec)

void FMemoryChunkStoreSpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	StoreSize = 15;
	for (int32 Count = 0; Count < 30; ++Count)
	{
		FGuid NewGuid = FGuid::NewGuid();
		SomeChunks.Add(NewGuid);
		switch (Count % 5)
		{
			case 0:
				CleanableChunks.Add(NewGuid);
				break;
			case 1:
				BootableChunks.Add(NewGuid);
				break;
			default:
				break;
		}
	}
	SomeChunk = *SomeChunks.CreateConstIterator();

	// Specs.
	BeforeEach([this]()
	{
		MockChunkEvictionPolicy.Reset(new FMockChunkEvictionPolicy());
		FakeChunkStore.Reset(new FFakeChunkStore());
		MockMemoryChunkStoreStat.Reset(new FMockMemoryChunkStoreStat());
		MemoryChunkStore.Reset(FMemoryChunkStoreFactory::Create(
			StoreSize,
			MockChunkEvictionPolicy.Get(),
			FakeChunkStore.Get(),
			MockMemoryChunkStoreStat.Get(),
			nullptr));
	});

	Describe("MemoryChunkStore", [this]()
	{
		Describe("DumpToOverflow", [this]()
		{
			Describe("when some chunks were previously Put", [this]()
			{
				BeforeEach([this]()
				{
					for (const FGuid& Chunk : SomeChunks)
					{
						MemoryChunkStore->Put(Chunk, TUniquePtr<IChunkDataAccess>(new FFakeChunkDataAccess()));
					}
				});

				It("should Put all chunks to the overflow store.", [this]()
				{
					MemoryChunkStore->DumpToOverflow();
					TSet<FGuid> PutChunks;
					Algo::Transform(FakeChunkStore->RxPut, PutChunks, [](const FMockChunkStore::FPut& Call) { return Call.Get<1>(); });
					TEST_TRUE(SetsAreEqual(SomeChunks, PutChunks));
				});

				Describe("when a chunks was previously Get", [this]()
				{
					BeforeEach([this]()
					{
						MemoryChunkStore->Get(SomeChunk);
					});

					It("should Put all chunks to the overflow store.", [this]()
					{
						MemoryChunkStore->DumpToOverflow();
						TSet<FGuid> PutChunks;
						Algo::Transform(FakeChunkStore->RxPut, PutChunks, [](const FMockChunkStore::FPut& Call) { return Call.Get<1>(); });
						TEST_TRUE(SetsAreEqual(SomeChunks, PutChunks));
					});
				});
			});
		});

		Describe("Put", [this]()
		{
			It("should query the chunk eviction policy.", [this]()
			{
				MemoryChunkStore->Put(SomeChunk, TUniquePtr<IChunkDataAccess>(new FFakeChunkDataAccess()));
				TEST_EQUAL(MockChunkEvictionPolicy->RxQuery.Num(), 1);
			});

			Describe("when some chunks were previously Put", [this]()
			{
				BeforeEach([this]()
				{
					for (const FGuid& Chunk : SomeChunks)
					{
						MemoryChunkStore->Put(Chunk, TUniquePtr<IChunkDataAccess>(new FFakeChunkDataAccess()));
					}
				});

				Describe("when there are cleanable chunks", [this]()
				{
					BeforeEach([this]()
					{
						MockChunkEvictionPolicy->Cleanable = CleanableChunks;
					});

					It("should release them.", [this]()
					{
						MemoryChunkStore->Put(SomeChunk, TUniquePtr<IChunkDataAccess>(new FFakeChunkDataAccess()));
						TSet<FGuid> ReleasedChunks;
						Algo::Transform(MockMemoryChunkStoreStat->RxChunkReleased, ReleasedChunks, [](const FMockMemoryChunkStoreStat::FChunkReleased& Call) { return Call.Get<1>(); });
						TEST_TRUE(SetsAreEqual(CleanableChunks, ReleasedChunks));
					});
				});

				Describe("when there are bootable chunks", [this]()
				{
					BeforeEach([this]()
					{
						MockChunkEvictionPolicy->Bootable = BootableChunks;
					});

					It("should boot them.", [this]()
					{
						MemoryChunkStore->Put(SomeChunk, TUniquePtr<IChunkDataAccess>(new FFakeChunkDataAccess()));
						TSet<FGuid> BootedChunks;
						Algo::Transform(MockMemoryChunkStoreStat->RxChunkBooted, BootedChunks, [](const FMockMemoryChunkStoreStat::FChunkBooted& Call) { return Call.Get<1>(); });
						TEST_TRUE(SetsAreEqual(BootableChunks, BootedChunks));
					});

					It("should Put them to the overflow store.", [this]()
					{
						MemoryChunkStore->Put(SomeChunk, TUniquePtr<IChunkDataAccess>(new FFakeChunkDataAccess()));
						TSet<FGuid> PutChunks;
						Algo::Transform(FakeChunkStore->RxPut, PutChunks, [](const FMockChunkStore::FPut& Call) { return Call.Get<1>(); });
						TEST_TRUE(SetsAreEqual(BootableChunks, PutChunks));
					});
				});
			});
		});

		Describe("Get", [this]()
		{
			Describe("when no chunks were previously Put", [this]()
			{
				It("should return nullptr for each call.", [this]()
				{
					for (const FGuid& Chunk : SomeChunks)
					{
						TEST_NULL(MemoryChunkStore->Get(Chunk));
					}
				});
			});

			Describe("when some chunks were previously Put", [this]()
			{
				BeforeEach([this]()
				{
					for (const FGuid& Chunk : SomeChunks)
					{
						MemoryChunkStore->Put(Chunk, TUniquePtr<IChunkDataAccess>(new FFakeChunkDataAccess()));
					}
				});

				It("should successfully return some chunks.", [this]()
				{
					for (const FGuid& Chunk : SomeChunks)
					{
						TEST_NOT_NULL(MemoryChunkStore->Get(Chunk));
					}
				});

				Describe("when some chunk was previously Get", [this]()
				{
					BeforeEach([this]()
					{
						MemoryChunkStore->Get(SomeChunk);
					});

					It("should return the chunk.", [this]()
					{
						TEST_NOT_NULL(MemoryChunkStore->Get(SomeChunk));
					});
				});

				Describe("when some chunk was previously Removed", [this]()
				{
					BeforeEach([this]()
					{
						MemoryChunkStore->Remove(SomeChunk);
					});

					It("should return nullptr.", [this]()
					{
						TEST_NULL(MemoryChunkStore->Get(SomeChunk));
					});
				});

				Describe("when some chunk was previously Get and Removed", [this]()
				{
					BeforeEach([this]()
					{
						MemoryChunkStore->Get(SomeChunk);
						MemoryChunkStore->Remove(SomeChunk);
					});

					It("should return nullptr.", [this]()
					{
						TEST_NULL(MemoryChunkStore->Get(SomeChunk));
					});
				});
			});
		});

		Describe("Remove", [this]()
		{
			Describe("when no chunks were previously Put", [this]()
			{
				It("should return invalid ptr for each call.", [this]()
				{
					for (const FGuid& Chunk : SomeChunks)
					{
						TEST_FALSE(MemoryChunkStore->Remove(Chunk).IsValid());
					}
				});
			});

			Describe("when some chunks were previously Put", [this]()
			{
				BeforeEach([this]()
				{
					for (const FGuid& Chunk : SomeChunks)
					{
						MemoryChunkStore->Put(Chunk, TUniquePtr<IChunkDataAccess>(new FFakeChunkDataAccess()));
					}
				});

				It("should return valid ptr for each call.", [this]()
				{
					for (const FGuid& Chunk : SomeChunks)
					{
						TEST_TRUE(MemoryChunkStore->Remove(Chunk).IsValid());
					}
				});
			});
		});
	});

	AfterEach([this]()
	{
		MemoryChunkStore.Reset();
	});
}

#endif //WITH_DEV_AUTOMATION_TESTS
