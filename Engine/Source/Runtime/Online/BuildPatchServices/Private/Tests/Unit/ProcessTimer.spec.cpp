// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Tests/TestHelpers.h"
#include "Tests/Mock/CyclesProvider.mock.inl"
#include "Core/ProcessTimer.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FProcessTimerSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
// Unit.
typedef BuildPatchServices::TProcessTimer<BuildPatchServices::FMockCyclesProvider> FTestProcessTimer;
TUniquePtr<FTestProcessTimer> ProcessTimer;
// Test helpers.
void AdvanceTime(double Seconds);
END_DEFINE_SPEC(FProcessTimerSpec)

void FProcessTimerSpec::Define()
{
	using namespace BuildPatchServices;

	// Specs.
	BeforeEach([this]()
	{
		FMockCyclesProvider::Reset();
		ProcessTimer.Reset(new FTestProcessTimer());
	});

	Describe("ProcessTimer", [this]()
	{
		Describe("GetSeconds", [this]()
		{
			It("should initially return 0.", [this]()
			{
				TEST_EQUAL(ProcessTimer->GetSeconds(), 0.0);
			});

			Describe("when started", [this]()
			{
				BeforeEach([this]()
				{
					ProcessTimer->Start();
				});

				Describe("and 3 seconds have passed", [this]()
				{
					BeforeEach([this]()
					{
						AdvanceTime(3.0);
					});

					It("should return 3.", [this]()
					{
						TEST_EQUAL(ProcessTimer->GetSeconds(), 3.0);
					});

					Describe("then paused", [this]()
					{
						BeforeEach([this]()
						{
							ProcessTimer->SetPause(true);
						});

						Describe("and 2 seconds have passed", [this]()
						{
							BeforeEach([this]()
							{
								AdvanceTime(2.0);
							});

							It("should return 3.", [this]()
							{
								TEST_EQUAL(ProcessTimer->GetSeconds(), 3.0);
							});

							Describe("then stopped", [this]()
							{
								BeforeEach([this]()
								{
									ProcessTimer->Stop();
								});

								It("should return 3.", [this]()
								{
									TEST_EQUAL(ProcessTimer->GetSeconds(), 3.0);
								});
							});

							Describe("then un-paused", [this]()
							{
								BeforeEach([this]()
								{
									ProcessTimer->SetPause(false);
								});

								Describe("and 2 seconds have passed", [this]()
								{
									BeforeEach([this]()
									{
										AdvanceTime(2.0);
									});

									It("should return 5.", [this]()
									{
										TEST_EQUAL(ProcessTimer->GetSeconds(), 5.0);
									});

									Describe("then stopped", [this]()
									{
										BeforeEach([this]()
										{
											ProcessTimer->Stop();
										});

										It("should return 5.", [this]()
										{
											TEST_EQUAL(ProcessTimer->GetSeconds(), 5.0);
										});
									});
								});
							});
						});
					});

					Describe("then stopped", [this]()
					{
						BeforeEach([this]()
						{
							ProcessTimer->Stop();
						});

						It("should return 3.", [this]()
						{
							TEST_EQUAL(ProcessTimer->GetSeconds(), 3.0);
						});

						Describe("and 4 seconds have passed", [this]()
						{
							BeforeEach([this]()
							{
								AdvanceTime(4.0);
							});

							It("should return 3.", [this]()
							{
								TEST_EQUAL(ProcessTimer->GetSeconds(), 3.0);
							});

							Describe("then stopped", [this]()
							{
								BeforeEach([this]()
								{
									ProcessTimer->Stop();
								});

								It("should return 3.", [this]()
								{
									TEST_EQUAL(ProcessTimer->GetSeconds(), 3.0);
								});
							});

							Describe("then started", [this]()
							{
								BeforeEach([this]()
								{
									ProcessTimer->Start();
								});

								Describe("and 3 seconds have passed", [this]()
								{
									BeforeEach([this]()
									{
										AdvanceTime(3.0);
									});

									It("should return 6.", [this]()
									{
										TEST_EQUAL(ProcessTimer->GetSeconds(), 6.0);
									});
								});
							});
						});
					});
				});
			});

			Describe("when paused", [this]()
			{
				BeforeEach([this]()
				{
					ProcessTimer->SetPause(true);
				});

				Describe("then started", [this]()
				{
					BeforeEach([this]()
					{
						ProcessTimer->Start();
					});

					Describe("and 3 seconds have passed", [this]()
					{
						BeforeEach([this]()
						{
							AdvanceTime(3.0);
						});

						It("should return 0.", [this]()
						{
							TEST_EQUAL(ProcessTimer->GetSeconds(), 0.0);
						});

						Describe("then stopped", [this]()
						{
							BeforeEach([this]()
							{
								ProcessTimer->Stop();
							});

							It("should return 0.", [this]()
							{
								TEST_EQUAL(ProcessTimer->GetSeconds(), 0.0);
							});
						});

						Describe("then un-paused", [this]()
						{
							BeforeEach([this]()
							{
								ProcessTimer->SetPause(false);
							});

							Describe("and 2 seconds have passed", [this]()
							{
								BeforeEach([this]()
								{
									AdvanceTime(2.0);
								});

								It("should return 2.", [this]()
								{
									TEST_EQUAL(ProcessTimer->GetSeconds(), 2.0);
								});
							});
						});
					});
				});
			});
		});
	});

	AfterEach([this]()
	{
		FMockCyclesProvider::Reset();
		ProcessTimer.Reset();
	});
}

void FProcessTimerSpec::AdvanceTime(double Seconds)
{
	using namespace BuildPatchServices;

	FMockCyclesProvider::CurrentCycles += Seconds / FMockCyclesProvider::SecondsPerCycle;
}

#endif //WITH_DEV_AUTOMATION_TESTS
