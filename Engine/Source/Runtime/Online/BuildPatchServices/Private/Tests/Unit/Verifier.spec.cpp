// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/SecureHash.h"
#include "Math/RandomStream.h"
#include "Tests/TestHelpers.h"
#include "Tests/Mock/VerifierStat.mock.h"
#include "Tests/Mock/Manifest.mock.h"
#include "Tests/Fake/FileSystem.fake.h"
#include "Tests/Fake/InstallerError.fake.h"
#include "Installer/Verifier.h"
#include "BuildPatchVerify.h"
#include "BuildPatchHash.h"
#include "IBuildManifestSet.h"
#include "BuildPatchSettings.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FVerifierSpec, "BuildPatchServices.Unit", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
// Unit.
TUniquePtr<BuildPatchServices::IVerifier> Verifier;
// Mock.
TUniquePtr<BuildPatchServices::FFakeFileSystem> FakeFileSystem;
TUniquePtr<BuildPatchServices::FFakeInstallerError> FakeInstallerError;
TUniquePtr<BuildPatchServices::FMockVerifierStat> MockVerificationStat;
BuildPatchServices::FMockManifestPtr MockManifest;
// Data.
FString VerifyDirectory;
FString StagedFileDirectory;
TSet<FString> AllFiles;
TSet<FString> SomeFiles;
TSet<FString> TouchedFiles;
TSet<FString> Tags;
TArray<FString> OutDatedFiles;
TMap<FString, FString> DiskFileToManifestFile;
bool bHasPaused;
TUniquePtr<BuildPatchServices::IBuildManifestSet> ManifestSet;
// Test helpers.
TFuture<void> PauseFor(float Seconds);
void MakeFileData();
void TouchAllFiles();
void TouchSomeFiles();
void CorruptSomeFiles();
void ResizeSomeFiles();
void StageSomeFiles();
void MakeUnit(BuildPatchServices::EVerifyMode Mode);
TSet<FString> LoadedFiles();
TSet<FString> FilesSizeCheckedFiles();
END_DEFINE_SPEC(FVerifierSpec)

void FVerifierSpec::Define()
{
	using namespace BuildPatchServices;

	// Data setup.
	FRollingHashConst::Init();
	VerifyDirectory = TEXT("VerDir");
	StagedFileDirectory = TEXT("StaFilDir");
	bHasPaused = false;

	// Specs.
	BeforeEach([this]()
	{
		FakeFileSystem.Reset(new FFakeFileSystem());
		FakeInstallerError.Reset(new FFakeInstallerError());
		MockVerificationStat.Reset(new FMockVerifierStat());
		MockManifest = MakeShareable(new FMockManifest());
		ManifestSet.Reset(FBuildManifestSetFactory::Create({ BuildPatchServices::FInstallerAction::MakeInstall(MockManifest.ToSharedRef(), Tags) }));
		MakeFileData();
	});

	xDescribe("Verifier", [this]()
	{
		Describe("Verify", [this]()
		{
			BeforeEach([this]()
			{
				TouchSomeFiles();
			});

			Describe("when SHA verifying all files", [this]()
			{
				BeforeEach([this]()
				{
					MakeUnit(EVerifyMode::ShaVerifyAllFiles);
				});

				It("should load and SHA check all files.", [this]()
				{
					Verifier->Verify(OutDatedFiles);
					TEST_EQUAL(OutDatedFiles.Num(), 0);
					TEST_TRUE(SetsAreEqual(LoadedFiles(), AllFiles));
				});
			});

			Describe("when SHA verifying only touched files", [this]()
			{
				BeforeEach([this]()
				{
					MakeUnit(EVerifyMode::ShaVerifyTouchedFiles);
				});

				It("should load and SHA check touched files.", [this]()
				{
					Verifier->Verify(OutDatedFiles);
					TEST_EQUAL(OutDatedFiles.Num(), 0);
					TEST_TRUE(SetsAreEqual(LoadedFiles(), TouchedFiles));
				});
			});

			Describe("when file size verifying all files", [this]()
			{
				BeforeEach([this]()
				{
					MakeUnit(EVerifyMode::FileSizeCheckAllFiles);
				});

				It("should check file size of all files.", [this]()
				{
					Verifier->Verify(OutDatedFiles);
					TEST_EQUAL(OutDatedFiles.Num(), 0);
					TEST_EQUAL(LoadedFiles().Num(), 0);
					TEST_TRUE(SetsAreEqual(FilesSizeCheckedFiles(), AllFiles));
				});
			});

			Describe("when file size verifying touched files", [this]()
			{
				BeforeEach([this]()
				{
					MakeUnit(EVerifyMode::FileSizeCheckTouchedFiles);
				});

				It("should check file size of touched files.", [this]()
				{
					Verifier->Verify(OutDatedFiles);
					TEST_EQUAL(OutDatedFiles.Num(), 0);
					TEST_EQUAL(LoadedFiles().Num(), 0);
					TEST_TRUE(SetsAreEqual(FilesSizeCheckedFiles(), TouchedFiles));
				});
			});

			Describe("when verifying with SHA and some files are corrupt", [this]()
			{
				BeforeEach([this]()
				{
					TouchAllFiles();
					CorruptSomeFiles();
					MakeUnit(EVerifyMode::ShaVerifyAllFiles);
				});

				It("should provide some files.", [this]()
				{
					Verifier->Verify(OutDatedFiles);
					TEST_TRUE(SetsAreEqual(TSet<FString>(OutDatedFiles), SomeFiles));
				});
			});

			Describe("when verifying by size and some files are corrupt", [this]()
			{
				BeforeEach([this]()
				{
					TouchAllFiles();
					ResizeSomeFiles();
					MakeUnit(EVerifyMode::FileSizeCheckAllFiles);
				});

				It("should provide some files.", [this]()
				{
					Verifier->Verify(OutDatedFiles);
					TEST_TRUE(SetsAreEqual(TSet<FString>(OutDatedFiles), SomeFiles));
				});
			});

			Describe("when some files were only staged", [this]()
			{
				BeforeEach([this]()
				{
					TouchAllFiles();
					StageSomeFiles();
					MakeUnit(EVerifyMode::ShaVerifyAllFiles);
				});

				It("load staged files instead of installed files.", [this]()
				{
					Verifier->Verify(OutDatedFiles);
					TEST_TRUE(SetsAreEqual(LoadedFiles(), AllFiles));
				});
			});

			Describe("when there are multiple types of file errors", [this]()
			{
				Describe("when we are using SHA verify mode", [this]()
				{
					Describe("when the first error is a corruption", [this]()
					{
						BeforeEach([this]()
						{
							CorruptSomeFiles();
							const FString LastFile = VerifyDirectory / SomeFiles.Array().Last();
							FakeFileSystem->DiskData.Remove(LastFile);
							MakeUnit(EVerifyMode::ShaVerifyAllFiles);
						});

						It("will return HashCheckFailed.", [this]()
						{
							EVerifyResult VerifyResult = Verifier->Verify(OutDatedFiles);
							TEST_EQUAL(VerifyResult, EVerifyResult::HashCheckFailed);
						});
					});

					Describe("the first error is missing file", [this]()
					{
						BeforeEach([this]()
						{
							CorruptSomeFiles();
							const FString FirstFile = VerifyDirectory / *SomeFiles.CreateIterator();
							FakeFileSystem->DiskData.Remove(FirstFile);
							MakeUnit(EVerifyMode::ShaVerifyAllFiles);
						});

						It("will return FileMissing.", [this]()
						{
							EVerifyResult VerifyResult = Verifier->Verify(OutDatedFiles);
							TEST_EQUAL(VerifyResult, EVerifyResult::FileMissing);
						});
					});

					Describe("the first error is a file failed to open", [this]()
					{
						BeforeEach([this]()
						{
							CorruptSomeFiles();
							const FString FirstFile = VerifyDirectory / *SomeFiles.CreateIterator();
							FakeFileSystem->DiskDataOpenFailure.Add(FirstFile);
							MakeUnit(EVerifyMode::ShaVerifyAllFiles);
						});

						It("will return OpenFileFailed.", [this]()
						{
							EVerifyResult VerifyResult = Verifier->Verify(OutDatedFiles);
							TEST_EQUAL(VerifyResult, EVerifyResult::OpenFileFailed);
						});
					});

					Describe("the first error is file was the wrong size", [this]()
					{
						BeforeEach([this]()
						{
							ResizeSomeFiles();
							const FString LastFile = VerifyDirectory / SomeFiles.Array().Last();
							FakeFileSystem->DiskData.Remove(LastFile);
							MakeUnit(EVerifyMode::ShaVerifyAllFiles);
						});

						It("will return FileSizeFailed.", [this]()
						{
							EVerifyResult VerifyResult = Verifier->Verify(OutDatedFiles);
							TEST_EQUAL(VerifyResult, EVerifyResult::FileSizeFailed);
						});
					});
				});

				Describe("when we are using file size verify mode", [this]()
				{
					Describe("the first error is missing file", [this]()
					{
						BeforeEach([this]()
						{
							ResizeSomeFiles();
							const FString FirstFile = VerifyDirectory / *SomeFiles.CreateIterator();
							FakeFileSystem->DiskData.Remove(FirstFile);
							MakeUnit(EVerifyMode::FileSizeCheckAllFiles);
						});

						It("will return FileMissing.", [this]()
						{
							EVerifyResult VerifyResult = Verifier->Verify(OutDatedFiles);
							TEST_EQUAL(VerifyResult, EVerifyResult::FileMissing);
						});
					});

					Describe("the first error is file was the wrong size", [this]()
					{
						BeforeEach([this]()
						{
							ResizeSomeFiles();
							const FString LastFile = VerifyDirectory / SomeFiles.Array().Last();
							FakeFileSystem->DiskData.Remove(LastFile);
							MakeUnit(EVerifyMode::FileSizeCheckAllFiles);
						});

						It("will return FileSizeFailed.", [this]()
						{
							EVerifyResult VerifyResult = Verifier->Verify(OutDatedFiles);
							TEST_EQUAL(VerifyResult, EVerifyResult::FileSizeFailed);
						});
					});
				});
			});
		});

		Describe("SetPaused", [this]()
		{
			Describe("when verifying with SHA", [this]()
			{
				BeforeEach([this]()
				{
					TouchAllFiles();
					MakeUnit(EVerifyMode::ShaVerifyAllFiles);
				});

				It("should delay the verification process.", [this]()
				{
					const float PauseTime = 0.15f;
					MockVerificationStat->OnFileCompletedFunc = [this, PauseTime](const FString&, EVerifyResult)
					{
						if (!bHasPaused)
						{
							bHasPaused = true;
							PauseFor(PauseTime);
						}
					};
					Verifier->Verify(OutDatedFiles);
					double LongestDelay = 0.0f;
					for (int32 Idx = 1; Idx < MockVerificationStat->RxOnFileStarted.Num(); ++Idx)
					{
						double ThisDelay = MockVerificationStat->RxOnFileStarted[Idx].Get<0>() - MockVerificationStat->RxOnFileStarted[Idx - 1].Get<0>();
						if (ThisDelay > LongestDelay)
						{
							LongestDelay = ThisDelay;
						}
					}
					TEST_TRUE(LongestDelay >= PauseTime);
				});
			});

			Describe("when verifying with size", [this]()
			{
				BeforeEach([this]()
				{
					TouchAllFiles();
					MakeUnit(EVerifyMode::FileSizeCheckAllFiles);
				});

				It("should delay the verification process.", [this]()
				{
					const float PauseTime = 0.15f;
					MockVerificationStat->OnFileCompletedFunc = [this, PauseTime](const FString&, EVerifyResult)
					{
						if (!bHasPaused)
						{
							bHasPaused = true;
							PauseFor(PauseTime);
						}
					};
					Verifier->Verify(OutDatedFiles);
					double LongestDelay = 0.0f;
					for (int32 Idx = 1; Idx < MockVerificationStat->RxOnFileStarted.Num(); ++Idx)
					{
						double ThisDelay = MockVerificationStat->RxOnFileStarted[Idx].Get<0>() - MockVerificationStat->RxOnFileStarted[Idx - 1].Get<0>();
						if (ThisDelay > LongestDelay)
						{
							LongestDelay = ThisDelay;
						}
					}
					TEST_TRUE(LongestDelay >= PauseTime);
				});
			});
		});

		Describe("Abort", [this]()
		{
			Describe("when verifying with SHA", [this]()
			{
				BeforeEach([this]()
				{
					TouchAllFiles();
					MakeUnit(EVerifyMode::ShaVerifyAllFiles);
				});

				It("should halt process and stop.", [this]()
				{
					MockVerificationStat->OnFileCompletedFunc = [this](const FString&, EVerifyResult)
					{
						Verifier->Abort();
					};
					Verifier->Verify(OutDatedFiles);
					TEST_TRUE(LoadedFiles().Num() < MockManifest->BuildFileList.Num());
				});

				It("should return aborted.", [this]()
				{
					MockVerificationStat->OnFileCompletedFunc = [this](const FString&, EVerifyResult)
					{
						Verifier->Abort();
					};
					EVerifyResult VerifyResult = Verifier->Verify(OutDatedFiles);
					TEST_EQUAL(VerifyResult, EVerifyResult::Aborted);
				});
			});

			Describe("when verifying by size", [this]()
			{
				BeforeEach([this]()
				{
					TouchAllFiles();
					MakeUnit(EVerifyMode::FileSizeCheckAllFiles);
				});

				It("should halt process and stop.", [this]()
				{
					MockVerificationStat->OnFileCompletedFunc = [this](const FString&, EVerifyResult)
					{
						Verifier->Abort();
					};
					Verifier->Verify(OutDatedFiles);
					TEST_TRUE(LoadedFiles().Num() < MockManifest->BuildFileList.Num());
				});

				It("should return aborted.", [this]()
				{
					MockVerificationStat->OnFileCompletedFunc = [this](const FString&, EVerifyResult)
					{
						Verifier->Abort();
					};
					EVerifyResult VerifyResult = Verifier->Verify(OutDatedFiles);
					TEST_EQUAL(VerifyResult, EVerifyResult::Aborted);
				});
			});
		});
	});

	AfterEach([this]()
	{
		Verifier.Reset();
		ManifestSet.Reset();
		MockManifest.Reset();
		MockVerificationStat.Reset();
		FakeFileSystem.Reset();
		AllFiles.Reset();
		TouchedFiles.Reset();
		Tags.Reset();
		OutDatedFiles.Reset();
		DiskFileToManifestFile.Reset();
		bHasPaused = false;
	});
}

TFuture<void> FVerifierSpec::PauseFor(float Seconds)
{
	using namespace BuildPatchServices;

	double PauseAt = FStatsCollector::GetSeconds();
	Verifier->SetPaused(true);
	TFunction<void()> Task = [this, PauseAt, Seconds]()
	{
		while ((FStatsCollector::GetSeconds() - PauseAt) < Seconds)
		{
			FPlatformProcess::Sleep(0.01f);
		}
		Verifier->SetPaused(false);
	};
	return Async(EAsyncExecution::Thread, MoveTemp(Task));
}

void FVerifierSpec::MakeFileData()
{
	using namespace BuildPatchServices;

	for (int32 Idx = 0; Idx < 25; ++Idx)
	{
		MockManifest->BuildFileList.Add(FString::Printf(TEXT("Some/Install/File%d.exe"), Idx));
		DiskFileToManifestFile.Add(VerifyDirectory / MockManifest->BuildFileList.Last(), MockManifest->BuildFileList.Last());
		DiskFileToManifestFile.Add(StagedFileDirectory / MockManifest->BuildFileList.Last(), MockManifest->BuildFileList.Last());
		SomeFiles.Add(MockManifest->BuildFileList.Last());
		MockManifest->BuildFileList.Add(FString::Printf(TEXT("Other/Install/File%d.exe"), Idx));
		DiskFileToManifestFile.Add(VerifyDirectory / MockManifest->BuildFileList.Last(), MockManifest->BuildFileList.Last());
		DiskFileToManifestFile.Add(StagedFileDirectory / MockManifest->BuildFileList.Last(), MockManifest->BuildFileList.Last());
	}
	AllFiles = TSet<FString>(MockManifest->BuildFileList);
	MockManifest->TaggedFileList = AllFiles;
	FRandomStream RandomData(0);
	FSHAHash SHAHashData;
	for (const FString& Filename : MockManifest->BuildFileList)
	{
		FString FullFilename = VerifyDirectory / Filename;
		TArray<uint8>& FileData = FakeFileSystem->DiskData.Add(FullFilename);
		FileData.AddUninitialized(100);
		uint8* Data = FileData.GetData();
		for (int32 DataIdx = 0; DataIdx <= (FileData.Num() - 4); DataIdx += 4)
		{
			*((uint32*)(Data + DataIdx)) = RandomData.GetUnsignedInt();
		}
		FSHA1::HashBuffer(Data, FileData.Num(), SHAHashData.Hash);
		MockManifest->FileNameToHashes.Add(Filename, SHAHashData);
		MockManifest->FileNameToFileSize.Add(Filename, FileData.Num());
	}
}

void FVerifierSpec::TouchAllFiles()
{
	TouchedFiles = TSet<FString>(MockManifest->BuildFileList);
}

void FVerifierSpec::TouchSomeFiles()
{
	TouchedFiles.Reset();
	for (const FString& Filename : SomeFiles)
	{
		TouchedFiles.Add(Filename);
	}
}

void FVerifierSpec::CorruptSomeFiles()
{
	for (const FString& Filename : SomeFiles)
	{
		FString FullFilename = VerifyDirectory / Filename;
		TArray<uint8>& FileData = FakeFileSystem->DiskData[FullFilename];
		for (int32 DataIdx = 0; DataIdx < 10; ++DataIdx)
		{
			FileData[DataIdx] = FileData[DataIdx + 1];
		}
	}
}

void FVerifierSpec::ResizeSomeFiles()
{
	for (const FString& Filename : SomeFiles)
	{
		FString FullFilename = VerifyDirectory / Filename;
		TArray<uint8>& FileData = FakeFileSystem->DiskData[FullFilename];
		FileData.Add(123);
		FSHA1::HashBuffer(FileData.GetData(), FileData.Num(), MockManifest->FileNameToHashes[Filename].Hash);
	}
}

void FVerifierSpec::StageSomeFiles()
{
	for (const FString& Filename : SomeFiles)
	{
		FString OldFilename = VerifyDirectory / Filename;
		FString NewFilename = StagedFileDirectory / Filename;
		FakeFileSystem->DiskData.Add(NewFilename, FakeFileSystem->DiskData[OldFilename]);
		FakeFileSystem->DiskData.Remove(OldFilename);
		DiskFileToManifestFile[OldFilename] = TEXT("Break");
	}
}

void FVerifierSpec::MakeUnit(BuildPatchServices::EVerifyMode Mode)
{
	using namespace BuildPatchServices;
	// TODO: Verifier behavioural change needs unit test update.
	TMap<FString, FString> PerFileSubdirectories;
	Verifier.Reset(FVerifierFactory::Create(
		FakeFileSystem.Get(),
		MockVerificationStat.Get(),
		Mode,
		nullptr,
		/*TouchedFiles,*/
		ManifestSet.Get(),
		VerifyDirectory,
		StagedFileDirectory,
		false,
		PerFileSubdirectories));

	if (EVerifyMode::FileSizeCheckTouchedFiles == Mode || EVerifyMode::ShaVerifyTouchedFiles == Mode)
	{
		Verifier->AddTouchedFiles(TouchedFiles);
	}
}

TSet<FString> FVerifierSpec::LoadedFiles()
{
	using namespace BuildPatchServices;

	TSet<FString> Result;
	for (const FMockFileSystem::FCreateFileReader& Call : FakeFileSystem->RxCreateFileReader)
	{
		Result.Add(DiskFileToManifestFile[Call.Get<2>()]);
	}
	return Result;
}

TSet<FString> FVerifierSpec::FilesSizeCheckedFiles()
{
	using namespace BuildPatchServices;

	TSet<FString> Result;
	for (const FMockFileSystem::FGetFileSize& Call : FakeFileSystem->RxGetFileSize)
	{
		Result.Add(DiskFileToManifestFile[Call.Get<1>()]);
	}
	return Result;
}
#endif //WITH_DEV_AUTOMATION_TESTS
