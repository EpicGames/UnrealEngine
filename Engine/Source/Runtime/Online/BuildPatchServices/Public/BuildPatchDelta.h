// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IBuildManifest.h"

namespace BuildPatchServices
{
	/**
	 * An enum defining the desired policy for requesting an optimised delta.
	 */
	enum class EDeltaPolicy : uint32
	{
		// Try to fetch, but continue without if request fail.
		TryFetchContinueWithout = 0,

		// Expect the delta to exist, hard fail the installation if it could not be retrieved.
		Expect,

		// Expect the delta to not exist, skipping any attempt to use one.
		Skip,

		InvalidOrMax
	};

	/**
	 * Based on the source and destination manifests, get the filename for the delta that optimises patching from source to destination.
	 * @param SourceManifest        The source manifest.
	 * @param DestinationManifest   The destination manifest.
	 * @param FilenameTrailer       This is an optional parameter to allow for multiple optimized deltas for the same build pairing. If unneeded, leave as "".
	 * @return the CloudDir relative delta filename.
	 */
	BUILDPATCHSERVICES_API FString GetChunkDeltaFilename(const IBuildManifestRef& SourceManifest, const IBuildManifestRef& DestinationManifest, const FString& FilenameTrailer);

	BUILDPATCHSERVICES_API IBuildManifestPtr MergeDeltaManifest(const IBuildManifestRef& Manifest, const IBuildManifestRef& Delta);
}

static_assert((uint32)BuildPatchServices::EDeltaPolicy::InvalidOrMax == 3, "Please add support for the extra values to the Lex functions below.");

inline const TCHAR* LexToString(BuildPatchServices::EDeltaPolicy DeltaPolicy)
{
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::EDeltaPolicy::Value: return TEXT(#Value)
	switch (DeltaPolicy)
	{
		CASE_ENUM_TO_STR(TryFetchContinueWithout);
		CASE_ENUM_TO_STR(Expect);
		CASE_ENUM_TO_STR(Skip);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::EDeltaPolicy& DeltaPolicy, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { DeltaPolicy = BuildPatchServices::EDeltaPolicy::Value; return; }
	const TCHAR* const Prefix = TEXT("EDeltaPolicy::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(TryFetchContinueWithout);
	RETURN_IF_EQUAL(Expect);
	RETURN_IF_EQUAL(Skip);
	// Did not match
	DeltaPolicy = BuildPatchServices::EDeltaPolicy::InvalidOrMax;
#undef RETURN_IF_EQUAL
}
