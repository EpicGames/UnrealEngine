// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace BuildPatchServices
{
	/**
	 * An enum type to describe supported features of a certain manifest.
	 */
	enum class EFeatureLevel : int32
	{
		// The original version.
		Original = 0,
		// Support for custom fields.
		CustomFields,
		// Started storing the version number.
		StartStoringVersion,
		// Made after data files where renamed to include the hash value, these chunks now go to ChunksV2.
		DataFileRenames,
		// Manifest stores whether build was constructed with chunk or file data.
		StoresIfChunkOrFileData,
		// Manifest stores group number for each chunk/file data for reference so that external readers don't need to know how to calculate them.
		StoresDataGroupNumbers,
		// Added support for chunk compression, these chunks now go to ChunksV3. NB: Not File Data Compression yet.
		ChunkCompressionSupport,
		// Manifest stores product prerequisites info.
		StoresPrerequisitesInfo,
		// Manifest stores chunk download sizes.
		StoresChunkFileSizes,
		// Manifest can optionally be stored using UObject serialization and compressed.
		StoredAsCompressedUClass,
		// These two features were removed and never used.
		UNUSED_0,
		UNUSED_1,
		// Manifest stores chunk data SHA1 hash to use in place of data compare, for faster generation.
		StoresChunkDataShaHashes,
		// Manifest stores Prerequisite Ids.
		StoresPrerequisiteIds,
		// The first minimal binary format was added. UObject classes will no longer be saved out when binary selected.
		StoredAsBinaryData,
		// Temporary level where manifest can reference chunks with dynamic window size, but did not serialize them. Chunks from here onwards are stored in ChunksV4.
		VariableSizeChunksWithoutWindowSizeChunkInfo,
		// Manifest can reference chunks with dynamic window size, and also serializes them.
		VariableSizeChunks,
		// Manifest uses a build id generated from its metadata.
		UsesRuntimeGeneratedBuildId,
		// Manifest uses a build id generated unique at build time, and stored in manifest.
		UsesBuildTimeGeneratedBuildId,
		// Manifests generated with this feature level onwards will store the MD5 hash and the calculated MIME type of each file.
		StoresFileMD5HashesAndMIMEType,
		// Manifests generated with this feature level onwards will store the SHA256 hash of each file.
		StoresFileSHA256Hashes,
		// Added support for Uninstall Actions.
		StoresUninstallActions,
		// Added full support for chunk encryption, additionally stores encyption secrets, as well as the CompressedDataSize and AuthTag
		// for each chunk. Chunks from here onwards are stored in ChunksV5.
		ChunkEncryptionSupport,
		// Added the secretID as part of the chunk pathing within a CloudDir
		ChunksStoredBySecret,
		// Completed full support for BPS data encryption, if a manifest got encrypted, it will additionally store encryption data
		// and most fields will have been replaced by empty or functional but not accurate data.
		ManifestEncryptionSupport,

		// !! Always after the latest version entry, signifies the latest version plus 1 to allow the following Latest alias.
		LatestPlusOne,
		// An alias for the actual latest version value.
		Latest = (LatestPlusOne - 1),
		// An alias to provide the latest version of a manifest supported by file data (nochunks).
		LatestNoChunks = StoresChunkFileSizes,
		// An alias to provide the latest version of a manifest supported by a json serialized format.
		LatestJson = StoresPrerequisiteIds,
		// An alias to provide the latest version of a manifest supported by generation runs on platforms without OpenSSL module support.
		LatestNoOpenSSL = StoresFileMD5HashesAndMIMEType,
		// An alias to provide the latest version of a manifest with unencrypted chunks.
		LatestUnencryptedChunks = StoresUninstallActions,
		// An alias to provide the first available version of optimised delta manifest saving.
		FirstOptimisedDelta = UsesRuntimeGeneratedBuildId,

		// More aliases, but this time for values that have been renamed
		StoresUniqueBuildId = UsesRuntimeGeneratedBuildId,

		// JSON manifests were stored with a version of 255 during a certain CL range due to a bug.
		// We will treat this as being StoresChunkFileSizes in code.
		BrokenJsonVersion = 255,
		// This is for UObject default, so that we always serialize it.
		Invalid = -1
	};
}

inline const TCHAR* LexToString(BuildPatchServices::EFeatureLevel FeatureLevel)
{
	static_assert((int32)BuildPatchServices::EFeatureLevel::Latest == 24, "Please add support for the extra values below.");
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::EFeatureLevel::Value: return TEXT(#Value)
	switch (FeatureLevel)
	{
		CASE_ENUM_TO_STR(Original);
		CASE_ENUM_TO_STR(CustomFields);
		CASE_ENUM_TO_STR(StartStoringVersion);
		CASE_ENUM_TO_STR(DataFileRenames);
		CASE_ENUM_TO_STR(StoresIfChunkOrFileData);
		CASE_ENUM_TO_STR(StoresDataGroupNumbers);
		CASE_ENUM_TO_STR(ChunkCompressionSupport);
		CASE_ENUM_TO_STR(StoresPrerequisitesInfo);
		CASE_ENUM_TO_STR(StoresChunkFileSizes);
		CASE_ENUM_TO_STR(StoredAsCompressedUClass);
		CASE_ENUM_TO_STR(UNUSED_0);
		CASE_ENUM_TO_STR(UNUSED_1);
		CASE_ENUM_TO_STR(StoresChunkDataShaHashes);
		CASE_ENUM_TO_STR(StoresPrerequisiteIds);
		CASE_ENUM_TO_STR(StoredAsBinaryData);
		CASE_ENUM_TO_STR(VariableSizeChunksWithoutWindowSizeChunkInfo);
		CASE_ENUM_TO_STR(VariableSizeChunks);
		CASE_ENUM_TO_STR(UsesRuntimeGeneratedBuildId);
		CASE_ENUM_TO_STR(UsesBuildTimeGeneratedBuildId);
		CASE_ENUM_TO_STR(StoresFileMD5HashesAndMIMEType);
		CASE_ENUM_TO_STR(StoresFileSHA256Hashes);
		CASE_ENUM_TO_STR(StoresUninstallActions);
		CASE_ENUM_TO_STR(ChunkEncryptionSupport);
		CASE_ENUM_TO_STR(ChunksStoredBySecret);
		CASE_ENUM_TO_STR(ManifestEncryptionSupport);
		CASE_ENUM_TO_STR(BrokenJsonVersion);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::EFeatureLevel& FeatureLevel, const TCHAR* Buffer)
{
	static_assert((int32)BuildPatchServices::EFeatureLevel::Latest == 24, "Please add support for the extra values below.");
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { FeatureLevel = BuildPatchServices::EFeatureLevel::Value; return; }
	const TCHAR* const Prefix = TEXT("EFeatureLevel::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(Original);
	RETURN_IF_EQUAL(CustomFields);
	RETURN_IF_EQUAL(StartStoringVersion);
	RETURN_IF_EQUAL(DataFileRenames);
	RETURN_IF_EQUAL(StoresIfChunkOrFileData);
	RETURN_IF_EQUAL(StoresDataGroupNumbers);
	RETURN_IF_EQUAL(ChunkCompressionSupport);
	RETURN_IF_EQUAL(StoresPrerequisitesInfo);
	RETURN_IF_EQUAL(StoresChunkFileSizes);
	RETURN_IF_EQUAL(StoredAsCompressedUClass);
	RETURN_IF_EQUAL(UNUSED_0);
	RETURN_IF_EQUAL(UNUSED_1);
	RETURN_IF_EQUAL(StoresChunkDataShaHashes);
	RETURN_IF_EQUAL(StoresPrerequisiteIds);
	RETURN_IF_EQUAL(StoredAsBinaryData);
	RETURN_IF_EQUAL(VariableSizeChunksWithoutWindowSizeChunkInfo);
	RETURN_IF_EQUAL(VariableSizeChunks);
	RETURN_IF_EQUAL(UsesRuntimeGeneratedBuildId);
	RETURN_IF_EQUAL(UsesBuildTimeGeneratedBuildId);
	RETURN_IF_EQUAL(StoresFileMD5HashesAndMIMEType);
	RETURN_IF_EQUAL(StoresFileSHA256Hashes);
	RETURN_IF_EQUAL(StoresUninstallActions);
	RETURN_IF_EQUAL(ChunkEncryptionSupport);
	RETURN_IF_EQUAL(ChunksStoredBySecret);
	RETURN_IF_EQUAL(ManifestEncryptionSupport);
	RETURN_IF_EQUAL(LatestPlusOne);
	RETURN_IF_EQUAL(Latest);
	RETURN_IF_EQUAL(LatestNoChunks);
	RETURN_IF_EQUAL(LatestJson);
	RETURN_IF_EQUAL(LatestUnencryptedChunks);
	RETURN_IF_EQUAL(FirstOptimisedDelta);
	RETURN_IF_EQUAL(StoresUniqueBuildId);
	RETURN_IF_EQUAL(BrokenJsonVersion);
	// Did not match
	FeatureLevel = BuildPatchServices::EFeatureLevel::Invalid;
#undef RETURN_IF_EQUAL
}
