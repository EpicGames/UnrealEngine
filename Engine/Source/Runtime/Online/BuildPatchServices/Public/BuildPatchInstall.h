// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IBuildManifest.h"

namespace BuildPatchServices
{
	/**
	 * An enum defining the installation mode that should be used.
	 */
	enum class EInstallMode : uint32
	{
		// Construct all required files, but only stage them ready to be completed later.
		StageFiles = 0,

		// Full installation, allowing immediate changes to be made to existing files. The installation is unusable until complete.
		DestructiveInstall,

		// Full installation, staging all required files before moving them all into place in a final step. The installation is still usable if canceled before the moving staging begins.
		NonDestructiveInstall,

		// Execute the prerequisite installer only, downloading it first if necessary. If the specified manifest has no prerequisites, this will result in an error.
		PrereqOnly,

		// Download chunk data to file only, to be unpacked as a later process. Chunk data will not be decrypted or decompressed.
		Preload,

		InvalidOrMax
	};

	BUILDPATCHSERVICES_API uint64 CalculateRequiredDiskSpace(const IBuildManifestPtr& CurrentManifest, const IBuildManifestRef& BuildManifest, const EInstallMode& InstallMode, const TSet<FString>& InstallTags);
}

static_assert((uint32)BuildPatchServices::EInstallMode::InvalidOrMax == 5, "Please add support for the extra values to the Lex functions below.");

inline const TCHAR* LexToString(BuildPatchServices::EInstallMode InstallMode)
{
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::EInstallMode::Value: return TEXT(#Value)
		switch (InstallMode)
		{
		CASE_ENUM_TO_STR(StageFiles);
		CASE_ENUM_TO_STR(DestructiveInstall);
		CASE_ENUM_TO_STR(NonDestructiveInstall);
		CASE_ENUM_TO_STR(PrereqOnly);
		CASE_ENUM_TO_STR(Preload);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::EInstallMode& InstallMode, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { InstallMode = BuildPatchServices::EInstallMode::Value; return; }
	const TCHAR* const Prefix = TEXT("EInstallMode::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(StageFiles);
	RETURN_IF_EQUAL(DestructiveInstall);
	RETURN_IF_EQUAL(NonDestructiveInstall);
	RETURN_IF_EQUAL(PrereqOnly);
	RETURN_IF_EQUAL(Preload);
	// Did not match
	InstallMode = BuildPatchServices::EInstallMode::InvalidOrMax;
#undef RETURN_IF_EQUAL
}
