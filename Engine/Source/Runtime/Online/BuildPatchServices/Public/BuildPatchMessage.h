// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Misc/Guid.h"
#include "Templates/Function.h"

namespace BuildPatchServices
{
	/**
	 * A message describing an event that occurred for a chunk source.
	 */
	struct FChunkSourceEvent
	{
		// Describes the event type.
		enum class EType : uint32
		{
			// Access was lost to the source.
			AccessLost = 0,
			// Access has been regained after being lost.
			AccessRegained,
			// HTTP request failed
			HttpRequestFailed,
			Retired,

			InvalidOrMax
		};

		// The type of event that occurred.
		EType Event;
		// The location context for the source, could be cloud root, install location, chunkdb file etc.
		FString Location;
	};

	/**
	 * A message describing an action taken to an installation file.
	 */
	struct FInstallationFileAction
	{
		// Describes the action type.
		enum class EType : uint32
		{
			// The file was completed in the staging location.
			Staged = 0,
			// The file was added to the final install location.
			Added,
			// The file was updated in the final install location.
			Updated,
			// The file was removed from the final install location.
			Removed,

			InvalidOrMax
		};

		// The type of action that occurred.
		EType Action;
		// The filename affected, relative to the install location.
		FString Filename;
	};

	/**
	 * A Request for a chunk
	 */
	struct FChunkUriRequest
	{
		FString CloudDirectory;
		FString ChunkCloudSubdirectory;
		FString ChunkPath;
		FString UniqueBuildId;
		bool bChunkFromDelta = false;
	};

	/**
	 * A Request for a delta file
	 */
	struct FDeltaUriRequest
	{
		FString CloudDirectory;
		FString DeltaCloudSubdirectory;
		FString DeltaPath;
		FString UniqueBuildId;
	};

	/**
	 * A Response containing the actual location of the chunk
	 */
	struct FChunkUriResponse
	{
		// Set this to true if the response can not be fulfilled. This will fail the chunk requests and subsequently
		// the installation.
		bool bFailed = false;
		FString Uri;
		
		// These headers <name, value> will be added to the HTTP request.
		TMap<FString, FString> AdditionalHeaders;
	};

	struct FGenericMessage
	{
		// Set of informational messages the installer can send to the client to update
		// UI or otherwise react.
		enum class EType : uint32
		{
			// Posted when a chunk is requested of the cloud source ONLY WHEN CHUNKDBS ARE PRESENT. This is useful if the installer
			// is not expected to be downloading anything independently and you want to log this case.
			// Note that even when chunks are fully provided, cancellation and resumption can lose "harvested"
			// chunks that were resident only in non-persistent backing store causing a download
			CloudSourceUsed,

			// Posted when a CDN/CloudDir has failed a download and is dropped in priority.
			// Payload1 = CDN that failed,
			// Payload2 = CDN that is considered "best" after this failure.
			CDNDownloadFailed,
		};

		EType Type;
		FGuid ChunkId;
		FString Payload1, Payload2;
	};
	

	// Flag for which Requests a message handler expects to receive, allows for internal implementation optimisation.
	enum class EMessageRequests : uint32
	{
		// Does not respond to any requests - message listener only.
		None = 0,
		// Will respond to chunk URI requests.
		ChunkUriRequest = 0x1,
		// Further request types to follow in future.

	};
	ENUM_CLASS_FLAGS(EMessageRequests);

	/**
	 * A Response containing the actual location of the file and URI's expiration time
	 */
	struct FExpiratedUriResponse
	{
		FString Uri;
		FString SignatureExpiration;
	};

	/**
	 * A Response containing an array of expirated URIs (e.g. different CDNs)
	 */
	struct FExpiratedUrisResponse
	{
		TArray<FExpiratedUriResponse> ExpiratedUri;
		bool bNeedRetry = true;
	};

	/**
	 * Base class of a message handler, this should be inherited from and passed to an installer to receive messages that you want to handle.
	 */
	class FMessageHandler
	{
	public:
		FMessageHandler(EMessageRequests InMessageRequests)
			: MessageRequests(InMessageRequests)
		{}

		virtual ~FMessageHandler() = default;

		/**
		* Handle generic information posting. This can be called from any thread and should not
		* take appreciable time as it blocks further installation.
		* See FGenericMessage for payload contents and types
		*/
		virtual void HandleMessage(const FGenericMessage& Message) {}

		/**
		 * Handles a chunk source event message.
		 * @param Message   The message to be handled.
		 */
		virtual void HandleMessage(const FChunkSourceEvent& Message) {}

		/**
		 * Handles an installation file action message.
		 * @param Message   The message to be handled.
		 */
		virtual void HandleMessage(const FInstallationFileAction& Message) {}

		/**
		 * Handles responding to a chunk Uri request
		 * @param Request   The request for a chunk
		 * @param OnResponse   The function to callback once the chunk has been found
		 * @return true if request is handled successfully
		 */
		virtual bool HandleRequest(const FChunkUriRequest& Request, TFunction<void(FChunkUriResponse)> OnResponse) { return false; }

		/**
		 * Handles responding to a delta Uri request
		 * @param Request   The request for a delta Uri
		 * @param OnResponse   The function to callback once the Uri has been found
		 */
		virtual bool HandleRequest(const FDeltaUriRequest& Request, TFunction<void(FExpiratedUrisResponse)> OnResponse) { return false; }

		/**
		 * @return the message request flags.
		 */
		EMessageRequests GetMessageRequests() const { return MessageRequests; }

	private:
		const EMessageRequests MessageRequests;
	};

	class FDefaultMessageHandler
		: public FMessageHandler
	{
	public:
		// Default constructor passes flags to always support all requests
		FDefaultMessageHandler()
			: FMessageHandler(static_cast<EMessageRequests>(0xFFFFFFFF))
		{}

		// This gets called if all registered handlers return false to HandleRequest. If there are
		// _no_ handlers then this is entirely sidestepped in the cloud chunk source.
		virtual bool HandleRequest(const FChunkUriRequest& Request, TFunction<void(FChunkUriResponse)> OnResponse) override
		{
			FChunkUriResponse Response;
			Response.Uri = Request.CloudDirectory / Request.ChunkCloudSubdirectory / Request.ChunkPath;
			OnResponse(MoveTemp(Response));
			return true;
		}

		virtual bool HandleRequest(const FDeltaUriRequest& Request, TFunction<void(FExpiratedUrisResponse)> OnResponse) override
		{
			FExpiratedUrisResponse Response;
			// Expiration time is not needed in the default case, setting this value explicitly empty
			Response.ExpiratedUri.Add({ Request.CloudDirectory / Request.DeltaCloudSubdirectory / Request.DeltaPath, "" });		
			OnResponse(MoveTemp(Response));
			return true;
		}
	};
}

static_assert((uint32)BuildPatchServices::FChunkSourceEvent::EType::InvalidOrMax == 4, "Please add support for the extra values to the Lex functions below.");
inline const TCHAR* LexToString(BuildPatchServices::FChunkSourceEvent::EType ChunkSourceEventType)
{
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::FChunkSourceEvent::EType::Value: return TEXT(#Value)
	switch (ChunkSourceEventType)
	{
		CASE_ENUM_TO_STR(AccessLost);
		CASE_ENUM_TO_STR(AccessRegained);
		CASE_ENUM_TO_STR(HttpRequestFailed);
		CASE_ENUM_TO_STR(Retired);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::FChunkSourceEvent::EType& ChunkSourceEventType, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { ChunkSourceEventType = BuildPatchServices::FChunkSourceEvent::EType::Value; return; }
	const TCHAR* const Prefix = TEXT("FChunkSourceEvent::EType::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(AccessLost);
	RETURN_IF_EQUAL(AccessRegained);
	RETURN_IF_EQUAL(HttpRequestFailed);
	RETURN_IF_EQUAL(Retired);
	// Did not match
	ChunkSourceEventType = BuildPatchServices::FChunkSourceEvent::EType::InvalidOrMax;
#undef RETURN_IF_EQUAL
}

inline const TCHAR* LexToString(BuildPatchServices::FInstallationFileAction::EType InstallationFileActionType)
{
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::FInstallationFileAction::EType::Value: return TEXT(#Value)
	static_assert((uint32)BuildPatchServices::FInstallationFileAction::EType::InvalidOrMax == 4, "Please add support for the extra values below.");
	switch (InstallationFileActionType)
	{
		CASE_ENUM_TO_STR(Staged);
		CASE_ENUM_TO_STR(Added);
		CASE_ENUM_TO_STR(Updated);
		CASE_ENUM_TO_STR(Removed);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::FInstallationFileAction::EType& InstallationFileActionType, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { InstallationFileActionType = BuildPatchServices::FInstallationFileAction::EType::Value; return; }
	const TCHAR* const Prefix = TEXT("FInstallationFileAction::EType::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	static_assert((uint32)BuildPatchServices::FInstallationFileAction::EType::InvalidOrMax == 4, "Please add support for the extra values below.");
	RETURN_IF_EQUAL(Staged);
	RETURN_IF_EQUAL(Added);
	RETURN_IF_EQUAL(Updated);
	RETURN_IF_EQUAL(Removed);
	// Did not match
	InstallationFileActionType = BuildPatchServices::FInstallationFileAction::EType::InvalidOrMax;
#undef RETURN_IF_EQUAL
}
