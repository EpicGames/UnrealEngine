// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BuildPatchServices.h: BuildPatchServices module public header file.
=============================================================================*/

#pragma once

/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

/* Dependencies
*****************************************************************************/
#include "Core.h"
#include "Modules/ModuleManager.h"

/* Private includes
*****************************************************************************/
#include "BuildPatchVerify.h"
#include "BuildPatchSettings.h"
#include "BuildPatchState.h"
#include "Interfaces/IBuildInstaller.h"
#include "Interfaces/IBuildManifest.h"
#include "Interfaces/IBuildPatchServicesModule.h"

