// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BuildPatchSettings.h"
#include "CoreMinimal.h"
#include "Interfaces/IBuildPatchServicesModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

class FBuildPatchServices
{
public:
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IBuildPatchServicesModule& Get()
	{
		return FModuleManager::Get().GetModuleChecked<IBuildPatchServicesModule>(ModuleName);
	}

	static const BuildPatchServices::FBuildPatchServicesInitSettings& GetSettings()
	{
		return InitSettings;
	}

	static void Set(
		const FName& Value, 
		const BuildPatchServices::FBuildPatchServicesInitSettings& BuildPatchServicesInitSettings = BuildPatchServices::FBuildPatchServicesInitSettings())
	{
		if(IsAvailable())
		{
			Shutdown();
		}
		ModuleName = Value;
		InitSettings = BuildPatchServicesInitSettings;
		FModuleManager::Get().LoadModuleChecked<IBuildPatchServicesModule>(ModuleName);
	}

	static void Shutdown()
	{
		if(IsAvailable())
		{
			FModuleManager::Get().UnloadModule(ModuleName);
		}
	}

private:

	static BUILDPATCHSERVICES_API FName ModuleName;
	static BUILDPATCHSERVICES_API BuildPatchServices::FBuildPatchServicesInitSettings InitSettings;
};
