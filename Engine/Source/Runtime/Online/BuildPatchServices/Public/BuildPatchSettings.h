// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BuildPatchDelta.h"
#include "BuildPatchFeatureLevel.h"
#include "BuildPatchInstall.h"
#include "BuildPatchMessage.h"
#include "BuildPatchVerify.h"
#include "BuildPatchState.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IBuildManifest.h"
#include "Interfaces/IBuildInstallerSharedContext.h"
#include "Misc/Optional.h"
#include "Misc/Variant.h"
#include "Templates/UnrealTemplate.h"

class FVariant;

namespace BuildPatchServices
{
	enum class EInstallActionIntent : int32;

	// A delegate returning a bool.
	DECLARE_DELEGATE_RetVal(bool, FChunkBuildBooleanDelegate);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FChunkBuildFilesEnumeratedDelegate, const TArray<FString>& /* Filenames */);
	/**
	 * An enum defining the method of sorting files in a build.
	 */
	enum class EFileSortOrder : uint8
	{
		AlphabeticalFilename = 0,
		AlphabeticalTagThenFilename,

		InvalidOrMax
	};

	/**
	 * Defines a list of all build patch services initialization settings, can be used to override default init behaviors.
	 */
	struct FBuildPatchServicesInitSettings
	{
	public:
		/**
		 * Default constructor. Initializes all members with default behavior values.
		 */
		BUILDPATCHSERVICES_API FBuildPatchServicesInitSettings();

	public:
		// The application settings directory.
		FString ApplicationSettingsDir;
		// The application project name.
		FString ProjectName;
		// The local machine config file name.
		FString LocalMachineConfigFileName;
	};

	struct FInstallerAction
	{
	public:

		/**
		 * Creates an install action.
		 * @param Manifest          The manifest for the build to be installed.
		 * @param InstallTags       The install tags to use if selectively installing files. If empty set, all files will be installed.
		 * @param InstallSubdirectory  The subdirectory to install this data to inside of the main install directory.
		 * @param CloudSubdirectory    The subdirectory of the cloud distribution root that this patch data should be sourced.
		 * @return the action setup for performing an installation.
		 */
		static BUILDPATCHSERVICES_API FInstallerAction MakeInstall(const IBuildManifestRef& Manifest, TSet<FString> InstallTags = TSet<FString>(), FString InstallSubdirectory = FString(), FString CloudSubdirectory = FString());

		/**
		 * Creates an update action.
		 * @param CurrentManifest   The manifest for the build currently installed.
		 * @param InstallManifest   The manifest for the build to be installed.
		 * @param InstallTags       The install tags to use if selectively installing files. If empty set, all files will be updated, or added if missing.
		 * @param InstallSubdirectory  The subdirectory to install this data to inside of the main install directory.
		 * @param CloudSubdirectory    The subdirectory of the cloud distribution root that this patch data should be sourced.
		 * @return the action setup for performing an update.
		 */
		static BUILDPATCHSERVICES_API FInstallerAction MakeUpdate(const IBuildManifestRef& CurrentManifest, const IBuildManifestRef& InstallManifest, TSet<FString> InstallTags = TSet<FString>(), FString InstallSubdirectory = FString(), FString CloudSubdirectory = FString());

		/**
		 * Creates a verify and repair action.
		 * @param Manifest          The manifest for the build being repaired.
		 * @param InstallTags       The install tags to use if selectively installing files. If empty set, all files will be repaired.
		 * @param InstallSubdirectory  The subdirectory to install this data to inside of the main install directory.
		 * @param CloudSubdirectory    The subdirectory of the cloud distribution root that this patch data should be sourced.
		 * @return the action setup forcing an SHA check, and repair of all tagged files.
		 */
		static BUILDPATCHSERVICES_API FInstallerAction MakeRepair(const IBuildManifestRef& Manifest, TSet<FString> InstallTags = TSet<FString>(), FString InstallSubdirectory = FString(), FString CloudSubdirectory = FString());

		/**
		 * Creates a verify only action. Note that MakeRepair should be used usually, which will verify and try to repair any issues right away.
		 * This action will only check integrity and fail or succeed based on results.
		 * @param Manifest          The manifest for the build being verified.
		 * @param InstallTags       The install tags to use if selectively installing files. If empty set, all files will be repaired.
		 * @param InstallSubdirectory  The subdirectory to install this data to inside of the main install directory.
		 * @param CloudSubdirectory    The subdirectory of the cloud distribution root that this patch data should be sourced.
		 * @return the action setup forcing an SHA check, and repair of all tagged files.
		 */
		static BUILDPATCHSERVICES_API FInstallerAction MakeVerifyOnly(const IBuildManifestRef& Manifest, TSet<FString> InstallTags = TSet<FString>(), FString InstallSubdirectory = FString(), FString CloudSubdirectory = FString());

		/**
		 * Creates an uninstall action.
		 * @param Manifest          The manifest for the build currently installed.
		 * @param InstallSubdirectory  The subdirectory to install this data to inside of the main install directory.
		 * @param CloudSubdirectory    The subdirectory of the cloud distribution root that this patch data should be sourced.
		 * @return the action setup for performing an uninstall, deleting all files referenced by the manifest.
		 */
		static BUILDPATCHSERVICES_API FInstallerAction MakeUninstall(const IBuildManifestRef& Manifest, FString InstallSubdirectory = FString(), FString CloudSubdirectory = FString());

		/**
		 * Helper for creating an install action or update action based on validity of CurrentManifest.
		 * @param CurrentManifest   The manifest for the build currently installed, invalid if no build installed.
		 * @param InstallManifest   The manifest for the build to be installed.
		 * @param InstallTags       The install tags to use if selectively installing files. If empty set, all files will be installed/updated, or added if missing.
		 * @param InstallSubdirectory  The subdirectory to install this data to inside of the main install directory.
		 * @param CloudSubdirectory    The subdirectory of the cloud distribution root that this patch data should be sourced.
		 * @return the action setup for performing an install or update.
		 */
		static FInstallerAction MakeInstallOrUpdate(const IBuildManifestPtr& CurrentManifest, const IBuildManifestRef& InstallManifest, TSet<FString> InstallTags = TSet<FString>(), FString InstallSubdirectory = FString(), FString CloudSubdirectory = FString())
		{
			if (!CurrentManifest.IsValid())
			{
				return MakeInstall(InstallManifest, MoveTemp(InstallTags), MoveTemp(InstallSubdirectory), MoveTemp(CloudSubdirectory));
			}
			else
			{
				return MakeUpdate(CurrentManifest.ToSharedRef(), InstallManifest, MoveTemp(InstallTags), MoveTemp(InstallSubdirectory), MoveTemp(CloudSubdirectory));
			}
		}

	public:
		/**
		 * @return true if this action intent is to perform a fresh installation.
		 */
		BUILDPATCHSERVICES_API bool IsInstall() const;

		/**
		 * @return true if this action intent is to update an existing installation.
		 */
		BUILDPATCHSERVICES_API bool IsUpdate() const;

		/**
		 * @return true if this action intent is to repair an existing installation.
		 */
		BUILDPATCHSERVICES_API bool IsRepair() const;

		/**
		 * @return true if this action intent is to only verify an existing installation, without repair or modification.
		 */
		BUILDPATCHSERVICES_API bool IsVerifyOnly() const;

		/**
		 * @return true if this action intent is to uninstall an installation.
		 */
		BUILDPATCHSERVICES_API bool IsUninstall() const;

		/**
		 * @return the install tags for the action.
		 */
		BUILDPATCHSERVICES_API const TSet<FString>& GetInstallTags() const;

		/**
		 * @return the install subdirectory for the action.
		 */
		BUILDPATCHSERVICES_API const FString& GetInstallSubdirectory() const;

		/**
		 * @return the cloud subdirectory for the action.
		 */
		BUILDPATCHSERVICES_API const FString& GetCloudSubdirectory() const;

		/**
		 * @return the manifest for the current installation, this will runtime assert if called invalidly (see TryGetCurrentManifest).
		 */
		BUILDPATCHSERVICES_API IBuildManifestRef GetCurrentManifest() const;

		/**
		 * @return the manifest for the desired installation, this will runtime assert if called invalidly (see TryGetInstallManifest).
		 */
		BUILDPATCHSERVICES_API IBuildManifestRef GetInstallManifest() const;

	public:
		/**
		 * Helper for getting the current manifest if nullable is preferred.
		 * @return the manifest for the current installation, if valid.
		 */
		IBuildManifestPtr TryGetCurrentManifest() const
		{
			if (IsUpdate() || IsRepair() || IsVerifyOnly() || IsUninstall())
			{
				return GetCurrentManifest();
			}
			return nullptr;
		}

		/**
		 * Helper for getting the install manifest if nullable is preferred.
		 * @return the manifest for the new installation, if valid.
		 */
		IBuildManifestPtr TryGetInstallManifest() const
		{
			if (IsInstall() || IsUpdate() || IsRepair() || IsVerifyOnly())
			{
				return GetInstallManifest();
			}
			return nullptr;
		}

		/**
		 * Helper for getting the current manifest, or install manifest if no current manifest. One will always be valid.
		 * @return the current, or the install manifest, based on validity respectively.
		 */
		IBuildManifestRef GetCurrentOrInstallManifest() const
		{
			return (CurrentManifest.IsValid() ? CurrentManifest : InstallManifest).ToSharedRef();
		}

		/**
		 * Helper for getting the install manifest, or current manifest if no install manifest. One will always be valid.
		 * @return the install, or the current manifest, based on validity respectively.
		 */
		IBuildManifestRef GetInstallOrCurrentManifest() const
		{
			return (InstallManifest.IsValid() ? InstallManifest : CurrentManifest).ToSharedRef();
		}

	private:
		BUILDPATCHSERVICES_API FInstallerAction();
		IBuildManifestPtr CurrentManifest;
		IBuildManifestPtr InstallManifest;
		TSet<FString> InstallTags;
		FString InstallSubdirectory;
		FString CloudSubdirectory;
		EInstallActionIntent ActionIntent;
	};

	/**
	 * DEPRECATED STRUCT. Please use FBuildInstallerConfiguration.
	 */
	struct FInstallerConfiguration
	{
		BUILDPATCHSERVICES_API FInstallerConfiguration(const IBuildManifestRef& InInstallManifest);
		BUILDPATCHSERVICES_API FInstallerConfiguration(const FInstallerConfiguration& CopyFrom);
		BUILDPATCHSERVICES_API FInstallerConfiguration(FInstallerConfiguration&& MoveFrom);

	public:
		IBuildManifestRef InstallManifest;
		IBuildManifestPtr CurrentManifest = nullptr;
		FString InstallDirectory;
		FString StagingDirectory;
		FString BackupDirectory;
		TArray<FString> ChunkDatabaseFiles;
		TArray<FString> CloudDirectories;
		TSet<FString> InstallTags;
		EInstallMode InstallMode = EInstallMode::NonDestructiveInstall;
		EVerifyMode VerifyMode = EVerifyMode::ShaVerifyAllFiles;
		EDeltaPolicy DeltaPolicy = EDeltaPolicy::Skip;
		bool bIsRepair = false;
		bool bRunRequiredPrereqs = true;
		bool bAllowConcurrentExecution = false;
	};

	/**
	 * Defines a list of all the options of an installation task.
	 */
	struct FBuildInstallerConfiguration
	{
		/**
		 * Construct with an array of action objects
		 */
		BUILDPATCHSERVICES_API FBuildInstallerConfiguration(TArray<FInstallerAction> InstallerActions);

	public:
		// The array of intended actions to perform.
		TArray<FInstallerAction> InstallerActions;
		// The context for allocating shared resources.
		IBuildInstallerSharedContextPtr SharedContext;
		// The directory to install to.
		FString InstallDirectory;
		// The directory for storing the intermediate files. This would usually be inside the InstallDirectory. Empty string will use module's global setting.
		FString StagingDirectory;
		// The directory for placing files that are believed to have local changes, before we overwrite them. Empty string will use module's global setting. If both empty, the feature disables.
		FString BackupDirectory;
		// The list of chunk database filenames that will be used to pull patch data from. These must be local (i.e. no URLs)
		TArray<FString> ChunkDatabaseFiles;
		// If set, when we've read everything we're going to read from a chunkdb file, we delete it to minimize required disk space for install/patch.
		bool bDeleteChunkDbFilesAfterUse = false;
		// If set, as we use chunks from a chunkdb file, we reduce it's size on disk to minimize required disk space for install/patch. This is only effective if the ChunkDB files are produced
		// with reverse chunk ordering.
		bool bTruncateChunkDbFilesAsUsed = false;

		// If set, don't actually do an installation, just figure out how much disk space is required if we are using entirely
		// chunkdbs and we delete them as we go.
		bool bCalculateDeleteChunkDbMaxDiskSpaceAndExit = false;

		// The list of cloud directory roots that will be used to pull patch data from. Empty array will use module's global setting. This is only hit if a chunk can not be satisfied by
		// other sources (i.e. chunkdbs or install directory for patches). If the chunk can not be serviced from here, it's a failure.
		TArray<FString> CloudDirectories;

		// Map of available encryption secrets.
		TMap<FGuid, TArray<uint8>> EncryptionSecrets;

		// The mode for installation.
		EInstallMode InstallMode = EInstallMode::NonDestructiveInstall;
		// The mode for verification. This will be overridden for any files referenced by a repair action.
		EVerifyMode VerifyMode = EVerifyMode::ShaVerifyAllFiles;
		// The policy to follow for requesting an optimised delta.
		EDeltaPolicy DeltaPolicy = EDeltaPolicy::Skip;
		// Optional specification for non-standard optimized delta
		FString DeltaFilenameTrailer;
		// Whether to run the prerequisite installer provided if it hasn't been ran before on this machine.
		bool bRunRequiredPrereqs = true;
		// Whether to skip the prerequisite installer provided if it already ran.
		bool bSkipPrereqIfAlreadyRan = true;
		// Whether to allow this installation to run concurrently with any existing installations.
		bool bAllowConcurrentExecution = false;
		// Download rate limit in bytes per second. (0 means unlimited)
		uint64 DownloadRateLimitBps = 0;
		// Use raw filenames for staging files
		bool bStageWithRawFilenames = false;
		// If true, fail the installation if the manifest contains a symlink.
		bool bRejectSymlinks = false;
		// Whether to run the custom uninstall action or not
		bool bSkipCustomUninstallAction = false;
		// Whether to gather individual file operation statistics during install
		bool bTrackFileOperations = false;

		// Whether the file constructor is allowed to spawn multiple threads for IO read/write servicing.
		// Unset uses Engine.ini value [Portal.BuildPatch]ConstructorSpawnAdditionalIOThreads if present, FFileConstructorConfig::bDefaultSpawnAdditionalIOThreads otherwise.
		TOptional<bool> ConstructorSpawnAdditionalIOThreads;
		// File constructor batch size, in MB. File will attempt to fill/write this chunk size to disk. 
		// Unset uses Engine.ini value [Portal.BuildPatch]ConstructorIOBatchSizeMB if present, FFileConstructorConfig::DefaultIOBatchSizeMB otherwise.
		TOptional<int32> ConstructorIOBatchSizeMB;

		// File constructor buffer size, in MB. This affects constructor performance heavily and must be at least big enough to fit a batch. 
		// For purely disk->disk installs, this can be relatively small - even 2x batch size will facilitate multiple GB/s installations. However
		// for cloud/download installs this needs to be fairly large to mitigate the effect of out of order download completion. 
		// When bConstructFilesInMemory or bInstallToMemory is set, this is used to determine how many active files can be in flight.
		// Unset uses Engine.ini value [Portal.BuildPatch]ConstructorIOBufferSizeMB if present, FFileConstructorConfig::DefaultIOBufferSizeMB otherwise.
		TOptional<int32> ConstructorIOBufferSizeMB;

		// Whether the file construction should adhere to file system write limits. If a write limit is hit, writing will stall until
		// sufficient quota exists for the write. Inheritance for this property is:
		// CVar BuildPatchFileConstructor.bStallWhenFileSystemThrottled, which defaults to true.
		// if set, this member.
		TOptional<bool> ConstructorStallWhenFileSystemThrottled;

		// Whether to disable resume on clean installs below a given threshold, in megabytes. 0 means "never disable".
		// Inheritance:
		// CVar BuildPatchFileConstructor.DisableResumeBelowMB, which defaults to 0.
		// if set, this member
		TOptional<int32> ConstructorDisableResumeBelowMB;

		// If set, files will be built entirely in memory - requiring at least the largest file's size worth of memory.
		// IO batch and buffer sizes are ignored as operations operate directly on the output memory buffer. If allowing
		// multiple files in flight, this causes the size of the entire installation to be allocated as there's no IO buffer
		// size limiting the number of in flight requests.
		// Once the file completes construction it is written to disk in the normal place defined by InstallMode. This occurs
		// after the old file is deleted for Destructive installations in order to manage peak disk space usage.
		//
		// This mode does not support resuming mid-file.
		bool bConstructFilesInMemory = false;

		// As bConstructFilesInMemory, except the file is never written to disk and is instead made available after via 
		// GetFilesInstalledToMemory 
		bool bInstallToMemory = false;

		// If set, no initial size check will be performed prior to starting the installation. Running out of disk space during installation is unchanged.
		bool bSkipInitialDiskSizeCheck = false;
	};

	/**
	 * Defines a list of all options for the build chunking task.
	 */
	struct FDirectoryChunkerConfiguration
	{
	public:
		// The client feature level to output data for.
		EFeatureLevel FeatureLevel = EFeatureLevel::Latest;
		// The directory to analyze.
		FString RootDirectory;
		// The ID of the app of this build.
		uint32 AppId = 0;
		// The name of the app of this build.
		FString AppName;
		// The version string for this build.
		FString BuildVersion;
		// The local exe path that would launch this build.
		FString LaunchExe;
		// The command line that would launch this build.
		FString LaunchCommand;
		// The path to a file containing a \r\n separated list of RootDirectory relative files to read.
		FString InputListFile;
		// The path to a file containing a \r\n separated list of RootDirectory relative files to ignore.
		FString IgnoreListFile;
		// The path to a file containing a \r\n separated list of RootDirectory relative files followed by attribute keywords.
		FString AttributeListFile;
		// The order in which to stream the build files.
		EFileSortOrder FileSortOrder = EFileSortOrder::AlphabeticalTagThenFilename;
		// The set of identifiers which the prerequisites satisfy.
		TSet<FString> PrereqIds;
		// The display name of the prerequisites installer.
		FString PrereqName;
		// The path to the prerequisites installer.
		FString PrereqPath;
		// The command line arguments for the prerequisites installer.
		FString PrereqArgs;
		// The path to the uninstall custom action executable.
		FString UninstallActionPath;
		// The command line arguments for the uninstall custom action executable.
		FString UninstallActionArgs;
		// The maximum age (in days) of existing data files which can be reused in this build.
		float DataAgeThreshold = TNumericLimits<float>::Max();
		// Indicates whether data age threshold should be honored. If false, ALL data files can be reused taking into
		// account the ReuseManifestPredicate.
		bool bShouldHonorReuseThreshold = false;
		// A callable that takes a const reference to an IBuildManifest and returns true to reuse the manifest
		// and its chunks, or false to fully ignore it.
		TFunction<bool(const IBuildManifest&)> ReuseManifestPredicate = [](const IBuildManifest&) { return true; };
		// The chunk window size to be used when saving out new data.
		uint32 OutputChunkWindowSize = 1024 * 1024;
		// Indicates whether any window size chunks should be matched, rather than just out output window size.
		bool bShouldMatchAnyWindowSize = true;
		// Indicates that if a chunk was already found in the cloud directory, to resave it anyway.
		bool bResaveExistingChunks = false;
		// Indicates that if a chunk was matched from a previous build, to resave it anyway.
		bool bResaveKnownChunks = false;
		// The encryption secret key to use for newly created chunk files, and the manifest itself.
		TArray<uint8> EncryptionSecretKey;
		// An ID that identifies the secret key above, used to know which secrets are required for each manifest & chunk.
		FGuid EncryptionSecretId;
		// Map of custom fields to add to the manifest.
		TMap<FString, FVariant> CustomFields;
		// The cloud directory that all patch data will be saved to. An empty value will use module's global setting.
		FString CloudDirectory;
		// The output manifest filename.
		FString OutputFilename;
		// Allow Manifest Creation for builds with no data
		bool bAllowEmptyBuild = false;
		// Whether to follow symlinks, or if false, replicate them. True would process the build as if all links were actual files and directories.
		// Note that this currently only affects mac and linux systems.
		bool bFollowSymlinks = false;
		// SHA256 and MD5 are expensive and can slow down the chunking process, set this true to skip calculation and save as zeroed values.
		bool bSHA1Only = false;
	};

	// Temporary for use with deprecated module functions.
	typedef FDirectoryChunkerConfiguration FGenerationConfiguration;
	typedef FDirectoryChunkerConfiguration FChunkBuildConfiguration;

	/**
	 * An enum defining the units of DiffAbortThreshold value.
	 */
	enum class EDiffAbortThresholdUnits : uint8
	{
		Absolute = 0,
		Percentage,
	};

	/**
	 * Defines the threshold for the original delta size, for which we would abort and not process.
	 */
	struct FDiffAbortThreshold
	{
	public:

		// A threshold for the original delta size, for which we would abort and not process.
		uint64 Value = 0ULL;

		// A units of measurement for Value field. 
		EDiffAbortThresholdUnits Unit = EDiffAbortThresholdUnits::Absolute;
	};

	/**
	 * Defines a namespace for threshold value limits.
	 */
	namespace DiffAbortThresholdLimits
	{
		// Minimum threshold value limit, expressed as a percentage
		constexpr uint64 MinPercentage = 1ULL;

		// Minimum threshold value limit, expressed in absolute units.
		constexpr uint64 MinAbsolute = 1000ULL * 1000ULL * 1000ULL;
	}

	/**
	 * Defines a list of all options for the chunk delta optimisation task.
	 */
	struct FChunkDeltaOptimiserConfiguration
	{
	public:
		/**
		 * Default constructor
		 */
		BUILDPATCHSERVICES_API FChunkDeltaOptimiserConfiguration();

	public:
		// A full file or http path for the manifest to be used as the source build.
		FString ManifestAUri;
		// A full file or http path for the manifest to be used as the destination build.
		FString ManifestBUri;
		// The cloud directory that all patch data will be saved to. An empty value will use ManifestB's directory.
		FString OutputCloudDirectory;
		// The cloud directories that existing patch data can be downloaded from. An empty array will use ManifestB's directory.
		TArray<FString> DownloadCloudDirectories;
		// The window size to use for find new matches.
		uint32 ScanWindowSize;
		// The chunk size to use for saving new diff data.
		uint32 OutputChunkSize;
		// Indicates that if a chunk was already found in the cloud directory, to resave it anyway.
		bool bResaveExistingChunks;
		// A threshold for the original delta size, for which we would abort and not process.
		TOptional<FDiffAbortThreshold> DiffAbortThreshold;
		// Map of available encryption secrets, if any are missing, the process will fail.
		TMap<FGuid, TArray<uint8>> EncryptionSecrets;
		// Array of message handlers to provide signed URLs for source build chunks downloads.
		TArray<FMessageHandler*> MessageHandlersA;
		// Array of message handlers to provide signed URLs for destination build chunks and delta file downloads.
		TArray<FMessageHandler*> MessageHandlersB;

		// The old and new build aliases. Tags in the old and new build will be wholesale replaced by any aliases
		// in these maps, allowing for control over what data is available for the patching system to match from. By default
		// matches are only allowed from the same tag, with this one can virtually replace a set of tags with the same tag, allowing
		// the patcher to select matches from all of the aliased tags.
		TMap<FString, FString> ManifestATagAliases;
		TMap<FString, FString> ManifestBTagAliases;
		TSet<FString> ManifestAIgnoreTags;
		TSet<FString> ManifestBIgnoreTags;
		
		// The custom optimized delta filename to use for generating this optimized delta. If used, all BPS modes
		// that request an optimized delta will need the same trailer passed in.
		FString DeltaFilenameTrailer;

		// If specified, the source information for where each chunk matches from will be output to this file.
		FString SourceDetailsLogFilename;
	};

	/**
	 * Defines a list of all options for the patch data enumeration task.
	 */
	struct FPatchDataEnumerationConfiguration
	{
	public:
		/**
		 * Default constructor
		 */
		BUILDPATCHSERVICES_API FPatchDataEnumerationConfiguration();

	public:
		// A full file path for the manifest or chunkdb to enumerate referenced data for.
		FString InputFile;
		// A full file path to a file where the list will be saved out to.
		FString OutputFile;
		// Whether to include files sizes.
		bool bIncludeSizes;
	};

	/**
	 * Defines a list of all options for the diff manifests task.
	 */
	struct FDiffManifestsConfiguration
	{
	public:
		// A full file or http path for the manifest to be used as the source build.
		FString ManifestAUri;
		// A full file or http path for the manifest to be used as the destination build.
		FString ManifestBUri;
		// List of directories or http paths that can be used to request patch data. If empty, then path to ManifestA will be used.
		TArray<FString> CloudDirs;
		// The tag set to use to filter desired files from ManifestA.
		TSet<FString> TagSetA;
		// The tag set to use to filter desired files from ManifestB.
		TSet<FString> TagSetB;
		// Tag sets that will be used to calculate additional differential size statistics between manifests.
		// They must all be a subset of anything used in TagSetA.
		TArray<TSet<FString>> CompareTagSetsA;
		// If set, the tagset comparison will use these as the new tags and CompareTagSets will be the old installed tags, to allow for
		// comparing the diff when the installed set of tags changes.
		TArray<TSet<FString>> CompareTagSetsB;
		// A full file path where a JSON object will be saved for the diff details.Empty string if not desired.
		FString OutputFilePath;
		// Array of message handlers to provide signed URIs for delta file.
		TArray<FMessageHandler*> MessageHandlers;

		// A destination base directory to use for writing out patch description files for the patch between the two
		// manifest.
		FString OutputPatchDescriptorPath;

		// By default, DiffManifest will use TryFetchContinueWithout and will, as a result, show large patches. If your workflows are such that
		// you always expect an optimized delta, set this to cause DiffManifest to fail instead of show irrelevant data.
		bool bRequireOptimizedDelta = false;

		// If true, only emit patch descriptors and not the entirety of the diff manifests output.
		bool bOnlyPatchDescriptors = false;

		// If true, emit simulated installation time estimations
		bool bSimulateInstallTimes = false;

		// The custom optimized delta filename to use, if desired.
		FString DeltaFilenameTrailer;
	};

	/**
	 * Defines a list of all options for the cloud directory compactifier task.
	 */
	struct FCompactifyConfiguration
	{
	public:
		/**
		 * Default constructor
		 */
		BUILDPATCHSERVICES_API FCompactifyConfiguration();

	public:
		// The path to the directory to compactify.
		FString CloudDirectory;
		// Chunks which are not referenced by a valid manifest, and which are older than this age(in days), will be deleted.
		float DataAgeThreshold;
		// The full path to a file to which a list of all chunk files deleted by compactify will be written.The output filenames will be relative to the cloud directory.
		FString DeletedChunkLogFile;
		// If ran in preview mode, then the process will run in logging mode only - no files will be deleted.
		bool bRunPreview;
	};

	/**
	 * Defines a list of all options for the chunk packaging task.
	 */
	struct FPackageChunksConfiguration
	{
	public:
		/**
		 * Default constructor
		 */
		BUILDPATCHSERVICES_API FPackageChunksConfiguration();

	public:
		// The client feature level to output data for.
		EFeatureLevel FeatureLevel;
		// A full file path to the manifest to enumerate chunks from.
		FString ManifestFilePath;
		// A full file path to a manifest describing a previous build, which will filter out saved chunks for patch only chunkdbs.
		FString PrevManifestFilePath;
		// Optional list of tagsets to split chunkdb files on. Empty array will include all data as normal.
		TArray<TSet<FString>> TagSetArray;
		// Optional tagset to filter the files used from PrevManifestFilePath, potentially increasing the number of chunks saved.
		TSet<FString> PrevTagSet;
		// A full file path to the chunkdb file to save. Extension of .chunkdb will be added if not present.
		FString OutputFile;
		// Cloud directory where chunks to be packaged can be found.
		FString CloudDir;
		// The maximum desired size for each chunkdb file.
		uint64 MaxOutputFileSize;
		// A full file path to use when saving the json output data.
		FString ResultDataFilePath;
		// Optional flag for ignoring manifest file invalid tags
		bool bIgnoreManifestFileInvalidTags;
		// Optional flag for ignoring manifest describing a previous build file invalid tags
		bool bIgnorePrevManifestFileInvalidTags;

		// Optional specification for non-standard optimized delta
		FString DeltaFilenameTrailer;
	};

	/**
	 * Defines a list of all options for the chunk harvesting task.
	 */
	struct FChunkHarvesterConfiguration
	{
	public:
		/**
		 * Default constructor
		 */
		BUILDPATCHSERVICES_API FChunkHarvesterConfiguration();

	public:
		// Set an override feature level to output newer format.
		EFeatureLevel FeatureLevelOverride;
		// A full file path to the manifest to enumerate chunks from.
		FString ManifestFilePath;
		// The directory containing the binary image to be read.
		FString BuildRoot;
		// The directory where existing data will be recognized from, and new data added to.
		FString CloudDir;
		// Indicates that if a chunk was already found in the cloud directory, to resave it anyway.
		bool bResaveExistingChunks;
	};
}

static_assert((uint32)BuildPatchServices::EFileSortOrder::InvalidOrMax == 2, "Please add support for the extra values to the Lex functions below.");

inline const TCHAR* LexToString(BuildPatchServices::EFileSortOrder FileSortOrder)
{
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::EFileSortOrder::Value: return TEXT(#Value)
	switch (FileSortOrder)
	{
		CASE_ENUM_TO_STR(AlphabeticalFilename);
		CASE_ENUM_TO_STR(AlphabeticalTagThenFilename);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::EFileSortOrder& FileSortOrder, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { FileSortOrder = BuildPatchServices::EFileSortOrder::Value; return; }
	const TCHAR* const Prefix = TEXT("EFileSortOrder::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(AlphabeticalFilename);
	RETURN_IF_EQUAL(AlphabeticalTagThenFilename);
	// Did not match
	FileSortOrder = BuildPatchServices::EFileSortOrder::InvalidOrMax;
#undef RETURN_IF_EQUAL
}
inline FString LexToString(const TOptional<BuildPatchServices::FDiffAbortThreshold>& DiffAbortThreshold)
{
	constexpr auto* DiffAbortThresholdString = TEXT("Never Abort");
	if (!DiffAbortThreshold.IsSet())
	{
		return DiffAbortThresholdString;
	}

	switch (DiffAbortThreshold.GetValue().Unit)
	{
	case BuildPatchServices::EDiffAbortThresholdUnits::Absolute:
		return FString::FormatAsNumber(int32(DiffAbortThreshold.GetValue().Value / (1024 * 1024))) + TEXT(" MiB");
	case BuildPatchServices::EDiffAbortThresholdUnits::Percentage:
		return FString::FormatAsNumber(int32(DiffAbortThreshold.GetValue().Value)) + TEXT("%");
	default:
		checkf(false, TEXT("Unexpected EDiffAbortThresholdUnits value: %hhu"), EnumToUnderlyingType(DiffAbortThreshold.GetValue().Unit));
		break;
	}

	return DiffAbortThresholdString;
}

inline FString LexToString(BuildPatchServices::EDiffAbortThresholdUnits Unit)
{
	#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::EDiffAbortThresholdUnits::Value: return TEXT(#Value)

	switch (Unit)
	{
		CASE_ENUM_TO_STR(Absolute);
		CASE_ENUM_TO_STR(Percentage);
		default: return FString::Printf(TEXT("Unexpected value: %hhu"), EnumToUnderlyingType(Unit));
	}

	#undef CASE_ENUM_TO_STR
}
