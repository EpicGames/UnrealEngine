// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"

class FString;
class FText;

namespace BuildPatchServices
{
	/**
	 * Namespace to declares the progress type enum
	 */
	enum class EBuildPatchState : uint32
	{
		// The patch process is waiting for other installs
		Queued = 0,

		// The patch process is initializing
		Initializing,

		// The patch process is enumerating existing staged data
		Resuming,

		// The patch process is downloading patch data
		Downloading,

		// The patch process is installing files
		Installing,

		// The patch process is running custom uninstall action
		RunningCustomUninstallAction,

		// The patch process is moving staged files to the install
		MovingToInstall,

		// The patch process is setting up attributes on the build
		SettingAttributes,

		// The patch process is verifying the build
		BuildVerification,

		// The patch process is cleaning temp files
		CleanUp,

		// The patch process is installing prerequisites
		PrerequisitesInstall,

		// A state to catch the UI when progress is 100% but UI still being displayed
		Completed,

		// The process has been set paused
		Paused,

		// Holds the number of states, for array sizes
		NUM_PROGRESS_STATES,
	};

	/**
	 * Returns the FText representation of the specified EBuildPatchState value. Used for displaying to the user.
	 * @param State - The error type value.
	 * @return The display text associated with the progress step.
	 */
	UE_DEPRECATED(4.21, "BuildPatchServices::StateToText(const EBuildPatchState& State) has been deprecated. It will no longer be supported by BuildPatchServices in the future.")
	BUILDPATCHSERVICES_API const FText& StateToText(const EBuildPatchState& State);
}

static_assert((uint32)BuildPatchServices::EBuildPatchState::NUM_PROGRESS_STATES == 13, "Please add support for the extra values to the Lex functions below.");

inline const TCHAR* LexToString(BuildPatchServices::EBuildPatchState State)
{
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::EBuildPatchState::Value: return TEXT(#Value)
	switch (State)
	{
		CASE_ENUM_TO_STR(Queued);
		CASE_ENUM_TO_STR(Initializing);
		CASE_ENUM_TO_STR(Resuming);
		CASE_ENUM_TO_STR(Downloading);
		CASE_ENUM_TO_STR(Installing);
		CASE_ENUM_TO_STR(RunningCustomUninstallAction);
		CASE_ENUM_TO_STR(MovingToInstall);
		CASE_ENUM_TO_STR(SettingAttributes);
		CASE_ENUM_TO_STR(BuildVerification);
		CASE_ENUM_TO_STR(CleanUp);
		CASE_ENUM_TO_STR(PrerequisitesInstall);
		CASE_ENUM_TO_STR(Completed);
		CASE_ENUM_TO_STR(Paused);
	default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::EBuildPatchState& Error, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { Error = BuildPatchServices::EBuildPatchState::Value; return; }
	const TCHAR* const Prefix = TEXT("EBuildPatchState::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(Queued);
	RETURN_IF_EQUAL(Initializing);
	RETURN_IF_EQUAL(Resuming);
	RETURN_IF_EQUAL(Downloading);
	RETURN_IF_EQUAL(Installing);
	RETURN_IF_EQUAL(RunningCustomUninstallAction);
	RETURN_IF_EQUAL(MovingToInstall);
	RETURN_IF_EQUAL(SettingAttributes);
	RETURN_IF_EQUAL(BuildVerification);
	RETURN_IF_EQUAL(CleanUp);
	RETURN_IF_EQUAL(PrerequisitesInstall);
	RETURN_IF_EQUAL(Completed);
	RETURN_IF_EQUAL(Paused);
	// Did not match
	Error = BuildPatchServices::EBuildPatchState::NUM_PROGRESS_STATES;
	return;
#undef RETURN_IF_EQUAL
}