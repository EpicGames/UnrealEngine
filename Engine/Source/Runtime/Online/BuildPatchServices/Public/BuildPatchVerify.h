// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumRange.h"

namespace BuildPatchServices
{
	/**
	 * An enum defining the verification mode that should be used.
	 */
	enum class EVerifyMode : uint32
	{
		// Fully SHA checks all files in the build.
		ShaVerifyAllFiles = 0,

		// Fully SHA checks only files touched by the install/patch process.
		ShaVerifyTouchedFiles,

		// Checks just the existence and file size of all files in the build.
		FileSizeCheckAllFiles,

		// Checks just the existence and file size of only files touched by the install/patch process.
		FileSizeCheckTouchedFiles,

		InvalidOrMax
	};

	/**
	 * An enum defining the possible causes for a verification failure.
	 */
	enum class EVerifyError : uint32
	{
		// The file was not found.
		FileMissing = 0,

		// The file failed to open.
		OpenFileFailed,

		// The file failed its hash check.
		HashCheckFailed,

		// The file was not the expected size.
		FileSizeFailed,

		InvalidOrMax
	};
}

ENUM_RANGE_BY_FIRST_AND_LAST(BuildPatchServices::EVerifyError, BuildPatchServices::EVerifyError::FileMissing, BuildPatchServices::EVerifyError::FileSizeFailed)

static_assert((uint32)BuildPatchServices::EVerifyMode::InvalidOrMax == 4, "Please add support for the extra values to the Lex functions below.");

inline const TCHAR* LexToString(BuildPatchServices::EVerifyMode VerifyMode)
{
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::EVerifyMode::Value: return TEXT(#Value)
	switch (VerifyMode)
	{
		CASE_ENUM_TO_STR(ShaVerifyAllFiles);
		CASE_ENUM_TO_STR(ShaVerifyTouchedFiles);
		CASE_ENUM_TO_STR(FileSizeCheckAllFiles);
		CASE_ENUM_TO_STR(FileSizeCheckTouchedFiles);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::EVerifyMode& VerifyMode, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { VerifyMode = BuildPatchServices::EVerifyMode::Value; return; }
	const TCHAR* const Prefix = TEXT("EVerifyMode::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(ShaVerifyAllFiles);
	RETURN_IF_EQUAL(ShaVerifyTouchedFiles);
	RETURN_IF_EQUAL(FileSizeCheckAllFiles);
	RETURN_IF_EQUAL(FileSizeCheckTouchedFiles);
	// Did not match
	VerifyMode = BuildPatchServices::EVerifyMode::InvalidOrMax;
#undef RETURN_IF_EQUAL
}

static_assert((uint32)BuildPatchServices::EVerifyError::InvalidOrMax == 4, "Please add support for the extra values to the Lex functions below.");

inline const TCHAR* LexToString(BuildPatchServices::EVerifyError VerifyError)
{
#define CASE_ENUM_TO_STR(Value) case BuildPatchServices::EVerifyError::Value: return TEXT(#Value)
	switch (VerifyError)
	{
		CASE_ENUM_TO_STR(FileMissing);
		CASE_ENUM_TO_STR(OpenFileFailed);
		CASE_ENUM_TO_STR(HashCheckFailed);
		CASE_ENUM_TO_STR(FileSizeFailed);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::EVerifyError& VerifyError, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { VerifyError = BuildPatchServices::EVerifyError::Value; return; }
	const TCHAR* const Prefix = TEXT("EVerifyError::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(FileMissing);
	RETURN_IF_EQUAL(OpenFileFailed);
	RETURN_IF_EQUAL(HashCheckFailed);
	RETURN_IF_EQUAL(FileSizeFailed);
	// Did not match
	VerifyError = BuildPatchServices::EVerifyError::InvalidOrMax;
#undef RETURN_IF_EQUAL
}
