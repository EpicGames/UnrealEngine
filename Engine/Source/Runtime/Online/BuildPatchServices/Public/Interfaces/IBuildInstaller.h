// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IBuildInstaller.h: Declares the IBuildInstaller interface.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "BuildPatchState.h"
#include "BuildPatchMessage.h"

class IBuildInstaller;

typedef TSharedPtr<IBuildInstaller, ESPMode::ThreadSafe> IBuildInstallerPtr;
typedef TSharedRef<IBuildInstaller, ESPMode::ThreadSafe> IBuildInstallerRef;
typedef TWeakPtr<IBuildInstaller, ESPMode::ThreadSafe> IBuildInstallerWeakPtr;

/**
 * Declares the error type enum for use with the error system
 */
enum class EBuildPatchInstallError
{
	// There has been no registered error
	NoError = 0,

	// A download request failed and ran out of allowed retries
	DownloadError = 1,

	// A file failed to construct properly
	FileConstructionFail = 2,

	// An error occurred trying to move the file to the install location
	MoveFileToInstall = 3,

	// The installed build failed to verify
	BuildVerifyFail = 4,

	// The user or some process has closed the application
	ApplicationClosing = 5,

	// An application error, such as module fail to load.
	ApplicationError = 6,

	// User canceled download
	UserCanceled = 7,

	// A prerequisites installer failed
	PrerequisiteError = 8,

	// An initialization error
	InitializationError = 9,

	// An error occurred creating a file due to excessive path length
	PathLengthExceeded = 10,

	// An error occurred creating a file due to their not being enough space left on the disk
	OutOfDiskSpace = 11,

	// The custom uninstall action failed
	CustomUninstallActionError = 12,

	// Used to help verify logic
	InvalidOrMax,
	// Deprecated value. Use InvalidOrMax instad of NumInstallErrors
	NumInstallErrors = InvalidOrMax,
};

/**
 * Declares the error code prefixes for each error type enum
 */
namespace InstallErrorPrefixes
{
	static const TCHAR* ErrorTypeStrings[] =
	{
		TEXT("OK"), // NoError
		TEXT("DL"), // DownloadError
		TEXT("FC"), // FileConstructionFail
		TEXT("MF"), // MoveFileToInstall
		TEXT("BV"), // BuildVerifyFail
		TEXT("SD"), // ApplicationClosing
		TEXT("FA"), // ApplicationError
		TEXT("UC"), // UserCanceled
		TEXT("PQ"), // PrerequisiteError
		TEXT("IZ"), // InitializationError
		TEXT("PL"), // PathLengthExceeded
		TEXT("DS"), // OutOfDiskSpace
		TEXT("UA"), // UninstallCustomActionError
	};
};

// Enum describing download health. The actual percentage values used are configurable in the engine ini.
enum class EBuildPatchDownloadHealth
{
	// All requests are in the retrying state. No progress currently. Possibly disconnected.
	Disconnected = 0,
	// More than 10% of requests are failing.
	Poor,
	// 10% or fewer requests are failing.
	OK,
	// 1% or fewer requests are failing.
	Good,
	// No requests are failing.
	Excellent,

	// Must be last value, only used for value counts.
	NUM_Values
};

/**
 * Tracks install stats for a given cloud directory.
 */
struct FCloudDirectoryStats
{
	// The identifier for this cloud directory, e.g. base URL.
	FString CloudDirectory;

	// Total number of bytes downloaded from this cloud directory.
	uint64 TotalReceivedData = 0;

	// Total number of errors encountered from this cloud directory.
	uint32 ErrorCount = 0;
};

/**
 * A struct to hold stats for a single install action.
 */
struct FInstallActionStats
{
public:
	bool UsedOptimizedDelta = false;
	uint64 OptimizedDeltaHTTPResponseCode = 0;
	uint64 NumLocalChunksExpected = 0;
	uint64 NumLocalChunksAvailable = 0;
	uint64 NumChunksRecycleFailures = 0;
};

/**
 * A struct to hold stats for the build process.
 */
struct FBuildInstallStats
{
	// Constructor
	FBuildInstallStats()
		: NumFilesInBuild(0)
		, NumFilesOutdated(0)
		, NumFilesToRemove(0)
		, NumChunksRequired(0)
		, ChunksQueuedForDownload(0)
		, ChunksLocallyAvailable(0)
		, ChunksInChunkDbs(0)
		, NumChunksDownloaded(0)
		, NumChunksRecycled(0)
		, NumChunksReadFromChunkDbs(0)
		, NumFailedDownloads(0)
		, NumBadDownloads(0)
		, NumAbortedDownloads(0)
		, NumRecycleFailures(0)
		, NumDriveStoreChunkLoads(0)
		, NumDriveStoreLoadFailures(0)
		, NumChunkDbChunksFailed(0)
		, TotalDownloadedData(0)
		, ActiveRequestCountPeak(0)
		, AverageDownloadSpeed(0.0)
		, PeakDownloadSpeed(0.0)
		, FinalDownloadSpeed(-1.0)
		, TheoreticalDownloadTime(0.0f)
		, TotalReadData(0)
		, AverageDiskReadSpeed(0.0)
		, PeakDiskReadSpeed(0.0)
		, TotalWrittenData(0)
		, AverageDiskWriteSpeed(0.0)
		, PeakDiskWriteSpeed(0.0)
		, NumFilesConstructed(0)
		, InitializeTime(0.0f)
		, ConstructTime(0.0f)
		, UninstallActionTime(0.0f)
		, MoveFromStageTime(0.0f)
		, FileAttributesTime(0.0f)
		, VerifyTime(0.0f)
		, CleanUpTime(0.0f)
		, PrereqTime(0.0f)
		, ProcessPausedTime(0.0f)
		, ProcessActiveTime(0.0f)
		, ProcessExecuteTime(0.0f)
		, ProcessSuccess(false)
		, NumInstallRetries(0)
		, FailureType(EBuildPatchInstallError::InitializationError)
		, RetryFailureTypes()
		, ErrorCode()
		, RetryErrorCodes()
		, FailureReasonText()
		, FinalProgress(0.0f)
		, OverallRequestSuccessRate(0.0f)
		, ExcellentDownloadHealthTime(0.0f)
		, GoodDownloadHealthTime(0.0f)
		, OkDownloadHealthTime(0.0f)
		, PoorDownloadHealthTime(0.0f)
		, DisconnectedDownloadHealthTime(0.0f)
		, ProcessRequiredDiskSpace(0)
		, ProcessAvailableDiskSpace(0)
		, CloudDirectoryStats()
		, ActionsStats()
	{}

	// The total number of files in the build.
	uint32 NumFilesInBuild;
	// The total number of files outdated.
	uint32 NumFilesOutdated;
	// The total number of files in the previous build that can be deleted.
	uint32 NumFilesToRemove;
	// The total number of chunks making up those files.
	uint32 NumChunksRequired;
	// The number of required chunks queued for download.
	uint32 ChunksQueuedForDownload;
	// The number of chunks locally available in the build.
	uint32 ChunksLocallyAvailable;
	// The number of chunks available in chunkdb files.
	uint32 ChunksInChunkDbs;
	// The total number of chunks that were downloaded.
	uint32 NumChunksDownloaded;
	// The number of chunks successfully recycled.
	uint32 NumChunksRecycled;
	// The number of chunks successfully read from chunkdbs.
	uint32 NumChunksReadFromChunkDbs;
	// The number of chunks we did not successfully receive.
	uint32 NumFailedDownloads;
	// The number of chunks we received but were determined bad data.
	uint32 NumBadDownloads;
	// The number of chunks we aborted as they were determined as taking too long.
	uint32 NumAbortedDownloads;
	// The number of chunks that failed to be recycled from existing build.
	uint32 NumRecycleFailures;
	// The number of chunks that had to be loaded from the drive store.
	uint32 NumDriveStoreChunkLoads;
	// The number of chunks that failed to load from the drive store.
	uint32 NumDriveStoreLoadFailures;
	// The max disk space used by the drive store
	uint32 DriveStorePeakBytes = 0;
	// The number of chunks that were lost due to drive space limits
	uint32 NumDriveStoreLostChunks = 0;
	// The number of chunks that were not successfully loaded from provided chunkdbs.
	uint32 NumChunkDbChunksFailed;
	// The total number of bytes downloaded.
	uint64 TotalDownloadedData;
	// The peak number of simultaneous download requests
	uint32 ActiveRequestCountPeak;
	// The average number of simultaneous download requests 
	uint32 ActiveRequestCountAvg;
	// The Experiment name if applicable for A/B Testing
	FString ExperimentName;
	// The Variant of the Experiment if applicable for A/B Testing
	FString ExerimentVariantName;
	// The average chunk download speed.
	double AverageDownloadSpeed;
	// The peak chunk download speed.
	double PeakDownloadSpeed;
	// The download speed registered at the end of the installation.
	double FinalDownloadSpeed;
	// The theoretical download time (data/speed).
	float TheoreticalDownloadTime;
	// The total number of bytes read to disk.
	uint64 TotalReadData;
	// The average disk read speed.
	double AverageDiskReadSpeed;
	// The peak disk read speed.
	double PeakDiskReadSpeed;
	// The total number of bytes written to disk.
	uint64 TotalWrittenData;
	// The average disk write speed.
	double AverageDiskWriteSpeed;
	// The peak disk write speed.
	double PeakDiskWriteSpeed;
	// The total number of files constructed.
	uint32 NumFilesConstructed;
	
	// We don't actually do any installation when requesting this value, we just start up an installer, compute it, then bail. See
	// bCalculateDeleteChunkDbMaxDiskSpaceAndExit in the installer configuration. Will be zero all other times.
	uint64 MaxDiskSpaceNeededWhenDeletingChunkDbsIfRequested = 0;

	// The time spent during the initialization stage.
	float InitializeTime;
	// The time spent during the construction stage.
	float ConstructTime;
	// The time spent on chunks decryption.
	uint64 TotalDecryptTime;
	// The time spent during the custom uninstall actiion run.
	float UninstallActionTime;
	// The time spent moving staged files into the installation location.
	float MoveFromStageTime;
	// The time spent during the file attribution stage.
	float FileAttributesTime;
	// The time spent during the verification stage.
	float VerifyTime;
	// The time spent during the clean up stage.
	float CleanUpTime;
	// The time spent during the prerequisite stage.
	float PrereqTime;
	// The amount of time that was spent paused.
	float ProcessPausedTime;
	// The amount of time that was spent active (un-paused).
	float ProcessActiveTime;
	// The total time that the install process took to complete.
	float ProcessExecuteTime;
	// Whether the process was successful.
	bool ProcessSuccess;
	// The number of times the system looped to retry.
	uint32 NumInstallRetries;
	// The failure type for the install.
	EBuildPatchInstallError FailureType;
	// If NumInstallRetries > 0, this will contain the list of retry reasons for retrying.
	TArray<EBuildPatchInstallError> RetryFailureTypes;
	// The error code. No error results in 'OK'.
	FString ErrorCode;
	// If NumInstallRetries > 0, this will contain the list of error codes for each retry.
	TArray<FString> RetryErrorCodes;
	// The localized, more generic failure reason.
	FText FailureReasonText;
	// Final progress state, this is the progress of the current retry attempt.
	float FinalProgress;
	// The overall rate of success for download requests.
	float OverallRequestSuccessRate;
	// The amount of time that was spent with Excellent download health.
	float ExcellentDownloadHealthTime;
	// The amount of time that was spent with Good download health.
	float GoodDownloadHealthTime;
	// The amount of time that was spent with OK download health.
	float OkDownloadHealthTime;
	// The amount of time that was spent with Poor download health.
	float PoorDownloadHealthTime;
	// The amount of time that was spent with Disconnected download health.
	float DisconnectedDownloadHealthTime;
	// Max memory usage for the memory store.
	uint64 MemoryStoreSizePeakBytes = 0; 
	// Memory limit for the memory backing store
	uint64 MemoryStoreSizeLimitBytes = 0;
	// The total number of bytes required to start the installation.
	uint64 ProcessRequiredDiskSpace;
	// The total number of bytes available at the time of checking ProcessRequiredDiskSpace.
	uint64 ProcessAvailableDiskSpace;
	// Stats tracked per cloud directory.
	TArray<FCloudDirectoryStats> CloudDirectoryStats;
	// Stats tracked per action. Array order matches actions order in configuration.
	TArray<FInstallActionStats> ActionsStats;
};

namespace BuildPatchServices
{
	struct FBuildInstallerConfiguration;
}

/**
 * Interface to a Build Installer, exposes installation control, progress, and state information.
 */
class IBuildInstaller
{
public:
	/**
	 * Virtual destructor.
	 */
	virtual ~IBuildInstaller() = default;
	
	/**
	 * Begin the installation process.
	 * @return true if the installation started successfully, or is already running.
	 */
	virtual bool StartInstallation() = 0;

	/**
	 * Get whether the install has complete
	 * @return	true if the thread completed
	 */
	virtual bool IsComplete() const = 0;

	/**
	 * Get whether the install was canceled. Only valid if complete.
	 * @return	true if installation was canceled
	 */
	virtual bool IsCanceled() const = 0;

	/**
	 * Get whether the install is currently paused.
	 * @return	true if installation is paused
	 */
	virtual bool IsPaused() const = 0;

	/**
	 * Get whether the install can be resumed.
	 * @return	true if installation is resumable
	 */
	virtual bool IsResumable() const = 0;

	/**
	 * Get whether the install is performing an update of an existing install.
	 * @return	true if installation is an update
	 */
	virtual bool IsUpdate() const = 0;

	/**
	 * Get whether the install is performing a repair of an existing install.
	 * @return	true if installation is a repair
	 */
	virtual bool IsRepair() const = 0;

	/**
	 * Get whether the install is performing a verify only of an existing install, without repair.
	 * @return	true if installation is a verify only.
	 */
	virtual bool IsVerifyOnly() const = 0;

	/**
	 * Get whether the install is actually uninstalling instead.
	 * @return	true if installation is actually an uninstallation
	 */
	virtual bool IsUninstall() const = 0;

	/**
	 * Get whether the process was successfully complete, meaning no errors, and was not cancelled. Only valid if complete.
	 * @return	true if installation process completed successfully.
	 */
	virtual bool CompletedSuccessfully() const = 0;

	/**
	 * Get whether the install failed. Only valid if complete.
	 * @return	true if installation was a failure
	 */
	virtual bool HasError() const = 0;

	/**
	 * Get the type of error for a failure that has occurred.
	 * @return	the enum representing the type of error
	 */
	virtual EBuildPatchInstallError GetErrorType() const = 0;

	UE_DEPRECATED(4.21, "GetPercentageText has been deprecated.  It will no longer be supported in the future.")
	virtual FText GetPercentageText() const = 0;

	UE_DEPRECATED(4.21, "GetDownloadSpeedText has been deprecated.  It will no longer be supported in the future.")
	virtual FText GetDownloadSpeedText() const = 0;

	/**
	 * Get the download speed for the current process
	 * @return	download speed progress
	 */
	virtual double GetDownloadSpeed() const = 0;

	UE_DEPRECATED(4.21, "GetInitialDownloadSize has been deprecated.  Please use GetTotalDownloadRequired instead.")
	virtual int64 GetInitialDownloadSize() const { return GetTotalDownloadRequired(); }

	/**
	 * Get the total download bytes required to complete
	 * @return	the total required bytes
	 */
	virtual int64 GetTotalDownloadRequired() const = 0;

	/**
	 * Get the total bytes downloaded
	 * @return	the total bytes downloaded
	 */
	virtual int64 GetTotalDownloaded() const = 0;

	/**
	 * Get the status of the install process.
	 * @return Status of the install process.
	 */
	virtual BuildPatchServices::EBuildPatchState GetState() const = 0;

	UE_DEPRECATED(4.21, "GetStatusText has been deprecated.  It will no longer be supported in the future.")
	virtual FText GetStatusText() const = 0;

	/**
	 * Get the update progress
	 * @return	A float describing progress: Between 0 and 1 for known progress, or less than 0 for unknown progress.
	 */
	virtual float GetUpdateProgress() const = 0;

	/**
	 * Get the build stats for the process. This should only be called after the install has completed
	 * @return	A struct containing information about the build
	 */
	virtual FBuildInstallStats GetBuildStatistics() const = 0;

	/**
	 * Get the current download health rating.
	 * @return	the enum representing the download health
	 */
	virtual EBuildPatchDownloadHealth GetDownloadHealth() const = 0;

	/**
	 * Get the display text for the error that occurred. Only valid to call after completion
	 * @return	display error text
	 */
	virtual FText GetErrorText() const = 0;

	/**
	 * Get the installation error code. This includes the failure type as well as specific code associated. The value is alphanumeric.
	 * This is only guaranteed to be set once the installation has completed.
	 * @returns the code as a string.
	 */
	virtual FString GetErrorCode() const = 0;

	/**
	 * Cancel the current install
	 */
	virtual void CancelInstall() = 0;

	/**
	 * Toggle the install paused state
	 * @return true if the installer is now paused
	 */
	virtual bool TogglePauseInstall() = 0;

	/**
	 * Registers a message handler with the installer.
	 * @param MessageHandler    Ptr to the message handler to add. Must not be null.
	 */
	virtual void RegisterMessageHandler(BuildPatchServices::FMessageHandler* MessageHandler) = 0;

	/**
	 * Unregisters a message handler, will no longer receive HandleMessage calls.
	 * @param MessageHandler    Ptr to the message handler to remove.
	 */
	virtual void UnregisterMessageHandler(BuildPatchServices::FMessageHandler* MessageHandler) = 0;

	/**
	 * Get the installation configuration object
	 * 
	 * @returns a const reference to the configuration
	 */
	virtual const BuildPatchServices::FBuildInstallerConfiguration& GetConfiguration() const = 0;

	/**
	 * Sets the download rate limit in Bytes per second. This throttles the download rate to the specified value.
	 * @param InDownloadRateLimitBps    throttle value in bytes per second. 
	 */
	virtual void SetDownloadBytesPerSecondLimit(uint64 InDownloadRateLimitBps) = 0;

	/**
	 * Gets the download rate limit in Bytes per second.
	 * @return the current download rate limit.
	 */
	virtual uint64 GetDownloadBytesPerSecondLimit() const = 0;
#if !UE_BUILD_SHIPPING
	/**
	 * Returns debug text for download screen to help troubleshoot download issues
	 */
	virtual void GetDebugText(TArray<FString>& Output) {}
#endif

	/**
	 * Get the completed files when installing to memory. This must not be called before installation
	 * completes, and is non-const so that clients can Move the memory as needed. Once installation is complete
	 * the installer does not touch this other than to delete it on destruction.
	 */
	virtual TMap<FString, TArray64<uint8>>& GetFilesInstalledToMemory() = 0; 

	/**
	* [Experimental]
	* Pass the installation folder on a per file basis. This path will be appended to the installation directory passed
	* in the FBuildInstallerConfiguration::InstallDirectory member. The old version is used for patch source data, the new
	* for everything else.
	*/
	virtual void SetPerFileSubdirectories(TMap<FString, FString>&& InOldPerFileSubdirectories, TMap<FString, FString>&& InNewPerFileSubdirectories) = 0;
};

static_assert((uint32)EBuildPatchInstallError::InvalidOrMax == 13, "Please add support for the extra values to the Lex functions below.");

inline const TCHAR* LexToString(EBuildPatchInstallError Error)
{
#define CASE_ENUM_TO_STR(Value) case EBuildPatchInstallError::Value: return TEXT(#Value)
	switch (Error)
	{
		CASE_ENUM_TO_STR(NoError);
		CASE_ENUM_TO_STR(DownloadError);
		CASE_ENUM_TO_STR(FileConstructionFail);
		CASE_ENUM_TO_STR(MoveFileToInstall);
		CASE_ENUM_TO_STR(BuildVerifyFail);
		CASE_ENUM_TO_STR(ApplicationClosing);
		CASE_ENUM_TO_STR(ApplicationError);
		CASE_ENUM_TO_STR(UserCanceled);
		CASE_ENUM_TO_STR(PrerequisiteError);
		CASE_ENUM_TO_STR(InitializationError);
		CASE_ENUM_TO_STR(PathLengthExceeded);
		CASE_ENUM_TO_STR(OutOfDiskSpace);
		CASE_ENUM_TO_STR(CustomUninstallActionError);
	default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(EBuildPatchInstallError& Error, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { Error = EBuildPatchInstallError::Value; return; }
	const TCHAR* const Prefix = TEXT("EBuildPatchInstallError::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(NoError);
	RETURN_IF_EQUAL(DownloadError);
	RETURN_IF_EQUAL(FileConstructionFail);
	RETURN_IF_EQUAL(MoveFileToInstall);
	RETURN_IF_EQUAL(BuildVerifyFail);
	RETURN_IF_EQUAL(ApplicationClosing);
	RETURN_IF_EQUAL(ApplicationError);
	RETURN_IF_EQUAL(UserCanceled);
	RETURN_IF_EQUAL(PrerequisiteError);
	RETURN_IF_EQUAL(InitializationError);
	RETURN_IF_EQUAL(PathLengthExceeded);
	RETURN_IF_EQUAL(OutOfDiskSpace);
	RETURN_IF_EQUAL(CustomUninstallActionError);
	// Did not match
	Error = EBuildPatchInstallError::InvalidOrMax;
	return;
#undef RETURN_IF_EQUAL
}

static_assert((uint32)EBuildPatchDownloadHealth::NUM_Values == 5, "Please add support for the extra values to the Lex functions below.");

inline const TCHAR* LexToString(EBuildPatchDownloadHealth Error)
{
#define CASE_ENUM_TO_STR(Value) case EBuildPatchDownloadHealth::Value: return TEXT(#Value)
	switch (Error)
	{
		CASE_ENUM_TO_STR(Disconnected);
		CASE_ENUM_TO_STR(Poor);
		CASE_ENUM_TO_STR(OK);
		CASE_ENUM_TO_STR(Good);
		CASE_ENUM_TO_STR(Excellent);
	default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(EBuildPatchDownloadHealth& Error, const TCHAR* Buffer)
{
#define RETURN_IF_EQUAL(Value) if (FCString::Stricmp(Buffer, TEXT(#Value)) == 0) { Error = EBuildPatchDownloadHealth::Value; return; }
	const TCHAR* const Prefix = TEXT("EBuildPatchDownloadHealth::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(Disconnected);
	RETURN_IF_EQUAL(Poor);
	RETURN_IF_EQUAL(OK);
	RETURN_IF_EQUAL(Good);
	RETURN_IF_EQUAL(Excellent);
	// Did not match
	Error = EBuildPatchDownloadHealth::NUM_Values;
	return;
#undef RETURN_IF_EQUAL
}
