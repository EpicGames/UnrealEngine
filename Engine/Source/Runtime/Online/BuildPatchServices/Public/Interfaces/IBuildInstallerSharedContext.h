// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

namespace BuildPatchServices
{
	/**
	 * Interface for a re-usable thread
	 */
	class IBuildInstallerThread
	{
	protected:
		/**
		 * Virtual destructor.
		 */
		virtual ~IBuildInstallerThread() = default;

	public:
		/**
		 * Adds a task to the thread's queue
		 */
		virtual void RunTask(TUniqueFunction<void()> Task) = 0;
	};
}

/**
 * An interface for sharing threads and components between multiple BPS installers
 */
class IBuildInstallerSharedContext
{
public:
	/**
	 * Virtual destructor.
	 */
	virtual ~IBuildInstallerSharedContext() = default;

	/**
	 * Return an existing free thread or allocate one.
	 * @return an IBuildInstallerThread
	 */
	virtual BuildPatchServices::IBuildInstallerThread* CreateThread() = 0;

	/**
	 * Relinquish the thread and add to the free list. It should not be used again after this.
	 */
	virtual void ReleaseThread(BuildPatchServices::IBuildInstallerThread* Thread) = 0;

	/**
	 * Preallocate thread free list with NumThreads.
	 * If more than NumThreads are required, a warning will be logged.
	 */
	virtual void PreallocateThreads(uint32 NumThreads) = 0;

	/**
	 * @return The number of threads required per installer
	 */
	virtual uint32 NumThreadsPerInstaller(bool bUseChunkDBs, bool bHasInstallSource) const = 0;
};

using IBuildInstallerSharedContextRef = TSharedRef<IBuildInstallerSharedContext>;
using IBuildInstallerSharedContextPtr = TSharedPtr<IBuildInstallerSharedContext>;
using IBuildInstallerSharedContextWeakPtr = TWeakPtr<IBuildInstallerSharedContext>;
