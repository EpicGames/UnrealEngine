// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IBuildManifest.h: Declares the IBuildManifest and IManifestField interfaces.
	This defines the functionality provided by a Build Manifest of a specific
	App, and also the interface to creating and reading custom fields.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "BuildPatchFeatureLevel.h"
#include "Misc/SecureHash.h"

class IBuildManifest;
class IManifestField;

typedef TSharedPtr< class IManifestField, ESPMode::ThreadSafe > IManifestFieldPtr;
typedef TSharedRef< class IManifestField, ESPMode::ThreadSafe > IManifestFieldRef;
typedef TSharedPtr< class IBuildManifest, ESPMode::ThreadSafe > IBuildManifestPtr;
typedef TSharedRef< class IBuildManifest, ESPMode::ThreadSafe > IBuildManifestRef;

/**
 * Declares flags for meta or attributes associated with a file.
 */
enum class EFileMetaFlags : uint8
{
	None = 0,
	// Flag for readonly file.
	ReadOnly = 1,
	// Flag for natively compressed.
	Compressed = 1 << 1,
	// Flag for unix executable.
	UnixExecutable = 1 << 2
};
ENUM_CLASS_FLAGS(EFileMetaFlags);

/**
 * Interface to a manifest field, which is used for accessing custom fields in the manifest
 */
class IManifestField
{
public:
	/**
	 * Virtual destructor.
	 */
	virtual ~IManifestField() = default;
	
	/**
	 * Get the fields value as an FString.
	 * @return	The string value.
	 */
	virtual FString AsString() const = 0;
	
	/**
	 * Get the fields value as a double.
	 * @return	The double value.
	 */
	virtual double AsDouble() const = 0;
	
	/**
	 * Get the fields value as an int.
	 * @return	The int value.
	 */
	virtual int64 AsInteger() const = 0;
};

/**
 * Interface to a Build Manifest.
 */
class IBuildManifest
{
public:
	/**
	 * Virtual destructor.
	 */
	virtual ~IBuildManifest() = default;

	/**
	 * Gets the manifest feature level
	 * @return		the feature level this manifest was built with
	 */
	virtual BuildPatchServices::EFeatureLevel GetFeatureLevel() const = 0;

	/**
	 * Get the App ID that this manifest belongs to
	 * @return		the app ID
	 */
	virtual uint32 GetAppID() const = 0;
	
	/**
	 * Get the name of the App that this manifest belongs to
	 * @return		the app name
	 */
	virtual const FString& GetAppName() const = 0;
	
	/**
	 * Get the string App version that this manifest describes
	 * @return		the version string
	 */
	virtual const FString& GetVersionString() const = 0;

	/**
	 * Get the unique string generated when producing the manifest
	 * @return		the unique build id string
	 */
	virtual const FString& GetUniqueBuildId() const = 0;

	/**
	 * If this manifest is or was encrypted, returns the GUID that identifies the secret key required for decryption.
	 * @param OUT	If the manifest in memory is currently encrypted.
	 * @return		the secret id
	 */
	virtual const FGuid& GetEncryptionSecretId(bool* bOutIsManifestEncrypted = nullptr) const = 0;

	/**
	 * Regardless of whether the manifest was ever encrypted, returns whether the manifest is currently encrypted.
	 * @return		If the manifest in memory is currently encrypted
	 */
	virtual bool IsManifestEncrypted() const = 0;

	/**
	 * Returns whether the manifest and all it chunks were originally encrypted when first loaded.
	 * This returns true even if the manifest has since been decrypted in memory.
	 * @return    true if the manifest was originally encrypted
	 */
	virtual bool IsOriginallyFullyEncrypted() const = 0;

	/**
	 * Get all necessary secret key IDs.
	 * @return		full set of necessary secret ids
	 */
	virtual TSet<FGuid> GetNecessaryEncryptionSecretIds() const = 0;
	
	/**
	 * Get the local install path to the exe that launches the App
	 * @return		local path to the launch exe
	 */
	virtual const FString& GetLaunchExe() const = 0;
	
	/**
	 * Get the command line arguments that the launch exe should be ran with
	 * @return		the launch command line
	 */
	virtual const FString& GetLaunchCommand() const = 0;

	/**
	 * Get the list of prereq ids that the prereq installer of this manifest satisfies
	 * @return		the set containing the prereq ids.
	 */
	virtual const TSet<FString>& GetPrereqIds() const = 0;

	/**
	 * Get the name of the prerequisites installer for the app
	 * @return		the prerequisites installer name
	 */
	virtual const FString& GetPrereqName() const = 0;
	
	/**
	 * Get the path to the prerequisites installer exe
	 * @return		local path to the prerequisites installer
	 */
	virtual const FString& GetPrereqPath() const = 0;
	
	/**
	 * Get the command line arguments that should be passed to the prerequisites installer
	 * @return		the prerequisites installer command line arguments
	 */
	virtual const FString& GetPrereqArgs() const = 0;

	/**
	 * Get the path to the custom uninstall action exe
	 * @return		local path to the custom uninstall action exe
	 */
	virtual const FString& GetUninstallActionPath() const = 0;

	/**
	 * Get the command line arguments that should be passed to the custom uninstall action exe
	 * @return		the custom uninstall action command line arguments
	 */
	virtual const FString& GetUninstallActionArgs() const = 0;

	/**
	 * Get the size of this download, assuming fresh install
	 * @return		the total download size in bytes
	 */
	virtual int64 GetDownloadSize() const = 0;

	/**
	 * Get the size of the download of this set of tags
	 * @param Tags	IN	A list of the tags we want to know the size of
	 * @return		the download size of the tags in bytes
	 */
	virtual int64 GetDownloadSize(const TSet<FString>& Tags) const = 0;

	/**
	 * Get the size of the download of this set of tags
	 * @param Tags              IN  The tags used for installation, will be applied to both manifests
	 * @param PreviousVersion   IN  The manifest for previous version to compare against
	 * @return the minimum download size required in bytes
	 */
	virtual int64 GetDeltaDownloadSize(const TSet<FString>& Tags, const IBuildManifestRef& PreviousVersion) const = 0;

	/**
	 * Get the size of the download of this set of tags
	 * @param Tags              IN  The tags used for installation
	 * @param PreviousVersion   IN  The manifest for previous version to compare against
	 * @param PreviousTags      IN  The tags used for previous installation
	 * @return the minimum download size required in bytes
	 */
	virtual int64 GetDeltaDownloadSize(const TSet<FString>& Tags, const IBuildManifestRef& PreviousVersion, const TSet<FString>& PreviousTags) const = 0;

	/**
	 * Get the size of this build
	 * @return		the build size in bytes
	 */
	virtual int64 GetBuildSize() const = 0;

	/**
	 * Get the size of the build of this set of tags
	 * @param Tags	IN	A list of the tags we want to know the size of
	 * @return		the build size of the tags in bytes
	 */
	virtual int64 GetBuildSize(const TSet<FString>& Tags) const = 0;

	/**
	 * Get the list of files in this build. Filenames are all relative to an install directory
	 * @return an array containing build files
	 */
	virtual TArray<FString> GetBuildFileList() const = 0;

	/**
	 * Get the list of files in this build. Filenames are all relative to an install directory.
	 * Filenames returned are a view over memory owned by the BuildManifest
	 * @return an array containing build files
	 */
	virtual TArray<FStringView> GetBuildFileListView() const = 0;

	/**
	 * Get the list of files in this build which match a tag from a given set. Filenames are all relative to an install directory
	 * @param		Tags	The set of tags to query
	 * @return an array containing build files
	 */
	virtual TArray<FString> GetBuildFileList(const TSet<FString>& Tags) const = 0;

	/**
	 * Get the list of files in this build which match a tag from a given set. Filenames are all relative to an install directory
	 * Filenames returned are a view over memory owned by the BuildManifest
	 * @return an array containing build files
	 */
	virtual TArray<FStringView> GetBuildFileListView(const TSet<FString>& Tags) const = 0;

	/**
	 * Returns the size of a particular file in the build
	 * @param Filename      The file.
	 * @return the file size.
	 */
	virtual int64 GetFileSize(FStringView Filename) const = 0;

	/**
	 * Returns the total size of all files in the array
	 * @param Filenames     The array of files.
	 * @return the total size of files in array.
	 */
	virtual int64 GetFileSize(const TArray<FString>& Filenames) const = 0;

	/**
	 * Returns the total size of all files in the set
	 * @param Filenames     The set of files.
	 * @return the total size of files in set.
	 */
	virtual int64 GetFileSize(const TSet<FString>& Filenames) const = 0;

	/**
	 * Get the file meta flags assigned to a particular file in the build.
	 * @param Filename      The file.
	 * @return		the file meta flags
	 */
	virtual EFileMetaFlags GetFileMetaFlags(const FString& Filename) const = 0;

	/**
	 * Gets the MD5 hash for a given file
	 * @param Filename      The file.
	 * @return ptr to the MD5 for the given file. Nullptr if not found.
	 */
	virtual const FMD5Hash* GetFileMD5Hash(const FString& Filename) const = 0;

	/**
	 * Gets the SHA1 hash for a given file
	 * @param Filename      The file.
	 * @return ptr to the SHA1 for the given file. Nullptr if not found.
	 */
	virtual const FSHAHash* GetFileSHA1Hash(const FString& Filename) const = 0;

	/**
	 * Gets the SHA256 hash for a given file
	 * @param Filename      The file.
	 * @return ptr to the SHA256 for the given file. Nullptr if not found.
	 */
	virtual const FSHA256Signature* GetFileSHA256Hash(const FString& Filename) const = 0;

	/**
	 * Gets the MIME type calculated for a given file
	 * @param Filename      The file.
	 * @return ptr to the MIME type string for the given file. Nullptr if not found.
	 */
	virtual const FString* GetFileMIMEType(const FString& Filename) const = 0;

	/**
	 * Get the list of install tags in this manifest
	 * @return the tags referenced.
	 */
	virtual TSet<FString> GetFileTagList() const = 0;

	/**
	 * Get the list of install tags in this manifest
	 * @param Tags			OUT		Receives the tags referenced.
	 */
	virtual void GetFileTagList(TSet<FString>& Tags) const = 0;

	/**
	 * Gets a list of files that were installed with the Old Manifest, but no longer required by this Manifest.
	 * @param OldManifest     IN    The Build Manifest that is currently installed.
	 * @param RemovableFiles  OUT   A list to receive the files that may be removed.
	 */
	virtual void GetOutdatedFiles(const IBuildManifestRef& OldManifest, TSet<FString>& OutdatedFiles) const = 0;

	/**
	 * Gets a list of files that were installed with the Old Manifest, but no longer required by this Manifest.
	 * @param OldManifest     IN    The Build Manifest that is currently installed.
	 * @param RemovableFiles  OUT   A list to receive the files that may be removed.
	 */
	virtual void GetRemovableFiles(const IBuildManifestRef& OldManifest, TArray<FString>& RemovableFiles) const = 0;

	/**
	 * Gets a list of files that are installed in InstallPath, but no longer required by this Manifest.
	 * @param InstallPath     IN    The path to the currently installed files.
	 * @param RemovableFiles  OUT   A list to receive the files that may be removed.
	 */
	virtual void GetRemovableFiles(const TCHAR* InstallPath, TArray<FString>& RemovableFiles) const = 0;

	/**
	 * Provides the set of chunks required to produce the given files.
	 * @param Filenames         IN      The set of files.
	 * @param RequiredChunks    OUT     The set of chunk GUIDs needed for those files.
	 */
	virtual void GetChunksRequiredForFiles(const TSet<FString>& Filenames, TSet<FGuid>& RequiredChunks) const = 0;

	/**
	 * Returns the size of a particular data file by it's GUID.
	 * @param DataGuid		The GUID for the data
	 * @return		File size, or 0 if this data is not in the manifest.
	 */
	virtual int64 GetDataSize(const FGuid& DataGuid) const = 0;

	/**
	 * Gets the SHA1 hash for a given chunk
	 * @param ChunkGuid		IN		The guid of the chunk to get hash for
	 * @param OutHash		OUT		Receives the hash value if found
	 * @return	true if we had the hash for this chunk
	 */
	virtual bool GetChunkShaHash(const FGuid& ChunkGuid, FSHAHash& OutHash) const = 0;

	/**
	 * Gets the secret ID of the chunk
	 * @param ChunkGuid					IN		The guid of the chunk to get secret ID for
	 * @param OutEncryptionSecretId		OUT		The guid value of chunk's secret ID if found
	 * @return true the chunk has encryption secret
	 */
	virtual bool GetChunkEncryptionSecretId(const FGuid& ChunkGuid, FGuid& OutEncryptionSecretId) const = 0;

	/**
	 * Checks the manifest format version to see if this manifest was loaded from latest data
	 * @return True if the manifest was created from the latest format
	 */
	virtual bool NeedsResaving() const = 0;

	/**
	 * Copy the custom fields from another manifest into this one. If this manifest has custom fields, matching keys will be overwritten but extras will remain
	 * @param Other		The manifest to copy from
	 * @param bClobber	Whether to overwrite any already existing fields
	 */
	virtual void CopyCustomFields(const IBuildManifestRef& Other, bool bClobber) = 0;

	/**
	 * Get the list of custom field names in this manifest
	 * @return an array containing custom field names
	 */
	virtual TArray<FString> GetCustomFieldNames() const = 0;

	/**
	 * Get a custom field from the manifest
	 * @param	FieldName	The name of the custom field
	 * @return An interface to the field
	 */
	virtual const IManifestFieldPtr GetCustomField(const FString& FieldName) const = 0;

	/**
	 * Various functions for setting a custom field in the manifest
	 * @param	FieldName	The name of the custom field
	 * @param	Value		The value for the field
	 * @return An interface to the field that was created
	 */
	virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const FString& Value) = 0;
	virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const double& Value) = 0;
	virtual const IManifestFieldPtr SetCustomField(const FString& FieldName, const int64& Value) = 0;

	/**
	 * Remove a custom field from the manifest
	 * @param	FieldName	The name of the custom field
	 */
	virtual void RemoveCustomField(const FString& FieldName) = 0;
	
	/**
	 * Duplicated this manifest to create a copy. Should be used if storing a received manifest as an installed
	 * manifest which would then be unique
	 * @return A shared ref to the new manifest
	 */
	virtual IBuildManifestRef Duplicate() const = 0;
};

