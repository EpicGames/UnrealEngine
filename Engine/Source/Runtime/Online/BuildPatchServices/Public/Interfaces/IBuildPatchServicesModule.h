// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Interfaces/IBuildInstaller.h"
#include "Interfaces/IBuildStatistics.h"
#include "Interfaces/IChunkDeltaOptimiser.h"
#include "Interfaces/IDirectoryChunker.h"
#include "Interfaces/IPatchDataEnumeration.h"
#include "Interfaces/IChunkHarvester.h"
#include "BuildPatchSettings.h"

class IAnalyticsProvider;

/**
 * Delegates that will be accepted and fired off by the implementation.
 */
DECLARE_DELEGATE_OneParam(FBuildPatchInstallerDelegate, const IBuildInstallerRef&);
DECLARE_DELEGATE_TwoParams(FBuildPatchBoolManifestDelegate, bool, IBuildManifestRef);
DECLARE_MULTICAST_DELEGATE_TwoParams(FSingleChunkWrittenDelegate, FString /* Filename */, FString /* MD5Hash */);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FManifestCreatedDelegate, FString /* Filename */, FString /* SHA1Hash */, FString /* MD5Hash */);

namespace ECompactifyMode
{
	enum Type
	{
		Preview,
		Full
	};
}

/**
 * Interface for the services manager.
 */
class IBuildPatchServicesModule
	: public IModuleInterface
{
public:
	/**
	 * Virtual destructor.
	 */
	virtual ~IBuildPatchServicesModule() = default;

	/**
	 * Factory providing construction of a build installer class.
	 * @param Configuration     The configuration for the installer.
	 * @param OnComplete        The delegate that will be called when the installer completes.
	 * @return an instance of an IBuildInstaller implementation.
	 */
	virtual IBuildInstallerRef CreateBuildInstaller(BuildPatchServices::FBuildInstallerConfiguration Configuration, FBuildPatchInstallerDelegate OnComplete) const = 0;

	/**
	 * Factory providing construction of a build installer shared context class.
	 * @param DebugName          Used to tag resources allocated with the shared context.
	 * @return an instance of an IBuildInstallerSharedContex implementation.
	 */
	virtual IBuildInstallerSharedContextRef CreateBuildInstallerSharedContext(const TCHAR* DebugName) const = 0;

	/**
	 * Factory providing construction of a build statistics class.
	 * @param Installer     The installer to create a build statistics for.
	 * @return an instance of an IBuildStatistics implementation.
	 */
	virtual BuildPatchServices::IBuildStatisticsRef CreateBuildStatistics(const IBuildInstallerRef& Installer) const = 0;

	/**
	 * Factory providing construction of a directory chunker class.
	 * @param Configuration         Specifies the settings for the operation.  See BuildPatchServices::FDirectoryChunkerConfiguration comments.
	 * @return an instance of an IDirectoryChunker implementation.
	 */
	virtual BuildPatchServices::IDirectoryChunkerRef CreateDirectoryChunker(BuildPatchServices::FDirectoryChunkerConfiguration Configuration) const = 0;

	/**
	 * Factory providing construction of a patch data enumeration class.
	 * @param Configuration         Specifies the settings for the operation.  See BuildPatchServices::FPatchDataEnumerationConfiguration comments.
	 * @return an instance of an IPatchDataEnumeration implementation.
	 */
	virtual BuildPatchServices::IPatchDataEnumerationRef CreatePatchDataEnumeration(BuildPatchServices::FPatchDataEnumerationConfiguration Configuration) const = 0;

	/**
	 * Factory providing construction of a chunk delta optimiser class.
	 * @param Configuration         Specifies the settings for the operation.  See BuildPatchServices::FChunkDeltaOptimiserConfiguration comments.
	 * @return an instance of an IChunkDeltaOptimiser implementation.
	 */
	virtual BuildPatchServices::IChunkDeltaOptimiserRef CreateChunkDeltaOptimiser(BuildPatchServices::FChunkDeltaOptimiserConfiguration Configuration) const = 0;

	/**
	 * Factory providing construction of a chunk harvester class.
	 * @param Configuration         Specifies the settings for the operation.  See BuildPatchServices::FChunkHarvesterConfiguration comments.
	 * @return an instance of an IChunkHarvester implementation.
	 */
	virtual BuildPatchServices::IChunkHarvesterRef CreateChunkHarvester(BuildPatchServices::FChunkHarvesterConfiguration Configuration) const = 0;

	/**
	 * Loads a Build Manifest from file and returns the interface
	 * @param Filename		The file to load from
	 * @return		a shared pointer to the manifest, which will be invalid if load failed.
	 */
	virtual IBuildManifestPtr LoadManifestFromFile( const FString& Filename ) = 0;

	/**
	 * Constructs a Build Manifest from a data
	 * @param ManifestData		The data received from a web api
	 * @return		a shared pointer to the manifest, which will be invalid if creation failed.
	 */
	virtual IBuildManifestPtr MakeManifestFromData( const TArray<uint8>& ManifestData ) = 0;

	/**
	 * Saves a Build Manifest to file
	 * @param Filename		The file to save to
	 * @param Manifest		The manifest to save out
	 * @return		If the save was successful.
	 */
	virtual bool SaveManifestToFile(const FString& Filename, IBuildManifestRef Manifest) = 0;

	/**
	 * Gets an array of prerequisite identifiers that are registered as installed on this system.
	 * @return a set containing all installed prerequisite identifiers.
	 */
	virtual TSet<FString> GetInstalledPrereqIds() const = 0;

	/**
	 * Gets a list of currently active installers
	 * @return all installers that are currently active.
	 */
	virtual const TArray<IBuildInstallerRef>& GetInstallers() const = 0;

	/**
	 * Sets the directory used for staging intermediate files.
	 * @param StagingDir	The staging directory
	 */
	virtual void SetStagingDirectory( const FString& StagingDir ) = 0;

	/**
	 * Sets the cloud directory where chunks and manifests will be pulled from and saved to.
	 * @param CloudDir		The cloud directory
	 */
	virtual void SetCloudDirectory( FString CloudDir ) = 0;

	/**
	 * Sets the cloud directory list where chunks and manifests will be pulled from and saved to.
	 * When downloading, if we get a failure, we move on to the next cloud option for that request.
	 * @param CloudDirs		The cloud directory list
	 */
	virtual void SetCloudDirectories( TArray<FString> CloudDirs ) = 0;

	/**
	 * Sets the backup directory where files that are being clobbered by repair/patch will be placed.
	 * @param BackupDir		The backup directory
	 */
	virtual void SetBackupDirectory( const FString& BackupDir ) = 0;

	/**
	 * Sets the Analytics provider that will be used to register errors with patch/build installs
	 * @param AnalyticsProvider		Shared ptr to an analytics interface to use. If NULL analytics will be disabled.
	 */
	virtual void SetAnalyticsProvider( TSharedPtr< IAnalyticsProvider > AnalyticsProvider ) = 0;

	/**
	 * Registers an installation on this machine. This information is used to gather a list of install locations that can be used as chunk sources.
	 * @param AppManifest			Ref to the manifest for this installation
	 * @param AppInstallDirectory	The install location
	 */
	virtual void RegisterAppInstallation(IBuildManifestRef AppManifest, const FString AppInstallDirectory) = 0;

	/**
	 * Unregisters an installation on this machine
	 * @param AppInstallDirectory	The install location which RegisterAppInstallation was executed with
	 * @return returns true if successfully removed installation directory
	 */
	virtual bool UnregisterAppInstallation(const FString AppInstallDirectory) = 0;

	/**
	 * Registers an encryption secret. This information is used by installers to decrypt patch data that references this secret.
	 * @param EncryptionSecretId    The ID for the secret key.
	 * @param EncryptionSecretKey   The secret key.
	 */
	virtual void RegisterEncryptionSecret(const FGuid& EncryptionSecretId, TArray<uint8> EncryptionSecretKey) = 0;

	/**
	 * Unregisters an encryption secret.
	 * @param EncryptionSecretId    The ID for the secret key.
	 * @return returns true if the secret existed and was removed.
	 */
	virtual bool UnregisterEncryptionSecret(const FGuid& EncryptionSecretId) = 0;

	/**
	 * Generates random data via OpenSSL.
	 * @param OutData    The object to fill with random data.
	 * @return returns true if OpenSSL was able to generate random data.
	*/
	virtual bool CreateRandomBytes(const TArrayView<uint8> OutData) = 0;

	/**
	 * Call to force the exit out of all current installers, optionally blocks until threads have exited and complete delegates are called.
	 * @param WaitForThreads		If true, will block on threads exit and completion delegates
	 */
	virtual void CancelAllInstallers(bool WaitForThreads) = 0;

	/**
	 * Processes a Cloud Directory to identify and delete any orphaned chunks or files.
	 * NOTE: THIS function is blocking and will not return until finished.
	 * @param Configuration         Specifies the settings for the operation. See BuildPatchServices::FCompactifyConfiguration comments.
	 * @return true if successful.
	 */
	virtual bool CompactifyCloudDirectory(const BuildPatchServices::FCompactifyConfiguration& Configuration) = 0;

	/**
	 * Searches a given directory for chunk and chunkdb files, and verifies their integrity uses the hashes in the files.
	 * NOTE: THIS function is blocking and will not return until finished. Don't run on main thread.
	 * @param SearchPath            A full file path for the directory to search.
	 * @param OutputFile            A full file path where to save the output text.
	 * @return true if successful and no corruptions detected.
	 */
	virtual bool VerifyChunkData(const FString& SearchPath, const FString& OutputFile) = 0;

	/**
	 * Packages data referenced by a manifest file into chunkdb files, supporting a maximum filesize per chunkdb.
	 * NOTE: THIS function is blocking and will not return until finished. Don't run on main thread.
	 * @param Configuration         Specifies the settings for the operation. See BuildPatchServices::FPackageChunksConfiguration comments.
	 * @return true if successful.
	 */
	virtual bool PackageChunkData(const BuildPatchServices::FPackageChunksConfiguration& Configuration) = 0;

	/**
	 * Takes two manifests as input, in order to merge together producing a new manifest containing all files.
	 * NOTE: THIS function is blocking and will not return until finished. Don't run on main thread.
	 * @param ManifestFilePathA         A full file path for the base manifest to be loaded.
	 * @param ManifestFilePathB         A full file path for the merge manifest to be loaded, by default files in B will stomp over A.
	 * @param ManifestFilePathC         A full file path for the manifest to be output.
	 * @param NewVersionString          The new version string for the build, all other meta will be copied from B.
	 * @param SelectionDetailFilePath   Optional full file path to a text file listing each build relative file required, followed by A or B to select which manifest to pull from.
	 *                                  The format should be \r\n separated lines of filename \t A|B. Example:
	 *                                  File/in/build1	A
	 *                                  File/in/build2	B
	 * @return true if successful.
	 */
	virtual bool MergeManifests(const FString& ManifestFilePathA, const FString& ManifestFilePathB, const FString& ManifestFilePathC, const FString& NewVersionString, const FString& SelectionDetailFilePath) = 0;

	/**
	 * Takes two manifests as input, in order to merge together producing a new manifest containing all files.
	 * NOTE: THIS function is blocking and will not return until finished. Don't run on main thread.
	 * @param ManifestFilePathA         A full file path for the base manifest to be loaded.
	 * @param ManifestFilePathB         A full file path for the merge manifest to be loaded, by default files in B will stomp over A.
	 * @param NewVersionString          The new version string for the build, all other meta will be copied from B.
	 * @param FilesFromA                A set containing the files that should be kept from ManifestA.
	 * @param FilesFromB                A set containing the files that should be kept from ManifestB.
	 * @return resulting manifest if successful.
	 */
	virtual IBuildManifestPtr MergeManifests(const IBuildManifestRef& ManifestA, const IBuildManifestRef& ManifestB, const FString& NewVersionString, const TSet<FString>& FilesFromA, const TSet<FString>& FilesFromB) = 0;

	/**
	 * Takes two manifests as input and outputs the details of the patch.
	 * NOTE: THIS function is blocking and will not return until finished. Don't run on main thread.
	 * @param Configuration         Specifies the settings for the operation. See BuildPatchServices::FDiffManifestsConfiguration comments.
	 * @return true if successful.
	 */
	virtual bool DiffManifests(const BuildPatchServices::FDiffManifestsConfiguration& Configuration) = 0;

	/**
	 * Returns an event which fires when we start a new build install.
	 */
	DECLARE_EVENT(IBuildPatchServicesModule, FSimpleEvent)
	virtual FSimpleEvent& OnStartBuildInstall() = 0;


	///// DEPRECATED FUNCTIONS

	UE_DEPRECATED(4.21, "MakeManifestFromJSON(const FString& ManifestJSON) has been deprecated.  Please use MakeManifestFromData(const TArray<uint8>& ManifestData) instead.")
	virtual IBuildManifestPtr MakeManifestFromJSON(const FString& ManifestJSON) = 0;

	UE_DEPRECATED(4.16, "EnumerateManifestData(const FString& ManifestFilePath, const FString& OutputFile, bool bIncludeSizes) has been deprecated.  Please use CreatePatchDataEnumeration(BuildPatchServices::FPatchDataEnumerationConfiguration Configuration) instead.")
	virtual bool EnumerateManifestData(const FString& ManifestFilePath, const FString& OutputFile, bool bIncludeSizes)
	{
		BuildPatchServices::FPatchDataEnumerationConfiguration Configuration;
		Configuration.InputFile = ManifestFilePath;
		Configuration.OutputFile = OutputFile;
		Configuration.bIncludeSizes = bIncludeSizes;
		BuildPatchServices::IPatchDataEnumerationRef PatchDataEnumeration = CreatePatchDataEnumeration(MoveTemp(Configuration));
		return PatchDataEnumeration->Run();
	}

	UE_DEPRECATED(4.23, "ChunkBuildDirectory(const BuildPatchServices::FChunkBuildConfiguration& Configuration) has been deprecated.  Please use CreateDirectoryChunker(BuildPatchServices::FDirectoryChunkerConfiguration Configuration) instead.")
	virtual bool ChunkBuildDirectory(const BuildPatchServices::FChunkBuildConfiguration& Configuration)
	{
		BuildPatchServices::IDirectoryChunkerRef DirectoryChunker = CreateDirectoryChunker(Configuration);
		return DirectoryChunker->Run();
	}

	UE_DEPRECATED(4.23, "EnumeratePatchData(const BuildPatchServices::FPatchDataEnumerationConfiguration& Configuration) has been deprecated.  Please use CreatePatchDataEnumeration(BuildPatchServices::FPatchDataEnumerationConfiguration Configuration) instead.")
	virtual bool EnumeratePatchData(const BuildPatchServices::FPatchDataEnumerationConfiguration& Configuration)
	{
		BuildPatchServices::IPatchDataEnumerationRef PatchDataEnumeration = CreatePatchDataEnumeration(Configuration);
		return PatchDataEnumeration->Run();
	}

	UE_DEPRECATED(4.23, "OptimiseChunkDelta(const BuildPatchServices::FChunkDeltaOptimiserConfiguration& Configuration) has been deprecated.  Please use CreateChunkDeltaOptimiser(BuildPatchServices::FChunkDeltaOptimiserConfiguration Configuration) instead.")
	virtual bool OptimiseChunkDelta(const BuildPatchServices::FChunkDeltaOptimiserConfiguration& Configuration)
	{
		BuildPatchServices::IChunkDeltaOptimiserRef ChunkDeltaOptimiser = CreateChunkDeltaOptimiser(Configuration);
		return ChunkDeltaOptimiser->Run();
	}

	UE_DEPRECATED(4.21, "GenerateChunksManifestFromDirectory(const BuildPatchServices::FGenerationConfiguration& Configuration) has been deprecated.  Please use CreateDirectoryChunker(BuildPatchServices::FDirectoryChunkerConfiguration Configuration) instead.")
	virtual bool GenerateChunksManifestFromDirectory(const BuildPatchServices::FGenerationConfiguration& Configuration)
	{
		BuildPatchServices::IDirectoryChunkerRef DirectoryChunker = CreateDirectoryChunker(Configuration);
		return DirectoryChunker->Run();
	}

	UE_DEPRECATED(4.23, "StartBuildInstall(IBuildManifestPtr, IBuildManifestPtr, const FString&, FBuildPatchBoolManifestDelegate, bool, TSet<FString>) has been deprecated.  Please use CreateBuildInstaller(BuildPatchServices::FBuildInstallerConfiguration, FBuildPatchInstallerDelegate) instead, followed by IBuildInstaller::StartInstallation.")
	IBuildInstallerPtr StartBuildInstall(IBuildManifestPtr CurrentManifest, IBuildManifestPtr InstallManifest, const FString& InstallDirectory, FBuildPatchBoolManifestDelegate OnCompleteDelegate, bool bIsRepair = false, TSet<FString> InstallTags = TSet<FString>())
	{
		BuildPatchServices::FBuildInstallerConfiguration InstallerConfiguration({ BuildPatchServices::FInstallerAction::MakeInstallOrUpdate(CurrentManifest, InstallManifest.ToSharedRef(), InstallTags) });
		FBuildPatchInstallerDelegate CompleteDelegate = FBuildPatchInstallerDelegate::CreateLambda([OnCompleteDelegate, InstallManifest](const IBuildInstallerRef& Installer)
		{
			OnCompleteDelegate.ExecuteIfBound(Installer->CompletedSuccessfully(), InstallManifest.ToSharedRef());
		});
		IBuildInstallerRef Installer = CreateBuildInstaller(InstallerConfiguration, CompleteDelegate);
		Installer->StartInstallation();
		return Installer;
	}

	UE_DEPRECATED(4.23, "StartBuildInstallStageOnly(IBuildManifestPtr, IBuildManifestPtr, const FString&, FBuildPatchBoolManifestDelegate, bool, TSet<FString>) has been deprecated.  Please use CreateBuildInstaller(BuildPatchServices::FBuildInstallerConfiguration, FBuildPatchInstallerDelegate) instead, followed by IBuildInstaller::StartInstallation.")
	IBuildInstallerPtr StartBuildInstallStageOnly(IBuildManifestPtr CurrentManifest, IBuildManifestPtr InstallManifest, const FString& InstallDirectory, FBuildPatchBoolManifestDelegate OnCompleteDelegate, bool bIsRepair = false, TSet<FString> InstallTags = TSet<FString>())
	{
		BuildPatchServices::FBuildInstallerConfiguration InstallerConfiguration({ BuildPatchServices::FInstallerAction::MakeInstallOrUpdate(CurrentManifest, InstallManifest.ToSharedRef(), InstallTags) });
		InstallerConfiguration.InstallMode = BuildPatchServices::EInstallMode::StageFiles;
		FBuildPatchInstallerDelegate CompleteDelegate = FBuildPatchInstallerDelegate::CreateLambda([OnCompleteDelegate, InstallManifest](const IBuildInstallerRef& Installer)
		{
			OnCompleteDelegate.ExecuteIfBound(Installer->CompletedSuccessfully(), InstallManifest.ToSharedRef());
		});
		IBuildInstallerRef Installer = CreateBuildInstaller(InstallerConfiguration, CompleteDelegate);
		Installer->StartInstallation();
		return Installer;
	}

	UE_DEPRECATED(4.23, "StartBuildInstall(BuildPatchServices::FInstallerConfiguration, FBuildPatchBoolManifestDelegate) has been deprecated.  Please use CreateBuildInstaller(BuildPatchServices::FBuildInstallerConfiguration, FBuildPatchInstallerDelegate) instead, followed by IBuildInstaller::StartInstallation.")
	IBuildInstallerRef StartBuildInstall(BuildPatchServices::FInstallerConfiguration Configuration, FBuildPatchBoolManifestDelegate OnCompleteDelegate)
	{
		FBuildPatchInstallerDelegate CompleteDelegate = FBuildPatchInstallerDelegate::CreateLambda([Configuration, OnCompleteDelegate](const IBuildInstallerRef& Installer)
		{
			OnCompleteDelegate.ExecuteIfBound(Installer->CompletedSuccessfully(), Configuration.InstallManifest);
		});
		TArray<BuildPatchServices::FInstallerAction> Actions;
		if (Configuration.bIsRepair)
		{
			Actions.Add(BuildPatchServices::FInstallerAction::MakeRepair(Configuration.InstallManifest, Configuration.InstallTags));
		}
		else
		{
			Actions.Add(BuildPatchServices::FInstallerAction::MakeInstallOrUpdate(Configuration.CurrentManifest, Configuration.InstallManifest, Configuration.InstallTags));
		}
		BuildPatchServices::FBuildInstallerConfiguration BuildInstallerConfiguration(MoveTemp(Actions));
		BuildInstallerConfiguration.InstallDirectory = Configuration.InstallDirectory;
		BuildInstallerConfiguration.StagingDirectory = Configuration.StagingDirectory;
		BuildInstallerConfiguration.BackupDirectory = Configuration.BackupDirectory;
		BuildInstallerConfiguration.ChunkDatabaseFiles = Configuration.ChunkDatabaseFiles;
		BuildInstallerConfiguration.CloudDirectories = Configuration.CloudDirectories;
		BuildInstallerConfiguration.InstallMode =  Configuration.InstallMode;
		BuildInstallerConfiguration.VerifyMode = Configuration.VerifyMode;
		BuildInstallerConfiguration.DeltaPolicy = Configuration.DeltaPolicy;
		BuildInstallerConfiguration.bRunRequiredPrereqs = Configuration.bRunRequiredPrereqs;
		BuildInstallerConfiguration.bAllowConcurrentExecution = Configuration.bAllowConcurrentExecution;
		IBuildInstallerRef Installer = CreateBuildInstaller(BuildInstallerConfiguration, CompleteDelegate);
		Installer->StartInstallation();
		return Installer;
	}
};
