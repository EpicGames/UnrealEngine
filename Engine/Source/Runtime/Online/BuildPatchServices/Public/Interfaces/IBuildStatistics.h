// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

namespace BuildPatchServices
{
	struct FBuildInstallerConfiguration;
	enum class EVerifyError : uint32;

	/**
	 * An enum describing the current state of the data for a file operation that has or will be processed.
	 */
	enum class EFileOperationState : int32
	{
		// Not yet evaluated.
		Unknown = 0,

		// The data for this operation has not yet been requested (from a local chunkdb source).
		PendingLocalChunkDbData,

		// The data is being acquired (from a local chunkdb source).
		RetrievingLocalChunkDbData,

		// The data for this operation has not yet been requested (from a local install source).
		PendingLocalInstallData,

		// The data is being acquired (from a local install source).
		RetrievingLocalInstallData,

		// The data for this operation has not yet been requested (from a remote cloud source).
		PendingRemoteCloudData,

		// The data is being acquired (from a remote cloud source).
		RetrievingRemoteCloudData,

		// The data is in a local store to be loaded later.
		PendingLocalDataStore,

		// The data is being acquired (from a local store).
		RetrievingLocalDataStore,

		// The data is in a memory chunk store ready for use.
		DataInMemoryStore,

		// The file operation has been written to the staging location.
		Staged,

		// The file operation has been written to the destination install location.
		Installed,

		// The file operation is being verified.
		Verifying,

		// The file operation has been verified and is corrupt.
		VerifiedFail,

		// The file operation has been verified and successful.
		VerifiedSuccess,

		// Helpers.
		NUM_States,
		Complete = NUM_States - 1
	};

	/**
	 * A struct representing a file operation to be completed.
	 */
	struct FFileOperation
	{
	public:
		FFileOperation(FString Filename, const FGuid& DataId, uint64 Offest, uint64 Size, EFileOperationState CurrentState);
		~FFileOperation();

	public:
		// The build filename where this data section goes.
		FString Filename;
		// The idea of the chunk where the data is taken from.
		FGuid DataId;
		// The offset into the file.
		uint64 Offest;
		// The size of the data to write.
		uint64 Size;
		// The current state of this operation.
		EFileOperationState CurrentState;
	};

	/**
	 * A struct representing a download.
	 */
	struct FDownload
	{
		// The uri for the download.
		FString Data;
		// The size of the download.
		int64 Size;
		// The amount of data received so far.
		int64 Received;
	};

	/**
	 * An interface for accessing runtime statistical information about an installer.
	 */
	class IBuildStatistics
	{
	public:
		/**
		 * Virtual destructor.
		 */
		virtual ~IBuildStatistics() = default;

		/**
		 * @return the configuration used when constructing the installer.
		 */
		virtual const FBuildInstallerConfiguration& GetConfiguration() const = 0;

		/**
		 * @return the total download size for the installation.
		 */
		virtual int64 GetDownloadSize() const = 0;

		/**
		 * @return the total install size for the installation.
		 */
		virtual int64 GetBuildSize() const = 0;

		/**
		 * @return the size in chunks of the memory store.
		 */
		virtual int32 GetMemoryChunkStoreSize() const { return 0; }

		/**
		 * @return the number of chunks currently in the memory store.
		 */
		virtual int32 GetMemoryChunksInStore() const { return 0; }

		/**
		 * @return the number of chunks that have been booted from the memory store.
		 */
		virtual int32 GetMemoryChunksBooted() const { return 0; }

		/**
		 * @return the number of chunks currently in the memory store which are held due to multiple referencing.
		 */
		virtual int32 GetMemoryChunksRetained() const { return 0; }

		/**
		 * @return the filename of the file currently being worked on.
		 */
		virtual FString GetCurrentWorkingFileName() const = 0;

		/**
		 * @return the progress of the file currently being worked on.
		 */
		virtual float GetCurrentWorkingFileProgress() const = 0;

		/**
		 * @return the chunk ID currently being used to write the current file.
		 */
		virtual FGuid GetCurrentWorkingData() const = 0;

		/**
		 * @return an array of currently active downloads.
		 */
		virtual TArray<FDownload> GetCurrentDownloads() const = 0;

		/**
		 * @return true if there are downloads currently active.
		 */
		virtual bool IsDownloadActive() const = 0;

		/**
		 * @return true if the current operation is creating or opening files.
		 */
		virtual bool IsHardDiskActiveAdministering() const = 0;

		/**
		 * @return true if currently writing data to disk.
		 */
		virtual bool IsHardDiskActiveWrite() const = 0;

		/**
		 * @return true if currently reading data from disk.
		 */
		virtual bool IsHardDiskActiveRead() const = 0;

		/**
		 * @return an array containing the states for all file write operations being performed by this installation.
		 */
		virtual const TArray<FFileOperation>& GetFileOperationStates() const = 0;

		/**
		 * @return the current download speed in bytes per second.
		 */
		virtual double GetDownloadByteSpeed() const = 0;

		/**
		 * @return the current disk read speed in bytes per second.
		 */
		virtual double GetDiskReadByteSpeed() const = 0;

		/**
		 * @return the current chunkdb read speed in bytes per second.
		 */
		UE_DEPRECATED(5.6, "Chunk db read speeds now included in DiskReadByteSpeed")
		double GetChunkDbReadByteSpeed() const { return 0; }

		/**
		 * @return the current disk write speed in bytes per second.
		 */
		virtual double GetDiskWriteByteSpeed() const = 0;

		/**
		 * @return the number of bytes downloaded so far.
		 */
		virtual int64 GetDownloadedBytes() const = 0;

		/**
		 * @return the number of bytes read so far.
		 */
		virtual int64 GetDiskReadBytes() const = 0;

		/**
		 * @return the number of bytes written so far.
		 */
		virtual int64 GetDiskWrittenBytes() const = 0;

		/**
		 * @return the total number of verify errors experienced during this installation.
		 */
		virtual int32 GetNumVerifyErrors() const = 0;
		
		/**
		 * @return the verify error counts per error type experienced during this installation.
		 */
		virtual TMap<EVerifyError, int32> GetVerifyErrorCounts() const = 0;
	};

	typedef TSharedPtr<IBuildStatistics> IBuildStatisticsPtr;
	typedef TSharedRef<IBuildStatistics> IBuildStatisticsRef;
}

inline const TCHAR* LexToString(BuildPatchServices::EFileOperationState Value)
{
	using namespace BuildPatchServices;
	static_assert(int32(EFileOperationState::NUM_States) == int32(EFileOperationState::VerifiedSuccess) + 1, "Please add support for the extra values below.");
#define CASE_ENUM_TO_STR(_X_) case EFileOperationState::_X_: return TEXT(#_X_)
	switch (Value)
	{
		CASE_ENUM_TO_STR(Unknown);
		CASE_ENUM_TO_STR(PendingLocalChunkDbData);
		CASE_ENUM_TO_STR(RetrievingLocalChunkDbData);
		CASE_ENUM_TO_STR(PendingLocalInstallData);
		CASE_ENUM_TO_STR(RetrievingLocalInstallData);
		CASE_ENUM_TO_STR(PendingRemoteCloudData);
		CASE_ENUM_TO_STR(RetrievingRemoteCloudData);
		CASE_ENUM_TO_STR(PendingLocalDataStore);
		CASE_ENUM_TO_STR(RetrievingLocalDataStore);
		CASE_ENUM_TO_STR(DataInMemoryStore);
		CASE_ENUM_TO_STR(Staged);
		CASE_ENUM_TO_STR(Installed);
		CASE_ENUM_TO_STR(Verifying);
		CASE_ENUM_TO_STR(VerifiedFail);
		CASE_ENUM_TO_STR(VerifiedSuccess);
		CASE_ENUM_TO_STR(NUM_States);
		default: return TEXT("InvalidOrMax");
	}
#undef CASE_ENUM_TO_STR
}

inline void LexFromString(BuildPatchServices::EFileOperationState& Value, const TCHAR* Buffer)
{
	using namespace BuildPatchServices;
	static_assert(int32(EFileOperationState::NUM_States) == int32(EFileOperationState::VerifiedSuccess) + 1, "Please add support for the extra values below.");
#define RETURN_IF_EQUAL(_X_) if (FCString::Stricmp(Buffer, TEXT(#_X_)) == 0) { Value = EFileOperationState::_X_; return; }
	const TCHAR* const Prefix = TEXT("EFileOperationState::");
	const SIZE_T PrefixLen = FCString::Strlen(Prefix);
	if (FCString::Strnicmp(Buffer, Prefix, PrefixLen) == 0)
	{
		Buffer += PrefixLen;
	}
	RETURN_IF_EQUAL(Unknown);
	RETURN_IF_EQUAL(PendingLocalChunkDbData);
	RETURN_IF_EQUAL(RetrievingLocalChunkDbData);
	RETURN_IF_EQUAL(PendingLocalInstallData);
	RETURN_IF_EQUAL(RetrievingLocalInstallData);
	RETURN_IF_EQUAL(PendingRemoteCloudData);
	RETURN_IF_EQUAL(RetrievingRemoteCloudData);
	RETURN_IF_EQUAL(PendingLocalDataStore);
	RETURN_IF_EQUAL(RetrievingLocalDataStore);
	RETURN_IF_EQUAL(DataInMemoryStore);
	RETURN_IF_EQUAL(Staged);
	RETURN_IF_EQUAL(Installed);
	RETURN_IF_EQUAL(Verifying);
	RETURN_IF_EQUAL(VerifiedFail);
	RETURN_IF_EQUAL(VerifiedSuccess);
	RETURN_IF_EQUAL(NUM_States);
	// Did not match
	Value = EFileOperationState::Unknown;
#undef RETURN_IF_EQUAL
}


