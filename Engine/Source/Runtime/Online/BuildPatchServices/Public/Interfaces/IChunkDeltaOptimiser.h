// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/SecureHash.h"
#include "CoreMinimal.h"

namespace BuildPatchServices
{
	class IChunkDeltaOptimiser
	{
	public:
		virtual ~IChunkDeltaOptimiser() {}

		/**
		 * Declares an event type exposed on this class for when files have been saved to the cloud directory.
		 * The event offers an MD5 hash of the file.
		 * @param FullFilePath   The full file path on disk for the file saved.
		 * @param MD5Hash        The MD5 checksum for the file.
		 */
		DECLARE_EVENT_TwoParams(IChunkDeltaOptimiser, FOnChunkFileWritten, const FString& /*FullFilePath*/, const FMD5Hash& /*MD5Hash*/);

		/**
		 * Declares an event type exposed on this class for when files have been saved to the cloud directory.
		 * The event offers two types of hash of the file.
		 * @param FullFilePath   The full file path on disk for the file saved.
		 * @param MD5Hash        The MD5 checksum for the file.
		 * @param SHA1Hash       The SHA1 checksum for the file.
		 */
		DECLARE_EVENT_ThreeParams(IChunkDeltaOptimiser, FOnDeltaFileWritten, const FString& /*FullFilePath*/, const FMD5Hash& /*MD5Hash*/, const FSHAHash& /*SHA1Hash*/);

		/**
		 * @return a delegate that will broadcast when a chunk was written to the cloud directory.
		 */
		virtual FOnChunkFileWritten& OnChunkFileWritten() = 0;

		/**
		 * @return a delegate that will broadcast when the delta file was written to the cloud directory.
		 */
		virtual FOnDeltaFileWritten& OnDeltaFileWritten() = 0;

		/**
		 * Executes the delta optimization. The process takes a pair of manifests to produce additional delta data which reduces the download size to patch directly between them.
		 * NOTE: This function is blocking and will not return until finished.
		 * @return true if successful.
		 */
		virtual bool Run() = 0;
	};

	typedef TSharedPtr<IChunkDeltaOptimiser> IChunkDeltaOptimiserPtr;
	typedef TSharedRef<IChunkDeltaOptimiser> IChunkDeltaOptimiserRef;
}
