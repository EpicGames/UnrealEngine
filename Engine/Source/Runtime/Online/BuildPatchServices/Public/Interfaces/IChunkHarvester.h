// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IBuildManifest.h"
#include "Misc/SecureHash.h"

namespace BuildPatchServices
{
	class IChunkHarvester
	{
	public:
		virtual ~IChunkHarvester() {}

		/**
		 * Declares an event type exposed on this class for when data files have been saved to the cloud directory.
		 * The event offers an MD5 hash of the file.
		 * @param FullFilePath   The full file path on disk for the file saved.
		 * @param MD5Hash        The MD5 checksum for the file.
		 */
		DECLARE_EVENT_TwoParams(IChunkHarvester, FOnChunkFileWritten, const FString& /*FullFilePath*/, const FMD5Hash& /*MD5Hash*/);

		/**
		 * Declares an event type exposed to this class when the manifest has been loaded, providing the result.
		 * @param Manifest        The loaded manifest.
		 */
		DECLARE_EVENT_OneParam(IChunkHarvester, FOnManifestLoaded, const IBuildManifest& /*Manifest*/);

		/**
		 * @return a delegate that will broadcast when a chunk was written to the cloud directory.
		 */
		virtual FOnChunkFileWritten& OnChunkFileWritten() = 0;

		/**
		 * @return a delegate that will broadcast when a chunk was written to the cloud directory.
		 */
		virtual FOnManifestLoaded& OnManifestLoaded() = 0;

		/**
		 * Uses the manifest file to rebuild the cloud directory from the orignal build data.
		 * NOTE: This function is blocking and will not return until finished.
		 * @return Success
		 */
		virtual bool Run() = 0;

		/**
		 * Signal to the process that it should abort.
		 */
		virtual void Abort() = 0;
	};

	typedef TSharedPtr<IChunkHarvester> IChunkHarvesterPtr;
	typedef TSharedRef<IChunkHarvester> IChunkHarvesterRef;
}