// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/SecureHash.h"
#include "CoreMinimal.h"

class IBuildManifest;

namespace BuildPatchServices
{
	class IDirectoryChunker
	{
	public:
		virtual ~IDirectoryChunker() {}

		/**
		 * Declares an event type exposed on this class for when the files from the build root have been enumerated.
		 * @param BuildFiles     The files enumerated for chunking.
		 */
		DECLARE_EVENT_OneParam(IDirectoryChunker, FOnBuildFilesEnumerated, const TArray<FString>& /*BuildFiles*/)

		/**
		 * Declares an event type exposed on this class for when data files have been saved to the cloud directory.
		 * The event offers an MD5 hash of the file.
		 * @param FullFilePath   The full file path on disk for the file saved.
		 * @param MD5Hash        The MD5 checksum for the file.
		 */
		DECLARE_EVENT_TwoParams(IDirectoryChunker, FOnChunkFileWritten, const FString& /*FullFilePath*/, const FMD5Hash& /*MD5Hash*/);

		/**
		 * Declares an event type exposed to this class when a chunk has been matched with the already existing one.
		 * @param ChunkGuid        The unique identifier of the chunk.
		 */
		DECLARE_EVENT_OneParam(IDirectoryChunker, FOnChunkMatch, const FGuid& /*ChunkGuid*/);

		/**
		 * Declares an event type exposed on this class for when the manifest has been finalized.
		 * @param AppManifest		The ref to resulted manifest.
		 */
		DECLARE_EVENT_OneParam(IDirectoryChunker, FOnManifestGenerated, const IBuildManifest& /*AppManifest*/);

		/**
		 * Declares an event type exposed on this class for when the manifest has been saved to the cloud directory.
		 * The event offers two types of hash of the file.
		 * @param FullFilePath   The full file path on disk for the file saved.
		 * @param MD5Hash        The MD5 checksum for the file.
		 * @param SHA1Hash       The SHA1 checksum for the file.
		 */
		DECLARE_EVENT_ThreeParams(IDirectoryChunker, FOnManifestFileWritten, const FString& /*FullFilePath*/, const FMD5Hash& /*MD5Hash*/, const FSHAHash& /*SHA1Hash*/);

		/**
		 * @return a delegate that will broadcast when the files from the build root have been enumerated.
		 */
		virtual FOnBuildFilesEnumerated& OnBuildFilesEnumerated() = 0;

		/**
		 * @return a delegate that will broadcast when a chunk was written to the cloud directory.
		 */
		virtual FOnChunkFileWritten& OnChunkFileWritten() = 0;

		/**
		 * @return a delegate that will broadcast when a chunk was matched with an already existing one.
		 */
		virtual FOnChunkMatch& OnChunkMatch() = 0;

		/**
		 * @return a delegate that will broadcast when the manifest file was finilazed.
		 */
		virtual FOnManifestGenerated& OnManifestGenerated() = 0;

		/**
		 * @return a delegate that will broadcast after the manifest has been saved, and so could be decrypted.
		 */
		virtual FOnManifestGenerated& OnManifestGeneratedDecrypted() = 0;

		/**
		 * @return a delegate that will broadcast when the manifest file was written to the cloud directory.
		 */
		virtual FOnManifestFileWritten& OnManifestFileWritten() = 0;

		/**
		 * Processes the directory to create chunks for new data and produce a manifest, saved to the cloud directory.
		 * NOTE: This function is blocking and will not return until finished.
		 * @return true if successful.
		 */
		virtual bool Run() = 0;

		/**
		 * Signal to the process that it should abort.
		 */
		virtual void Abort() = 0;
	};

	typedef TSharedPtr<IDirectoryChunker> IDirectoryChunkerPtr;
	typedef TSharedRef<IDirectoryChunker> IDirectoryChunkerRef;
}
