// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace BuildPatchServices
{
	class IPatchDataEnumeration
	{
	public:
		virtual ~IPatchDataEnumeration() = default;

		/**
		 * Enumerates the data files used and outputs them to the OutputFile specified in the configuration
		 * NOTE: THIS function is blocking and will not return until finished.
		 * @return Success
		 */
		virtual bool Run() = 0;

		/**
		 * Enumerates the data files used and outputs them into the array provided
		 * NOTE: THIS function is blocking and will not return until finished.
		 * @param OutFiles		OUT Array of enumerated files
		 * @return Success
		 */
		virtual bool Run(TArray<FString>& OutFiles) = 0;
	};

	typedef TSharedPtr<IPatchDataEnumeration> IPatchDataEnumerationPtr;
	typedef TSharedRef<IPatchDataEnumeration> IPatchDataEnumerationRef;
}
