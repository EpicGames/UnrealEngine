// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EventLoop : ModuleRules
{
	public EventLoop(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"NetCommon"
			});

		PublicDefinitions.Add(string.Format("HAS_EVENTLOOP_PLATFORM_SOCKET_IMPLEMENTATION={0}", bHasPlatformSocketImplementation ? 1 : 0));
		PublicDefinitions.Add(string.Format("HAS_EVENTLOOP_PLATFORM_BSD_SOCKET_HEADER={0}", bHasPlatformBSDSocketHeader ? 1 : 0));
		PublicDefinitions.Add(string.Format("HAS_PLATFORM_EVENTLOOP_NOTIFIER_EVENTFD={0}", bHasPlatformEventLoopNotifierEventFd ? 1 : 0));
		PublicDefinitions.Add(string.Format("HAS_EVENTLOOP_NOTIFIER_EVENTFD={0}", bHasEventLoopNotifierEventFd ? 1 : 0));
	}

	public virtual bool bHasPlatformSocketImplementation
	{
		get
		{
			return false;
			// Other platforms may override this property.
		}
	}

	public virtual bool bHasPlatformBSDSocketHeader
	{
		get
		{
			return false;
			// Other platforms may override this property.
		}
	}

	public virtual bool bHasPlatformEventLoopNotifierEventFd { get { return false; } }
	public virtual bool bHasEventLoopNotifierEventFd { get { return Target.IsInPlatformGroup(UnrealPlatformGroup.Linux); } }
}
