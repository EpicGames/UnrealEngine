// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoop/BSDSocket/EventLoopIOManagerBSDSocketSelect.h"
#include "EventLoop/BSDSocket/BSDSocketTypesPrivate.h"
#include "EventLoop/EventLoopLog.h"
#include "EventLoop/IEventLoop.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "Stats/Stats.h"

#if PLATFORM_HAS_BSD_SOCKETS && PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT

namespace UE::EventLoop {

class FIOManagerBSDSocketSelectImpl final : public IIOManager
{
public:
	FIOManagerBSDSocketSelectImpl(IEventLoop& EventLoop);
	virtual ~FIOManagerBSDSocketSelectImpl() = default;
	virtual bool Init() override;
	virtual void Shutdown() override;
	virtual void Notify() override;
	virtual void Poll(FTimespan WaitTime) override;

	void PollInternal(FTimespan WaitTime);

	FIOAccessBSDSocket& GetIOAccess()
	{
		return IOAccess;
	}

private:
	using FStorageType = FIOAccessBSDSocket::FStorageType;
	using FInternalHandleArryType = FStorageType::FInternalHandleArryType;
	using FInternalHandle = FStorageType::FInternalHandle;

	struct FSelectData
	{
		fd_set SocketReadSet;
		fd_set SocketWriteSet;
		fd_set SocketExceptionSet;
		SOCKET MaxFd = INVALID_SOCKET;
	};

	void RebuildSelectData(FSelectData& SelectData) const;
	int32 PollSelectData(const FSelectData& SelectData, FSelectData& OutSignaledData, FTimespan WaitTime) const;

	FSelectData SelectData;

	IEventLoop& EventLoop;
	FStorageType IORequestStorage;
	FIOAccessBSDSocket IOAccess;
	TAtomic<bool> bAsyncSignal;

	// Indices
	TMap<FInternalHandle, SOCKET> SocketInternalHandleIndex;
	TMap<SOCKET, FInternalHandle> SocketIndex;
};

FIOManagerBSDSocketSelectImpl::FIOManagerBSDSocketSelectImpl(IEventLoop& InEventLoop)
	: EventLoop(InEventLoop)
	, IOAccess(IORequestStorage)
	, bAsyncSignal(false)
{
}

bool FIOManagerBSDSocketSelectImpl::Init()
{
	IORequestStorage.Init();
	return true;
}

void FIOManagerBSDSocketSelectImpl::Shutdown()
{
}

void FIOManagerBSDSocketSelectImpl::Notify()
{
	bAsyncSignal = true;
}

void FIOManagerBSDSocketSelectImpl::Poll(FTimespan WaitTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_FIOManagerBSDSocketSelectImpl_Poll);

	// The base BSD socket polling implementation relies on using socket select which cannot be
	// woken by another thread. To mitigate this issue, many smaller calls to PollInternal are done
	// while no signal is received.

	const FTimespan MaxIterationWaitTime = FTimespan::FromMilliseconds(10);
	FTimespan AccumulatedWaitTime;

	// If a signal was received before polling, still poll for any ready events using a timeout of 0.
	if (bAsyncSignal)
	{
		WaitTime = FTimespan::Zero();
	}

	// Run at least one iteration. Keep running while WaitTime has not been reached.
	for (int32 Index = 0; Index == 0 || AccumulatedWaitTime < WaitTime; ++Index)
	{
		FTimespan CurrentWaitTime = FMath::Min(WaitTime - AccumulatedWaitTime, MaxIterationWaitTime);
		PollInternal(CurrentWaitTime);

		if (bAsyncSignal)
		{
			break;
		}

		AccumulatedWaitTime += MaxIterationWaitTime;
	}

	// Reset signal. Resetting is safe here as setting the async signal indicates that the poll should be exited.
	bAsyncSignal = false;
}

void FIOManagerBSDSocketSelectImpl::PollInternal(FTimespan WaitTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_FIOManagerBSDSocketSelectImpl_PollInternal);

	// Process queued actions.
	FInternalHandleArryType AddedHandles;
	FInternalHandleArryType RemovedHandles;
	uint32 NumChanges = IORequestStorage.Update(&AddedHandles, &RemovedHandles);

	// Remove active requests before adding more to have an accurate count of available resources.
	for (FInternalHandle RemovedHandle : RemovedHandles)
	{
		SOCKET RemovedSocket;
		if (SocketInternalHandleIndex.RemoveAndCopyValue(RemovedHandle, RemovedSocket))
		{
			UE_LOGF(LogEventLoop, VeryVerbose, "[FIOManagerBSDSocketSelectImpl::PollInternal] Request removed: Socket: %p, Handle: %ls", reinterpret_cast<void*>(RemovedSocket), *RemovedHandle.ToString());
			SocketIndex.Remove(RemovedSocket);
		}
	}

	for (FInternalHandle AddedHandle : AddedHandles)
	{
		// Request may have been added and removed in the same update.
		if (FIORequestBSDSocket* NewRequest = IORequestStorage.Find(AddedHandle))
		{
			// Check whether there are enough resources available to handle the request.
			if (SocketIndex.Num() >= FD_SETSIZE)
			{
				UE_LOGF(LogEventLoop, Verbose, "[FIOManagerBSDSocketSelectImpl::PollInternal] Request failed: Socket: %p, Reason: %ls", reinterpret_cast<void*>(NewRequest->Socket), *LexToString(ESocketIoRequestStatus::NoResources));
				NewRequest->Callback(NewRequest->Socket, ESocketIoRequestStatus::NoResources, EIOFlags::None);
				IORequestStorage.Remove(AddedHandle);
				continue;
			}

			// Only one request may be active for a socket at one time.
			if (SocketIndex.Contains(NewRequest->Socket))
			{
				UE_LOGF(LogEventLoop, Verbose, "[FIOManagerBSDSocketSelectImpl::PollInternal] Request failed: Socket: %p, Reason: %ls", reinterpret_cast<void*>(NewRequest->Socket), *LexToString(ESocketIoRequestStatus::Invalid));
				NewRequest->Callback(NewRequest->Socket, ESocketIoRequestStatus::Invalid, EIOFlags::None);
				IORequestStorage.Remove(AddedHandle);
				continue;
			}

			UE_LOGF(LogEventLoop, VeryVerbose, "[FIOManagerBSDSocketSelectImpl::PollInternal] Request added: Socket: %p, Handle: %ls, Flags: 0x%08X", reinterpret_cast<void*>(NewRequest->Socket), *AddedHandle.ToString(), uint32{EnumToUnderlyingType(NewRequest->Flags)});

			SocketInternalHandleIndex.Add(AddedHandle, NewRequest->Socket);
			SocketIndex.Add(NewRequest->Socket, AddedHandle);
		}
	}

	// Check whether the cached FD_SETs need to be rebuilt.
	if (NumChanges > 0)
	{
		RebuildSelectData(SelectData);
	}

	// When there is no work, block the loop here.
	if (IORequestStorage.IsEmpty())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_FIOManagerBSDSocketSelectImpl_PollInternal_IdleSleep);
		FPlatformProcess::SleepNoStats(WaitTime.GetTotalSeconds());
		return;
	}

	// Poll for socket status.
	FSelectData SignaledSelectData;
	int32 SelectStatus = PollSelectData(SelectData, SignaledSelectData, WaitTime);

	// Check whether any events triggered.
	if (SelectStatus > 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_FIOManagerBSDSocketSelectImpl_PollInternal_HandleIOState);

		// Check request state and trigger callbacks if needed.
		for (TPair<FInternalHandle, FIORequestBSDSocket&>& IORequestEntry : IORequestStorage)
		{
			FIORequestBSDSocket& IORequest = IORequestEntry.Value;
			EIOFlags SignaledFlags = EIOFlags::None;

			if (EnumHasAnyFlags(IORequest.Flags, EIOFlags::Read) && FD_ISSET(IORequest.Socket, &SignaledSelectData.SocketReadSet))
			{
				SignaledFlags |= EIOFlags::Read;
			}

			if (EnumHasAnyFlags(IORequest.Flags, EIOFlags::Write) &&
				(FD_ISSET(IORequest.Socket, &SignaledSelectData.SocketWriteSet) || FD_ISSET(IORequest.Socket, &SignaledSelectData.SocketExceptionSet)))
			{
				SignaledFlags |= EIOFlags::Write;
			}

			if (SignaledFlags != EIOFlags::None)
			{
				// Signal event status.
				UE_LOGF(LogEventLoop, VeryVerbose, "[FIOManagerBSDSocketSelectImpl::PollInternal] Request signaled: Socket: %p, Handle: %ls, Flags: 0x%08X", reinterpret_cast<void*>(IORequest.Socket), *IORequestEntry.Key.ToString(), uint32{EnumToUnderlyingType(SignaledFlags)});
				IORequest.Callback(IORequest.Socket, ESocketIoRequestStatus::Ok, SignaledFlags);
			}
		}
	}
	else if (SelectStatus < 0)
	{
		UE_LOGF(LogEventLoop, Fatal, "[FIOManagerBSDSocketSelectImpl::PollInternal] Select error: %ls", *GetLastErrorString());
	}
}

void FIOManagerBSDSocketSelectImpl::RebuildSelectData(FSelectData& OutSelectData) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_FIOManagerBSDSocketSelectImpl_RebuildSelectData);
	FD_ZERO(&OutSelectData.SocketReadSet);
	FD_ZERO(&OutSelectData.SocketWriteSet);
	FD_ZERO(&OutSelectData.SocketExceptionSet);
	OutSelectData.MaxFd = INVALID_SOCKET;

	for (const TPair<SOCKET, FInternalHandle>& IndexEntry : SocketIndex)
	{
		if (const FIORequestBSDSocket* IORequest = IORequestStorage.Find(IndexEntry.Value))
		{
			if (OutSelectData.MaxFd == INVALID_SOCKET)
			{
				OutSelectData.MaxFd = IORequest->Socket;
			}
			else
			{
				OutSelectData.MaxFd = FMath::Max(SelectData.MaxFd, IORequest->Socket);
			}

			if (EnumHasAnyFlags(IORequest->Flags, EIOFlags::Read))
			{
				FD_SET(IORequest->Socket, &OutSelectData.SocketReadSet);
			}

			if (EnumHasAnyFlags(IORequest->Flags, EIOFlags::Write))
			{
				FD_SET(IORequest->Socket, &OutSelectData.SocketWriteSet);
				FD_SET(IORequest->Socket, &OutSelectData.SocketExceptionSet);
			}
		}
	}
}

int32 FIOManagerBSDSocketSelectImpl::PollSelectData(const FSelectData& InSelectData, FIOManagerBSDSocketSelectImpl::FSelectData& OutSignaledData, FTimespan WaitTime) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_FIOManagerBSDSocketSelectImpl_PollSelectData);

	// Copy cached FD sets.
	OutSignaledData = InSelectData;

	// convert WaitTime to a timeval
	timeval Time;
	Time.tv_sec = (int32)WaitTime.GetTotalSeconds();
	Time.tv_usec = WaitTime.GetFractionMicro();

	timeval* TimePointer = WaitTime.GetTicks() >= 0 ? &Time : nullptr;

	// Poll for socket status.
	return select(IntCastChecked<int>(OutSignaledData.MaxFd + 1), &OutSignaledData.SocketReadSet, &OutSignaledData.SocketWriteSet, &OutSignaledData.SocketExceptionSet, TimePointer);
}

//-----------------------
// Pimpl implementation.
//-----------------------

FIOManagerBSDSocketSelect::FIOManagerBSDSocketSelect(IEventLoop& EventLoop, FParams&&)
	: Impl(MakeShared<FIOManagerBSDSocketSelectImpl>(EventLoop))
{
}

bool FIOManagerBSDSocketSelect::Init()
{
	return Impl->Init();
}

void FIOManagerBSDSocketSelect::Shutdown()
{
	Impl->Shutdown();
}

void FIOManagerBSDSocketSelect::Notify()
{
	Impl->Notify();
}

void FIOManagerBSDSocketSelect::Poll(FTimespan WaitTime)
{
	Impl->Poll(WaitTime);
}

FIOManagerBSDSocketSelect::FIOAccess& FIOManagerBSDSocketSelect::GetIOAccess()
{
	return Impl->GetIOAccess();
}

/* UE::EventLoop */ }

#endif // PLATFORM_HAS_BSD_SOCKETS && PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT

