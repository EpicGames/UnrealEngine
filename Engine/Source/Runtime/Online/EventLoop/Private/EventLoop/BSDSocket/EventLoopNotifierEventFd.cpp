// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoop/BSDSocket/EventLoopNotifierEventFd.h"

#include "Async/Async.h"
#include "EventLoop/EventLoopLog.h"
#include "HAL//IConsoleManager.h"
#include "Misc/QueuedThreadPool.h"
#include "Stats/Stats.h"

#if HAS_EVENTLOOP_NOTIFIER_EVENTFD
#include "BSDSocketTypesPrivate.h"

namespace UE::EventLoop {

static bool bHttpEventLoopNotifierUseThreadPool = true;
static FAutoConsoleVariableRef CVarHttpEventLoopNotifierUseThreadPool(
	TEXT("http.EventLoopNotifierUseThreadPool"),
	bHttpEventLoopNotifierUseThreadPool,
	TEXT("When enabled, in event loop it will use thread pool to notify other thread to wake up"),
	ECVF_SaveForNextBoot
);

bool FEventLoopNotifierEventFd::Init()
{
	FileDescriptorRead = eventfd(0, EFD_NONBLOCK);
	return FileDescriptorRead != INVALID_SOCKET;
}

void FEventLoopNotifierEventFd::Shutdown()
{
	// Wait for any in-flight async notify to complete before closing the fd,
	// so the lambda cannot access a closed or destroyed fd.
#if PLATFORM_WRITES_ARE_SLOW
	while (bNotified)
	{
		FPlatformProcess::Yield();
	}
#endif

	if (FileDescriptorRead != INVALID_SOCKET)
	{
		close(FileDescriptorRead);
		FileDescriptorRead = INVALID_SOCKET;
	}
}

namespace EventLoopNotifierEventFd
{

static void NotifyImpl(int32 Fd)
{
	eventfd_t Value = 1;
	if (eventfd_write(Fd, Value) == SOCKET_RESULT_FAILED)
	{
		int Err = errno;
		UE_CLOGF(Err != EAGAIN, LogEventLoop, Warning, "Failed to notify by event_fd_write %d with error %d", Fd, Err);
	}
}

}

void FEventLoopNotifierEventFd::Notify()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FEventLoopNotifierEventFd_Notify);

#if PLATFORM_WRITES_ARE_SLOW
	if (bHttpEventLoopNotifierUseThreadPool && GIOThreadPool)
	{ 
		// When Notify() get called several times in same frame, we only need to notify once
		if (!bNotified.exchange(true))
		{
			AsyncPool(*GIOThreadPool, [Fd = FileDescriptorRead, &bNotified = this->bNotified] {
				EventLoopNotifierEventFd::NotifyImpl(Fd);
				bNotified = false;
			});
		}
	}
	else
#endif
	{
		EventLoopNotifierEventFd::NotifyImpl(FileDescriptorRead);
	}
}

void FEventLoopNotifierEventFd::Clear()
{
	eventfd_t Value = 0;
	ensure(eventfd_read(FileDescriptorRead, &Value) != SOCKET_RESULT_FAILED);
}

/* UE::EventLoop */ }

#endif // HAS_EVENTLOOP_NOTIFIER_EVENTFD

