// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoop/BSDSocket/EventLoopNotifierSocket.h"
#include "BSDSocketTypesPrivate.h"
#include "EventLoop/EventLoopLog.h"
#include "Stats/Stats.h"

#if PLATFORM_HAS_BSD_SOCKETS && !PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS

namespace UE::EventLoop {

namespace EventLoopNotifierSocket
{

static void SetSocketNonBlock(int32 FileDescriptor)
{
	int32 Flags = fcntl(FileDescriptor, F_GETFL, 0);
	fcntl(FileDescriptor, F_SETFL, O_NONBLOCK | Flags );
}

}

bool FEventLoopNotifierSocket::Init()
{
	// TODO: Validate and handle the case for socket recreation in case of suspend and resume

	// TODO: Investigate if it's possible to use pipe or IPPROTO_RAW, or socketpair instead
	FileDescriptorListener = socket(AF_INET, SOCK_STREAM, IPPROTO_TCP);
	if (FileDescriptorListener == INVALID_SOCKET)
	{
		return false;
	}
	sockaddr_in AddrIn;
	memset(&AddrIn, 0, sizeof(AddrIn));
	AddrIn.sin_family = AF_INET;
	AddrIn.sin_addr.s_addr = htonl(INADDR_LOOPBACK);
	AddrIn.sin_port = 0;

	if (bind(FileDescriptorListener, reinterpret_cast<sockaddr*>(&AddrIn), sizeof(AddrIn)) == SOCKET_RESULT_FAILED)
	{
		close(FileDescriptorListener);
		FileDescriptorListener = INVALID_SOCKET;

		return false;
	}

	if (listen(FileDescriptorListener, /*backlog*/1) == SOCKET_RESULT_FAILED)
	{
		close(FileDescriptorListener);
		FileDescriptorListener = INVALID_SOCKET;

		return false;
	}

	EventLoopNotifierSocket::SetSocketNonBlock(FileDescriptorListener);




	// TODO: Set the write buffer to 1
	sockaddr Addr;
	memset(&Addr, 0, sizeof(Addr));

	socklen_t AddrLen = sizeof(Addr);
	getsockname(FileDescriptorListener, &Addr, &AddrLen);

	FileDescriptorRead = socket(AF_INET, SOCK_STREAM, IPPROTO_TCP);
	if (FileDescriptorRead == INVALID_SOCKET)
	{
		close(FileDescriptorListener);
		FileDescriptorListener = INVALID_SOCKET;
		return false;
	}
	if (connect(FileDescriptorRead, &Addr, sizeof(Addr)) == SOCKET_RESULT_FAILED)
	{
		close(FileDescriptorListener);
		FileDescriptorListener = INVALID_SOCKET;

		close(FileDescriptorRead);
		FileDescriptorRead = INVALID_SOCKET;
		return false;
	}

	EventLoopNotifierSocket::SetSocketNonBlock(FileDescriptorRead);




	// TODO: Set the read buffer to 1
	FileDescriptorWrite = accept(FileDescriptorListener, NULL, NULL);
	if (FileDescriptorWrite == INVALID_SOCKET)
	{
		close(FileDescriptorListener);
		FileDescriptorListener = INVALID_SOCKET;

		close(FileDescriptorRead);
		FileDescriptorRead = INVALID_SOCKET;
		return false;
	}

	EventLoopNotifierSocket::SetSocketNonBlock(FileDescriptorWrite);




	return true;
}

void FEventLoopNotifierSocket::Shutdown()
{
	if (FileDescriptorWrite != INVALID_SOCKET)
	{
		close(FileDescriptorWrite);
		FileDescriptorWrite = INVALID_SOCKET;
	}

	if (FileDescriptorRead != INVALID_SOCKET)
	{
		close(FileDescriptorRead);
		FileDescriptorRead = INVALID_SOCKET;
	}

	if (FileDescriptorListener != INVALID_SOCKET)
	{
		close(FileDescriptorListener);
		FileDescriptorListener = INVALID_SOCKET;
	}
}

void FEventLoopNotifierSocket::Notify()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FEventLoopNotifierSocket_Notify);
	if (FileDescriptorWrite != INVALID_SOCKET)
	{
		// TODO: A failure to write may not be critical. Its possible that the receive buffer is just full. 
		// The notifier just cares that the socket is set to a readable state.
		if (write(FileDescriptorWrite, "1", 1) == SOCKET_RESULT_FAILED)
		{
			int Err = errno;
			UE_CLOGF(Err != EAGAIN, LogEventLoop, Warning, "Failed to notify by socket fd write %d with error %d", FileDescriptorWrite, Err);
		}
	}
}

void FEventLoopNotifierSocket::Clear()
{
	if (FileDescriptorRead != INVALID_SOCKET)
	{
		char Buffer[1];
		ensure(read(FileDescriptorRead, &Buffer[0], 1) != SOCKET_RESULT_FAILED);
	}
}

/* UE::EventLoop */ }

#endif // PLATFORM_HAS_BSD_SOCKETS && !PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS

