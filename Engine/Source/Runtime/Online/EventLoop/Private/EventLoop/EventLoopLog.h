// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

namespace UE::EventLoop {

DECLARE_LOG_CATEGORY_EXTERN(LogEventLoop, Log, All);

/* UE::EventLoop */ }
