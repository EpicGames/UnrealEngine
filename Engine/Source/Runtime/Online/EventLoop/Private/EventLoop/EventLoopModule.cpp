// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoop/EventLoopModule.h"

IMPLEMENT_MODULE(UE::EventLoop::FEventLoopModule, EventLoop);

namespace UE::EventLoop {

void FEventLoopModule::StartupModule()
{
}

void FEventLoopModule::ShutdownModule()
{
}

/* UE::EventLoop */ }
