// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_HAS_BSD_SOCKETS

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	typedef UPTRINT SOCKET;
	#define INVALID_SOCKET  (SOCKET)(~0)
#else // PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
	typedef int32 SOCKET;
	#define INVALID_SOCKET -1
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS

#define SOCKET_RESULT_FAILED -1

#endif // PLATFORM_HAS_BSD_SOCKETS