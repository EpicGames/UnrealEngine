// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_HAS_BSD_SOCKETS

#include "Containers/UnrealString.h"
#include "EventLoop/BSDSocket/BSDSocketTypes.h"
#include "EventLoop/IEventLoopIOManager.h"
#include "EventLoop/EventLoopManagedStorage.h"
#include "Templates/Function.h"

namespace UE::EventLoop {

enum class ESocketIoRequestStatus : uint8
{
	/**
	 * No error occurred, signaled flags are valid.
	 * After signaling the handle associated with the request will remain valid until the user
	 * deletes the associated request.
	 */
	Ok,

	/**
	 * There are no resources available to serve the IO request.
	 * This will occur if more requests have been made than the system is capable of handling.
	 * After signaling the handle associated with the request is removed.
	 */
	NoResources,

	/**
	 * The request is invalid.
	 * A request may be invalid if another request for the same socket is already in progress.
	 * After signaling the handle associated with the request is removed.
	 */
	Invalid,
};

using FSocketIOCallback = TUniqueFunction<void(SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags)>;
using FSocketIORequestDestroyedCallback = FManagedStorageOnRemoveComplete;

struct FIORequestBSDSocket
{
	SOCKET Socket = INVALID_SOCKET;
	EIOFlags Flags = EIOFlags::None;
	FSocketIOCallback Callback;
};

class FIOAccessBSDSocket final : public FNoncopyable
{
public:
	struct FStorageTraits : public FManagedStorageDefaultTraits
	{
		using FExternalHandle = FIORequestHandle;
	};

	using FStorageType = TManagedStorage<FIORequestBSDSocket, FStorageTraits>;

	FIOAccessBSDSocket(FStorageType& InIORequestStorage)
		: IORequestStorage(InIORequestStorage)
	{
	}

	FIORequestHandle CreateSocketIORequest(FIORequestBSDSocket&& Request)
	{
		if (Request.Socket == INVALID_SOCKET || Request.Flags == EIOFlags::None || !Request.Callback)
		{
			return FIORequestHandle();
		}

		return IORequestStorage.Add({MoveTemp(Request)});
	}

	void DestroyIORequest(FIORequestHandle& Handle, FSocketIORequestDestroyedCallback&& OnDestroyComplete = FSocketIORequestDestroyedCallback())
	{
		IORequestStorage.Remove(Handle, MoveTemp(OnDestroyComplete));
	}

private:
	FStorageType& IORequestStorage;
};

/* UE::EventLoop */ }

inline FString LexToString(UE::EventLoop::ESocketIoRequestStatus Status)
{
	switch (Status)
	{
	case UE::EventLoop::ESocketIoRequestStatus::Ok: return TEXT("Ok");
	case UE::EventLoop::ESocketIoRequestStatus::NoResources: return TEXT("NoResources");
	case UE::EventLoop::ESocketIoRequestStatus::Invalid: return TEXT("Invalid");

	default:
		checkNoEntry();
		return TEXT("Unknown");
	}
}

#endif // PLATFORM_HAS_BSD_SOCKETS
