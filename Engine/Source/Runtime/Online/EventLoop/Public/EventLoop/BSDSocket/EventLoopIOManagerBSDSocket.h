// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EventLoop/BSDSocket/EventLoopIOManagerBSDSocketSelect.h"

namespace UE::EventLoop {

#if PLATFORM_HAS_BSD_SOCKETS
#if HAS_EVENTLOOP_PLATFORM_SOCKET_IMPLEMENTATION
	#include COMPILED_PLATFORM_HEADER(EventLoopIOManagerBSDSocket.h)
#else
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT
	// Default implementation.
	using FIOManagerBSDSocket = FIOManagerBSDSocketSelect;
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT
#endif // EVENTLOOP_PLATFORM_IMPLEMENTATION
#endif // PLATFORM_HAS_BSD_SOCKETS

/* UE::EventLoop */ }
