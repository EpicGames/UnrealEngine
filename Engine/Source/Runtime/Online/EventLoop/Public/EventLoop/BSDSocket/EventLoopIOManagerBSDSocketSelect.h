// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if PLATFORM_HAS_BSD_SOCKETS && PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT

#include "EventLoop/BSDSocket/BSDSocketTypes.h"
#include "EventLoop/BSDSocket/EventLoopIOAccessBSDSocket.h"
#include "EventLoop/IEventLoopIOManager.h"
#include "EventLoop/EventLoopManagedStorage.h"
#include "Templates/SharedPointer.h"

namespace UE::EventLoop {

class FIOManagerBSDSocketSelect final : public IIOManager
{
public:
	using FIOAccess = FIOAccessBSDSocket;

	struct FParams
	{
	};

	struct FConfig
	{
	};

	EVENTLOOP_API FIOManagerBSDSocketSelect(IEventLoop& EventLoop, FParams&& Params);
	virtual ~FIOManagerBSDSocketSelect() = default;
	EVENTLOOP_API virtual bool Init() override;
	EVENTLOOP_API virtual void Shutdown() override;
	EVENTLOOP_API virtual void Notify() override;
	EVENTLOOP_API virtual void Poll(FTimespan WaitTime) override;

	EVENTLOOP_API FIOAccess& GetIOAccess();

private:
	TSharedRef<class FIOManagerBSDSocketSelectImpl> Impl;
};

/* UE::EventLoop */ }

#endif // PLATFORM_HAS_BSD_SOCKETS && PLATFORM_HAS_BSD_SOCKET_FEATURE_SELECT
