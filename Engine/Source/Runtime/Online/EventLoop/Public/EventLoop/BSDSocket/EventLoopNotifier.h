// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if HAS_EVENTLOOP_NOTIFIER_EVENTFD
#include "EventLoop/BSDSocket/EventLoopNotifierEventFd.h"
namespace UE::EventLoop
{
using FEventLoopNotifier = FEventLoopNotifierEventFd;
}
#elif PLATFORM_HAS_BSD_SOCKETS && !PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
#include "EventLoop/BSDSocket/EventLoopNotifierSocket.h"
namespace UE::EventLoop
{
using FEventLoopNotifier = FEventLoopNotifierSocket;
}
#endif
