// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BSDSocketTypes.h"
#include "CoreTypes.h"
#include <atomic>

#if HAS_EVENTLOOP_NOTIFIER_EVENTFD

namespace UE::EventLoop {

class FEventLoopNotifierEventFd
{
public:
	bool Init();
	void Shutdown();

	void Notify();
	void Clear();

	int32 GetFileDescriptorRead() const { return FileDescriptorRead; }

private:
	int32 FileDescriptorRead = INVALID_SOCKET;
	std::atomic<bool> bNotified = false;
};

/* UE::EventLoop */ }

#endif // HAS_EVENTLOOP_NOTIFIER_EVENTFD
