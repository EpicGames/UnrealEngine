// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BSDSocketTypes.h"
#include "CoreTypes.h"

#if PLATFORM_HAS_BSD_SOCKETS && !PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS

namespace UE::EventLoop {

class FEventLoopNotifierSocket
{
public:
	bool Init();
	void Shutdown();

	void Notify();
	void Clear();

	int32 GetFileDescriptorRead() const { return FileDescriptorRead; }

private:
	int32 FileDescriptorListener = INVALID_SOCKET;
	int32 FileDescriptorWrite = INVALID_SOCKET;
	int32 FileDescriptorRead = INVALID_SOCKET;
};

/* UE::EventLoop */ }

#endif // PLATFORM_HAS_BSD_SOCKETS && !PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS

