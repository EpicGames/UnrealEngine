// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Algo/ForEach.h"
#include "Async/Mutex.h"
#include "Containers/MpscQueue.h"
#include "EventLoop/IEventLoop.h"
#include "EventLoop/EventLoopTimer.h"
#include "EventLoop/IEventLoopIOManager.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/ScopeExit.h"
#include "Stats/Stats.h"
#include <limits>

namespace UE::EventLoop {

struct FEventLoopTraitsBase
{
	static inline uint32 GetCurrentThreadId()
	{
		return FPlatformTLS::GetCurrentThreadId();
	}

	static inline FTimespan GetCurrentTime()
	{
		return FTimespan(FPlatformTime::Seconds() * ETimespan::TicksPerSecond);
	}

	static inline bool IsEventLoopThread(uint32 EventLoopThreadId)
	{
		return EventLoopThreadId == GetCurrentThreadId();
	}

	static inline bool IsInitialized(uint32 EventLoopThreadId)
	{
		return (EventLoopThreadId != 0);
	}

	static inline void CheckInitialized(uint32 EventLoopThreadId)
	{
		check(IsInitialized(EventLoopThreadId));
	}

	static inline void CheckNotInitialized(uint32 EventLoopThreadId)
	{
		check(!IsInitialized(EventLoopThreadId));
	}

	static inline void CheckIsEventLoopThread(uint32 EventLoopThreadId)
	{
		check(IsEventLoopThread(EventLoopThreadId));
	}

	static inline bool IsShutdownRequested(bool bShutdownRequested)
	{
		return bShutdownRequested;
	}
};

struct FEventLoopDefaultTraits : public FEventLoopTraitsBase
{
	using FTimerManagerTraits = FTimerManagerDefaultTraits;
};

template <typename IOManagerType, typename Traits = FEventLoopDefaultTraits>
class TEventLoop final : public IEventLoop
{
private:
	IOManagerType IOManager;
	TTimerManager<typename Traits::FTimerManagerTraits> TimerManager;
	TMpscQueue<FAsyncTask> AsyncTasks;
	TArray<FOnShutdownComplete> ShutdownRequests;
	FTimespan LoopTime;
	TAtomic<uint32> EventLoopThreadId;
	TAtomic<bool> bShutdownRequested;
	TAtomic<bool> bShutdownCompleted;
	TAtomic<bool> bNotifyOnInit;
	UE::FMutex InitMutex;
	UE::FMutex ShutdownMutex;

public:
	struct FParams
	{
		// Initial IO manager parameters.
		typename IOManagerType::FParams IOManagerParams;
	};

	TEventLoop(FParams&& Params = FParams())
		: IOManager(*this, MoveTemp(Params.IOManagerParams))
		, TimerManager()
		, AsyncTasks()
		, LoopTime()
		, EventLoopThreadId(0)
		, bShutdownRequested(false)
		, bShutdownCompleted(false)
		, bNotifyOnInit(false)
	{
		// Make sure request manager type derives from IIOManager.
		static_assert(std::is_base_of_v<IIOManager, IOManagerType> == true);
	}

	virtual ~TEventLoop() = default;

	typename IOManagerType::FIOAccess& GetIOAccess()
	{
		return IOManager.GetIOAccess();
	}

	virtual bool Init() override
	{
		bool bInitSuccessful = false;

		{
			InitMutex.Lock();
			ON_SCOPE_EXIT
			{
				InitMutex.Unlock();
			};

			Traits::CheckNotInitialized(EventLoopThreadId);
			LoopTime = Traits::GetCurrentTime();
			TimerManager.Init();
			bInitSuccessful = IOManager.Init();
			EventLoopThreadId = Traits::GetCurrentThreadId();
		}

		if (bNotifyOnInit)
		{
			IOManager.Notify();
		}

		return bInitSuccessful;
	}

	virtual void RequestShutdown(FOnShutdownComplete&& OnShutdownComplete = FOnShutdownComplete()) override
	{
		bool bRunLocally = false;

		{
			ShutdownMutex.Lock();
			ON_SCOPE_EXIT
			{
				ShutdownMutex.Unlock();
			};

			// Check if this call to RequestShutdown is the first call.
			bool bExpectedValue = false;
			if (bShutdownRequested.CompareExchange(bExpectedValue, true))
			{
				// This is the first call to RequestShutdown. Notify the request manager to wake
				// it so that the loop will see the request.
				ShutdownRequests.Add(MoveTemp(OnShutdownComplete));
				TryNotify();
			}
			else if (!bShutdownCompleted)
			{
				// Shutdown has been requested, but not yet acknowledged.
				ShutdownRequests.Add(MoveTemp(OnShutdownComplete));
			}
			else
			{
				// Loop shutdown has already completed. Run the callback locally outside of the lock.
				bRunLocally = true;
			}
		}

		// Loop shutdown has already completed.
		if (bRunLocally && OnShutdownComplete)
		{
			OnShutdownComplete();
		}
	}

	virtual FTimerHandle SetTimer(FTimerCallback&& Callback, FTimespan InRate, bool InbRepeat = false, TOptional<FTimespan> InFirstDelay = TOptional<FTimespan>()) override
	{
		FTimerHandle OutHandle = TimerManager.SetTimer(MoveTemp(Callback), InRate, InbRepeat, InFirstDelay);
		TryNotify();
		return OutHandle;
	}

	virtual void ClearTimer(FTimerHandle& InHandle, FOnTimerCleared&& OnTimerCleared = FOnTimerCleared()) override
	{
		const bool bWasValid = InHandle.IsValid();
		TimerManager.ClearTimer(InHandle, MoveTemp(OnTimerCleared));
		if (bWasValid)
		{
			TryNotify();
		}
	}

	virtual void PostAsyncTask(FAsyncTask&& Task) override
	{
		AsyncTasks.Enqueue(MoveTemp(Task));
		TryNotify();
	}

	virtual void Run() override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_TEventLoop_Run);
		Traits::CheckInitialized(EventLoopThreadId);

		while (true)
		{
			const FTimespan InfiniteWait(std::numeric_limits<int64>::max());
			FTimespan WaitTime = InfiniteWait;

			if (!RunOnce(WaitTime))
			{
				break;
			}
		};
	}

	virtual bool RunOnce(FTimespan WaitTime) override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_TEventLoop_RunOnce);
		Traits::CheckInitialized(EventLoopThreadId);

		if (bShutdownCompleted)
		{
			// Terminated.
			return false;
		}

		// Check if there are any running timers. The maximum wait time will then be shortened
		// to the expiration of the next timer.
		FTimespan TimerExpirationTime;
		if (TimerManager.GetNextTimeout(TimerExpirationTime))
		{
			WaitTime = FMath::Min(WaitTime, TimerExpirationTime);
		}

		IOManager.Poll(WaitTime);

		// Run timers.
		FTimespan CurrentTime = Traits::GetCurrentTime();
		FTimespan ElapsedTime = FMath::Max(CurrentTime - LoopTime, 0.0);
		LoopTime = CurrentTime;
		TimerManager.Tick(ElapsedTime);

		// If any timers ran due to the above call to Tick, check whether any repeat timers are
		// waiting to be rescheduled. Ticking with zero delta reschedules them into the heap
		// without advancing internal time, which prevents short-interval timers from
		// immediately re-firing in the same iteration.
		if (TimerManager.HasPendingRepeatTimer())
		{
			TimerManager.Tick(FTimespan::Zero());
		}

		RunAsyncTasks();

		if (Traits::IsShutdownRequested(bShutdownRequested))
		{
			IOManager.Shutdown();

			TArray<FOnShutdownComplete> LocalShutdownRequests;
			{
				ShutdownMutex.Lock();
				ON_SCOPE_EXIT
				{
					ShutdownMutex.Unlock();
				};

				bShutdownCompleted = true;
				LocalShutdownRequests = MoveTemp(ShutdownRequests);
			}

			// Run post-shutdown tasks.
			Algo::ForEach(LocalShutdownRequests, [](FOnShutdownComplete& OnShutdownComplete){
				if (OnShutdownComplete)
				{
					OnShutdownComplete();
				}
			});

			// Terminated.
			return false;
		}

		// Continue running.
		return true;
	}

	virtual FTimespan GetLoopTime() const override
	{
		Traits::CheckIsEventLoopThread(EventLoopThreadId);
		return LoopTime;
	}

private:
	void RunAsyncTasks()
	{
		FAsyncTask AsyncTask;
		while (AsyncTasks.Dequeue(AsyncTask))
		{
			AsyncTask();
		}
	}

	void TryNotify()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_TEventLoop_TryNotify);

		if (Traits::IsInitialized(EventLoopThreadId))
		{
			// Init already completed. Notify the IO manager.
			IOManager.Notify();
		}
		else
		{
			InitMutex.Lock();
			ON_SCOPE_EXIT
			{
				InitMutex.Unlock();
			};

			if (!Traits::IsInitialized(EventLoopThreadId))
			{
				// Init has not yet started. Let it know to notify the IO manager of
				// pending actions once it has completed.
				bNotifyOnInit = true;
			}
			else
			{
				// Init competed before the lock was acquired. Notify the IO manager.
				IOManager.Notify();
			}
		}
	}
};

/* UE::EventLoop */ }
