// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Templates/Atomic.h"

namespace UE::EventLoop {

template <typename Traits>
struct TResourceHandle final
{
	enum EGenerateNewHandleType
	{
		GenerateNewHandle
	};

	/** Creates an initially unset handle */
	TResourceHandle()
		: ID(0)
	{
	}

	/** Creates a handle pointing to a new instance */
	explicit TResourceHandle(EGenerateNewHandleType)
		: ID(GenerateNewID())
	{
	}

	/** True if this handle was ever initialized by the timer manager */
	bool IsValid() const
	{
		return ID > 0;
	}

	/** Explicitly clear handle */
	void Invalidate()
	{
		ID = 0;
	}

	bool operator==(const TResourceHandle& Other) const
	{
		return ID == Other.ID;
	}

	bool operator!=(const TResourceHandle& Other) const
	{
		return ID != Other.ID;
	}

	friend inline uint32 GetTypeHash(const TResourceHandle& InHandle)
	{
		return GetTypeHash(InHandle.ID);
	}

	FString ToString() const
	{
		return IsValid() ? FString::Printf(TEXT("%s:%llu"), Traits::Name, ID) : FString::Printf(TEXT("%s:<invalid>"), Traits::Name);
	}

	static const TCHAR* GetTypeName()
	{
		return Traits::Name;
	}

private:
	/**
	 * Generates a new ID for use the delegate handle.
	 *
	 * @return A unique ID for the delegate.
	 */
	static uint64 GenerateNewID()
	{
		// Just increment a counter to generate an ID.
		uint64 Result = NextID++;

		// Check for the next-to-impossible event that we wrap round to 0, because we reserve 0 for null delegates.
		if (Result == 0)
		{
			// Increment it again - it might not be zero, so don't just assign it to 1.
			Result = ++NextID;
		}

		return Result;
	}

	static TAtomic<uint64> NextID;

	uint64 ID;
};

template <typename Traits>
TAtomic<uint64> TResourceHandle<Traits>::NextID = 1;

/* UE::EventLoop */ }
