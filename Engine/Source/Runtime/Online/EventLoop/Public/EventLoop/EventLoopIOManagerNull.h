// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Async/ManualResetEvent.h"
#include "EventLoop/IEventLoop.h"
#include "EventLoop/IEventLoopIOManager.h"
#include "Misc/Timespan.h"

namespace UE::EventLoop {

class FIOAccessNull final : public FNoncopyable
{
public:
	// Nothing needed for now.
};

class FIOManagerNull final : public IIOManager
{
public:
	using FIOAccess = FIOAccessNull;

	struct FParams
	{
	};

	FIOManagerNull(IEventLoop& EventLoop, FParams&& Params)
	{
	}

	virtual ~FIOManagerNull() = default;

	virtual bool Init() override
	{
		return true;
	}

	virtual void Shutdown() override
	{
	}

	virtual void Notify() override
	{
		Event.Notify();
	}

	virtual void Poll(FTimespan WaitTime) override
	{
		Event.WaitFor(FMonotonicTimeSpan::FromSeconds(WaitTime.GetTotalSeconds()));
		Event.Reset();
	}

	FIOAccess& GetIOAccess()
	{
		return IOAccess;
	}

private:
	FIOAccess IOAccess;
	UE::FManualResetEvent Event;
};

/* UE::EventLoop */ }
