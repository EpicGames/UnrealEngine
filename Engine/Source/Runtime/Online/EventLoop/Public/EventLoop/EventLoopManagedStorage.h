// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Containers/MpscQueue.h"
#include "EventLoop/EventLoopHandle.h"
#include "HAL/PlatformTLS.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/Optional.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"

namespace UE::EventLoop {

using FManagedStorageOnRemoveComplete = TUniqueFunction<void()>;

struct FManagedStorageDefaultExternalHandleTraits
{
	static constexpr TCHAR Name[] = TEXT("ManagedStorageDefault");
};

struct FManagedStorageTraitsBase
{
	static inline uint32 GetCurrentThreadId()
	{
		return FPlatformTLS::GetCurrentThreadId();
	}

	static inline bool IsManagerThread(uint32 ManagerThreadId)
	{
		return ManagerThreadId == GetCurrentThreadId();
	}

	static inline void CheckNotInitialized(uint32 ManagerThreadId)
	{
		check(ManagerThreadId == 0);
	}

	static inline void CheckIsManagerThread(uint32 ManagerThreadId)
	{
		check(IsManagerThread(ManagerThreadId));
	}
};

/*
 * Default traits for managed storage. Traits are used to implement functionality which can be
 * mocked to allow testing the class.
 *
 * In most cases the default traits can be used without modification.
 */
struct FManagedStorageDefaultTraits : public FManagedStorageTraitsBase
{

	/**
	 * The allocator to use for the internal handle array.
	 * 
	 * An internal handle array is passed to Update to get the handles added since the last call.
	 * Using inline storage can prevent an unnecessary allocation when retrieving this info.
	 */
	using InternalHandleArryAllocatorType = TInlineAllocator<32>;

	/**
	 * Whether to check thread safety when accessing internal storage.
	 * Defaulted to off for performance, but can be added to implementation traits for debugging.
	 */
	static constexpr bool bStorageAccessThreadChecksEnabled = false;

	/**
	 * The type of handle which will ne externally usable.
	 */
	using FExternalHandle = TResourceHandle<FManagedStorageDefaultExternalHandleTraits>;
};

/**
 * Managed storage class which allows resource creation and removal from any thread.
 * The owning thread may access the storage without thread safety concerns.
 */
template <typename ElementType, typename Traits = FManagedStorageDefaultTraits>
class TManagedStorage final : public FNoncopyable
{
public:
	/*
	 * Handle which extends the external resource handle to allow O(1) access into the internal storage array.
	 */
	template <typename ExternalHandleType>
	struct TManagedStorageInternalHandle
	{
		enum EGenerateNewHandleType
		{
			GenerateNewHandle
		};

		/** Creates an initially unset handle */
		TManagedStorageInternalHandle()
			: ExternalHandle()
			, InternalIndex(INDEX_NONE)
		{
		}

		/** Creates a handle pointing to a new instance */
		explicit TManagedStorageInternalHandle(EGenerateNewHandleType, ExternalHandleType InExternalHandle, int32 InInternalIndex)
			: ExternalHandle(InExternalHandle)
			, InternalIndex(InInternalIndex)
		{
		}

		/** True if this handle was ever initialized by the timer manager */
		bool IsValid() const
		{
			return ExternalHandle.IsValid();
		}

		/** Explicitly clear handle */
		void Invalidate()
		{
			ExternalHandle.Invalidate();
		}

		bool operator==(const TManagedStorageInternalHandle& Other) const
		{
			return ExternalHandle == Other.ExternalHandle;
		}

		bool operator!=(const TManagedStorageInternalHandle& Other) const
		{
			return ExternalHandle != Other.ExternalHandle;
		}

		friend inline uint32 GetTypeHash(const TManagedStorageInternalHandle& InHandle)
		{
			return GetTypeHash(InHandle.ExternalHandle);
		}

		FString ToString() const
		{
			return IsValid() ? FString::Printf(TEXT("Internal:[ext:%s, idx:%d]"), *ExternalHandle.ToString(), InternalIndex) : FString("<invalid>");
		}

		/** Gets the external view of this handle. */
		ExternalHandleType GetExternalHandle() const
		{
			return ExternalHandle;
		}

		/** Gets the internal storage index for the handles data. */
		int32 GetInternalIndex() const
		{
			return InternalIndex;
		}

	private:
		ExternalHandleType ExternalHandle;
		int32 InternalIndex;
	};

	using FExternalHandle = typename Traits::FExternalHandle;
	using FInternalHandle = TManagedStorageInternalHandle<FExternalHandle>;
	using FInternalHandleArryType = TArray<FInternalHandle, typename Traits::InternalHandleArryAllocatorType>;

private:
	struct FAddRequest
	{
		FExternalHandle Handle;
		ElementType Data;
	};

	struct FRemoveRequest
	{
		FExternalHandle Handle;
		FManagedStorageOnRemoveComplete OnComplete;
	};

	struct FStorageEntry
	{
		FInternalHandle InternalHandle;
		ElementType Data;
	};

	using FStorageType = TSparseArray<FStorageEntry>;

	/** The internal storage of entries. */
	FStorageType Storage;
	/* Mapping of external handles to the array index of the element in Storage. */
	TMap<FExternalHandle, int32> StorageHandleIndex;
	/** Requests to add entries to storage. Processed in Update(). */
	TMpscQueue<FAddRequest> AddRequests;
	/** Requests to remove entries from storage. Processed in Update(). */
	TMpscQueue<FRemoveRequest> RemoveRequests;
	/**
	 * The thread id set when calling Init. This thread becomes the 'Manager' thread and is safe
	 * to call non-thread safe methods from.
	 */
	TAtomic<uint32> ManagerThreadId;

public:
	TManagedStorage()
		: Storage()
		, StorageHandleIndex()
		, AddRequests()
		, RemoveRequests()
		, ManagerThreadId(0)
	{
	}

	~TManagedStorage() = default;

	/**
	 * External access for checking whether the current thread is the manager thread.
	 */
	inline bool IsManagerThread() const
	{
		return Traits::IsManagerThread(ManagerThreadId);
	}

	/**
	 * Initialize storage.
	 * 
	 * NOT thread safe.
	 */
	void Init()
	{
		Traits::CheckNotInitialized(ManagerThreadId);
		ManagerThreadId = Traits::GetCurrentThreadId();
	}

	 /**
	  * Process all pending storage modifications.
	  * 
	  * NOT thread safe.
	  * 
	  * @param OutAddedHandles	Handles which were added since the last call to Update.
	  * 						Element may not be present in the internal storage if element was
	  *							also removed since the last call to update.
	  * @param OutRemovedHandles Handles which were removed since the last update.
	  * @return The number of changes which occurred.
	  */
	uint32 Update(FInternalHandleArryType* OutAddedHandles = nullptr, FInternalHandleArryType* OutRemovedHandles = nullptr)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_TManagedStorage_Update);

		Traits::CheckIsManagerThread(ManagerThreadId);

		uint32 NumChanges = 0;

		// Add new queued items.
		if (OutAddedHandles)
		{
			FAddRequest AddRequest;
			while (AddRequests.Dequeue(AddRequest))
			{
				OutAddedHandles->Add(AddImpl(AddRequest.Handle, MoveTemp(AddRequest.Data)));
				++NumChanges;
			}
		}
		else
		{
			FAddRequest AddRequest;
			while (AddRequests.Dequeue(AddRequest))
			{
				AddImpl(AddRequest.Handle, MoveTemp(AddRequest.Data));
				++NumChanges;
			}
		}

		// Remove queued items.
		if (OutRemovedHandles)
		{
			FRemoveRequest RemoveRequest;
			while (RemoveRequests.Dequeue(RemoveRequest))
			{
				OutRemovedHandles->Add(RemoveImpl(RemoveRequest.Handle, RemoveRequest.OnComplete));
				++NumChanges;
			}
		}
		else
		{
			FRemoveRequest RemoveRequest;
			while (RemoveRequests.Dequeue(RemoveRequest))
			{
				RemoveImpl(RemoveRequest.Handle, RemoveRequest.OnComplete);
				++NumChanges;
			}
		}

		return NumChanges;
	}

	 /**
	  * Requests to add a resource.
	  * 
	  * Thread safe.
	  *
	  * @param Data The data to be added and managed.
	  * @return A new handle for the added resource.
	  */
	FExternalHandle Add(ElementType&& Data)
	{
		// Assign new handle for data.
		FExternalHandle Handle(FExternalHandle::GenerateNewHandle);
		AddRequests.Enqueue(FAddRequest{Handle, MoveTemp(Data)});
		return Handle;
	}

	/**
	 * Requests to remove a resource.
	 * 
	 * Thread safe.
	 *
	 * @param Handle The handle for the resource to be removed.
	 * @param OnComplete A callback to be signaled once the resource has been destroyed.
	 */
	void Remove(const FExternalHandle Handle, FManagedStorageOnRemoveComplete&& OnRemoveComplete = FManagedStorageOnRemoveComplete())
	{
		RemoveRequests.Enqueue(FRemoveRequest{Handle, MoveTemp(OnRemoveComplete)});
	}

	/**
	 * Removes a resource directly using its internal handle.
	 * 
	 * NOT thread safe.
	 *
	 * @param Handle The handle for the resource to be removed.
	 */
	void Remove(const FInternalHandle Handle)
	{
		RemoveImpl(Handle.GetExternalHandle(), nullptr);
	}

public:
	/*
	 * STORAGE ACCESS
	 * 
	 * NOT thread safe.
	 */

	inline int32 Num() const
	{
		CheckThreadForStorageAccess();
		return Storage.Num();
	}

	inline bool IsEmpty() const
	{
		CheckThreadForStorageAccess();
		return Storage.IsEmpty();
	}

	inline ElementType* Find(FExternalHandle Handle)
	{
		CheckThreadForStorageAccess();

		// When using an external handle the internal index will be valid if present.
		if (int32* InternalIndex = StorageHandleIndex.Find(Handle))
		{
			FStorageEntry& Entry = Storage[*InternalIndex];
			return &Entry.Data;
		}

		return nullptr;
	}

	inline const ElementType* Find(FExternalHandle Handle) const
	{
		return const_cast<TManagedStorage*>(this)->Find(Handle);
	}

	inline ElementType* Find(FInternalHandle Handle)
	{
		CheckThreadForStorageAccess();

		// When using an internal handle the index must be verified in case the handle is stale.
		const int32 InternalIndex = Handle.GetInternalIndex();
		if (Storage.IsValidIndex(InternalIndex))
		{
			// Check that the internal handle matches the expected value in case
			// the slot in the array was reused.
			FStorageEntry& Entry = Storage[InternalIndex];
			return Entry.InternalHandle == Handle ? &Entry.Data : nullptr;
		}

		return nullptr;
	}

	inline const ElementType* Find(FInternalHandle Handle) const
	{
		return const_cast<TManagedStorage*>(this)->Find(Handle);
	}

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 * 
	 * NOT thread safe.
	 */
	template <bool bConst>
	class TBaseRangeForIterator
	{
	private:
		using TInternalIterator = typename FStorageType::TRangedForIterator;
		using StorageIteratorType = std::conditional_t<bConst, typename FStorageType::TRangedForConstIterator, typename FStorageType::TRangedForIterator>;
		using InternalElementType = std::conditional_t<bConst, const ElementType, ElementType>;
		using ItElementType = TPair<FInternalHandle, InternalElementType&>;

	public:
		explicit TBaseRangeForIterator(StorageIteratorType InStorageIterator)
			: StorageIterator(InStorageIterator)
		{
			if (StorageIterator)
			{
				ElementAccess.Emplace(ItElementType{StorageIterator->InternalHandle, StorageIterator->Data});
			}
		}

		inline TBaseRangeForIterator& operator++()
		{
			++StorageIterator;
			if (StorageIterator)
			{
				ElementAccess.Emplace(ItElementType{StorageIterator->InternalHandle, StorageIterator->Data});
			}
			return *this;
		}

		inline bool operator==(const TBaseRangeForIterator& Rhs) const { return StorageIterator == Rhs.StorageIterator; }
		inline bool operator!=(const TBaseRangeForIterator& Rhs) const { return StorageIterator != Rhs.StorageIterator; }

		/** conversion to "bool" returning true if the iterator is valid. */
		inline explicit operator bool() const
		{
			return !!StorageIterator;
		}

		/** inverse of the "bool" operator */
		inline bool operator !() const
		{
			return !(bool)*this;
		}

		inline ItElementType& operator*() const { return *ElementAccess; }
		inline ItElementType* operator->() const { return &*ElementAccess; }

	private:
		StorageIteratorType StorageIterator;
		mutable TOptional<ItElementType> ElementAccess;
	};

	using TRangedForIterator = TBaseRangeForIterator<false>;
	using TRangedForConstIterator = TBaseRangeForIterator<true>;

	inline TRangedForIterator begin()
	{
		CheckThreadForStorageAccess();
		return TRangedForIterator(Storage.begin());
	}

	inline TRangedForConstIterator begin() const
	{
		CheckThreadForStorageAccess();
		return TRangedForConstIterator(Storage.begin());
	}

	inline TRangedForIterator end()
	{
		CheckThreadForStorageAccess();
		return TRangedForIterator(Storage.end());
	}

	inline TRangedForConstIterator end() const
	{
		CheckThreadForStorageAccess();
		return TRangedForConstIterator(Storage.end());
	}

private:
	inline void CheckThreadForStorageAccess() const
	{
		if constexpr (Traits::bStorageAccessThreadChecksEnabled)
		{
			Traits::CheckIsManagerThread(ManagerThreadId);
		}
	}

	FInternalHandle AddImpl(const FExternalHandle Handle, ElementType&& Data)
	{
		FSparseArrayAllocationInfo Allocation = Storage.AddUninitialized();
		FInternalHandle InternalHandle(FInternalHandle::GenerateNewHandle, Handle, Allocation.Index);
		new(Allocation) FStorageEntry{ InternalHandle, MoveTemp(Data)};
		StorageHandleIndex.Add(Handle, Allocation.Index);
		return InternalHandle;
	}

	FInternalHandle RemoveImpl(const FExternalHandle Handle, const FManagedStorageOnRemoveComplete& OnComplete)
	{
		int32 FoundIndex = INDEX_NONE;
		FInternalHandle InternalHandle;
		if (StorageHandleIndex.RemoveAndCopyValue(Handle, FoundIndex) && Storage.IsValidIndex(FoundIndex))
		{
			InternalHandle = Storage[FoundIndex].InternalHandle;
			Storage.RemoveAt(FoundIndex);
		}

		if (OnComplete)
		{
			OnComplete();
		}

		return InternalHandle;
	}
};

/* UE::EventLoop */ }
