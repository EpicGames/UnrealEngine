// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::EventLoop {

class FEventLoopModule : public IModuleInterface
{
public:
	// IModuleInterface

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 * Overloaded to allow the default subsystem a chance to load
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 * Overloaded to shut down all loaded online subsystems
	 */
	virtual void ShutdownModule() override;

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return	Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
};

/* UE::EventLoop */ }
