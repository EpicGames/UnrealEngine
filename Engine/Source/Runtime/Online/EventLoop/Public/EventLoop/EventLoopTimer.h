// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "EventLoop/EventLoopManagedStorage.h"
#include "Templates/Function.h"
#include "Misc/ScopeExit.h"
#include "Misc/Timespan.h"
#include "Stats/Stats.h"

namespace UE::EventLoop {

struct FTimerHandleTraits
{
	static constexpr TCHAR Name[] = TEXT("TimerHandle");
};

using FTimerCallback = TUniqueFunction<void()>;

using FOnTimerCleared = FManagedStorageOnRemoveComplete;

using FTimerHandle = TResourceHandle<FTimerHandleTraits>;

enum class ETimerStatus : uint8
{
	/**
	 * Timer has been queued, but has not started running yet.
	 * Timer will become active the next time the timer manager is ticked.
	 */
	Pending,
	/**
	 * Timer is active and will be triggered once its expiration elapses in Tick.
	 */
	Active,
	/**
	 * Timer will be removed during the next tick.
	 */
	PendingRemoval,
	/**
	 * Timer will be rescheduled during the next tick.
	 */
	PendingReschedule,
	/**
	 * The timer has elapsed and its callback is executing.
	 */
	Executing,
};

struct FTimerData
{
	FTimerData(FTimerCallback&& InCallback, TOptional<FTimespan> InFirstDelay, FTimespan InRate, bool bInRepeat)
		: Callback(MoveTemp(InCallback))
		, FirstDelay(InFirstDelay)
		, Rate(InRate)
		, bRepeat(bInRepeat)
	{
	}

	/** Holds the callback to call. */
	FTimerCallback Callback;

	/** Time to delay the first execution of the timer. Relevant to looping timers. */
	TOptional<FTimespan> FirstDelay;

	/** Time between set and fire, or repeat frequency if looping. */
	FTimespan Rate;

	/** When the timer is active, the expiration is the absolute time relative to the Timer Manager at which the timer should be fired. */
	FTimespan Expiration;

	ETimerStatus Status = ETimerStatus::Pending;

	/** If true, this timer will repeat indefinitely. Otherwise, it will be destroyed when it expires. */
	uint8 bRepeat : 1;

	FTimerData() = default;

	// Movable only
	FTimerData(FTimerData&&) = default;
	FTimerData(const FTimerData&) = delete;
	FTimerData& operator=(FTimerData&&) = default;
	FTimerData& operator=(const FTimerData&) = delete;
};

struct FTimerManagerTraitsBase
{
	static inline void CheckIsManagerThread(bool bIsManagerThread)
	{
		check(bIsManagerThread);
	}
};

/*
 * Default traits for timer manager. Traits are used to implement functionality which can be
 * mocked to allow testing the class.
 *
 * In most cases the default traits can be used without modification.
 */
struct FTimerManagerDefaultTraits : public FTimerManagerTraitsBase
{
	/**
	 * The allocator to use for the timer heap.
	 */
	using FTimerHeapAllocatorType = TInlineAllocator<32>;

	/**
	 * The allocator to use for timers waiting to be rescheduled.
	 */
	using FTimerRepeatAllocatorType = TInlineAllocator<32>;

	/**
	 * Default traits used for timer storage.
	 */
	struct FStorageTraits : public FManagedStorageDefaultTraits
	{
		using FExternalHandle = FTimerHandle;
	};
};

/**
 * Timer manager for posting timers to an event loop.
 * 
 * The event loop timer manager provides the guarantee that its timers will never fire early while
 * only progressing its view of time based on the delta passed to its Tick method. To provide this
 * guarantee, timers may fire later than requested which is by design.
 * 
 * Adding and removing timers are thread-safe, however the timer callback and clear callbacks will
 * both be called from within the Tick method, which may be a different thread. It is up to the
 * user to handle thread safety within their callbacks.
 */
template <typename Traits = FTimerManagerDefaultTraits>
class TTimerManager final : public FNoncopyable
{
public:
	using FStorageType = TManagedStorage<FTimerData, typename Traits::FStorageTraits>;
	using FTimerHandle = typename FStorageType::FExternalHandle;
	using FInternalHandle = typename FStorageType::FInternalHandle;
	using FInternalHandleArryType = typename FStorageType::FInternalHandleArryType;

public:
	/**
	 * Initialize the timer manager and its storage.
	 *
	 * NOT thread safe.
	 */
	void Init()
	{
		Storage.Init();
	}

	/**
	 * Accumulate the timer managers elapsed time and fire timers which have elapsed.
	 *
	 * NOT thread safe.
	 * 
	 * @param DeltaTime The amount of time which has elapsed since the last call to Tick.
	 */
	void Tick(FTimespan DeltaTime);

	/**
	 * Retrieve the remaining time since the last call to Tick before the next timer will fire.
	 *
	 * NOT thread safe.
	 * 
	 * @param OutTimeout Time amount of time relative to TimerManagers' internal time until the next valid timer will run.
	 * @return true if a valid timer is currently set.
	 */
	inline bool GetNextTimeout(FTimespan& OutTimeout) const
	{
		if (!ActiveTimerHeap.IsEmpty())
		{
			FInternalHandle TopHandle = ActiveTimerHeap.HeapTop();
			const FTimerData* TimerData = Storage.Find(TopHandle);
			OutTimeout = TimerData->Expiration - InternalTime;
			return true;
		}

		return false;
	}

	/**
	 * Returns the total number of timers. Includes active timers and timers waiting to be rescheduled.
	 *
	 * NOT thread safe.
	 * 
	 * @return the total number of timers.
	 */
	inline uint32 GetNumTimers() const
	{
		return Storage.Num();
	}

	/**
	 * Set a new timer. The timer callback will be triggered from within the call to Tick,
	 * which may occur on a different thread from the one which set the timer.
	 * 
	 * Thread safe.
	 *
	 * @param Callback Callback to call when timer fires.
	 * @param InRate The amount of time between set and firing.
	 * @param InbRepeat true to keep firing at Rate intervals, false to fire only once.
	 * @param InFirstDelay The time for the first iteration of a looping timer.
	 * @return handle to the registered timer.
	 */
	inline FTimerHandle SetTimer(FTimerCallback&& Callback, FTimespan InRate, bool InbRepeat = false, TOptional<FTimespan> InFirstDelay = TOptional<FTimespan>())
	{
		// Callback must be set.
		const bool bValidCallback = Callback.operator bool();
		// Rate cannot be less than zero.
		const bool bValidRate = InRate >= FTimespan::Zero();
		// First delay is only valid when repeating.
		const bool bFirstDelaySetValid = !InFirstDelay.IsSet() || InbRepeat;
		// First delay cannot be less than zero.
		const bool bFirstDelayValidTime = !InFirstDelay.IsSet() || (*InFirstDelay >= FTimespan::Zero());

		if (!(bValidCallback && bValidRate && bFirstDelaySetValid && bFirstDelayValidTime))
		{
			return FTimerHandle();
		}

		return Storage.Add(FTimerData(MoveTemp(Callback), InFirstDelay, InRate, InbRepeat));
	}

	/**
	* Clears a previously set timer.
	* 
	* Thread safe.
	*
	* @param InHandle The handle of the timer to clear.
	* @param OnTimerCleared Callback to be fired when the timer has been removed. The callback will
	*                       be fired within the Tick method which may be a different thread from
	*                       which the timer clear was requested.
	*/
	inline void ClearTimer(FTimerHandle& InHandle, FOnTimerCleared&& OnTimerCleared = FOnTimerCleared())
	{
		// If ClearTimer is run from the manager thread and the timer manager is currently ticking,
		// mark the timer as pending removal so that it will not be run.
		if (Storage.IsManagerThread() && bIsTicking)
		{
			if (FTimerData* TimerData = Storage.Find(InHandle))
			{
				TimerData->Status = ETimerStatus::PendingRemoval;
			}
		}

		// Queue removal request.
		Storage.Remove(InHandle, MoveTemp(OnTimerCleared));

		// Invalidate the handle.
		InHandle.Invalidate();
	}

	/**
	 * Returns whether any timers are waiting to be rescheduled.
	 *
	 * NOT thread safe.
	 * 
	 * @return true if a valid timer is waiting to be rescheduled.
	 */
	inline bool HasPendingRepeatTimer() const
	{
		return !PendingRepeatTimers.IsEmpty();
	}

private:
	struct FTimerHeapOrder
	{
		explicit FTimerHeapOrder(const FStorageType& InStorage)
			: Storage(InStorage)
			, NumTimers(InStorage.Num())
		{
		}

		bool operator()(FInternalHandle LhsHandle, FInternalHandle RhsHandle) const
		{
			const FTimerData* LhsData = Storage.Find(LhsHandle);
			const FTimerData* RhsData = Storage.Find(RhsHandle);

			// Move to top if empty.
			if (LhsData == nullptr)
			{
				return true;
			}

			// Move to top if empty.
			if (RhsData == nullptr)
			{
				return false;
			}

			return LhsData->Expiration < RhsData->Expiration;
		}

		const FStorageType& Storage;
		int32 NumTimers;
	};

	/** Storage for timer entries. */
	FStorageType Storage;
	/** Heap of actively running timers. */
	TArray<FInternalHandle, typename Traits::FTimerHeapAllocatorType> ActiveTimerHeap;
	/** Timers which will be rescheduled on the next call to Tick. */
	TArray<FInternalHandle, typename Traits::FTimerRepeatAllocatorType> PendingRepeatTimers;
	/** An internally consistent clock. Advances during ticking. */
	FTimespan InternalTime;
	/** Track whether the timer manager is in the process of ticking. */
	bool bIsTicking = false;
};

template <typename Traits>
void TTimerManager<Traits>::Tick(FTimespan DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_TTimerManager_Tick);

	Traits::CheckIsManagerThread(Storage.IsManagerThread());

	bIsTicking = true;
	ON_SCOPE_EXIT{ bIsTicking = false; };

	// Progress internal time.
	InternalTime += DeltaTime;

	// Process queued actions.
	FInternalHandleArryType AddedHandles;
	FInternalHandleArryType RemovedHandles;
	Storage.Update(&AddedHandles, &RemovedHandles);

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_TTimerManager_Tick_AddTimers);

		// Activate new timers.
		// Timers are always added on the next call to Tick regardless of thread to provide the
		// guarantee that a timer may fire late, but never early.
		for (FInternalHandle InternalHandle : AddedHandles)
		{
			// Timer data may not be found if the timer was added and removed before Tick was called.
			if (FTimerData* TimerData = Storage.Find(InternalHandle))
			{
				const FTimespan EffectiveInitialRate = TimerData->FirstDelay ? *TimerData->FirstDelay : TimerData->Rate;
				TimerData->Expiration = InternalTime + EffectiveInitialRate;
				TimerData->Status = ETimerStatus::Active;
				ActiveTimerHeap.HeapPush(InternalHandle, FTimerHeapOrder(Storage));
			}
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_TTimerManager_Tick_RemoveTimers);

		// Clear removed timers.
		for (FInternalHandle InternalHandle : RemovedHandles)
		{
			// Index may not be found if the timer was added and removed before Tick was called.
			int32 InternalHandleIndex = ActiveTimerHeap.Find(InternalHandle);
			if (InternalHandleIndex != INDEX_NONE)
			{
				ActiveTimerHeap.HeapRemoveAt(InternalHandleIndex, FTimerHeapOrder(Storage), EAllowShrinking::No);
			}
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_TTimerManager_Tick_RescheduleTimers);

		// Schedule timers waiting to be repeated.
		for (FInternalHandle InternalHandle : PendingRepeatTimers)
		{
			// Index may not be found if the timer was removed before Tick was called.
			if (FTimerData* TimerData = Storage.Find(InternalHandle))
			{
				TimerData->Expiration = InternalTime + TimerData->Rate;
				TimerData->Status = ETimerStatus::Active;
				ActiveTimerHeap.HeapPush(InternalHandle, FTimerHeapOrder(Storage));
			}
		}
		PendingRepeatTimers.Empty();
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_EventLoop_TTimerManager_Tick_RunTimers);

		while (ActiveTimerHeap.Num() > 0)
		{
			FInternalHandle TopHandle = ActiveTimerHeap.HeapTop();
			FTimerData* TimerData = Storage.Find(TopHandle);

			// TimerHeap is now resolved with Storage. All entries are valid.
			check(TimerData);

			// Pop from heap and continue if timer was removed during iteration.
			// The timer data will be removed on the next call to Update.
			if (TimerData->Status == ETimerStatus::PendingRemoval)
			{
				FInternalHandle TimerHandle;
				ActiveTimerHeap.HeapPop(TimerHandle, FTimerHeapOrder(Storage), EAllowShrinking::No);
				continue;
			}

			if (InternalTime >= TimerData->Expiration)
			{
				FInternalHandle TimerHandle;
				ActiveTimerHeap.HeapPop(TimerHandle, FTimerHeapOrder(Storage), EAllowShrinking::No);

				// Fire user timer callback.
				TimerData->Status = ETimerStatus::Executing;
				TimerData->Callback();

				// Reschedule repeating timers.
				if (TimerData->bRepeat)
				{
					TimerData->Status = ETimerStatus::PendingReschedule;
					PendingRepeatTimers.Add(TimerHandle);
				}
				else
				{
					// Timer has been fired and is non-repeating. Remove the timer.
					Storage.Remove(TopHandle);
				}
			}
			else
			{
				break;
			}
		}
	}
}

/* UE::EventLoop */ }
