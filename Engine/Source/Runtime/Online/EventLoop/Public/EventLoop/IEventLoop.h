// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EventLoop/EventLoopTimer.h"
#include "Misc/Timespan.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

namespace UE::EventLoop {

using FAsyncTask = TUniqueFunction<void()>;
using FOnShutdownComplete = TUniqueFunction<void()>;

class IEventLoop
{
public:
	virtual ~IEventLoop() = default;

	/**
	 * Initialize the event loop.
	 * 
	 * NOT thread safe.
	 */
	virtual bool Init() = 0;

	/**
	 * Signals a shutdown request to the event loop. May be signaled from any thread.
	 * Once called, RunOnce will return false after completing its current iteration.
	 *
	 * Thread safe.
	 */
	virtual void RequestShutdown(FOnShutdownComplete&& OnShutdownComplete = FOnShutdownComplete()) = 0;

	/**
	 * Set a new timer. The timer callback will be triggered from within the call to RunOnce,
	 * which may occur on a different thread from the one which set the timer.
	 * 
	 * Thread safe.
	 *
	 * @param Callback Callback to call when timer fires.
	 * @param InRate The amount of time between set and firing.
	 * @param InbRepeat true to keep firing at Rate intervals, false to fire only once.
	 * @param InFirstDelay The time for the first iteration of a looping timer.
	 * @return handle to the registered timer.
	 */
	virtual FTimerHandle SetTimer(FTimerCallback&& Callback, FTimespan InRate, bool InbRepeat = false, TOptional<FTimespan> InFirstDelay = TOptional<FTimespan>()) = 0;

	/**
	* Clears a previously set timer.
	* 
	* Thread safe.
	*
	* @param InHandle The handle of the timer to clear.
	* @param OnTimerCleared Callback to be fired when the timer has been removed. The callback will
	*                       be fired from within the event loops RunOnce method.
	*/
	virtual void ClearTimer(FTimerHandle& InHandle, FOnTimerCleared&& OnTimerCleared = FOnTimerCleared()) = 0;

	/**
	* Post a task to be run by the event loop. Queues the task to be run and signals the event loop to wake.
	* 
	* Thread safe.
	*
	* @param Task The task to be run.
	*/
	virtual void PostAsyncTask(FAsyncTask&& Task) = 0;

	/*
	 * Run the event loop until shutdown is called.
	 * The event loop will wake up when a timer needs to run or an async task has been posted.
	 * 
	 * NOT thread safe.
	 */
	virtual void Run() = 0;

	/*
	 * Run one iteration of the event loop.
	 * 
	 * NOT thread safe.
	 * 
	 * @param WaitTime Maximum amount of time to wait for events.
	 * @return true if RunOnce should be called again.
	 */
	virtual bool RunOnce(FTimespan WaitTime) = 0;

	/*
	 * Return the event loops current loop time.
	 * 
	 * NOT thread safe.
	 * 
	 * @return the time the event loop last woke up.
	 */
	virtual FTimespan GetLoopTime() const = 0;
};

/* UE::EventLoop */ }
