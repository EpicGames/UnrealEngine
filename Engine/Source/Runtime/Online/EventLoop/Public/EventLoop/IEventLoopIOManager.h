// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EventLoop/EventLoopHandle.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Timespan.h"

namespace UE::EventLoop {

class IEventLoop;

struct FIORequestHandleTraits
{
	static constexpr auto Name = TEXT("EventLoopIORequestHandle");
};
using FIORequestHandle = TResourceHandle<FIORequestHandleTraits>;

enum class EIOFlags : uint32
{
	None = 0,
	Read = 1 << 0,
	Write = 1 << 1,
};
ENUM_CLASS_FLAGS(EIOFlags);

class IIOManager : public FNoncopyable
{
public:

	/**
	 * Parameters for starting an IO manager. Derived classes are expected to implement the
	 * structure with any needed values. Will be passed from the event loop constructor into the
	 * IO manager.
	 */
	struct FParams
	{
	};

	/**
	 * IIOManager(IEventLoop& EventLoop, FParams&& Params);
	 */

	virtual ~IIOManager() = default;


	/**
	 * Initialize the IO manager. Called from within IEventLoop::Init.
	 *
	 * @return true if successfully initialized.
	 */
	virtual bool Init() = 0;

	/**
	 * Cleanup any resources and prepare for shutdown. Called from IEventLoop::RunOnce when the
	 * event loop has received a shutdown request.
	 */
	virtual void Shutdown() = 0;

	/**
	 * Notify the IO manager to interrupt a waiting call to Poll. Used to wake the request
	 * manager when activity is required.
	 * 
	 * Thread safe.
	 */
	virtual void Notify() = 0;

	/**
	 * Poll the request manager for activity.
	 * 
	 * @param WaitTime The maximum amount of time to wait for request activity.
	 */
	virtual void Poll(FTimespan WaitTime) = 0;
};

/* UE::EventLoop */ }
