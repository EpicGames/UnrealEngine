// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class EventLoopUnitTests : TestModuleRules
{
    static EventLoopUnitTests()
    {
		TestMetadata = new Metadata();
        TestMetadata.TestName = "EventLoop";
        TestMetadata.TestShortName = "EventLoop";
    }

    public EventLoopUnitTests(ReadOnlyTargetRules Target) : base(Target)
	{
		OptimizeCode = CodeOptimization.Never;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"EventLoop",
				"Sockets"
			});
	}
}