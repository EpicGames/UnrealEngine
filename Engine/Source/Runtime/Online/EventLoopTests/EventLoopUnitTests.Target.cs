// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class EventLoopUnitTestsTarget : TestTargetRules
{
	public EventLoopUnitTestsTarget(TargetInfo Target) : base(Target)
	{
		bCompileAgainstCoreUObject = true;
		bCompileAgainstApplicationCore = true;

		// Optionally add global definitions for Catch2 benchmarking etc.
		GlobalDefinitions.Add("CATCH_CONFIG_ENABLE_BENCHMARKING=1");
	}
}