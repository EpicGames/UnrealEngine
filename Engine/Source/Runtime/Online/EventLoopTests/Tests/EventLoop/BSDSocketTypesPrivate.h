// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EventLoop/BSDSocket/BSDSocketTypes.h"

#if PLATFORM_HAS_BSD_SOCKETS

#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
#include "Windows/AllowWindowsPlatformTypes.h"

#include <winsock2.h>
#include <ws2tcpip.h>

typedef int32 SOCKLEN;

#include "Windows/HideWindowsPlatformTypes.h"
#else // PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
#if HAS_EVENTLOOP_PLATFORM_BSD_SOCKET_HEADER
#include COMPILED_PLATFORM_HEADER(BSDSocketTypesPrivate.h)
#else // HAS_EVENTLOOP_PLATFORM_BSD_SOCKET_HEADER
#include <unistd.h>
#include <sys/socket.h>
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL
#include <fcntl.h>
#include <sys/types.h>
#include <sys/ioctl.h>
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_IOCTL
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL
#include <poll.h>
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_POLL
#include <netinet/in.h>
#include <arpa/inet.h>
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_GETHOSTNAME
#include <netdb.h>
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_GETHOSTNAME
#if PLATFORM_HAS_BSD_SOCKET_FEATURE_NODELAY
#include <netinet/tcp.h>
#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_NODELAY

#define ioctlsocket ioctl
#endif // HAS_EVENTLOOP_PLATFORM_BSD_SOCKET_HEADER

#define SOCKET_ERROR -1
#define INVALID_SOCKET -1

typedef socklen_t SOCKLEN;
typedef int32 SOCKET;
typedef sockaddr_in SOCKADDR_IN;
typedef struct timeval TIMEVAL;

inline int32 closesocket(SOCKET Socket)
{
	shutdown(Socket, SHUT_RDWR); // gracefully shutdown if connected
	return close(Socket);
}

#endif // PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
#endif // PLATFORM_HAS_BSD_SOCKETS
