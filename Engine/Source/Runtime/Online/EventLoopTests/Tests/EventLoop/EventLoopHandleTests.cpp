// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoop/EventLoopHandle.h"

#include "TestHarness.h"

namespace UE::EventLoop
{
	struct FTestHandleTraits
	{
		static constexpr auto Name = TEXT("test");
	};

	TEST_CASE("EventLoop::EventLoopHandle::Invalid Handle", "[Online][EventLoop][Smoke]")
	{
		TResourceHandle<FTestHandleTraits> Handle;
		CHECK(!Handle.IsValid());
	}

	TEST_CASE("EventLoop::EventLoopHandle::Valid Handle", "[Online][EventLoop][Smoke]")
	{
		TResourceHandle<FTestHandleTraits> Handle(TResourceHandle<FTestHandleTraits>::GenerateNewHandle);
		CHECK(Handle.IsValid());
	}

	struct FCount1HandleTraits
	{
		static constexpr auto Name = TEXT("count1");
	};

	struct FCount2HandleTraits
	{
		static constexpr auto Name = TEXT("count2");
	};

	TEST_CASE("EventLoop::EventLoopHandle::Counting", "[Online][EventLoop][Smoke]")
	{
		TResourceHandle<FCount1HandleTraits> Handle1(TResourceHandle<FCount1HandleTraits>::GenerateNewHandle);
		TResourceHandle<FCount1HandleTraits> Handle2(TResourceHandle<FCount1HandleTraits>::GenerateNewHandle);
		TResourceHandle<FCount2HandleTraits> Handle3(TResourceHandle<FCount2HandleTraits>::GenerateNewHandle);
		FString ExpectedString;

		// Check that handles count as expected.
		ExpectedString = FString::Printf(TEXT("%s:1"), FCount1HandleTraits::Name);
		CHECK(Handle1.ToString() == ExpectedString);

		ExpectedString = FString::Printf(TEXT("%s:2"), FCount1HandleTraits::Name);
		CHECK(Handle2.ToString() == ExpectedString);

		// A handle using a different traits time uses its own counter.
		ExpectedString = FString::Printf(TEXT("%s:1"), FCount2HandleTraits::Name);
		CHECK(Handle3.ToString() == ExpectedString);
	}
} // UE::EventLoop
