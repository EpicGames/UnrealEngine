// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#if PLATFORM_HAS_BSD_SOCKETS

#include "BSDSocketTypesPrivate.h"
#include "EventLoop/IEventLoop.h"
#include "EventLoop/BSDSocket/EventLoopIOManagerBSDSocketSelect.h"
#include "TestHarness.h"

#include "SocketSubsystem.h"

namespace UE::EventLoop
{
	struct FBSDSocketFuncs
	{
		static SOCKET Open(int32 Family, int32 Type, int32 Protocol)
		{
			return socket(Family, Type, Protocol);
		}

		static void Close(SOCKET Socket)
		{
			closesocket(Socket);
		}

		static bool SetSendBufferSize(SOCKET Socket, int32 Size)
		{
			SOCKLEN SizeSize = sizeof(int32);
			bool bOk = setsockopt(Socket, SOL_SOCKET, SO_SNDBUF, (char*)&Size, sizeof(int32)) == 0;

			// Read the value back in case the size was modified
			int32 NewSize;
			getsockopt(Socket, SOL_SOCKET, SO_SNDBUF, (char*)&NewSize, &SizeSize);

			return bOk && Size == NewSize;
		}

		static bool SetRecvBufferSize(SOCKET Socket, int32 Size)
		{
			SOCKLEN SizeSize = sizeof(int32);
			bool bOk = setsockopt(Socket, SOL_SOCKET, SO_RCVBUF, (char*)&Size, sizeof(int32)) == 0;

			// Read the value back in case the size was modified
			int32 NewSize;
			getsockopt(Socket, SOL_SOCKET, SO_RCVBUF, (char*)&NewSize, &SizeSize);

			return bOk && Size == NewSize;
		}

		static bool SetNonBlocking(SOCKET Socket, bool bIsNonBlocking)
		{
	#if PLATFORM_HAS_BSD_SOCKET_FEATURE_WINSOCKETS
			u_long Value = bIsNonBlocking ? true : false;
			return ioctlsocket(Socket, FIONBIO, &Value) == 0;
	#else
			int Flags = fcntl(Socket, F_GETFL, 0);
			//Set the flag or clear it, without destroying the other flags.
			Flags = bIsNonBlocking ? Flags | O_NONBLOCK : Flags ^ (Flags & O_NONBLOCK);
			int err = fcntl(Socket, F_SETFL, Flags);
			return (err == 0 ? true : false);
	#endif
		}

		static bool SetReusePort(SOCKET Socket, bool bIsReusePort)
		{
			int32 Resuse = bIsReusePort ? true : false;
			bool bOk = setsockopt(Socket, SOL_SOCKET, SO_BROADCAST, (char*)&Resuse, sizeof(int32)) == 0;
			bOk &= setsockopt(Socket, SOL_SOCKET, SO_REUSEADDR, (char*)&Resuse, sizeof(int32)) == 0;
			return bOk;
		}

		static bool BindLoopback(SOCKET Socket)
		{
			sockaddr_in Addr;
			FMemory::Memzero(&Addr, sizeof(Addr));
			Addr.sin_addr.s_addr = htonl(INADDR_LOOPBACK);
			Addr.sin_family = AF_INET;

			return Bind(Socket, Addr);
		}

		static bool Bind(SOCKET Socket, const sockaddr_in& Addr)
		{
			return bind(Socket, (const sockaddr*)&Addr, sizeof(sockaddr_in)) == 0;
		}

		static bool Listen(SOCKET Socket, int32 MaxBacklog)
		{
			return listen(Socket, MaxBacklog) == 0;
		}

		static bool GetAddress(SOCKET Socket, sockaddr_in& OutAddr)
		{
			SOCKLEN Size = sizeof(sockaddr_in);

			// Figure out what ip/port we are bound to
			return getsockname(Socket, (sockaddr*)&OutAddr, &Size) == 0;
		}

		static bool Connect(SOCKET Socket, const sockaddr_in& Addr)
		{
			SOCKLEN Size = sizeof(sockaddr_in);
			int32 Return = connect(Socket, (const sockaddr*)&Addr, Size);

			ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			check(SocketSubsystem);
			ESocketErrors Error = SocketSubsystem->TranslateErrorCode(Return);

			// EWOULDBLOCK is not an error, and EINPROGRESS is fine on initial connection as it may still be creating for nonblocking sockets
			return ((Error == SE_NO_ERROR) || (Error == SE_EWOULDBLOCK) || (Error == SE_EINPROGRESS));
		}

		static SOCKET Accept(SOCKET Socket)
		{
			return accept(Socket, NULL, NULL);
		}

		static bool Send(SOCKET Socket, const uint8* Data, int32 Count, int32& BytesSent)
		{
			BytesSent = send(Socket,(const char*)Data,Count,0);
			return BytesSent >= 0;
		}

		static bool Recv(SOCKET Socket, uint8* Data, int32 BufferSize, int32& BytesRead)
		{
			int SocketType = 0;
			SOCKLEN SocketTypeLength = sizeof(int32);
			if (getsockopt(Socket, SOL_SOCKET, SO_TYPE, (char*)&SocketType, &SocketTypeLength) == SOCKET_ERROR)
			{
				return false;
			}

			bool bSuccess = false;
			const bool bStreamSocket = (SocketType == SOCK_STREAM);
			BytesRead = recv(Socket, (char*)Data, BufferSize, 0);

			if (BytesRead >= 0)
			{
				// For Streaming sockets, 0 indicates a graceful failure
				bSuccess = !bStreamSocket || (BytesRead > 0);
			}
			else
			{
				// For Streaming sockets, don't treat SE_EWOULDBLOCK as an error
				ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
				check(SocketSubsystem);
				bSuccess = bStreamSocket && (SocketSubsystem->TranslateErrorCode(BytesRead) == SE_EWOULDBLOCK);
				BytesRead = 0;
			}

			return bSuccess;
		}

		static bool SendTo(SOCKET Socket, const uint8* Data, int32 Count, int32& BytesSent, const sockaddr_in& Addr)
		{
			SOCKLEN Size = sizeof(sockaddr_in);
			// Write the data and see how much was written
			BytesSent = sendto(Socket, (const char*)Data, Count, 0, (const sockaddr*)&(Addr), Size);
			return BytesSent >= 0;
		}

		static bool RecvFrom(SOCKET Socket, uint8* Data, int32 BufferSize, int32& BytesRead, sockaddr_in& Addr)
		{
			int SocketType = 0;
			SOCKLEN SocketTypeLength = sizeof(int32);
			if (getsockopt(Socket, SOL_SOCKET, SO_TYPE, (char*)&SocketType, &SocketTypeLength) == SOCKET_ERROR)
			{
				return false;
			}

			bool bSuccess = false;
			const bool bStreamSocket = (SocketType == SOCK_STREAM);
			SOCKLEN Size = sizeof(sockaddr_in);

			// Read into the buffer and set the source address
			BytesRead = recvfrom(Socket, (char*)Data, BufferSize, 0, (sockaddr*)&Addr, &Size);

			if (BytesRead >= 0)
			{
				// For Streaming sockets, 0 indicates a graceful failure
				bSuccess = !bStreamSocket || (BytesRead > 0);
			}
			else
			{
				ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
				check(SocketSubsystem);
				// For Streaming sockets, don't treat SE_EWOULDBLOCK as an error
				bSuccess = bStreamSocket && (SocketSubsystem->TranslateErrorCode(BytesRead) == SE_EWOULDBLOCK);
				BytesRead = 0;
			}

			return bSuccess;
		}
	};

	// Todo - remove eventloop parameter from IOManager.
	class FMockEventLoop : public IEventLoop
	{
	public:
		FMockEventLoop() = default;
		virtual ~FMockEventLoop() = default;
		virtual bool Init() override { return false; }
		virtual void RequestShutdown(FOnShutdownComplete&& OnShutdownComplete) override {}
		virtual FTimerHandle SetTimer(FTimerCallback&& Callback, FTimespan InRate, bool InbRepeat, TOptional<FTimespan> InFirstDelay) override { return FTimerHandle(); }
		virtual void ClearTimer(FTimerHandle& InHandle, FOnTimerCleared&& OnTimerCleared) override {}
		virtual void PostAsyncTask(FAsyncTask&& Task) override {}
		virtual void Run() override {}
		virtual bool RunOnce(FTimespan WaitTime) override { return false; }
		virtual FTimespan GetLoopTime() const override { return FTimespan(); }
	};

	TEST_CASE("IOManagerBSDSocketSelect", "[Online][EventLoop][IOManager][Smoke]")
	{
		FMockEventLoop MockEventLoop;
		FIOManagerBSDSocketSelect IOManager(MockEventLoop, FIOManagerBSDSocketSelect::FParams());

		REQUIRE(IOManager.Init());

		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		check(SocketSubsystem);

		SECTION("Notify tests")
		{
			double StartTime = 0;
			FTimespan Duration;

			// 1. Waiting while no events are received will wait for at least the requested duration.
			StartTime = FPlatformTime::Seconds();
			IOManager.Poll(FTimespan::FromMilliseconds(50));
			Duration = FTimespan::FromSeconds(FPlatformTime::Seconds() - StartTime);
			CHECK(Duration > FTimespan::FromMilliseconds(50));

			// 2. Notifying the IO manager will wake the loop up early.
			StartTime = FPlatformTime::Seconds();
			IOManager.Notify();
			IOManager.Poll(FTimespan::FromMilliseconds(50));
			Duration = FTimespan::FromSeconds(FPlatformTime::Seconds() - StartTime);
			CHECK(Duration < FTimespan::FromMilliseconds(1));
		}

		SECTION("Single socket tests")
		{
			// Size of buffer used for sending test data.
			constexpr uint64 WriteBufferSize = 128;
			TArray<uint8> WriteBuffer;
			WriteBuffer.AddDefaulted(WriteBufferSize);

			// Size of buffer used for receiving test data.
			constexpr uint64 ReadBufferSize = 128;
			TArray<uint8> ReadBuffer;
			ReadBuffer.AddDefaulted(ReadBufferSize);

			// Socket buffer sizes for send / recv
			constexpr int32 SocketSendBufferSize = 64;
			constexpr int32 SocketReceiveBufferSize = 64;

			// Create listen socket.
			SOCKET ListenSocket = FBSDSocketFuncs::Open(AF_INET, SOCK_STREAM, IPPROTO_TCP);
			REQUIRE(ListenSocket != INVALID_SOCKET);
			REQUIRE(FBSDSocketFuncs::SetSendBufferSize(ListenSocket, SocketSendBufferSize));
			REQUIRE(FBSDSocketFuncs::SetRecvBufferSize(ListenSocket, SocketReceiveBufferSize));
			REQUIRE(FBSDSocketFuncs::SetNonBlocking(ListenSocket, true));
			REQUIRE(FBSDSocketFuncs::BindLoopback(ListenSocket));
			REQUIRE(FBSDSocketFuncs::Listen(ListenSocket, 1));
			sockaddr_in HostAddr;
			FMemory::Memzero(&HostAddr, sizeof(HostAddr));
			REQUIRE(FBSDSocketFuncs::GetAddress(ListenSocket, HostAddr));

			// Create remote socket.
			SOCKET RemoteSocket1 = FBSDSocketFuncs::Open(AF_INET, SOCK_STREAM, IPPROTO_TCP);
			REQUIRE(RemoteSocket1 != INVALID_SOCKET);
			REQUIRE(FBSDSocketFuncs::SetSendBufferSize(RemoteSocket1, SocketSendBufferSize));
			REQUIRE(FBSDSocketFuncs::SetRecvBufferSize(RemoteSocket1, SocketReceiveBufferSize));
			REQUIRE(FBSDSocketFuncs::SetNonBlocking(RemoteSocket1, true));
			REQUIRE(FBSDSocketFuncs::Connect(RemoteSocket1, HostAddr));

			// Create local socket.
			ESocketErrors ErrorCode1 = SE_NO_ERROR;
			SOCKET LocalSocket1 = INVALID_SOCKET;
			do
			{
				LocalSocket1 = FBSDSocketFuncs::Accept(ListenSocket);
				ErrorCode1 = SocketSubsystem->GetLastErrorCode();
			}
			while (ErrorCode1 == SE_EWOULDBLOCK);
			REQUIRE(LocalSocket1 != INVALID_SOCKET);

			SECTION("Single waiting socket tests")
			{
				SECTION("Wait on socket readable")
				{
					SOCKET TriggeredSocket = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags = EIOFlags::None;
					uint32 CallbackTriggerCount = 0;

					FIORequestBSDSocket Request;
					Request.Socket = LocalSocket1;
					Request.Flags = EIOFlags::Read;
					Request.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket = Socket;
						TriggeredStatus = Status;
						TriggeredFlags = SignaledFlags;
						++CallbackTriggerCount;
					};

					FIORequestHandle RequestHandle = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request));
					CHECK(RequestHandle.IsValid());

					// 1. No events triggered when no data has arrived at the socket.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 0);

					// 2. Send data to the socket.
					int32 BytesSent = 0;
					CHECK(FBSDSocketFuncs::Send(RemoteSocket1, WriteBuffer.GetData(), 1, BytesSent));
					CHECK(BytesSent == 1);

					// 3. See that socket has a read event.
					TriggeredFlags = EIOFlags::None;
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 1);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags == EIOFlags::Read);

					// 4. See that socket still has a read event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 2);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags == EIOFlags::Read);

					// 5. Read the waiting data from the socket.
					int32 BytesRead = 0;
					CHECK(FBSDSocketFuncs::Recv(LocalSocket1, ReadBuffer.GetData(), ReadBufferSize, BytesRead));
					CHECK(BytesRead == 1);

					// 6. See that socket no longer has a read event.
					TriggeredFlags = EIOFlags::None;
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 2);
				}

				SECTION("Wait on socket writeable")
				{
					SOCKET TriggeredSocket = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags = EIOFlags::None;
					uint32 CallbackTriggerCount = 0;

					FIORequestBSDSocket Request;
					Request.Socket = LocalSocket1;
					Request.Flags = EIOFlags::Write;
					Request.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket = Socket;
						TriggeredStatus = Status;
						TriggeredFlags = SignaledFlags;
						++CallbackTriggerCount;
					};

					FIORequestHandle RequestHandle = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request));
					CHECK(RequestHandle.IsValid());

					// 1. See that socket has a write event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 1);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags == EIOFlags::Write);

					// 2. Write data to the socket until it returns an error indicating that it would block.
					int32 BytesSent = 0;
					int32 TotalBytesSent = 0;
					while (FBSDSocketFuncs::Send(LocalSocket1, WriteBuffer.GetData(), WriteBufferSize, BytesSent));

					// 3. See that socket no longer has a write event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 1);

					// 4. Read the data from the remote end of the connection.
					int32 BytesRead = 0;
					while (FBSDSocketFuncs::Recv(RemoteSocket1, ReadBuffer.GetData(), ReadBufferSize, BytesRead))
					{
						if (BytesRead == 0)
						{
							break;
						}
					}

					// 5. See that socket now has a write event.
					IOManager.Poll(FTimespan::FromMilliseconds(10));
					CHECK(CallbackTriggerCount == 2);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags == EIOFlags::Write);
				}

				SECTION("Receive both read and write events with one request")
				{
					SOCKET TriggeredSocket = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags = EIOFlags::None;
					uint32 CallbackTriggerCount = 0;

					FIORequestBSDSocket Request;
					Request.Socket = LocalSocket1;
					Request.Flags = EIOFlags::Read | EIOFlags::Write;
					Request.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket = Socket;
						TriggeredStatus = Status;
						TriggeredFlags = SignaledFlags;
						++CallbackTriggerCount;
					};

					FIORequestHandle RequestHandle = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request));
					CHECK(RequestHandle.IsValid());

					// 1. No events triggered when no data has arrived at the socket.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 1);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(EnumHasAnyFlags(TriggeredFlags, EIOFlags::Write));
					CHECK(!EnumHasAnyFlags(TriggeredFlags, EIOFlags::Read));

					// 2. Send data to the socket.
					int32 BytesSent = 0;
					CHECK(FBSDSocketFuncs::Send(RemoteSocket1, WriteBuffer.GetData(), 1, BytesSent));
					CHECK(BytesSent == 1);

					// 3. See that socket has a read event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 2);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(EnumHasAnyFlags(TriggeredFlags, EIOFlags::Write));
					CHECK(EnumHasAnyFlags(TriggeredFlags, EIOFlags::Read));
				}

				SECTION("Canceling a request stops signaling events")
				{
					SOCKET TriggeredSocket = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags = EIOFlags::None;
					uint32 CallbackTriggerCount = 0;

					FIORequestBSDSocket Request;
					Request.Socket = LocalSocket1;
					Request.Flags = EIOFlags::Read;
					Request.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket = Socket;
						TriggeredStatus = Status;
						TriggeredFlags = SignaledFlags;
						++CallbackTriggerCount;
					};

					FIORequestHandle RequestHandle = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request));
					CHECK(RequestHandle.IsValid());

					// 1. No events triggered when no data has arrived at the socket.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 0);

					// 2. Send data to the socket.
					int32 BytesSent = 0;
					CHECK(FBSDSocketFuncs::Send(RemoteSocket1, WriteBuffer.GetData(), 1, BytesSent));
					CHECK(BytesSent == 1);

					// 3. See that socket has a read event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 1);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags == EIOFlags::Read);

					// 4. Cancel the request.
					IOManager.GetIOAccess().DestroyIORequest(RequestHandle);

					// 5. See that event is not triggered.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 1);
				}

				SECTION("TCP socket remote end closing triggers writeable wakeup")
				{
					SOCKET TriggeredSocket = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags = EIOFlags::None;
					uint32 CallbackTriggerCount = 0;

					FIORequestBSDSocket Request;
					Request.Socket = LocalSocket1;
					Request.Flags = EIOFlags::Write;
					Request.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket = Socket;
						TriggeredStatus = Status;
						TriggeredFlags = SignaledFlags;
						++CallbackTriggerCount;
					};

					FIORequestHandle RequestHandle = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request));
					CHECK(RequestHandle.IsValid());

					// 1. See that socket has a write event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 1);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags == EIOFlags::Write);

					// 2. Write data to the socket until it returns an error indicating that it would block.
					int32 BytesSent = 0;
					while (FBSDSocketFuncs::Send(LocalSocket1, WriteBuffer.GetData(), WriteBufferSize, BytesSent));

					// 3. See that socket no longer has a write event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 1);

					// 4. Close the remote end of the connection.
					FBSDSocketFuncs::Close(RemoteSocket1);

					// 5. See that socket now has a write event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount == 2);
					CHECK(TriggeredSocket == LocalSocket1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags == EIOFlags::Write);
				}

				SECTION("More than one request for the same socket")
				{
					SOCKET TriggeredSocket = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags = EIOFlags::None;
					uint32 CallbackTriggerCount1 = 0;
					uint32 CallbackTriggerCount2 = 0;

					FIORequestBSDSocket Request1;
					Request1.Socket = LocalSocket1;
					Request1.Flags = EIOFlags::Read;
					Request1.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket = Socket;
						TriggeredStatus = Status;
						TriggeredFlags = SignaledFlags;
						++CallbackTriggerCount1;
					};

					FIORequestHandle RequestHandle1 = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request1));
					CHECK(RequestHandle1.IsValid());

					FIORequestBSDSocket Request2;
					Request2.Socket = LocalSocket1;
					Request2.Flags = EIOFlags::Write;
					Request2.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket = Socket;
						TriggeredStatus = Status;
						TriggeredFlags = SignaledFlags;
						++CallbackTriggerCount2;
					};

					FIORequestHandle RequestHandle2 = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request2));
					CHECK(RequestHandle2.IsValid());

					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 0);
					CHECK(CallbackTriggerCount2 == 1);
					CHECK(TriggeredStatus == ESocketIoRequestStatus::Invalid);
				}
			}

			SECTION("Multiple waiting TCP socket tests")
			{
				// Create remote socket.
				SOCKET RemoteSocket2 = FBSDSocketFuncs::Open(AF_INET, SOCK_STREAM, IPPROTO_TCP);
				REQUIRE(RemoteSocket2 != INVALID_SOCKET);
				REQUIRE(FBSDSocketFuncs::SetSendBufferSize(RemoteSocket2, SocketSendBufferSize));
				REQUIRE(FBSDSocketFuncs::SetRecvBufferSize(RemoteSocket2, SocketReceiveBufferSize));
				REQUIRE(FBSDSocketFuncs::SetNonBlocking(RemoteSocket2, true));
				REQUIRE(FBSDSocketFuncs::Connect(RemoteSocket2, HostAddr));

				// Create local socket.
				ESocketErrors ErrorCode2 = SE_NO_ERROR;
				SOCKET LocalSocket2 = INVALID_SOCKET;
				do
				{
					LocalSocket2 = FBSDSocketFuncs::Accept(ListenSocket);
					ErrorCode2 = SocketSubsystem->GetLastErrorCode();
				}
				while (ErrorCode2 == SE_EWOULDBLOCK);
				REQUIRE(LocalSocket2 != INVALID_SOCKET);

				SECTION("Triggering LocalSocket1 does not trigger LocalSocket2")
				{
					SOCKET TriggeredSocket1 = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus1 = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags1 = EIOFlags::None;
					uint32 CallbackTriggerCount1 = 0;
					FIORequestBSDSocket Request1;
					Request1.Socket = LocalSocket1;
					Request1.Flags = EIOFlags::Read;
					Request1.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket1 = Socket;
						TriggeredStatus1 = Status;
						TriggeredFlags1 = SignaledFlags;
						++CallbackTriggerCount1;
					};
					FIORequestHandle RequestHandle1 = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request1));
					CHECK(RequestHandle1.IsValid());

					SOCKET TriggeredSocket2 = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus2 = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags2 = EIOFlags::None;
					uint32 CallbackTriggerCount2 = 0;
					FIORequestBSDSocket Request2;
					Request2.Socket = LocalSocket2;
					Request2.Flags = EIOFlags::Read;
					Request2.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket2 = Socket;
						TriggeredStatus2 = Status;
						TriggeredFlags2 = SignaledFlags;
						++CallbackTriggerCount2;
					};
					FIORequestHandle RequestHandle2 = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request2));
					CHECK(RequestHandle2.IsValid());

					// 1. No events triggered when no data has arrived at the socket.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 0);
					CHECK(CallbackTriggerCount2 == 0);

					// 2. Send data to the socket.
					int32 BytesSent = 0;
					CHECK(FBSDSocketFuncs::Send(RemoteSocket1, WriteBuffer.GetData(), 1, BytesSent));
					CHECK(BytesSent == 1);

					// 3. See that socket has a read event.
					TriggeredFlags1 = EIOFlags::None;
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 1);
					CHECK(CallbackTriggerCount2 == 0);
					CHECK(TriggeredSocket1 == LocalSocket1);
					CHECK(TriggeredStatus1 == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags1 == EIOFlags::Read);

					// 4. See that socket still has a read event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 2);
					CHECK(CallbackTriggerCount2 == 0);
					CHECK(TriggeredSocket1 == LocalSocket1);
					CHECK(TriggeredStatus1 == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags1 == EIOFlags::Read);

					// 5. Read the waiting data from the socket.
					int32 BytesRead = 0;
					CHECK(FBSDSocketFuncs::Recv(LocalSocket1, ReadBuffer.GetData(), ReadBufferSize, BytesRead));
					CHECK(BytesRead == 1);

					// 6. See that socket no longer has a read event.
					TriggeredFlags1 = EIOFlags::None;
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 2);
					CHECK(CallbackTriggerCount2 == 0);
				}

				FBSDSocketFuncs::Close(RemoteSocket2);
				FBSDSocketFuncs::Close(LocalSocket2);
			}

			SECTION("Waiting on a TCP and UDP socket")
			{
				// Create local socket.
				SOCKET LocalSocket2 = FBSDSocketFuncs::Open(AF_INET, SOCK_DGRAM, IPPROTO_UDP);
				REQUIRE(LocalSocket2 != INVALID_SOCKET);

				REQUIRE(FBSDSocketFuncs::SetNonBlocking(LocalSocket2, true));
				REQUIRE(FBSDSocketFuncs::BindLoopback(LocalSocket2));
				sockaddr_in HostAddr2;
				FMemory::Memzero(&HostAddr2, sizeof(HostAddr2));
				REQUIRE(FBSDSocketFuncs::GetAddress(LocalSocket2, HostAddr2));

				// Create remote socket.
				SOCKET RemoteSocket2 = FBSDSocketFuncs::Open(AF_INET, SOCK_DGRAM, IPPROTO_UDP);
				REQUIRE(RemoteSocket2 != INVALID_SOCKET);
				REQUIRE(FBSDSocketFuncs::SetSendBufferSize(RemoteSocket2, SocketSendBufferSize));
				REQUIRE(FBSDSocketFuncs::SetRecvBufferSize(RemoteSocket2, SocketReceiveBufferSize));
				REQUIRE(FBSDSocketFuncs::SetNonBlocking(RemoteSocket2, true));

				SECTION("Recevie events for TCP and UDP socket types.")
				{
					SOCKET TriggeredSocket1 = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus1 = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags1 = EIOFlags::None;
					uint32 CallbackTriggerCount1 = 0;
					FIORequestBSDSocket Request1;
					Request1.Socket = LocalSocket1;
					Request1.Flags = EIOFlags::Read;
					Request1.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket1 = Socket;
						TriggeredStatus1 = Status;
						TriggeredFlags1 = SignaledFlags;
						++CallbackTriggerCount1;
					};
					FIORequestHandle RequestHandle1 = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request1));
					CHECK(RequestHandle1.IsValid());

					SOCKET TriggeredSocket2 = INVALID_SOCKET;
					ESocketIoRequestStatus TriggeredStatus2 = ESocketIoRequestStatus::Ok;
					EIOFlags TriggeredFlags2 = EIOFlags::None;
					uint32 CallbackTriggerCount2 = 0;
					FIORequestBSDSocket Request2;
					Request2.Socket = LocalSocket2;
					Request2.Flags = EIOFlags::Read;
					Request2.Callback = [&](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
						TriggeredSocket2 = Socket;
						TriggeredStatus2 = Status;
						TriggeredFlags2 = SignaledFlags;
						++CallbackTriggerCount2;
					};
					FIORequestHandle RequestHandle2 = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request2));
					CHECK(RequestHandle2.IsValid());

					// 1. No events triggered when no data has arrived at the socket.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 0);
					CHECK(CallbackTriggerCount2 == 0);

					// 2. Send data to the socket.
					int32 BytesSent = 0;
					CHECK(FBSDSocketFuncs::Send(RemoteSocket1, WriteBuffer.GetData(), 1, BytesSent));
					CHECK(BytesSent == 1);

					// 3. See that socket has a read event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 1);
					CHECK(CallbackTriggerCount2 == 0);
					CHECK(TriggeredSocket1 == LocalSocket1);
					CHECK(TriggeredStatus1 == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags1 == EIOFlags::Read);

					// 4. Send data to the UDP socket.
					BytesSent = 0;
					CHECK(FBSDSocketFuncs::SendTo(RemoteSocket2, WriteBuffer.GetData(), 1, BytesSent, HostAddr2));
					CHECK(BytesSent == 1);

					// 5. See that both sockets have a read event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 2);
					CHECK(CallbackTriggerCount2 == 1);
					CHECK(TriggeredSocket1 == LocalSocket1);
					CHECK(TriggeredStatus1 == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags1 == EIOFlags::Read);
					CHECK(TriggeredSocket2 == LocalSocket2);
					CHECK(TriggeredStatus2 == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags2 == EIOFlags::Read);

					// 5. Read the waiting data from the TCP socket.
					int32 BytesRead = 0;
					CHECK(FBSDSocketFuncs::Recv(LocalSocket1, ReadBuffer.GetData(), ReadBufferSize, BytesRead));
					CHECK(BytesRead == 1);

					// 6. See that the UDP socket still has a read event.
					IOManager.Poll(FTimespan::Zero());
					CHECK(CallbackTriggerCount1 == 2);
					CHECK(CallbackTriggerCount2 == 2);
					CHECK(TriggeredSocket2 == LocalSocket2);
					CHECK(TriggeredStatus2 == ESocketIoRequestStatus::Ok);
					CHECK(TriggeredFlags2 == EIOFlags::Read);
				}

				FBSDSocketFuncs::Close(RemoteSocket2);
				FBSDSocketFuncs::Close(LocalSocket2);
			}

			FBSDSocketFuncs::Close(ListenSocket);
			FBSDSocketFuncs::Close(RemoteSocket1);
			FBSDSocketFuncs::Close(LocalSocket1);
		}

		SECTION("Wait on too many sockets")
		{
			struct FCallbackData
			{
				SOCKET Socket = INVALID_SOCKET;
				ESocketIoRequestStatus Status = ESocketIoRequestStatus::Ok;
				EIOFlags SignaledFlags = EIOFlags::None;
			};

			TArray<SOCKET> Sockets;
			TArray<FCallbackData> CallbackData;

			SOCKET BaseSocket = FBSDSocketFuncs::Open(AF_INET, SOCK_DGRAM, IPPROTO_UDP);
			REQUIRE(BaseSocket != INVALID_SOCKET);
			REQUIRE(FBSDSocketFuncs::SetNonBlocking(BaseSocket, true));
			REQUIRE(FBSDSocketFuncs::SetReusePort(BaseSocket, true));
			REQUIRE(FBSDSocketFuncs::BindLoopback(BaseSocket));

			sockaddr_in BaseAddr;
			FMemory::Memzero(&BaseAddr, sizeof(BaseAddr));
			REQUIRE(FBSDSocketFuncs::GetAddress(BaseSocket, BaseAddr));

			const int32 MaxTestSockets = 10000;
			for (int32 index = 0; index < MaxTestSockets; ++index)
			{
				SOCKET Socket = FBSDSocketFuncs::Open(AF_INET, SOCK_DGRAM, IPPROTO_UDP);
				REQUIRE(Socket != INVALID_SOCKET);
				REQUIRE(FBSDSocketFuncs::SetNonBlocking(Socket, true));
				REQUIRE(FBSDSocketFuncs::SetReusePort(Socket, true));
				REQUIRE(FBSDSocketFuncs::Bind(Socket, BaseAddr));
				Sockets.Add(Socket);

				FIORequestBSDSocket Request;
				Request.Socket = Socket;
				Request.Flags = EIOFlags::Read;
				Request.Callback = [&CallbackData](SOCKET Socket, ESocketIoRequestStatus Status, EIOFlags SignaledFlags){
					CallbackData.Add({Socket, Status, SignaledFlags});
				};
				FIORequestHandle RequestHandle = IOManager.GetIOAccess().CreateSocketIORequest(MoveTemp(Request));
				CHECK(RequestHandle.IsValid());

				IOManager.Poll(FTimespan::Zero());

				if (!CallbackData.IsEmpty())
				{
					CHECK(CallbackData.Num() == 1);
					CHECK(CallbackData[0].Socket == Socket);
					CHECK(CallbackData[0].Status == ESocketIoRequestStatus::NoResources);
					CHECK(CallbackData[0].SignaledFlags == EIOFlags::None);
					break;
				}
			}

			// Check that a failure occurred.
			CHECK(!CallbackData.IsEmpty());

			for (SOCKET Socket : Sockets)
			{
				FBSDSocketFuncs::Close(Socket);
			}

			FBSDSocketFuncs::Close(BaseSocket);
		}

		IOManager.Shutdown();
	}

} // UE::EventLoop

#endif // PLATFORM_HAS_BSD_SOCKETS
