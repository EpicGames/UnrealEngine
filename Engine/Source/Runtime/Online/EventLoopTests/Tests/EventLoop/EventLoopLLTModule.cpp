// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Modules/ModuleManager.h"
#include "TestRunner.h"

namespace UE::EventLoop
{

TArray<FString> GetRequiredModules()
{
	static TArray<FString> Modules({
		FString("EventLoop"),
		FString("Sockets")
	});

	return Modules;
}

void InitializeEventLoopTests()
{
	for (const FString& ModuleName : GetRequiredModules())
	{
		FModuleManager::LoadModulePtr<IModuleInterface>(*ModuleName);
	}
}

void CleanupEventLoopTests()
{
	for (const FString& ModuleName : GetRequiredModules())
	{
		if (IModuleInterface* Module = FModuleManager::Get().GetModule(*ModuleName))
		{
			Module->ShutdownModule();
		}
	}
}

class EventLoopTestGlobalSetup
{
public:
	EventLoopTestGlobalSetup()
	{
		FTestDelegates::GetGlobalSetup().BindStatic(&InitializeEventLoopTests);
		FTestDelegates::GetGlobalTeardown().BindStatic(&CleanupEventLoopTests);
	}
} GEventLoopTestGlobalSetup;

} // UE::EventLoop
