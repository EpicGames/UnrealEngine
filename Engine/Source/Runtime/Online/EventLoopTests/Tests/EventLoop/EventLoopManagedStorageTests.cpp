// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoop/EventLoopManagedStorage.h"

#include "TestHarness.h"

namespace UE::EventLoop
{
	struct FMockManagedStorageHandleTraits
	{
		static constexpr const TCHAR* Name = TEXT("MockManagedStorageHandle");
	};

	struct FMockEventLoopManagedStorageTraits
	{
		static uint32 GetCurrentThreadId()
		{
			return CurrentThreadId;
		}

		static bool IsManagerThread(uint32 ManagerThreadId)
		{
			return ManagerThreadId == GetCurrentThreadId();
		}

		static void CheckNotInitialized(uint32 ManagerThreadId)
		{
			bCheckNotInitializedTriggered = (ManagerThreadId != 0);
		}

		static void CheckIsManagerThread(uint32 ManagerThreadId)
		{
			bCheckIsManagerThreadTriggered = !IsManagerThread(ManagerThreadId);
		}

		static constexpr bool bStorageAccessThreadChecksEnabled = true;
		using InternalHandleArryAllocatorType = TInlineAllocator<32>;
		using FExternalHandle = TResourceHandle<FMockManagedStorageHandleTraits>;

		static void ResetTestConditions()
		{
			CurrentThreadId = 1;
			bCheckNotInitializedTriggered = false;
			bCheckIsManagerThreadTriggered = false;
		}

		static uint32 CurrentThreadId;
		static bool bCheckNotInitializedTriggered;
		static bool bCheckIsManagerThreadTriggered;
	};

	uint32 FMockEventLoopManagedStorageTraits::CurrentThreadId = 1;
	bool FMockEventLoopManagedStorageTraits::bCheckNotInitializedTriggered = false;
	bool FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered = false;

	TEST_CASE("EventLoop::EventLoopManagedStorage", "[Online][EventLoop][Smoke]")
	{
		const uint32 ManagerThreadId = 1;
		const uint32 WorkerThreadId = 2;
		FMockEventLoopManagedStorageTraits::ResetTestConditions();
		FMockEventLoopManagedStorageTraits::CurrentThreadId = ManagerThreadId;

		using MockStorageType = TManagedStorage<int32, FMockEventLoopManagedStorageTraits>;
		using FMockStorageExternalHandle = MockStorageType::FExternalHandle;
		using FMockStorageInternalHandle = MockStorageType::FInternalHandle;
		using FInternalHandleArryType = MockStorageType::FInternalHandleArryType;

		MockStorageType ManagedStorage;
		const TManagedStorage<int32, FMockEventLoopManagedStorageTraits>& ConstManagedStorage = ManagedStorage;

		SECTION("Single init call")
		{
			ManagedStorage.Init();
			CHECK(!FMockEventLoopManagedStorageTraits::bCheckNotInitializedTriggered);
		}

		SECTION("Double init call")
		{
			ManagedStorage.Init();
			ManagedStorage.Init();
			CHECK(FMockEventLoopManagedStorageTraits::bCheckNotInitializedTriggered);
		}

		SECTION("Call update before init.")
		{
			ManagedStorage.Update();
			CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
		}

		SECTION("Storage access pre-init.")
		{
			SECTION("Num called before init.")
			{
				ManagedStorage.Num();
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("IsEmpty called before init.")
			{
				CHECK(!FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				ManagedStorage.IsEmpty();
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("begin called before init.")
			{
				ManagedStorage.begin();
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("const begin called before init.")
			{
				ConstManagedStorage.begin();
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("end called before init.")
			{
				ManagedStorage.end();
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("const end called before init.")
			{
				ConstManagedStorage.end();
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("Find external handle called before init.")
			{
				ManagedStorage.Find(FMockStorageExternalHandle());
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("const Find external handle called before init.")
			{
				ConstManagedStorage.Find(FMockStorageExternalHandle());
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("Find internal handle called before init.")
			{
				ManagedStorage.Find(FMockStorageInternalHandle());
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("const Find internal handle called before init.")
			{
				ConstManagedStorage.Find(FMockStorageInternalHandle());
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}
		}

		SECTION("Initialized")
		{
			int32 Element1Value = 77;
			int32 Element2Value = 88;
			ManagedStorage.Init();

			SECTION("Call update from correct thread.")
			{
				ManagedStorage.Update();
				CHECK(!FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("Call update from wrong thread.")
			{
				FMockEventLoopManagedStorageTraits::CurrentThreadId = WorkerThreadId;
				ManagedStorage.Update();
				CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("Storage container access.")
			{
				SECTION("Add and remove behavior.")
				{
					FMockStorageInternalHandle InternalHandle1;
					FMockStorageExternalHandle Handle1 = ManagedStorage.Add(MoveTemp(Element1Value));
					FMockStorageExternalHandle Handle2 = ManagedStorage.Add(MoveTemp(Element2Value));
					FMockStorageExternalHandle StaleHandle1 = Handle1;
					FMockStorageExternalHandle InvalidHandle;
					int32 Count = 0;
					FInternalHandleArryType AddedHandles;
					FInternalHandleArryType RemovedHandles;

					// Element will not be added until Update is called.
					CHECK(ManagedStorage.Num() == 0);
					CHECK(ManagedStorage.IsEmpty());

					AddedHandles.Empty();
					ManagedStorage.Update(&AddedHandles);

					REQUIRE(AddedHandles.Num() == 2);
					InternalHandle1 = AddedHandles[0];
					CHECK(InternalHandle1.GetExternalHandle() == Handle1);

					// Element has been added.
					CHECK(ManagedStorage.Num() == 2);
					CHECK(!ManagedStorage.IsEmpty());

					// Request to remove element.
					ManagedStorage.Remove(Handle1);

					// Element has not yet been removed.
					CHECK(ManagedStorage.Num() == 2);
					CHECK(!ManagedStorage.IsEmpty());

					RemovedHandles.Empty();
					ManagedStorage.Update(nullptr, &RemovedHandles);

					REQUIRE(RemovedHandles.Num() == 1);
					InternalHandle1 = RemovedHandles[0];
					CHECK(InternalHandle1.GetExternalHandle() == Handle1);

					// Element has been removed.
					CHECK(ManagedStorage.Num() == 1);
					CHECK(!ManagedStorage.IsEmpty());

					// Element added and removed before Update called.
					Handle1 = ManagedStorage.Add(MoveTemp(Element1Value));
					ManagedStorage.Remove(Handle1);

					AddedHandles.Empty();
					RemovedHandles.Empty();
					ManagedStorage.Update(&AddedHandles, &RemovedHandles);

					// Managed storage still contains the second element.
					CHECK(ManagedStorage.Num() == 1);

					REQUIRE(AddedHandles.Num() == 1);
					InternalHandle1 = AddedHandles[0];
					CHECK(InternalHandle1.GetExternalHandle() == Handle1);

					REQUIRE(RemovedHandles.Num() == 1);
					InternalHandle1 = RemovedHandles[0];
					CHECK(InternalHandle1.GetExternalHandle() == Handle1);

					// Try to remove an element using a stale handle.
					ManagedStorage.Remove(StaleHandle1);
					RemovedHandles.Empty();
					ManagedStorage.Update(nullptr, &RemovedHandles);
					CHECK(!RemovedHandles.IsEmpty());
					CHECK(ManagedStorage.Num() == 1);

					// Try to remove an element using an invalid handle.
					ManagedStorage.Remove(InvalidHandle);
					RemovedHandles.Empty();
					ManagedStorage.Update(nullptr, &RemovedHandles);
					CHECK(!RemovedHandles.IsEmpty());
					CHECK(ManagedStorage.Num() == 1);

					// Remove second element.
					ManagedStorage.Remove(Handle2);
					ManagedStorage.Update();
					CHECK(ManagedStorage.IsEmpty());
					CHECK(ManagedStorage.Num() == 0);
				}

				SECTION("Ranged for behavior.")
				{
					FMockStorageExternalHandle Handle1 = ManagedStorage.Add(MoveTemp(Element1Value));
					FMockStorageExternalHandle Handle2 = ManagedStorage.Add(MoveTemp(Element2Value));
					int32 Count = 0;

					FInternalHandleArryType AddedHandles;
					ManagedStorage.Update(&AddedHandles);
					REQUIRE(AddedHandles.Num() == 2);

					FMockStorageInternalHandle InternalHandle1 = AddedHandles[0];
					FMockStorageInternalHandle InternalHandle2 = AddedHandles[1];

					// Ranged for access.
					Count = 0;
					bool bHandle1Tested = false;
					bool bHandle2Tested = false;
					for (TPair<FMockStorageInternalHandle, int32&>& Entry : ManagedStorage)
					{
						if (Count == 0)
						{
							CHECK(Entry.Key == InternalHandle1);
							CHECK(Entry.Key.GetExternalHandle() == Handle1);
							CHECK(Entry.Value == Element1Value);
							bHandle1Tested = true;
						}
						if (Count == 1)
						{
							CHECK(Entry.Key == InternalHandle2);
							CHECK(Entry.Key.GetExternalHandle() == Handle2);
							CHECK(Entry.Value == Element2Value);
							bHandle2Tested = true;
						}

						++Count;
					}
					CHECK(Count == ManagedStorage.Num());
					CHECK(bHandle1Tested);
					CHECK(bHandle2Tested);

					// Modify element during ranged iteration.
					const int32 ModifiedValue = 56;
					for (TPair<FMockStorageInternalHandle, int32&>& Entry : ManagedStorage)
					{
						if (Entry.Key == InternalHandle1)
						{
							Entry.Value = ModifiedValue;
						}
					}

					// Verify element 1 changed.
					int32* Element1 = ManagedStorage.Find(InternalHandle1);
					REQUIRE(Element1);
					CHECK(*Element1 == ModifiedValue);

					// Verify element 2 did not change.
					int32* Element2 = ManagedStorage.Find(InternalHandle2);
					REQUIRE(Element2);
					CHECK(*Element2 == Element2Value);

					// Remove elements.
					ManagedStorage.Remove(Handle1);
					ManagedStorage.Remove(Handle2);
					ManagedStorage.Update();
					REQUIRE(ManagedStorage.IsEmpty());

					// Ranged for access.
					Count = 0;
					for (TPair<FMockStorageInternalHandle, int32&>& Entry : ManagedStorage)
					{
						++Count;
					}
					CHECK(Count == 0);
				}

				SECTION("Ranged for const behavior.")
				{
					FMockStorageExternalHandle Handle1 = ManagedStorage.Add(MoveTemp(Element1Value));
					FMockStorageExternalHandle Handle2 = ManagedStorage.Add(MoveTemp(Element2Value));
					int32 Count = 0;

					FInternalHandleArryType AddedHandles;
					ManagedStorage.Update(&AddedHandles);
					REQUIRE(AddedHandles.Num() == 2);

					FMockStorageInternalHandle InternalHandle1 = AddedHandles[0];
					FMockStorageInternalHandle InternalHandle2 = AddedHandles[1];

					// Ranged for const access.
					Count = 0;
					bool bHandle1Tested = false;
					bool bHandle2Tested = false;
					for (const TPair<FMockStorageInternalHandle, const int32&>& Entry : ConstManagedStorage)
					{
						if (Count == 0)
						{
							CHECK(Entry.Key == InternalHandle1);
							CHECK(Entry.Key.GetExternalHandle() == Handle1);
							CHECK(Entry.Value == Element1Value);
							bHandle1Tested = true;
						}
						if (Count == 1)
						{
							CHECK(Entry.Key == InternalHandle2);
							CHECK(Entry.Key.GetExternalHandle() == Handle2);
							CHECK(Entry.Value == Element2Value);
							bHandle2Tested = true;
						}

						++Count;
					}
					CHECK(Count == ManagedStorage.Num());
					CHECK(bHandle1Tested);
					CHECK(bHandle2Tested);

					// Remove elements.
					ManagedStorage.Remove(Handle1);
					ManagedStorage.Remove(Handle2);
					ManagedStorage.Update();
					REQUIRE(ManagedStorage.IsEmpty());

					// Ranged for const access.
					Count = 0;
					for (const TPair<FMockStorageInternalHandle, const int32&>& Entry : ConstManagedStorage)
					{
						++Count;
					}
					CHECK(Count == 0);
				}
			}

			SECTION("Element access.")
			{
				FInternalHandleArryType AddedHandles;

				SECTION("New element added.")
				{
					FMockStorageExternalHandle ExternalHandle1 = ManagedStorage.Add(MoveTemp(Element1Value));
					FMockStorageExternalHandle ExternalHandle2 = ManagedStorage.Add(MoveTemp(Element2Value));
					FMockStorageExternalHandle StaleExternalHandle2 = ExternalHandle2;

					AddedHandles.Empty();
					ManagedStorage.Update(&AddedHandles);

					CHECK(AddedHandles.Num() == 2);
					FMockStorageInternalHandle InternalHandle1 = AddedHandles[0];
					FMockStorageInternalHandle InternalHandle2 = AddedHandles[1];
					FMockStorageInternalHandle StaleInternalHandle2 = InternalHandle2;

					ManagedStorage.Remove(ExternalHandle2);
					ManagedStorage.Update();

					SECTION("Mutable access.")
					{
						int32* Element = nullptr;

						// Retrieve value using internal handle.
						Element = ManagedStorage.Find(InternalHandle1);
						REQUIRE(Element);
						CHECK(*Element == Element1Value);

						// Retrieve value using external handle.
						Element = ManagedStorage.Find(ExternalHandle1);
						REQUIRE(Element);
						CHECK(*Element == Element1Value);

						// Retrieve value using stale internal handle.
						Element = ManagedStorage.Find(StaleInternalHandle2);
						CHECK(Element == nullptr);

						// Retrieve value using stale external handle.
						Element = ManagedStorage.Find(StaleExternalHandle2);
						CHECK(Element == nullptr);
					}

					SECTION("Const access.")
					{
						const int32* Element = nullptr;

						// Retrieve value using internal handle.
						Element = ConstManagedStorage.Find(InternalHandle1);
						REQUIRE(Element);
						CHECK(*Element == Element1Value);

						// Retrieve value using external handle.
						Element = ConstManagedStorage.Find(ExternalHandle1);
						REQUIRE(Element);
						CHECK(*Element == Element1Value);

						// Retrieve value using stale internal handle.
						Element = ManagedStorage.Find(StaleInternalHandle2);
						CHECK(Element == nullptr);

						// Retrieve value using stale external handle.
						Element = ManagedStorage.Find(StaleExternalHandle2);
						CHECK(Element == nullptr);
					}
				}

				SECTION("New element removed before update called.")
				{
					FMockStorageExternalHandle ExternalHandle = ManagedStorage.Add(MoveTemp(Element1Value));
					ManagedStorage.Remove(ExternalHandle);
					ManagedStorage.Update(&AddedHandles);
					REQUIRE(AddedHandles.Num() == 1);
					FMockStorageInternalHandle InternalHandle = AddedHandles[0];

					SECTION("Mutable access.")
					{
						int32* Element = nullptr;

						// Retrieve value using internal handle.
						Element = ManagedStorage.Find(InternalHandle);
						CHECK(Element == nullptr);

						// Retrieve value using external handle.
						Element = ManagedStorage.Find(ExternalHandle);
						CHECK(Element == nullptr);
					}

					SECTION("Const access.")
					{
						const int32* Element = nullptr;

						// Retrieve value using internal handle.
						Element = ConstManagedStorage.Find(InternalHandle);
						CHECK(Element == nullptr);

						// Retrieve value using external handle.
						Element = ConstManagedStorage.Find(InternalHandle);
						CHECK(Element == nullptr);
					}
				}
			}

			SECTION("Storage access thread safety.")
			{
				FMockStorageExternalHandle ExternalHandle = ManagedStorage.Add(MoveTemp(Element1Value));
				FInternalHandleArryType AddedHandles;
				ManagedStorage.Update(&AddedHandles);
				REQUIRE(AddedHandles.Num() == 1);
				FMockStorageInternalHandle InternalHandle = AddedHandles[0];

				FMockEventLoopManagedStorageTraits::ResetTestConditions();
				FMockEventLoopManagedStorageTraits::CurrentThreadId = WorkerThreadId;

				SECTION("Num called from wrong thread.")
				{
					ManagedStorage.Num();
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("IsEmpty called from wrong thread.")
				{
					ManagedStorage.IsEmpty();
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("begin called from wrong thread.")
				{
					ManagedStorage.begin();
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("const begin called from wrong thread.")
				{
					ConstManagedStorage.begin();
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("end called from wrong thread.")
				{
					ManagedStorage.end();
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("const end called from wrong thread.")
				{
					ConstManagedStorage.end();
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("Find external handle called from wrong thread.")
				{
					ManagedStorage.Find(ExternalHandle);
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("const Find external handle called from wrong thread.")
				{
					ConstManagedStorage.Find(ExternalHandle);
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("Find internal handle called from wrong thread.")
				{
					ManagedStorage.Find(InternalHandle);
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}

				SECTION("const Find internal handle called from wrong thread.")
				{
					ConstManagedStorage.Find(InternalHandle);
					CHECK(FMockEventLoopManagedStorageTraits::bCheckIsManagerThreadTriggered);
				}
			}

			SECTION("Handle string conversion.")
			{
				FInternalHandleArryType AddedHandles;
				FMockStorageExternalHandle ExternalHandle;
				FString ExternalStringValue;
				FString InternalStringValue;
				FString ExpectedStringValue;

				// Test that ToString resolves name from traits.
				ExternalHandle = ManagedStorage.Add(MoveTemp(Element1Value));
				ExternalStringValue = ExternalHandle.ToString();
				CHECK(ExternalStringValue.Contains(FMockEventLoopManagedStorageTraits::FExternalHandle::GetTypeName()));

				// Test that internal handle as a string contains the external handle string.
				ManagedStorage.Update(&AddedHandles);
				REQUIRE(AddedHandles.Num() == 1);
				InternalStringValue = AddedHandles[0].ToString();
				CHECK(InternalStringValue.Contains(ExternalStringValue));
			}
		}
	}

	TEST_CASE("EventLoop::EventLoopManagedStorage::MoveOnlyType", "[Online][EventLoop][Smoke]")
	{
		FMockEventLoopManagedStorageTraits::ResetTestConditions();

		struct FManagedData
		{
			FManagedData(const FManagedData&) = delete;
			FManagedData& operator=(const FManagedData&) = delete;

			FManagedData()
				: IntValue(0)
			{
			}

			FManagedData(FManagedData&& Other)
				: IntValue(MoveTemp(Other.IntValue))
			{
			}

			FManagedData& operator=(FManagedData&& Other)
			{
				IntValue = MoveTemp(Other.IntValue);
				return *this;
			}

			int32 IntValue;
		};

		using MockStorageType = TManagedStorage<FManagedData, FMockEventLoopManagedStorageTraits>;
		using FMockStorageExternalHandle = MockStorageType::FExternalHandle;
		using FMockStorageInternalHandle = MockStorageType::FInternalHandle;
		using FInternalHandleArryType = MockStorageType::FInternalHandleArryType;

		MockStorageType ManagedStorage;

		SECTION("Element access.")
		{
			const int32 ExpectedValue = 5;
			FManagedData ManagedData;
			ManagedData.IntValue = ExpectedValue;
			FMockStorageExternalHandle Handle = ManagedStorage.Add(MoveTemp(ManagedData));

			FInternalHandleArryType AddedHandles;
			ManagedStorage.Update(&AddedHandles);
			REQUIRE(!AddedHandles.IsEmpty());

			const int32 ModifiedValue = 33;
			for (TPair<FMockStorageInternalHandle, FManagedData&>& Entry : ManagedStorage)
			{
				CHECK(Entry.Value.IntValue == ExpectedValue);
				Entry.Value.IntValue = ModifiedValue;
			}

			FManagedData* FoundManagedData = ManagedStorage.Find(AddedHandles[0]);
			REQUIRE(FoundManagedData);
			CHECK(FoundManagedData->IntValue == ModifiedValue);
		}
	}
} // UE::EventLoop
