// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoop/EventLoop.h"
#include "Templates/SharedPointer.h"

#include "TestHarness.h"

namespace UE::EventLoop
{
	struct FMockTimerHandleTraits
	{
		static constexpr const TCHAR* Name = TEXT("MockTimerHandle");
	};

	struct FEventLoopMockTraits
	{
		static FORCEINLINE uint32 GetCurrentThreadId()
		{
			return CurrentThreadId;
		}

		static FORCEINLINE FTimespan GetCurrentTime()
		{
			return CurrentTime;
		}

		static FORCEINLINE bool IsEventLoopThread(uint32 EventLoopThreadId)
		{
			return EventLoopThreadId == GetCurrentThreadId();
		}

		static FORCEINLINE bool IsInitialized(uint32 EventLoopThreadId)
		{
			return (EventLoopThreadId != 0);
		}

		static FORCEINLINE void CheckInitialized(uint32 EventLoopThreadId)
		{
			bCheckInitializedTriggered = (EventLoopThreadId == 0);
		}

		static FORCEINLINE void CheckNotInitialized(uint32 EventLoopThreadId)
		{
			bCheckNotInitializedTriggered = (EventLoopThreadId != 0);
		}

		static FORCEINLINE void CheckIsEventLoopThread(uint32 EventLoopThreadId)
		{
			bCheckIsEventLoopThreadTriggered = !IsEventLoopThread(EventLoopThreadId);
		}

		static FORCEINLINE bool IsShutdownRequested(bool bInShutdownRequested)
		{
			return bShutdownRequested || bInShutdownRequested;
		}

		struct FTimerManagerTraits : public FTimerManagerTraitsBase
		{
			static void CheckIsManagerThread(bool bIsManagerThread)
			{
				bCheckIsManagerThreadTriggered = !bIsManagerThread;
			}

			using FTimerHeapAllocatorType = TInlineAllocator<32>;
			using FTimerRepeatAllocatorType = TInlineAllocator<32>;

			struct FStorageTraits
			{
				static uint32 GetCurrentThreadId()
				{
					return CurrentThreadId;
				}

				static bool IsManagerThread(uint32 ManagerThreadId)
				{
					return ManagerThreadId == GetCurrentThreadId();
				}

				static void CheckNotInitialized(uint32 ManagerThreadId)
				{
					bCheckNotInitializedTriggered = (ManagerThreadId != 0);
				}

				static void CheckIsManagerThread(uint32 ManagerThreadId)
				{
					bCheckIsManagerThreadTriggered = !IsManagerThread(ManagerThreadId);
				}

				static constexpr bool bStorageAccessThreadChecksEnabled = true;
				using InternalHandleArryAllocatorType = TInlineAllocator<32>;
				using FExternalHandle = FTimerHandle;

				static void ResetTestConditions()
				{
					CurrentThreadId = 1;
					bCheckNotInitializedTriggered = false;
					bCheckIsManagerThreadTriggered = false;
				}

				static uint32 CurrentThreadId;
				static bool bCheckNotInitializedTriggered;
				static bool bCheckIsManagerThreadTriggered;
			};

			static void ResetTestConditions()
			{
				FStorageTraits::ResetTestConditions();
				bCheckIsManagerThreadTriggered = false;
			}

			static bool bCheckIsManagerThreadTriggered;
		};

		static void ResetTestConditions()
		{
			FTimerManagerTraits::ResetTestConditions();
			CurrentThreadId = 1;
			CurrentTime = FTimespan::Zero();
			bCheckInitializedTriggered = false;
			bCheckNotInitializedTriggered = false;
			bCheckIsEventLoopThreadTriggered = false;
			bShutdownRequested = false;
		}

		static uint32 CurrentThreadId;
		static FTimespan CurrentTime;
		static bool bCheckInitializedTriggered;
		static bool bCheckNotInitializedTriggered;
		static bool bCheckIsEventLoopThreadTriggered;
		static bool bShutdownRequested;
	};

	uint32 FEventLoopMockTraits::CurrentThreadId = 1;
	FTimespan FEventLoopMockTraits::CurrentTime = FTimespan::Zero();
	bool FEventLoopMockTraits::bCheckInitializedTriggered = false;
	bool FEventLoopMockTraits::bCheckNotInitializedTriggered = false;
	bool FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered = false;
	bool FEventLoopMockTraits::bShutdownRequested = false;
	bool FEventLoopMockTraits::FTimerManagerTraits::bCheckIsManagerThreadTriggered = false;
	uint32 FEventLoopMockTraits::FTimerManagerTraits::FStorageTraits::CurrentThreadId = 1;
	bool FEventLoopMockTraits::FTimerManagerTraits::FStorageTraits::bCheckNotInitializedTriggered = false;
	bool FEventLoopMockTraits::FTimerManagerTraits::FStorageTraits::bCheckIsManagerThreadTriggered = false;

	class FIOAccessMock final : public FNoncopyable
	{
	public:
		// Nothing needed for now.
	};

	class FIOManagerMock final : public IIOManager
	{
	public:
		using FIOAccess = FIOAccessMock;

		struct FParams
		{
		};

		FIOManagerMock(IEventLoop&, FParams&&)
		{
		}

		virtual ~FIOManagerMock() = default;

		virtual bool Init() override
		{
			bHasInitialized = true;
			return bInitReturnValue;
		}

		virtual void Shutdown() override
		{
			bHasShutdown = true;
		}

		virtual void Notify() override
		{
			bHasNotified = true;
		}

		virtual void Poll(FTimespan WaitTime) override
		{
			CHECK(bHasNotified == bExpectedNotify);
			CHECK(WaitTime == ExpectedWaitTime);
		}

		FIOAccess& GetIOAccess()
		{
			return IOAccess;
		}

		static void ResetTestConditions()
		{
			bHasInitialized = false;
			bInitReturnValue = true;
			bHasNotified = false;
			bExpectedNotify = false;
			bHasShutdown = false;
			ExpectedWaitTime = FTimespan::Zero();
		}

		static bool bHasInitialized;
		static bool bInitReturnValue;
		static bool bHasNotified;
		static bool bExpectedNotify;
		static bool bHasShutdown;
		static FTimespan ExpectedWaitTime;

		FIOAccess IOAccess;
	};

	bool FIOManagerMock::bHasInitialized = false;
	bool FIOManagerMock::bInitReturnValue = true;
	bool FIOManagerMock::bHasNotified = false;
	bool FIOManagerMock::bExpectedNotify = false;
	bool FIOManagerMock::bHasShutdown = false;
	FTimespan FIOManagerMock::ExpectedWaitTime = FTimespan::Zero();

	TEST_CASE("EventLoop", "[Online][EventLoop][Smoke]")
	{
		FIOManagerMock::ResetTestConditions();
		FEventLoopMockTraits::ResetTestConditions();
		TEventLoop<FIOManagerMock, FEventLoopMockTraits> EventLoop;

		const FTimespan InfiniteWait(std::numeric_limits<int64>::max());
		const uint32 EventLoopThreadId = 1;
		FEventLoopMockTraits::CurrentThreadId = EventLoopThreadId;
		FEventLoopMockTraits::FTimerManagerTraits::FStorageTraits::CurrentThreadId = EventLoopThreadId;

		SECTION("Single init call")
		{
			EventLoop.Init();
			CHECK(!FEventLoopMockTraits::bCheckNotInitializedTriggered);
			CHECK(FIOManagerMock::bHasInitialized);
		}

		SECTION("Double init call")
		{
			EventLoop.Init();
			EventLoop.Init();
			CHECK(FEventLoopMockTraits::bCheckNotInitializedTriggered);
		}

		SECTION("Call Run before Init")
		{
			FEventLoopMockTraits::bShutdownRequested = true;
			FIOManagerMock::ExpectedWaitTime = InfiniteWait;

			EventLoop.Run();
			CHECK(FEventLoopMockTraits::bCheckInitializedTriggered);
		}

		SECTION("Call RunOnce before Init")
		{
			FEventLoopMockTraits::bShutdownRequested = true;

			EventLoop.RunOnce(FTimespan::Zero());
			CHECK(FEventLoopMockTraits::bCheckInitializedTriggered);
		}

		SECTION("Call RequestShutdown before Init")
		{
			EventLoop.RequestShutdown();

			// Request shutdown is allowed to be called before init.
			CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);

			// Check that loop is shutdown when run after calling init.
			CHECK(EventLoop.Init());

			FIOManagerMock::bExpectedNotify = true;
			FIOManagerMock::ExpectedWaitTime = InfiniteWait;
			CHECK(!EventLoop.RunOnce(InfiniteWait));
		}

		SECTION("Call SetTimer before Init")
		{
			bool bTimerRan = false;
			FTimerHandle Handle = EventLoop.SetTimer([&bTimerRan](){ bTimerRan = true; }, FTimespan::FromSeconds(1));
			CHECK(Handle.IsValid());

			// Request shutdown is allowed to be called before init.
			CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);

			// Check that timer is run when run after calling init.
			CHECK(EventLoop.Init());

			// The first run schedules the timer. It will not wait because it has been interrupted.
			FIOManagerMock::bExpectedNotify = true;
			FIOManagerMock::ExpectedWaitTime = InfiniteWait;
			CHECK(EventLoop.RunOnce(InfiniteWait));

			// The second run will run until the timer duration and fire the timer.
			FIOManagerMock::bHasNotified = false;
			FIOManagerMock::bExpectedNotify = false;
			FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
			FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(1);
			CHECK(EventLoop.RunOnce(InfiniteWait));
			CHECK(bTimerRan);
		}

		SECTION("Call ClearTimer before Init")
		{
			bool bTimerRan = false;
			FTimerHandle Handle = EventLoop.SetTimer([&bTimerRan](){ bTimerRan = true; }, FTimespan::FromSeconds(1));
			CHECK(Handle.IsValid());

			EventLoop.ClearTimer(Handle);

			// Request shutdown is allowed to be called before init.
			CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);

			// Check that timer is not run when run after calling init.
			CHECK(EventLoop.Init());

			// The first run schedules the timer. It will not wait because it has been interrupted.
			FIOManagerMock::bExpectedNotify = true;
			FIOManagerMock::ExpectedWaitTime = InfiniteWait;
			CHECK(EventLoop.RunOnce(InfiniteWait));

			// The second run would have fired the timer, but it is no longer expected to be present.
			FIOManagerMock::bHasNotified = false;
			FIOManagerMock::bExpectedNotify = false;
			FIOManagerMock::ExpectedWaitTime = InfiniteWait;
			FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(1);
			CHECK(EventLoop.RunOnce(InfiniteWait));
			CHECK(!bTimerRan);
		}

		SECTION("Call PostAsyncTask before Init")
		{
			bool bTaskTriggered = false;
			EventLoop.PostAsyncTask([&bTaskTriggered](){ bTaskTriggered = true; });

			// PostAsyncTask is allowed to be called before init.
			CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);

			// Check that task is run when run after calling init.
			CHECK(EventLoop.Init());

			// Loop is expected to have been interrupted due to scheduling a task.
			FIOManagerMock::bExpectedNotify = true;
			FIOManagerMock::ExpectedWaitTime = InfiniteWait;
			CHECK(EventLoop.RunOnce(InfiniteWait));
			CHECK(bTaskTriggered);
		}

		SECTION("Init - IOManager failure propagated")
		{
			FIOManagerMock::bInitReturnValue = false;
			CHECK(!EventLoop.Init());
		}

		SECTION("GetLoopTime - returns time captured at Init")
		{
			FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(5);
			EventLoop.Init();
			CHECK(EventLoop.GetLoopTime() == FTimespan::FromSeconds(5));
		}

		SECTION("GetLoopTime - updates after each RunOnce")
		{
			EventLoop.Init();

			FIOManagerMock::ExpectedWaitTime = InfiniteWait;
			FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(2);
			EventLoop.RunOnce(InfiniteWait);
			CHECK(EventLoop.GetLoopTime() == FTimespan::FromSeconds(2));

			FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(5);
			EventLoop.RunOnce(InfiniteWait);
			CHECK(EventLoop.GetLoopTime() == FTimespan::FromSeconds(5));
		}

		SECTION("GetLoopTime - fires check when called from wrong thread")
		{
			EventLoop.Init(); // EventLoopThreadId set to CurrentThreadId (1).
			FEventLoopMockTraits::CurrentThreadId = 2;
			EventLoop.GetLoopTime();
			CHECK(FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
		}

		SECTION("Initialized")
		{
			EventLoop.Init();

			SECTION("Call Run")
			{
				FEventLoopMockTraits::bShutdownRequested = true;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;

				EventLoop.Run();

				CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);
				CHECK(!FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
			}

			SECTION("Call RunOnce - no shutdown")
			{
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);

				bool bContinueRunning = EventLoop.RunOnce(FIOManagerMock::ExpectedWaitTime);
				CHECK(bContinueRunning == true);

				CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);
				CHECK(!FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
			}

			SECTION("Call RunOnce - shutdown")
			{
				FEventLoopMockTraits::bShutdownRequested = true;
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);

				bool bContinueRunning = EventLoop.RunOnce(FIOManagerMock::ExpectedWaitTime);
				CHECK(bContinueRunning == false);

				CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);
				CHECK(!FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
				CHECK(FIOManagerMock::bHasShutdown);
			}

			SECTION("Call SetTimer")
			{
				bool bTimerRan = false;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				FIOManagerMock::bExpectedNotify = true;

				EventLoop.SetTimer([&](){ bTimerRan = true; }, FTimespan::FromSeconds(1));
				EventLoop.RunOnce(InfiniteWait);
				CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);
				CHECK(!FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
				CHECK(!bTimerRan);

				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(1);
				FIOManagerMock::bHasNotified = false;
				FIOManagerMock::bExpectedNotify = false;
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bTimerRan);

				bTimerRan = false;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(2);
				FIOManagerMock::bExpectedNotify = false;
				EventLoop.RunOnce(InfiniteWait);
				CHECK(!bTimerRan);
			}

			SECTION("Call SetTimer - repeating")
			{
				bool bTimerRan = false;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				FIOManagerMock::bExpectedNotify = true;
				EventLoop.SetTimer([&](){ bTimerRan = true; }, FTimespan::FromSeconds(1), true);
				EventLoop.RunOnce(InfiniteWait);
				CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);
				CHECK(!FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
				CHECK(!bTimerRan);

				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(1);
				FIOManagerMock::bHasNotified = false;
				FIOManagerMock::bExpectedNotify = false;
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bTimerRan);

				bTimerRan = false;
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(2);
				FIOManagerMock::bExpectedNotify = false;
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bTimerRan);
			}

			SECTION("Call ClearTimer")
			{
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				FIOManagerMock::bExpectedNotify = false; // Invalid handle — no notify expected.

				FTimerHandle Handle;
				EventLoop.ClearTimer(Handle);
				EventLoop.RunOnce(FIOManagerMock::ExpectedWaitTime);

				CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);
				CHECK(!FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
			}

			SECTION("Call PostAsyncTask")
			{
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				FIOManagerMock::bExpectedNotify = true;

				bool bTaskRan = false;
				EventLoop.PostAsyncTask([&bTaskRan](){ bTaskRan = true; });
				EventLoop.RunOnce(FIOManagerMock::ExpectedWaitTime);

				CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);
				CHECK(!FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
				CHECK(bTaskRan);
			}

			SECTION("Call RequestShutdown")
			{
				bool bContinueRunning = false;

				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				FIOManagerMock::bExpectedNotify = true;

				bool bShutdownCompleted = false;
				EventLoop.RequestShutdown([&bShutdownCompleted](){ bShutdownCompleted = true; });
				bContinueRunning = EventLoop.RunOnce(InfiniteWait);

				CHECK(!FEventLoopMockTraits::bCheckInitializedTriggered);
				CHECK(!FEventLoopMockTraits::bCheckIsEventLoopThreadTriggered);
				CHECK(bShutdownCompleted);
				CHECK(!bContinueRunning);

				// Request again after requested.
				bShutdownCompleted = false;
				EventLoop.RequestShutdown([&bShutdownCompleted](){ bShutdownCompleted = true; });
				CHECK(bShutdownCompleted);
				CHECK(!bContinueRunning);

				EventLoop.RunOnce(InfiniteWait);
			}

			SECTION("Call RunOnce - returns false without polling after shutdown")
			{
				// Drive the loop to shutdown.
				FEventLoopMockTraits::bShutdownRequested = true;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				CHECK(!EventLoop.RunOnce(InfiniteWait));

				// A subsequent RunOnce must return false immediately without calling Poll or
				// Shutdown again. If Poll is incorrectly called it will receive a WaitTime that
				// mismatches ExpectedWaitTime, causing the internal CHECK inside Poll to fire.
				FIOManagerMock::bHasShutdown = false;
				FIOManagerMock::ExpectedWaitTime = FTimespan::Zero();
				CHECK(!EventLoop.RunOnce(FTimespan::FromSeconds(999)));
				CHECK(!FIOManagerMock::bHasShutdown);
			}

			SECTION("Call SetTimer - invalid parameters return invalid handle")
			{
				// None of these should produce a valid timer. TryNotify is still called
				// by TEventLoop::SetTimer regardless, but we do not RunOnce so Poll is
				// never reached and the bHasNotified state does not matter.
				FTimerHandle Handle;

				// Null callback.
				Handle = EventLoop.SetTimer(FTimerCallback(), FTimespan::FromSeconds(1));
				CHECK(!Handle.IsValid());

				// Negative rate.
				Handle = EventLoop.SetTimer([](){ }, FTimespan::FromSeconds(-1));
				CHECK(!Handle.IsValid());

				// FirstDelay on a non-repeating timer.
				Handle = EventLoop.SetTimer([](){ }, FTimespan::FromSeconds(1), false, FTimespan::FromSeconds(1));
				CHECK(!Handle.IsValid());

				// Negative FirstDelay.
				Handle = EventLoop.SetTimer([](){ }, FTimespan::FromSeconds(1), true, FTimespan::FromSeconds(-1));
				CHECK(!Handle.IsValid());
			}

			SECTION("Call SetTimer - repeating with FirstDelay")
			{
				bool bTimerRan = false;

				// Activate the timer. The heap is empty at this point so Poll waits infinitely.
				FIOManagerMock::bExpectedNotify = true;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				EventLoop.SetTimer([&bTimerRan](){ bTimerRan = true; }, FTimespan::FromSeconds(1), true, FTimespan::FromSeconds(2));
				EventLoop.RunOnce(InfiniteWait); // activates timer with Expiration = 0 + FirstDelay(2s)
				CHECK(!bTimerRan);

				// Poll wait is clamped to FirstDelay (2s), not Rate (1s).
				FIOManagerMock::bHasNotified = false;
				FIOManagerMock::bExpectedNotify = false;
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(2);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(1); // Rate elapsed, but not FirstDelay.
				EventLoop.RunOnce(InfiniteWait);
				CHECK(!bTimerRan);

				// FirstDelay elapsed — timer fires and reschedules at Rate.
				// Poll wait = Expiration(2s) - InternalTime(1s) = 1s.
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(2);
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bTimerRan);

				// Timer repeats at Rate going forward.
				bTimerRan = false;
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(3);
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bTimerRan);
			}

			SECTION("Call SetTimer - multiple timers fire in order")
			{
				bool bFirstTimerRan = false;
				bool bSecondTimerRan = false;

				// Both SetTimer calls notify; bHasNotified ends up true either way.
				FIOManagerMock::bExpectedNotify = true;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				EventLoop.SetTimer([&bFirstTimerRan](){ bFirstTimerRan = true; }, FTimespan::FromSeconds(1));
				EventLoop.SetTimer([&bSecondTimerRan](){ bSecondTimerRan = true; }, FTimespan::FromSeconds(2));
				EventLoop.RunOnce(InfiniteWait); // activates both timers
				CHECK(!bFirstTimerRan);
				CHECK(!bSecondTimerRan);

				// Poll is clamped to the nearest expiration (1s).
				FIOManagerMock::bHasNotified = false;
				FIOManagerMock::bExpectedNotify = false;
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(1);
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bFirstTimerRan);
				CHECK(!bSecondTimerRan);

				// Poll clamped to remaining time until second timer:
				// Expiration(2s) - InternalTime(1s) = 1s.
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(2);
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bSecondTimerRan);
			}

			SECTION("Call ClearTimer - valid handle")
			{
				bool bTimerRan = false;
				bool bTimerCleared = false;

				// Activate the timer.
				FIOManagerMock::bExpectedNotify = true;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				FTimerHandle Handle = EventLoop.SetTimer([&bTimerRan](){ bTimerRan = true; }, FTimespan::FromSeconds(1));
				CHECK(Handle.IsValid());
				EventLoop.RunOnce(InfiniteWait); // timer is now active in the heap

				// Clear the valid handle — should notify. The timer is still in the heap
				// during GetNextTimeout so Poll is clamped to its remaining time (1s).
				FIOManagerMock::bHasNotified = false;
				FIOManagerMock::bExpectedNotify = true;
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				EventLoop.ClearTimer(Handle, [&bTimerCleared](){ bTimerCleared = true; });
				CHECK(!Handle.IsValid());
				EventLoop.RunOnce(InfiniteWait); // processes removal and fires OnTimerCleared
				CHECK(bTimerCleared);
				CHECK(!bTimerRan);

				// Advance past the original expiration — the cleared timer must not fire.
				FIOManagerMock::bHasNotified = false;
				FIOManagerMock::bExpectedNotify = false;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait; // heap is now empty
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(1);
				EventLoop.RunOnce(InfiniteWait);
				CHECK(!bTimerRan);
			}

			SECTION("Call RequestShutdown - multiple callbacks queued before RunOnce")
			{
				bool bFirstShutdownCompleted = false;
				bool bSecondShutdownCompleted = false;

				// Only the first RequestShutdown calls TryNotify; the second queues its
				// callback without notifying since shutdown is already requested.
				FIOManagerMock::bExpectedNotify = true;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				EventLoop.RequestShutdown([&bFirstShutdownCompleted](){ bFirstShutdownCompleted = true; });
				EventLoop.RequestShutdown([&bSecondShutdownCompleted](){ bSecondShutdownCompleted = true; });

				bool bContinueRunning = EventLoop.RunOnce(InfiniteWait);
				CHECK(!bContinueRunning);
				CHECK(bFirstShutdownCompleted);
				CHECK(bSecondShutdownCompleted);
			}

			SECTION("Call PostAsyncTask - task posts another task in same iteration")
			{
				bool bFirstTaskRan = false;
				bool bSecondTaskRan = false;

				// The second task is enqueued while RunAsyncTasks is draining the queue.
				// The while loop picks it up before returning, so both run in one RunOnce.
				FIOManagerMock::bExpectedNotify = true;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				EventLoop.PostAsyncTask([&](){
					bFirstTaskRan = true;
					EventLoop.PostAsyncTask([&bSecondTaskRan](){ bSecondTaskRan = true; });
				});
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bFirstTaskRan);
				CHECK(bSecondTaskRan);
			}

			SECTION("Timer callback posts async task in same RunOnce iteration")
			{
				bool bTimerRan = false;
				bool bTaskRan = false;

				// Activate the timer.
				FIOManagerMock::bExpectedNotify = true;
				FIOManagerMock::ExpectedWaitTime = InfiniteWait;
				EventLoop.SetTimer([&](){
					bTimerRan = true;
					EventLoop.PostAsyncTask([&bTaskRan](){ bTaskRan = true; });
				}, FTimespan::FromSeconds(1));
				EventLoop.RunOnce(InfiniteWait);

				// Timer fires during Tick; callback posts a task. RunAsyncTasks runs
				// immediately after Tick in the same RunOnce, so the task executes here too.
				FIOManagerMock::bHasNotified = false;
				FIOManagerMock::bExpectedNotify = false;
				FIOManagerMock::ExpectedWaitTime = FTimespan::FromSeconds(1);
				FEventLoopMockTraits::CurrentTime = FTimespan::FromSeconds(1);
				EventLoop.RunOnce(InfiniteWait);
				CHECK(bTimerRan);
				CHECK(bTaskRan);
			}
		}
	}
} // UE::EventLoop
