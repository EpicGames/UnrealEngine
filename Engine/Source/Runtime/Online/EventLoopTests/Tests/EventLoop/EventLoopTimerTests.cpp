// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoop/EventLoopTimer.h"

#include "TestHarness.h"

namespace UE::EventLoop
{
	struct FMockTimerHandleTraits
	{
		static constexpr const TCHAR* Name = TEXT("MockTimerHandle");
	};

	struct FEventLoopTimerManagerMockTraits : public FTimerManagerTraitsBase
	{
		static void CheckIsManagerThread(bool bIsManagerThread)
		{
			bCheckIsManagerThreadTriggered = !bIsManagerThread;
		}

		using FTimerHeapAllocatorType = TInlineAllocator<32>;
		using FTimerRepeatAllocatorType = TInlineAllocator<32>;

		struct FStorageTraits
		{
			static uint32 GetCurrentThreadId()
			{
				return CurrentThreadId;
			}

			static bool IsManagerThread(uint32 ManagerThreadId)
			{
				return ManagerThreadId == GetCurrentThreadId();
			}

			static void CheckNotInitialized(uint32 ManagerThreadId)
			{
				bCheckNotInitializedTriggered = (ManagerThreadId != 0);
			}

			static void CheckIsManagerThread(uint32 ManagerThreadId)
			{
				bCheckIsManagerThreadTriggered = !IsManagerThread(ManagerThreadId);
			}

			static constexpr bool bStorageAccessThreadChecksEnabled = true;
			using InternalHandleArryAllocatorType = TInlineAllocator<32>;
			using FExternalHandle = TResourceHandle<FMockTimerHandleTraits>;

			static void ResetTestConditions()
			{
				CurrentThreadId = 1;
				bCheckNotInitializedTriggered = false;
				bCheckIsManagerThreadTriggered = false;
			}

			static uint32 CurrentThreadId;
			static bool bCheckNotInitializedTriggered;
			static bool bCheckIsManagerThreadTriggered;
		};

		static void ResetTestConditions()
		{
			FStorageTraits::ResetTestConditions();
			bCheckIsManagerThreadTriggered = false;
		}

		static bool bCheckIsManagerThreadTriggered;
	};

	bool FEventLoopTimerManagerMockTraits::bCheckIsManagerThreadTriggered = false;
	uint32 FEventLoopTimerManagerMockTraits::FStorageTraits::CurrentThreadId = 1;
	bool FEventLoopTimerManagerMockTraits::FStorageTraits::bCheckNotInitializedTriggered = false;
	bool FEventLoopTimerManagerMockTraits::FStorageTraits::bCheckIsManagerThreadTriggered = false;

	TEST_CASE("EventLoop::Timer", "[Online][EventLoop][Smoke]")
	{
		using FMockEventLoopTimerManager = TTimerManager<FEventLoopTimerManagerMockTraits>;
		using FMockEventLoopTimerHandle = FMockEventLoopTimerManager::FTimerHandle;
		FMockEventLoopTimerManager TimerManager;
		const uint32 ManagerThreadId = 1;
		FEventLoopTimerManagerMockTraits::ResetTestConditions();
		FEventLoopTimerManagerMockTraits::FStorageTraits::CurrentThreadId = ManagerThreadId;

		SECTION("Single init call")
		{
			TimerManager.Init();
			CHECK(!FEventLoopTimerManagerMockTraits::FStorageTraits::bCheckNotInitializedTriggered);
		}

		SECTION("Double init call")
		{
			TimerManager.Init();
			TimerManager.Init();
			CHECK(FEventLoopTimerManagerMockTraits::FStorageTraits::bCheckNotInitializedTriggered);
		}

		SECTION("Call Tick before Init")
		{
			TimerManager.Tick(FTimespan());
			CHECK(FEventLoopTimerManagerMockTraits::bCheckIsManagerThreadTriggered);
		}

		SECTION("Initialized")
		{
			uint32 TimerRunCount1 = 0;
			auto TimerFunc1 = [&]() { ++TimerRunCount1; };

			uint32 TimerRunCount2 = 0;
			auto TimerFunc2 = [&]() { ++TimerRunCount2; };

			TimerManager.Init();

			SECTION("Call Tick after Init")
			{
				TimerManager.Tick(FTimespan());
				CHECK(!FEventLoopTimerManagerMockTraits::bCheckIsManagerThreadTriggered);
			}

			SECTION("Timer paramater validation.")
			{
				SECTION("Timer with zero rate.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::Zero());
					CHECK(Handle.IsValid());
				}

				SECTION("Timer with positive rate.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
					CHECK(Handle.IsValid());
				}

				SECTION("Timer with negative rate.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(-1));
					CHECK(!Handle.IsValid());
				}

				SECTION("Timer with zero first delay.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), true, FTimespan::Zero());
					CHECK(Handle.IsValid());
				}

				SECTION("Timer with positive first delay.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), true, FTimespan::FromSeconds(1));
					CHECK(Handle.IsValid());
				}

				SECTION("Timer with negative first delay.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), true, FTimespan::FromSeconds(-1));
					CHECK(!Handle.IsValid());
				}

				SECTION("Timer with invalid first delay.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), false, FTimespan::FromSeconds(1));
					CHECK(!Handle.IsValid());
				}

				SECTION("Timer with invalid callback.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(nullptr, FTimespan::FromSeconds(1));
					CHECK(!Handle.IsValid());
				}
			}

			SECTION("Non-repeating")
			{
				SECTION("Simple timer")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);

					// Tick just before expiration time.
					TimerManager.Tick(FTimespan::FromMilliseconds(999));
					CHECK(TimerRunCount1 == 0);

					// Tick to make the timer run.
					TimerManager.Tick(FTimespan::FromMilliseconds(1));
					CHECK(TimerRunCount1 == 1);

					// Make sure timer does not run again.
					TimerRunCount1 = 0;
					TimerManager.Tick(FTimespan::FromSeconds(10));
					CHECK(TimerRunCount1 == 0);
				}

				SECTION("Multiple timers fired during tick.")
				{
					FMockEventLoopTimerHandle Handle1 = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
					REQUIRE(Handle1.IsValid());

					FMockEventLoopTimerHandle Handle2 = TimerManager.SetTimer(MoveTemp(TimerFunc2), FTimespan::FromSeconds(2));
					REQUIRE(Handle2.IsValid());

					// Tick to add the timers.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);
					CHECK(TimerRunCount2 == 0);

					// Tick to fire the timers.
					TimerManager.Tick(FTimespan::FromSeconds(5));
					CHECK(TimerRunCount1 == 1);
					CHECK(TimerRunCount2 == 1);
				}

				SECTION("Tiemr with zero rate called when scheduled.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::Zero());
					REQUIRE(Handle.IsValid());

					// Tick to add and fire the timer.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 1);
				}

				SECTION("Timer removed before firing.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);

					TimerManager.ClearTimer(Handle);

					// Make sure timer does not run.
					TimerManager.Tick(FTimespan::FromSeconds(10));
					CHECK(TimerRunCount1 == 0);
				}

				SECTION("Timer not fired more than once.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerManager.Tick(FTimespan::Zero());

					// Check that timer ran.
					TimerManager.Tick(FTimespan::FromSeconds(10));
					CHECK(TimerRunCount1 == 1);
				}

				SECTION("Timer pending reschedule is not triggered.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerManager.Tick(FTimespan::Zero());

					// Check that no timer is pending reschedule.
					CHECK(!TimerManager.HasPendingRepeatTimer());

					// Check that timer ran.
					TimerManager.Tick(FTimespan::FromSeconds(10));
					CHECK(TimerRunCount1 == 1);

					// Check that no timer is pending reschedule.
					CHECK(!TimerManager.HasPendingRepeatTimer());
				}

				SECTION("Timer storage")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);
					CHECK(TimerManager.GetNumTimers() == 1);

					// Tick to fire the timer.
					TimerRunCount1 = 0;
					TimerManager.Tick(FTimespan::FromSeconds(10));
					CHECK(TimerRunCount1 == 1);
					CHECK(TimerManager.GetNumTimers() == 0);
				}
			}

			SECTION("Repeating")
			{
				SECTION("Timer with zero rate does not block tick")
				{
					FMockEventLoopTimerHandle Handle1 = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
					REQUIRE(Handle1.IsValid());

					// Tick to add first timer.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);

					FMockEventLoopTimerHandle Handle2 = TimerManager.SetTimer(MoveTemp(TimerFunc2), FTimespan::Zero());
					REQUIRE(Handle2.IsValid());

					// Check that both timers ran.
					TimerManager.Tick(FTimespan::FromSeconds(10));
					CHECK(TimerRunCount1 == 1);
					CHECK(TimerRunCount2 == 1);
				}

				SECTION("Timer with zero rate called each tick")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::Zero(), true);
					REQUIRE(Handle.IsValid());

					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 1);

					TimerRunCount1 = 0;
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 1);

					TimerManager.ClearTimer(Handle);

					TimerRunCount1 = 0;
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);
				}

				SECTION("Timer with positive rate called when elapsed")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), true);
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);

					// Tick not enough time to fire.
					TimerManager.Tick(FTimespan::FromMilliseconds(500));
					CHECK(TimerRunCount1 == 0);

					// Tick enough time to fire.
					TimerManager.Tick(FTimespan::FromMilliseconds(500));
					CHECK(TimerRunCount1 == 1);
					TimerRunCount1 = 0;

					// Tick to reschedule timer.
					// This looks funky, the intent is that a timer will never fire early.
					// The way to prevent the extra call to tick to reschedule while providing that
					// guarantee is to get the current time again when rescheduling which this
					// implementation wants to avoid.
					TimerManager.Tick(FTimespan::FromMilliseconds(500));
					CHECK(TimerRunCount1 == 0);

					// Tick not enough time to fire.
					TimerManager.Tick(FTimespan::FromMilliseconds(500));
					CHECK(TimerRunCount1 == 0);

					// Tick enough time to fire.
					TimerManager.Tick(FTimespan::FromMilliseconds(500));
					CHECK(TimerRunCount1 == 1);

					// Timer not fired after cleared.
					TimerManager.ClearTimer(Handle);
					TimerRunCount1 = 0;
					TimerManager.Tick(FTimespan::FromSeconds(10));
					CHECK(TimerRunCount1 == 0);
				}

				SECTION("Timer with zero first delay fires as expected.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), true, FTimespan::Zero());
					REQUIRE(Handle.IsValid());

					// Tick to add and fire the timer.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 1);

					// Tick to reschedule the timer.
					TimerRunCount1 = 0;
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);

					// Tick to see timer does not fire again at zero interval.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);

					// Tick to fire the timer.
					TimerManager.Tick(FTimespan::FromSeconds(1));
					CHECK(TimerRunCount1 == 1);
				}

				SECTION("Timer with positive first delay fires as expected.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), true, FTimespan::FromSeconds(5));
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);

					// Tick to see timer not fire once rate has elapsed.
					TimerManager.Tick(FTimespan::FromSeconds(1));
					CHECK(TimerRunCount1 == 0);

					// Tick near remaining time to see timer has still not fired.
					TimerManager.Tick(FTimespan::FromMilliseconds(3999));
					CHECK(TimerRunCount1 == 0);

					// Tick remaining time to see timer has fired.
					TimerManager.Tick(FTimespan::FromMilliseconds(1));
					CHECK(TimerRunCount1 == 1);
				}

				SECTION("Timer pending reschedule is triggered.")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), true);
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerManager.Tick(FTimespan::Zero());

					// Check that no timer is pending reschedule.
					CHECK(!TimerManager.HasPendingRepeatTimer());

					// Check that timer ran.
					TimerManager.Tick(FTimespan::FromSeconds(10));
					CHECK(TimerRunCount1 == 1);

					// Check that timer is pending reschedule.
					CHECK(TimerManager.HasPendingRepeatTimer());
				}

				SECTION("Timer storage")
				{
					FMockEventLoopTimerHandle Handle = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1), true);
					REQUIRE(Handle.IsValid());

					// Tick to add the timer.
					TimerRunCount1 = 0;
					TimerManager.Tick(FTimespan::Zero());
					CHECK(TimerRunCount1 == 0);
					CHECK(TimerManager.GetNumTimers() == 1);

					// Tick to fire the timer.
					TimerRunCount1 = 0;
					TimerManager.Tick(FTimespan::FromSeconds(2));
					CHECK(TimerRunCount1 == 1);
					CHECK(TimerManager.GetNumTimers() == 1);

					SECTION("Remove before reschedule")
					{
						// Tick to clear the timer.
						TimerRunCount1 = 0;
						TimerManager.ClearTimer(Handle);
						TimerManager.Tick(FTimespan::FromSeconds(2));
						CHECK(TimerRunCount1 == 0);
						CHECK(TimerManager.GetNumTimers() == 0);
					}

					SECTION("Remove after reschedule")
					{
						// Tick to reschedule the timer.
						TimerRunCount1 = 0;
						TimerManager.Tick(FTimespan::Zero());
						CHECK(TimerRunCount1 == 0);
						CHECK(TimerManager.GetNumTimers() == 1);

						// Tick to clear the timer.
						TimerRunCount1 = 0;
						TimerManager.ClearTimer(Handle);
						TimerManager.Tick(FTimespan::FromSeconds(2));
						CHECK(TimerRunCount1 == 0);
						CHECK(TimerManager.GetNumTimers() == 0);
					}
				}
			}

			SECTION("Primed timer removed before it is processed during tick.")
			{
				FMockEventLoopTimerHandle Handle1 = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(2));
				REQUIRE(Handle1.IsValid());

				bool Timer1Removed = false;
				auto Timer1RemovedCallback = [&]()
				{
					Timer1Removed = true;
				};

				auto Timer2Callback = [&]()
				{
					TimerManager.ClearTimer(Handle1, MoveTemp(Timer1RemovedCallback));
				};

				FMockEventLoopTimerHandle Handle2 = TimerManager.SetTimer(MoveTemp(Timer2Callback), FTimespan::FromSeconds(1));
				REQUIRE(Handle1.IsValid());

				// Tick to add the timers.
				TimerManager.Tick(FTimespan::Zero());
				CHECK(TimerRunCount1 == 0);

				// Tick so that both timers are expired.
				// See that timer 1 did not fire, but will not be removed until the following tick.
				TimerManager.Tick(FTimespan::FromSeconds(2));
				CHECK(TimerRunCount1 == 0);
				CHECK(Timer1Removed == false);

				// Tick again to see that timer has been removed.
				TimerManager.Tick(FTimespan::Zero());
				CHECK(Timer1Removed);
			}

			SECTION("Timer with zero rate added during tick is not fired until the next tick.")
			{
				FMockEventLoopTimerHandle Handle1;
				FMockEventLoopTimerHandle Handle2;

				auto Timer1Callback = [&]()
				{
					Handle2 = TimerManager.SetTimer(MoveTemp(TimerFunc2), FTimespan::Zero());
					REQUIRE(Handle2.IsValid());
				};

				FString value = Handle1.ToString();

				Handle1 = TimerManager.SetTimer(MoveTemp(Timer1Callback), FTimespan::Zero());
				REQUIRE(Handle1.IsValid());

				value = Handle1.ToString();

				// Tick to add and fire the timers.
				TimerManager.Tick(FTimespan::Zero());

				// Timer 2 has been set, but has not fired.
				CHECK(Handle2.IsValid());
				CHECK(TimerRunCount2 == 0);

				// Tick to fire timer 2.
				TimerManager.Tick(FTimespan::Zero());
				CHECK(TimerRunCount2 == 1);
			}

			SECTION("GetNextTimeout.")
			{
				FTimespan NextTimeout = FTimespan::Zero();

				FMockEventLoopTimerHandle Handle1 = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::FromSeconds(1));
				REQUIRE(Handle1.IsValid());

				FMockEventLoopTimerHandle Handle2 = TimerManager.SetTimer(MoveTemp(TimerFunc2), FTimespan::FromSeconds(2));
				REQUIRE(Handle2.IsValid());

				// No next timeout set yet.
				CHECK(!TimerManager.GetNextTimeout(NextTimeout));

				// Tick to add the timers.
				TimerManager.Tick(FTimespan::Zero());
				CHECK(TimerRunCount1 == 0);
				CHECK(TimerRunCount2 == 0);

				CHECK(TimerManager.GetNextTimeout(NextTimeout));
				CHECK(NextTimeout == FTimespan::FromSeconds(1));

				TimerManager.Tick(FTimespan::FromMilliseconds(500));
				CHECK(TimerRunCount1 == 0);
				CHECK(TimerRunCount2 == 0);

				CHECK(TimerManager.GetNextTimeout(NextTimeout));
				CHECK(NextTimeout == FTimespan::FromMilliseconds(500));

				TimerManager.Tick(NextTimeout);
				CHECK(TimerRunCount1 == 1);
				CHECK(TimerRunCount2 == 0);

				CHECK(TimerManager.GetNextTimeout(NextTimeout));
				CHECK(NextTimeout == FTimespan::FromSeconds(1));

				TimerManager.Tick(NextTimeout);
				CHECK(TimerRunCount2 == 1);

				// There is no longer any valid timer.
				CHECK(!TimerManager.GetNextTimeout(NextTimeout));
			}

			SECTION("Handle string conversion.")
			{
				FMockEventLoopTimerHandle Handle1 = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::Zero());
				REQUIRE(Handle1.IsValid());

				// Test that ToString resolves name from traits.
				FString StringValue = Handle1.ToString();
				CHECK(StringValue.Contains(FEventLoopTimerManagerMockTraits::FStorageTraits::FExternalHandle::GetTypeName()));
			}

			SECTION("Timer handle is cleared when timer is removed.")
			{
				FMockEventLoopTimerHandle Handle1 = TimerManager.SetTimer(MoveTemp(TimerFunc1), FTimespan::Zero());
				REQUIRE(Handle1.IsValid());

				TimerManager.ClearTimer(Handle1);
				CHECK(!Handle1.IsValid());
			}
		}
	}
}