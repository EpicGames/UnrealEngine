// Copyright Epic Games, Inc. All Rights Reserved.

#include "BandwidthDebugDelegates.h"


namespace CBWD
{
	static UE::ClientBandwidthDelegates::FOnAdditionToDebugTextDisplay OnAdditionToDebugTextDisplay;

	static UE::ClientBandwidthDelegates::FOnClearForTick OnClearDebugInfoForTick;
}


void IClientBandwidthGlobalDelegates::AddTextToDebugDisplay(FStringView CategoryName, FStringView TextToPrint, float Scale, FColor Color, bool bIsSubHeader)
{
	CBWD::OnAdditionToDebugTextDisplay.Broadcast(CategoryName, TextToPrint, Scale, Color, bIsSubHeader);
}

FDelegateHandle IClientBandwidthGlobalDelegates::BindToTextAdditionForDebugDisplay(TFunction<void(FStringView, FStringView, float, FColor, bool)> Callback)
{
	return CBWD::OnAdditionToDebugTextDisplay.AddLambda(Callback);
}

void IClientBandwidthGlobalDelegates::ClearDebugInfoForTick(FStringView CategoryName)
{
	CBWD::OnClearDebugInfoForTick.Broadcast(CategoryName);
}

FDelegateHandle IClientBandwidthGlobalDelegates::BindToClearDebugInfoForTick(TFunction<void(FStringView)> Callback)
{
	return CBWD::OnClearDebugInfoForTick.AddLambda(Callback);
}

void IClientBandwidthGlobalDelegates::UnbindAllToClearDebugInfoForTick(FDelegateHandle Handle)
{
	CBWD::OnClearDebugInfoForTick.Remove(Handle);
}

void IClientBandwidthGlobalDelegates::UnbindAllToTextAdditionForDebugDisplay(FDelegateHandle Handle)
{
	CBWD::OnAdditionToDebugTextDisplay.Remove(Handle);
}