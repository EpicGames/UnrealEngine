// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

// BandwidthDebugDelegates module
// Debug Visualizer for client side bandwidth data

class FBandwidthDebugDelegatesModule : public IModuleInterface
{
	void StartupModule()
	{
		ClientBandwidthDebugger = FModuleManager::Get().LoadModule("ClientBandwidthDebugVisualizer");
	}

	void ShutdownModule()
	{
		FModuleManager::Get().UnloadModule("ClientBandwidthDebugVisualizer", true);
		ClientBandwidthDebugger = nullptr;
	}

	IModuleInterface* ClientBandwidthDebugger = nullptr;
};

IMPLEMENT_MODULE(FBandwidthDebugDelegatesModule, BandwidthDebugDelegates)