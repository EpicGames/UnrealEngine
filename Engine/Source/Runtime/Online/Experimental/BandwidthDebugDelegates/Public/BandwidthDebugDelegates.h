// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"	
#include "UObject/Interface.h"
#include "Containers/StringView.h"

#include "BandwidthDebugDelegates.generated.h"

#define UE_API BANDWIDTHDEBUGDELEGATES_API

namespace UE::ClientBandwidthDelegates
{
	DECLARE_MULTICAST_DELEGATE_FiveParams(FOnAdditionToDebugTextDisplay, FStringView, FStringView, float, FColor, bool);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnClearForTick, FStringView);
}

UINTERFACE(MinimalAPI)
class UClientBandwidthGlobalDelegates : public UInterface
{
	GENERATED_BODY()
};

class IClientBandwidthGlobalDelegates
{
	GENERATED_BODY()

public:
	UE_API static void AddTextToDebugDisplay(FStringView CategoryName, FStringView TextToPrint, float Scale, FColor Color, bool bIsSubHeader = false);

	UE_API static FDelegateHandle BindToTextAdditionForDebugDisplay(TFunction<void(FStringView, FStringView, float, FColor, bool)> Callback);

	UE_API static void ClearDebugInfoForTick(FStringView CategoryName);

	UE_API static FDelegateHandle BindToClearDebugInfoForTick(TFunction<void(FStringView)> Callback);

	UE_API static void UnbindAllToClearDebugInfoForTick(FDelegateHandle Handle);

	UE_API static void UnbindAllToTextAdditionForDebugDisplay(FDelegateHandle Handle);
};
#undef UE_API