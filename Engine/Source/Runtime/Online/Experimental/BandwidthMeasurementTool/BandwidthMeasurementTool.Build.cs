// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BandwidthMeasurementTool : ModuleRules
{
	public BandwidthMeasurementTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(new string[]{
			"Core",
			});

		PrivateDependencyModuleNames.AddRange(new string[]{
			"CoreUObject",
			"HTTP",
			"BandwidthDebugDelegates",
			});
	}
}