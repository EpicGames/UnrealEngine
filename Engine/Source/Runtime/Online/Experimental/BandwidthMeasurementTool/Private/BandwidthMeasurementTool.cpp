// Copyright Epic Games, Inc. All Rights Reserved.

#include "BandwidthMeasurementTool.h"
#include "BandwidthDebugDelegates.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "Math/NumericLimits.h"
#include "Misc/ConfigCacheIni.h"
#include "Templates/UniquePtr.h"

DEFINE_LOG_CATEGORY(LogBandwidthMeasurementTool);

namespace BWMT
{
	static bool GEnableMeasurementTool = false;
	static FAutoConsoleVariableRef CVarBWMTEnableMeasurementTool(
		TEXT("net.BWMT.Enable"),
		GEnableMeasurementTool,
		TEXT("Turns on/off the Bandwidth Measurement Tool."),
		ECVF_Default);

	static int32 GMaxThroughputHistory = 10;
	static FAutoConsoleVariableRef CVarBWMTMaxThroughputHistory(
		TEXT("net.BWMT.HistoryCount"),
		GMaxThroughputHistory,
		TEXT("Value used to denote the maximum number of previous measurements the tool can average against."),
		ECVF_Default);

	static int32 GMaxNumHttpRequests = 5;
	static FAutoConsoleVariableRef CVarBWMTMaxNumHttpRequests(
		TEXT("net.BWMT.RequestCount"),
		GMaxNumHttpRequests,
		TEXT("The number of network requests to send out to flood the network for measurement."),
		ECVF_Default);

	static float GMaxTimeInSecondsToMeasure = 5.0f;
	static FAutoConsoleVariableRef CVarBWMTMaxTimeInSecondsToMeasure(
		TEXT("net.BWMT.MeasurementTime"),
		GMaxTimeInSecondsToMeasure,
		TEXT("The time in seconds that we want to allow the measurement tool to collect data from the download requests."),
		ECVF_Default);

	static float GWarmupPeriodTimeInSeconds = 0.0f; 
	static FAutoConsoleVariableRef CVarBWMTWarmupPeriodTimeInSeconds(
		TEXT("net.BWMT.WarmupPeriod"),
		GWarmupPeriodTimeInSeconds,
		TEXT("The time in seconds we want to not capture download data to let the CDN warmup and provide a more accurate measurement."),
		ECVF_Default);

	static FString GMeasurementURL = TEXT("");
	static FAutoConsoleVariableRef CVarBWMTMeasurementURL(
		TEXT("net.BWMT.MeasurementURL"),
		GMeasurementURL,
		TEXT("The CDN URL to download from when measuring bandwidth."),
		ECVF_Default);

	// Command to manually run a measurement and log the outcome
	FAutoConsoleCommand FManualBandwidthMeasurementTest(
		TEXT("net.BWMT.ManualMeasure"), 
		TEXT("Starts a manual run of the bandwidth measurement tool and will log the resulting value."),
		FConsoleCommandDelegate::CreateLambda([]() 
			{
				FBandwidthMeasurementTool& MeasurementTool = FBandwidthMeasurementTool::Get();
				MeasurementTool.RegisterForCompletedMeasurement([](uint64 MeasurementInBps) 
					{
						UE_LOGF(LogBandwidthMeasurementTool, Log, "FBandwidthMeasurementTool has measured a bandwidth of %llu Bps.", MeasurementInBps);
					});

				MeasurementTool.BeginMeasurement();
			}),
		ECVF_Cheat);

	// Command to manually clear the accumulated historical measurement data
	FAutoConsoleCommand FClearMeasurementHistory(
		TEXT("net.BWMT.ClearHistory"),
		TEXT("Clears the collected history of bandwidth measurements."),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				FBandwidthMeasurementTool& MeasurementTool = FBandwidthMeasurementTool::Get();
				MeasurementTool.ClearHistory();
			}),
		ECVF_Cheat);

	// Command to print telemetry data to the log
	FAutoConsoleCommand FPrintTelemetryData(
		TEXT("net.BWMT.PrintTelemetryData"),
		TEXT("Prints most recent snapshot of bandwidth measurement data."),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				FBandwidthMeasurementTool& MeasurementTool = FBandwidthMeasurementTool::Get();
				MeasurementTool.Log(true);
			}),
		ECVF_Cheat);
}

// ---------------------------
// Singleton backing pointer
// ---------------------------
TUniquePtr<FBandwidthMeasurementTool> FBandwidthMeasurementTool::Instance = nullptr;

/// Static Functions
FBandwidthMeasurementTool& FBandwidthMeasurementTool::Get()
{
	if (!Instance)
	{
		Instance = MakeUnique<FBandwidthMeasurementTool>();
	}
	check(Instance);
	return *Instance.Get();
}

/// Member Functions
FBandwidthMeasurementTool::FBandwidthMeasurementTool()
{

}

FBandwidthMeasurementTool::~FBandwidthMeasurementTool()
{

}

void FBandwidthMeasurementTool::Initialize()
{
	FString MeasurementUrl;
	GConfig->GetString(TEXT("BandwidthMeasurementTool"), TEXT("MeasurementUrl"), MeasurementUrl, GEngineIni);
	BWMT::CVarBWMTMeasurementURL->Set(*MeasurementUrl, ECVF_SetByCode);

	if (!EnableChangedDelegate.IsValid())
	{
		EnableChangedDelegate = BWMT::CVarBWMTEnableMeasurementTool->OnChangedDelegate().AddLambda([this](IConsoleVariable* Var)
			{
				if (Var->GetBool())
				{
					BeginMeasurement();
				}
			});
	}
}

void FBandwidthMeasurementTool::Shutdown()
{
	OnBandwidthMeasurementComplete.Clear();

	BWMT::CVarBWMTEnableMeasurementTool->OnChangedDelegate().Remove(EnableChangedDelegate);
}

bool FBandwidthMeasurementTool::IsBandwidthBeingMeasured() const
{
	return MeasurementTimer.IsValid();
}

void FBandwidthMeasurementTool::RegisterForCompletedMeasurement(TFunction<void(uint64)>&& FunctionPtr)
{
	if (FunctionPtr.IsSet())
	{
		OnBandwidthMeasurementComplete.AddLambda(MoveTemp(FunctionPtr));
	}
}

float FBandwidthMeasurementTool::GetTimeNeededForMeasurement() const
{
	return BWMT::GMaxTimeInSecondsToMeasure;
}

void FBandwidthMeasurementTool::BeginMeasurement()
{
	if (IsBandwidthBeingMeasured() || !BWMT::GEnableMeasurementTool)
	{
		// We already have a measurement being performed
		return;
	}

	CreateRequestTraffic();

	// We only want this tick to finish when we meet our requirements for stopping
	MeasurementTimer = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float InDeltaSeconds)->bool
		{
			UE::TScopeLock Lock(Mutex);

			bool bTimeToStop = bHasTheLastRequestFinishedEarly;
			if (!bTimeToStop && MeasurementStartTimestamp > 0.0)
			{
				const double ElapsedMeasurementTime = FPlatformTime::Seconds() - MeasurementStartTimestamp;
				bTimeToStop = ElapsedMeasurementTime >= GetTimeNeededForMeasurement();
			}

			if (bTimeToStop)
			{
				StopMeasuring();
			}

			return !bTimeToStop;
		}));

	// This will create the delay time we want to allow the CDN to warm up
	WarmupTimer = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float InDeltaSeconds)->bool
		{
			UE::TScopeLock Lock(Mutex);

			bReadyToMeasureAfterWarmupPeriod = true;
			MeasurementStartTimestamp = FPlatformTime::Seconds();
			return false;
		}), BWMT::GWarmupPeriodTimeInSeconds);
}

uint64 FBandwidthMeasurementTool::SafeIncrementForUInt64(const uint64 Base, const uint64 Increase) const
{
	if (Base <= TNumericLimits<uint64>::Max() - Increase)
	{
		// adding is within or at numerical max
		return Base + Increase;
	}
	else
	{
		// adding the average would take us outside the max, so we are capping
		return TNumericLimits<uint64>::Max();
	}
}

void FBandwidthMeasurementTool::CreateRequestTraffic()
{
	UE::TScopeLock Lock(Mutex);

	TelemetryData.FinishRequestCount = 0;
	TelemetryData.CompleteRequestCount = 0;
	TelemetryData.PostWarmupCompleteRequestCount = 0;

	// Need to repeatedly send requests to fill line
	int32 LastFailedRequest = 0;
	constexpr int32 SpamPreventingRange = 0;
	for (int32 RequestCount = 0; RequestCount < BWMT::GMaxNumHttpRequests; RequestCount++)
	{
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
		HttpRequest->SetVerb(TEXT("GET"));
		HttpRequest->SetURL(BWMT::GMeasurementURL);
		HttpRequest->SetTimeout(GetTimeNeededForMeasurement());
		HttpRequest->SetResponseBodyReceiveStreamDelegateV2(FHttpRequestStreamDelegateV2::CreateLambda([this](void* Ptr, int64& Length)
			{
				UE::TScopeLock Lock(Mutex);

				if (bReadyToMeasureAfterWarmupPeriod && IsBandwidthBeingMeasured())
				{
					TotalBytesReceived = SafeIncrementForUInt64(TotalBytesReceived, Length);

					UE_LOGF(LogBandwidthMeasurementTool, Verbose, "Packet Data received! BytesReceived: %llu \t CurrentTotal: %llu", Length, TotalBytesReceived.load());
				}
			}));

		HttpRequest->OnProcessRequestComplete().BindLambda(
			[this](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bProcessedSuccessfully)
			{
				UE::TScopeLock Lock(Mutex);

				if (IsBandwidthBeingMeasured())
				{
					TelemetryData.FinishRequestCount++;

					if (bProcessedSuccessfully)
					{
						TelemetryData.CompleteRequestCount++;

						if (bReadyToMeasureAfterWarmupPeriod)
						{
							TelemetryData.PostWarmupCompleteRequestCount++;
						}
					}

					if (TelemetryData.FinishRequestCount >= static_cast<uint64>(ActiveRequests.Num()))
					{
						bHasTheLastRequestFinishedEarly = true;
					}
				}
			});

		if (HttpRequest->ProcessRequest())
		{
			ActiveRequests.Add(HttpRequest);
		}
		else
		{
			if (RequestCount >= LastFailedRequest + SpamPreventingRange)
			{
				// Log a failed processing attempt
				UE_LOGF(LogBandwidthMeasurementTool, Warning, "%ls - Request attempt %i Failed to process!",
					UE_SOURCE_LOCATION, RequestCount);

				LastFailedRequest = RequestCount;
			}
		}
	}
}

void FBandwidthMeasurementTool::StopMeasuring()
{
	UE::TScopeLock Lock(Mutex);

	// We have had the data collection period and need to remove the timers to prevent unnecessary execution
	FTSTicker::GetCoreTicker().RemoveTicker(MeasurementTimer);
	MeasurementTimer.Reset();
	FTSTicker::GetCoreTicker().RemoveTicker(WarmupTimer);
	WarmupTimer.Reset();

	for (TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& HttpRequest : ActiveRequests)
	{
		if (HttpRequest->GetStatus() == EHttpRequestStatus::Processing)
		{
			HttpRequest->CancelRequest();
		}
	}

	CalculateAverageBandwidth();
	// Broadcast the measurement to any subscribers
	OnBandwidthMeasurementComplete.Broadcast(CurrentBandwidthMeasurementInBps);
	Log(false);

	// Reset Values for next cycle
	ActiveRequests.Empty();
	MeasurementStartTimestamp = 0.0;
	bHasTheLastRequestFinishedEarly = false;
	bReadyToMeasureAfterWarmupPeriod = false;
	TotalBytesReceived = 0;
}

void FBandwidthMeasurementTool::CalculateAverageBandwidth()
{
	// Calculate raw throughput for this measurement window
	double ActualMeasurementWindowInSeconds = (MeasurementStartTimestamp > 0.0)
		? (FPlatformTime::Seconds() - MeasurementStartTimestamp)
		: 0.0;
	ActualMeasurementWindowInSeconds = FMath::Max(ActualMeasurementWindowInSeconds, UE_DOUBLE_SMALL_NUMBER);
	uint64 RawThroughputBps = static_cast<uint64>(static_cast<double>(TotalBytesReceived.load()) / ActualMeasurementWindowInSeconds);

	TelemetryData.RecentReportedBytesReceived = TotalBytesReceived.load();
	TelemetryData.RecentMeasurementTime = ActualMeasurementWindowInSeconds;
	TelemetryData.RecentMeasurementInBps = RawThroughputBps;

	AugmentBandwidthMeasurement(RawThroughputBps);
	RecordBandwidthHistory(RawThroughputBps);

	// Get history for averaging
	const uint64 Count = static_cast<uint64>(RequestMemoryThroughputHistory.Num());
	uint64 AverageBps = 0;
	for (uint64 Measurement : RequestMemoryThroughputHistory)
	{
		// Divide each entry before accumulating to prevent rollover on the sum
		AverageBps = SafeIncrementForUInt64(AverageBps, Measurement / Count);
	}

	TelemetryData.RecentMeasurementCount = Count;
	TelemetryData.RecentAverageInBps = AverageBps;
	TelemetryData.RecentHistory = RequestMemoryThroughputHistory;

	CurrentBandwidthMeasurementInBps = AverageBps;
}

void FBandwidthMeasurementTool::AugmentBandwidthMeasurement(uint64& MeasurementToAlterInBytes) const
{
	// This will do nothing at the moment, but as we have need for certain cases,
	// like platform considerations, they will go here
}

void FBandwidthMeasurementTool::RecordBandwidthHistory(const uint64 CalculatedBytesPerSecond)
{
	if (CalculatedBytesPerSecond == 0)
	{
		return;
	}

	if (RequestMemoryThroughputHistory.Num() >= BWMT::GMaxThroughputHistory)
	{
		// If we are at the limit we need to remove the oldest element
		RequestMemoryThroughputHistory.RemoveAt(0);
	}

	RequestMemoryThroughputHistory.Add(CalculatedBytesPerSecond);
}

void FBandwidthMeasurementTool::ClearHistory()
{
	RequestMemoryThroughputHistory.Empty(BWMT::GMaxThroughputHistory);
}

TOptional<FBandwidthMeasurementTool::FMeasurementQualityStats> FBandwidthMeasurementTool::ComputeMeasurementQuality() const
{
	const TArray<uint64>& History = TelemetryData.RecentHistory;
	if (History.Num() < 2)
	{
		return {};
	}

	FMeasurementQualityStats Stats;
	Stats.MinBps = TNumericLimits<uint64>::Max();
	Stats.MaxBps = 0;
	double Sum = 0.0;
	for (uint64 Val : History)
	{
		Sum += static_cast<double>(Val);
		Stats.MinBps = FMath::Min(Stats.MinBps, Val);
		Stats.MaxBps = FMath::Max(Stats.MaxBps, Val);
	}

	const double Mean = Sum / History.Num();
	if (Mean > 0.0)
	{
		double SumSqDiff = 0.0;
		for (uint64 Val : History)
		{
			SumSqDiff += FMath::Square(static_cast<double>(Val) - Mean);
		}
		const double StdDev = FMath::Sqrt(SumSqDiff / (History.Num() - 1));
		Stats.VariancePercent = static_cast<float>((StdDev / Mean) * 100.0);
	}

	return Stats;
}

void FBandwidthMeasurementTool::Log(bool bPrint)
{
	constexpr const FStringView CategoryName = TEXT("Bandwidth Measurement Tool");
	constexpr const float CategoryScale = 2.0f;
	constexpr const float EntryScale = 1.25;
	const FColor CategoryColor = FColor::Emerald;
	const FColor EntryColor = FColor::Cyan;
	TArray<FString> LogText;

	auto WriteLogEntry = [bPrint, &LogText, &CategoryName](FString& Message, float Scale, FColor Color, bool bIsSubHeader = false)
		{
			IClientBandwidthGlobalDelegates::AddTextToDebugDisplay(CategoryName, Message, Scale, Color, bIsSubHeader);
			if (bPrint)
			{
				LogText.Add(Message);
			}
		};

	// Safety since multiple calls can act like a tick
	IClientBandwidthGlobalDelegates::ClearDebugInfoForTick(CategoryName);

	FString PrintMsg = TEXT("FBandwidthMeasurementTool Telemetry Data:");
	WriteLogEntry(PrintMsg, CategoryScale, CategoryColor, true);

	PrintMsg = FString::Printf(TEXT("Finished Requests: %llu\t\t Completed Requests: %llu\t\t Post-Warmup Completed: %llu"), 
		TelemetryData.FinishRequestCount, TelemetryData.CompleteRequestCount, TelemetryData.PostWarmupCompleteRequestCount);
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	PrintMsg = FString::Printf(TEXT("Final Received Bytes: %llu\t\t Raw Bps: %llu"),
		TelemetryData.RecentReportedBytesReceived, TelemetryData.RecentMeasurementInBps);
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	PrintMsg = FString::Printf(TEXT("Measured Time: %f\t\t Measurement Count: %llu"),
		TelemetryData.RecentMeasurementTime, TelemetryData.RecentMeasurementCount);
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	PrintMsg = TEXT("FBandwidthMeasurementTool Historical Measurements:");
	WriteLogEntry(PrintMsg, CategoryScale, CategoryColor, true);

	for (const uint64& Measurement : TelemetryData.RecentHistory)
	{
		PrintMsg = FString::Printf(TEXT("Previous Bps: %llu"), Measurement);
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);
	}

	PrintMsg = FString::Printf(TEXT("Historical Avg Bps: %llu"), TelemetryData.RecentAverageInBps);
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	if (bPrint)
	{
		for (FString& Text : LogText)
		{
			UE_LOGF(LogBandwidthMeasurementTool, Log, "%ls", *Text);
		}
	}
}