// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "BandwidthMeasurementTool.h"

// BandwidthMeasurementTool module
// Module to manage the initialization and tear down for the Bandwidth measuring tool

class FBandwidthMeasurementToolModule : public IModuleInterface
{
	void StartupModule()
	{
		FBandwidthMeasurementTool::Get().Initialize();
	}

	void ShutdownModule()
	{
		FBandwidthMeasurementTool::Get().Shutdown();
	}
};

IMPLEMENT_MODULE(FBandwidthMeasurementToolModule, BandwidthMeasurementTool)