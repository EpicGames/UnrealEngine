// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/CircularQueue.h"
#include "Containers/Ticker.h"
#include "Templates/UnrealTemplate.h"

#define UE_API BANDWIDTHMEASUREMENTTOOL_API

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogBandwidthMeasurementTool, Log, All);

class IHttpRequest;

namespace UE::BandwidthMeasurement
{
	// Delegate receives a measurement in Bytes Per Second
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMeasurementComplete, uint64);
}

class FBandwidthMeasurementTool
{
public:
	UE_NONCOPYABLE(FBandwidthMeasurementTool);

	FBandwidthMeasurementTool();
	~FBandwidthMeasurementTool();

	// --- Singleton access ---
	UE_API static FBandwidthMeasurementTool& Get();

	// --- Lifecycle (called by module) ---
	UE_API void Initialize();
	void Shutdown();

	// --- API Functions ---

	// Assign to call back function to report when measurement is complete
	UE_API void RegisterForCompletedMeasurement(TFunction<void(uint64)>&& FunctionPtr);

	// Trigger measurement
	UE_API void BeginMeasurement();

	// Report out measurement completion
	UE_API bool IsBandwidthBeingMeasured() const;

	// Get delay time needed for measurement
	UE_API float GetTimeNeededForMeasurement() const;

	// --- Exposed Utility Functions ---
	void ClearHistory();

	void Log(bool bPrint);

private:
	// --- Internal Functions needed ---
	
	// Request Traffic from CDN
	void CreateRequestTraffic();

	// Tick-able Stop to call after time for measurement has passed
	void StopMeasuring();

	// Queue up bandwidth history
	void RecordBandwidthHistory(const uint64 CalculatedBytesPerSecond);

	// Calculate base average throughput
	void CalculateAverageBandwidth();

	// Apply any alterations to average throughput on case basis, maybe? Think platform needs
	void AugmentBandwidthMeasurement(uint64& MeasurementToAlterInBytes) const;

	// Helper function to manage uint64 rollover
	uint64 SafeIncrementForUInt64(const uint64 Base, const uint64 Increase) const;

	// --- Internal Members needed ---

	// Singleton instance pointer
	static TUniquePtr<FBandwidthMeasurementTool> Instance;

	// Queue of fixed size for history average
	TArray<uint64> RequestMemoryThroughputHistory;

	// External reporting callback
	UE::BandwidthMeasurement::FOnMeasurementComplete OnBandwidthMeasurementComplete;

	// Current average throughput
	uint64 CurrentBandwidthMeasurementInBps = 0;

	// The total number of bytes received via http requests this measurement
	std::atomic<uint64> TotalBytesReceived = 0;

	// Fixed array of TSharedRef<IHttpRequest> to track request states
	TArray<TSharedRef<IHttpRequest>> ActiveRequests;

	// Tick handle needed to track current measurements time passage
	FTSTicker::FDelegateHandle MeasurementTimer;

	// Used to tell if the final request finished before hitting the maximum run time
	bool bHasTheLastRequestFinishedEarly = false;

	// Tick handle to track time for the "warm up" period
	FTSTicker::FDelegateHandle WarmupTimer;

	// State Tracking used to allow for communication line warmup
	bool bReadyToMeasureAfterWarmupPeriod = false;

	// Wall-clock timestamp (FPlatformTime::Seconds) when post-warmup measurement began
	double MeasurementStartTimestamp = 0.0;

	// Mutex to keep data consistent across multiple potential thread executions
	mutable FTransactionallySafeCriticalSection Mutex;

public:
	// --- Telemetry Data ---
	struct FBandwidthSnapshotData
	{
		uint64 FinishRequestCount = 0;
		uint64 CompleteRequestCount = 0;
		uint64 PostWarmupCompleteRequestCount = 0;
		uint64 RecentReportedBytesReceived = 0;
		double RecentMeasurementTime = 0;
		uint64 RecentMeasurementInBps = 0;
		uint64 RecentMeasurementCount = 0;
		uint64 RecentAverageInBps = 0;
		TArray<uint64> RecentHistory;
	};

	// Returns Telemetry Data for reporting
	const FBandwidthSnapshotData& GetTelemetryData() const { return TelemetryData; };

	// Measurement quality stats
	struct FMeasurementQualityStats
	{
		float VariancePercent = 0.f; // Coefficient of variation as percentage
		uint64 MinBps = 0;
		uint64 MaxBps = 0;
	};

	// Computes quality stats from recent measurement history. Returns empty if insufficient data.
	UE_API TOptional<FMeasurementQualityStats> ComputeMeasurementQuality() const;

private:
	// Storage structure for telemetry reporting
	FBandwidthSnapshotData TelemetryData;

	// Tracker for Enable state changes
	FDelegateHandle EnableChangedDelegate;
};

#undef UE_API