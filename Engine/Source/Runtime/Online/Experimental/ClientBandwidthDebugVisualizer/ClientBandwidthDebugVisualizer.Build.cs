// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ClientBandwidthDebugVisualizer : ModuleRules
{
	public ClientBandwidthDebugVisualizer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(new string[] {
			"Core",
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"CoreUObject",
			"Engine",
			"BandwidthDebugDelegates",
			"EngineSettings",
		});
	}
}
