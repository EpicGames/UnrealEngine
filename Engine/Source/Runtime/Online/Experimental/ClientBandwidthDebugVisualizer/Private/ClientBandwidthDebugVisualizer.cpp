// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientBandwidthDebugVisualizer.h"
#include "BandwidthDebugDelegates.h"
#include "Kismet/KismetRenderingLibrary.h"	
#include "GameFramework/HUD.h"
#include "Engine/Canvas.h"
#include "Engine/Console.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Texture2D.h"
#include "Widgets/SWindow.h"
#include "UnrealEngine.h"

DEFINE_LOG_CATEGORY(LogBandwidthDebugger);

namespace CBDV
{
	static FName GDebugCategoryName = "ClientBandwidth";

	// Command to Cycle Debug Categories
	FAutoConsoleCommand FCycleDebugCategory(
		TEXT("net.Debug.Cycle"),
		TEXT("Cycles to the next category or provided category name."),
		FConsoleCommandWithArgsDelegate::CreateStatic([](const TArray<FString>& Args)
			{
				UClientBandwidthDebugVisualizer& Visualizer = UClientBandwidthDebugVisualizer::Get();
				if (Args.IsEmpty())
				{
					Visualizer.CycleToNextCategory(FString());
				}
				else
				{
					Visualizer.CycleToNextCategory(Args[0]);
				}
			}),
		ECVF_Cheat);
}

// ---------------------------
// Singleton backing pointer
// ---------------------------
TStrongObjectPtr<UClientBandwidthDebugVisualizer> UClientBandwidthDebugVisualizer::Instance = nullptr;

/// Static Functions
UClientBandwidthDebugVisualizer& UClientBandwidthDebugVisualizer::Get()
{
	if (!Instance)
	{
		Instance = TStrongObjectPtr<UClientBandwidthDebugVisualizer>(NewObject<UClientBandwidthDebugVisualizer>());
	}

	check(Instance);
	return *Instance.Get();
}

/// Member Functions
void UClientBandwidthDebugVisualizer::Initialize()
{
	if (!WorldActorInitialization.IsValid())
	{
		WorldActorInitialization = FWorldDelegates::OnWorldInitializedActors.AddLambda([this](const FActorsInitializedParams& InitializationParams)
			{
				if (InitializationParams.World && !IsRunningCommandlet())
				{
					if (InitializationParams.World->WorldType != EWorldType::Game && InitializationParams.World->WorldType != EWorldType::PIE)
					{
						return;
					}

					if (CurrentWorld.IsValid() && CurrentWorld.Get() != InitializationParams.World)
					{
						// We have a new world, but an old world reference
						CurrentWorld->RemoveOnActorSpawnedHandler(OnActorSpawnedHandle);
					}

					CurrentWorld = InitializationParams.World;
					OnActorSpawnedHandle = CurrentWorld->GetOnBeginPlayEvent().AddUObject(this, &UClientBandwidthDebugVisualizer::ClientPlayerControllerCollection);
				}
			});
	}

	if (!AddDebugTextHandle.IsValid())
	{
		AddDebugTextHandle = IClientBandwidthGlobalDelegates::BindToTextAdditionForDebugDisplay([this](FStringView CategoryName, FStringView TextToPrint, float Scale, FColor Color, bool bIsSubHeader)
			{
				AddDebugMessageToVisualizer(CategoryName, TextToPrint, Scale, Color, bIsSubHeader);
			});
	}

	if (!ClearDebugTextHandle.IsValid())
	{
		ClearDebugTextHandle = IClientBandwidthGlobalDelegates::BindToClearDebugInfoForTick([this](FStringView CategoryName)
			{
				ClearCategory(CategoryName);
			});
	}

	if (!ConsoleAutocomplete.IsValid())
	{
		ConsoleAutocomplete = UConsole::RegisterConsoleAutoCompleteEntries.AddLambda([](TArray<FAutoCompleteCommand>& AutoCompleteList)
			{
				const UConsoleSettings* ConsoleSettings = GetDefault<UConsoleSettings>();

				AutoCompleteList.AddDefaulted();
				FAutoCompleteCommand& AutoCompleteCommand = AutoCompleteList.Last();

				AutoCompleteCommand.Command = TEXT("showdebug ClientBandwidth");
				AutoCompleteCommand.Desc = TEXT("Toggles display of Client Bandwidth Debug Information.");
				AutoCompleteCommand.Color = ConsoleSettings->AutoCompleteCommandColor;
			});
	}
}

void UClientBandwidthDebugVisualizer::Shutdown()
{
	IClientBandwidthGlobalDelegates::UnbindAllToTextAdditionForDebugDisplay(AddDebugTextHandle);
	IClientBandwidthGlobalDelegates::UnbindAllToClearDebugInfoForTick(ClearDebugTextHandle);

	if (CurrentWorld.IsValid())
	{
		// We have a new world, but an old world reference
		CurrentWorld->RemoveOnActorSpawnedHandler(OnActorSpawnedHandle);
	}

	FWorldDelegates::OnWorldInitializedActors.Remove(WorldActorInitialization);
	UConsole::RegisterConsoleAutoCompleteEntries.Remove(ConsoleAutocomplete);

	if (Instance)
	{
		Instance.Reset();
	}
}

void UClientBandwidthDebugVisualizer::ClientPlayerControllerCollection(bool bHasBegun)
{
	if (CurrentWorld.IsValid() && !IsRunningCommandlet())
	{
		// We are not guaranteed the controller passed is the first/clients controller
		if (APlayerController* ClientPC = Cast<APlayerController>(CurrentWorld->GetFirstPlayerController()))
		{
			if (ClientPlayerController.Get() != ClientPC)
			{
				ClientPlayerController = ClientPC;
			}
			else if (ClientPlayerController.Get() == ClientPC)
			{
				// we have already bound to this controller's HUD
				return;
			}

			AHUD* PlayerHUD = Cast<AHUD>(ClientPlayerController->GetHUD());
			if (PlayerHUD)
			{
				PlayerHUD->OnShowDebugInfo.AddUObject(this, &UClientBandwidthDebugVisualizer::DisplayDebugInformation);

			}
		}
	}
}

void UClientBandwidthDebugVisualizer::ClearCategory(const FStringView& CategoryName)
{
	TArray<UE::ClientBandwidthDebug::FDebugTextInformation>* DebugTextList = DebugPrintText.Find(FString(CategoryName));
	if (DebugTextList)
	{
		DebugTextList->Empty();
	}
}

void UClientBandwidthDebugVisualizer::DisplayDebugInformation(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	if (DisplayInfo.IsDisplayOn(CBDV::GDebugCategoryName))
	{
		constexpr float TextPadding = 5.0f;
		UFont* Font = GEngine->GetMediumFont();
		float StartXPos = Canvas->DisplayDebugManager.GetXPos();
		float BoxMaxX = StartXPos;
		float BoxMinY = YPos;
		TArray<FCanvasTextItem> TextToDraw;

		FString TitleText = FString(CurrentCategory);
		TitleText += TEXT(" Debug Information:");

		const float ScaleDenominator = (static_cast<float>(Canvas->SizeY) * Canvas->GetDPIScale());
		const float MeasuredTextScale = (ScaleDenominator > 0.0f) ? static_cast<float>(Canvas->SizeX) / ScaleDenominator : 1.0f;
		float SafeScaling = FMath::Max(MeasuredTextScale, 1.0f);

		//Top Padding
		YPos += TextPadding;

		//Calculate Text positioning and space needed for background box for the title
		FCanvasTextItem TitleTextItem(FVector2D(StartXPos + TextPadding, YPos), FText::FromString(TitleText), Font, FColor::Yellow);
		TitleTextItem.Scale = FVector2D(SafeScaling, SafeScaling);
		TextToDraw.Add(TitleTextItem);

		float TitleTextWidth = 0.0f;
		float TitleTextHeight = 0.0f;
		HUD->GetTextSize(TitleText, TitleTextWidth, TitleTextHeight, Font, SafeScaling);

		BoxMaxX = (TitleTextWidth + StartXPos > BoxMaxX) ? TitleTextWidth + StartXPos : BoxMaxX;
		YPos += TitleTextHeight;

		// Calculate Text positioning and space needed for background box
		FString CurrentCategoryString = FString(CurrentCategory);
		TArray<UE::ClientBandwidthDebug::FDebugTextInformation>* DebugTextList = DebugPrintText.Find(CurrentCategoryString);
		if (!DebugTextList)
		{
			UE_LOGF(LogBandwidthDebugger, Error, "Bandwidth Debugger attempted to display with an invalid Current Category: %ls", *CurrentCategoryString);
			return;
		}

		for (UE::ClientBandwidthDebug::FDebugTextInformation& TextInfo : *DebugTextList)
		{
			float TextScaleToUse = (TextInfo.Scale != 0.0f) ? TextInfo.Scale : MeasuredTextScale;
			SafeScaling = FMath::Max(TextScaleToUse, 1.0f);

			if (TextInfo.bIsSubHeader)
			{
				YPos += YL;
			}

			FCanvasTextItem TextItem(FVector2D(StartXPos + TextPadding, YPos), FText::FromString(TextInfo.Text), Font, TextInfo.TextColor);
			TextItem.Scale = FVector2D(SafeScaling, SafeScaling);
			TextToDraw.Add(TextItem);

			float TextWidth = 0.0f;
			float TextHeight = 0.0f;
			HUD->GetTextSize(TextInfo.Text, TextWidth, TextHeight, Font, SafeScaling);

			BoxMaxX = (TextWidth + StartXPos > BoxMaxX) ? TextWidth + StartXPos : BoxMaxX;
			YPos += TextHeight;
		}

		BoxMaxX += TextPadding;
		YPos += TextPadding;

		// Send Draw Instructions
		Canvas->SetDrawColor(FColor(0,0,0,192));
		Canvas->DrawTile(Cast<UTexture>(Canvas->DefaultTexture.Get()), StartXPos, BoxMinY, BoxMaxX, YPos - BoxMinY, 0.0f, 0.0f, 1.0f, 1.0f, EBlendMode::BLEND_Translucent);
		Canvas->SetDrawColor(FColor::White);

		for (FCanvasTextItem& TextItem : TextToDraw)
		{
			Canvas->DrawItem(TextItem);
		}
	}
}

void UClientBandwidthDebugVisualizer::AddDebugMessageToVisualizer(FStringView CategoryName, FStringView TextToPrint, float Scale, FColor Color, bool bIsSubHeader)
{
	TArray<UE::ClientBandwidthDebug::FDebugTextInformation>& TextList = DebugPrintText.FindOrAdd(FString(CategoryName));
	UE::ClientBandwidthDebug::FDebugTextInformation TextInfo;
	TextInfo.Text = FString(TextToPrint); 
	TextInfo.Scale = Scale;
	TextInfo.TextColor = Color;
	TextInfo.bIsSubHeader = bIsSubHeader;
	TextList.Add(TextInfo);

	if (CurrentCategory.IsEmpty())
	{
		// Makes sure the String VIew is the stored string in DebugPrintText
		CurrentCategory = *DebugPrintText.FindKey(TextList);
	}
}

void UClientBandwidthDebugVisualizer::CycleToNextCategory(const FString& JumpCategory)
{
	if ((!JumpCategory.IsEmpty() && DebugPrintText.Find(JumpCategory)))
	{
		CurrentCategory = JumpCategory;
	}
	else
	{
		TArray<FString> CategoryList;
		DebugPrintText.GetKeys(CategoryList);
		if (!CategoryList.IsEmpty())
		{
			int32 CurrentIndex = CategoryList.IndexOfByKey(CurrentCategory);

			if (CurrentIndex == INDEX_NONE)
			{
				CurrentIndex = 0;
			}

			int32 NextIndex = (CurrentIndex + 1) % CategoryList.Num();
			CurrentCategory = CategoryList[NextIndex];
		}
	}
}