// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ClientBandwidthDebugVisualizer.h"

// ClientBandwidthDebugVisualizer module
// Debug Visualizer for client side bandwidth data

class FClientBandwidthDebugVisualizerModule : public IModuleInterface
{
	void StartupModule()
	{
		UClientBandwidthDebugVisualizer::Get().Initialize();
	}

	void ShutdownModule()
	{
		UClientBandwidthDebugVisualizer::Get().Shutdown();
	}
};

IMPLEMENT_MODULE(FClientBandwidthDebugVisualizerModule, ClientBandwidthDebugVisualizer)