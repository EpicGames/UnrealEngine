// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DisplayDebugHelpers.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"

#include "ClientBandwidthDebugVisualizer.generated.h"

#define UE_API CLIENTBANDWIDTHDEBUGVISUALIZER_API

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogBandwidthDebugger, Log, All);

class AHUD;
class UCanvas;

namespace UE::ClientBandwidthDebug
{
	//typedef TArray<char> FAnsiString;

	struct FDebugTextInformation
	{
		FDebugTextInformation() {};

		FDebugTextInformation(const FDebugTextInformation& Copy) 
			:Scale(Copy.Scale),
			TextColor(Copy.TextColor),
			bIsSubHeader(Copy.bIsSubHeader)
		{
			Text.Append(*(const_cast<FDebugTextInformation*>(&Copy)->Text));
		};

		bool operator==(const FDebugTextInformation& Compare) const
		{
			return Scale == Compare.Scale &&
				TextColor == Compare.TextColor &&
				bIsSubHeader == Compare.bIsSubHeader &&
				Text == Compare.Text;
		};

		FString Text;
		float Scale = 1.0f;
		FColor TextColor;
		bool bIsSubHeader = false;
	};
}

UCLASS(MinimalApi)
class UClientBandwidthDebugVisualizer : public UObject
{
	GENERATED_BODY()

public:
	// --- Static Functions ---
	UE_API static UClientBandwidthDebugVisualizer& Get();

	// --- Lifecycle (called by module) ---
	void Initialize();
	void Shutdown();

	// Displays Categorized Information to Debug HUD
	UE_API void DisplayDebugInformation(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	// Adds the supplied message to the Debug Text that is displayed, if category name does not exist it will be added
	UE_API void AddDebugMessageToVisualizer(FStringView CategoryName, FStringView TextToPrint, float Scale, FColor Color, bool bIsSubHeader);

	// Cycles to the next category/page of text to display
	UE_API void CycleToNextCategory(const FString& JumpCategory);

private:
	// Function to set trackable player controller and HUD bindings
	void ClientPlayerControllerCollection(bool bHasBegun);

	// Clears Category for debug entries
	void ClearCategory(const FStringView& CategoryName);

	// Singleton instance pointer
	static TStrongObjectPtr<UClientBandwidthDebugVisualizer> Instance;

	TMap<FString, TArray<UE::ClientBandwidthDebug::FDebugTextInformation>> DebugPrintText;

	FStringView CurrentCategory;

	TWeakObjectPtr<APlayerController> ClientPlayerController;

	FDelegateHandle OnActorSpawnedHandle;

	TWeakObjectPtr<UWorld> CurrentWorld;

	FDelegateHandle AddDebugTextHandle;
	FDelegateHandle ClearDebugTextHandle;
	FDelegateHandle WorldActorInitialization;
	FDelegateHandle ConsoleAutocomplete;
};
#undef UE_API