// Copyright Epic Games, Inc. All Rights Reserved.

#include "DownlinkBandwidthManager.h"
#include "BandwidthMeasurementTool.h"
#include "BandwidthDebugDelegates.h"
#include "DownlinkBandwidthManagerConfig.h"
#include "HAL/IConsoleManager.h"
#include "Math/NumericLimits.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectGlobals.h"

#if CSV_PROFILER_STATS
#include "ProfilingDebugging/CsvProfiler.h"
#endif

#include <random>

DEFINE_LOG_CATEGORY(LogDownlinkBandwidthManager);

#if CSV_PROFILER_STATS
CSV_DEFINE_CATEGORY_MODULE(DOWNLINKBANDWIDTHMANAGER_API, BandwidthAllocator, true);
#endif

namespace DLBW
{
	/// Console Variables to determine Manager behavior
	int32 GDefaultBandwidthToAllocate = 300000; // 300 Mbps
	static FAutoConsoleVariableRef CVarDLBWDefaultBandwidthToAllocate(
		TEXT("net.DLBW.DefaultBandwidth"),
		GDefaultBandwidthToAllocate,
		TEXT("Default value to use for bandwidth cap distribution in Kbps. Example: Average American home internet is 200-300Mbps, so we would use 200,000 - 300,000 for this value."),
		ECVF_Default);

	int32 GMinRequiredBandwidthDuringGameplay = 50000; // 50 Mbps
	static FAutoConsoleVariableRef CVarDLBWMinRequiredBandwidthDuringGameplay(
		TEXT("net.DLBW.MinGameplayBandwidth"),
		GMinRequiredBandwidthDuringGameplay,
		TEXT("This is a failsafe value, meant to act as the reserved size minimum we want to set aside should the value collection from the UNetDriver be too small."),
		ECVF_Default);

	static TAutoConsoleVariable<int32> CVarDLBWPriorityWeightExponent(
		TEXT("net.DLBW.PriorityWeightExponent"),
		3,
		TEXT("Bias weight used as an exponent for bucket distribution."),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarDLBWDistributionPolling(
		TEXT("net.DLBW.DistributionPolling"),
		1.0f,
		TEXT("Delay Time, per frame cycle to poll for changes needed in distribution."),
		ECVF_Default);

	static float GBandwidthPercentageToUse = 0.8f; 
	static FAutoConsoleVariableRef CVarDLBWBandwidthPercentToUse(
		TEXT("net.DLBW.BandwidthPercentage"),
		GBandwidthPercentageToUse,
		TEXT("0 - 1 value used to control the amount of measured bandwidth the manager will utilize."),
		ECVF_Default);

	static int32 GRolloutPercentage = -1;
	static FAutoConsoleVariableRef CVarDLBWRolloutPercentage(
		TEXT("net.DLBW.RolloutChance"),
		GRolloutPercentage,
		TEXT("A value from 0-100 that will be a percentage chance for the user to use values from the Distribution Manager. This is rolled per launch! For more guaranteed testing, enable the individual services for DLBW directly."),
		ECVF_Default);

	static bool GEnforceDistributionAllocation = false;
	static FAutoConsoleVariableRef CVarDLBWEnforcement(
		TEXT("net.DLBW.Enforcement"),
		GEnforceDistributionAllocation,
		TEXT("A true/false value used to denote whether we are in the rollout phase for applying the distributor's calculated allocations to subscribed systems."),
		ECVF_Default);

	static int32 GAutomationOverrideRolloutPercentage = -1;
	static FAutoConsoleVariableRef CVarDLBWOverrideRolloutPercentage(
		TEXT("net.DLBW.Auto.RolloutChance"),
		GAutomationOverrideRolloutPercentage,
		TEXT("A value from 0-100 that will override net.DLBW.RolloutChance and the Config.ini RolloutPercentage, set to -1 to disable."),
		ECVF_Default);

	static bool GAutomationOverrideEnforceDistributionAllocation = false;
	static FAutoConsoleVariableRef CVarDLBWOverrideEnforcement(
		TEXT("net.DLBW.Auto.Enforcement"),
		GAutomationOverrideEnforceDistributionAllocation,
		TEXT("A true/false value the will override net.DLBW.Enforcement and the Config.ini EnforceDistributionAllocation."),
		ECVF_Default);

	static bool GEnableLogPrinting = false;
	static FAutoConsoleVariableRef CVarDLBWEnableLogPrinting(
		TEXT("net.DLBW.PrintLog"),
		GEnableLogPrinting,
		TEXT("Enables Debug Information to print to the log."),
		ECVF_Default);

	// Command to print logging information
	FAutoConsoleCommand FDLBWShowBandwidthTelemetry(
		TEXT("net.DLBW.PrintTelemetry"),
		TEXT("Prints a snapshot of the manager data for logging purposes."),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				FDownlinkBandwidthManager& DLBWManager = FDownlinkBandwidthManager::Get();
				DLBWManager.Log(true);
			}),
		ECVF_Cheat);
}

/// Non-Member Functions
FString LexToString(UE::BandwidthManager::EBandwidthPriority Value)
{
	switch (Value)
	{
	case UE::BandwidthManager::EBandwidthPriority::HighPriority:
		return TEXT("High Priority");
	case UE::BandwidthManager::EBandwidthPriority::MediumPriority:
		return TEXT("Medium Priority");
	case UE::BandwidthManager::EBandwidthPriority::LowPriority:
		return TEXT("Low Priority");
	default:
		return TEXT("When Available");
	}
}

// ---------------------------
// Singleton backing pointer
// ---------------------------
TUniquePtr<FDownlinkBandwidthManager> FDownlinkBandwidthManager::Instance = nullptr;

/// Static Functions
FDownlinkBandwidthManager& FDownlinkBandwidthManager::Get()
{
	if (!Instance)
	{
		Instance = MakeUnique<FDownlinkBandwidthManager>();
	}
	check(Instance);
	return *Instance.Get();
}

bool FDownlinkBandwidthManager::HasLocalUserPassedRolloutCheck()
{
	int32 PassPercentage = (DLBW::GAutomationOverrideRolloutPercentage != -1) ? DLBW::GAutomationOverrideRolloutPercentage : DLBW::GRolloutPercentage;

	FDownlinkBandwidthManager& DLBWManager = FDownlinkBandwidthManager::Get();
	return static_cast<float>(PassPercentage) >= DLBWManager.RolloutPercentage;
}

bool FDownlinkBandwidthManager::HasValueEnforcementPassed()
{
	FDownlinkBandwidthManager& DLBWManager = FDownlinkBandwidthManager::Get();
	return (DLBW::GAutomationOverrideEnforceDistributionAllocation || DLBW::GEnforceDistributionAllocation) && HasLocalUserPassedRolloutCheck();
}

void BroadcastCVarSubscription()
{
	FDownlinkBandwidthManager& DLBWManager = FDownlinkBandwidthManager::Get();
	DLBWManager.CVarSubscriptionDelegate.Broadcast();
}

/// Member Functions
FDownlinkBandwidthManager::FDownlinkBandwidthManager()
{

}

FDownlinkBandwidthManager::~FDownlinkBandwidthManager()
{
	TrackedServices.Empty();
}

FDelegateHandle FDownlinkBandwidthManager::BindToCVarSubscription(TFunction<void()> CallBack)
{
	return CVarSubscriptionDelegate.AddLambda(CallBack);
}

void FDownlinkBandwidthManager::Initialize()
{
	// Get Rollout Data
	if (UClass* ConfigClass = UDownlinkBandwidthManagerConfig::StaticClass())
	{
		if (UDownlinkBandwidthManagerConfig* Config = ConfigClass->GetDefaultObject<UDownlinkBandwidthManagerConfig>())
		{
			Config->SetConsoleVariablesFromConfigurables();
		}
	}

	std::random_device Device;
	std::mt19937 Generator(Device());
	std::uniform_real_distribution<float> Distribution(0.0f, 100.0f);

	// A value in the range [0..100), EG. inclusive of 0, exclusive of 100.
	RolloutPercentage = Distribution(Generator);

	// Set an initial value
	SetIncomingBandwidthCap(DLBW::GDefaultBandwidthToAllocate);
	
	FBandwidthMeasurementTool& BandwidthTool = FBandwidthMeasurementTool::Get();
	BandwidthTool.RegisterForCompletedMeasurement([this](uint64 MeasurementInBps)
		{
			const int32 RestrictedCap = static_cast<int32>(static_cast<float>(MeasurementInBps) *
				FMath::Clamp(DLBW::GBandwidthPercentageToUse, 0.0f, 1.0f));

			const uint64 MeasurementInKbps = RestrictedCap / UE::BandwidthManager::KilobitsToBytesConversionValue;
			SetIncomingBandwidthCap(static_cast<int32>(MeasurementInKbps));

			// Track measurement outcome for telemetry so we can report whether bandwidth
			// estimation ran and produced a valid result, independent of the cap applied above.
			bTelemetryMeasurementCompleted = true;
			bTelemetryMeasurementSuccess = (MeasurementInBps > 0);
		});

	DLBW::CVarDLBWOverrideRolloutPercentage->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda(
		[](IConsoleVariable* ConsoleVariable)
		{
			BroadcastCVarSubscription();
		}));

	DLBW::CVarDLBWDefaultBandwidthToAllocate->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda(
		[this](IConsoleVariable* ConsoleVariable) 
		{
			int32 NewCap;
			ConsoleVariable->GetValue(NewCap);
			SetIncomingBandwidthCap(NewCap);
		}));

	// return true for reschedule
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float InDeltaSeconds)->bool
		{
			// let's us evaluate distribution on time cycle needs should anything change
			Tick();
			return true;
		}), DLBW::CVarDLBWDistributionPolling.GetValueOnAnyThread());
}

void FDownlinkBandwidthManager::Shutdown()
{
	FTSTicker::GetCoreTicker().RemoveTicker(TickHandle);
	PriorityBucketMapping.Empty();
}

void FDownlinkBandwidthManager::SetIncomingBandwidthCap(int32 InKbps)
{
	UE::TScopeLock Lock(Mutex);

	if (CapKbps != InKbps)
	{
		CapKbps = InKbps;

		// a change to the cap value means we need to perform a full re-distribution
		GeneratePriorityBucketMap();
	}
}

UE::BandwidthManager::FServiceHandleID FDownlinkBandwidthManager::RegisterMonitoredService(
	const FString& ServiceName,
	UE::BandwidthManager::FManagedServiceConfigurableData& ServiceConfigData,
	bool StartActive /* = false*/,
	FOnBandwidthUpdate&& Callback /* = nullptr*/)
{
	UE::TScopeLock Lock(Mutex);

	FManagedService NewService(MoveTemp(Callback));
	NewService.DebugName = ServiceName;
	NewService.ConfigData = ServiceConfigData;
	NewService.bActiveConnection = StartActive;

	NextServiceId++;
	TrackedServices.Emplace(NextServiceId, NewService);

	TrackedServices.ValueSort([&](const FManagedService& ServiceA, const FManagedService& ServiceB) 
		{
			uint8 ValueA = (CurrentGameplayPriorityState == UE::BandwidthManager::EBandwidthDistributionPriorityState::IdleGameplay) ? ServiceA.ConfigData.IdleGameplayPriority : ServiceA.ConfigData.InGameplayPriority;
			uint8 ValueB = (CurrentGameplayPriorityState == UE::BandwidthManager::EBandwidthDistributionPriorityState::IdleGameplay) ? ServiceB.ConfigData.IdleGameplayPriority : ServiceB.ConfigData.InGameplayPriority;
			// Sorts in ascending priority, so lower numerical (higher listed) services can be allotted to first
			return ValueA < ValueB;
		});

	if (StartActive)
	{
		GeneratePriorityBucketMap();
	}

	return NextServiceId;
}


void FDownlinkBandwidthManager::UnregisterMonitoredService(UE::BandwidthManager::FServiceHandleID& ServiceID)
{
	UE::TScopeLock Lock(Mutex);

	if (ServiceID == UE_INVALID_BANDWIDTH_SERVICE_HANDLE)
	{
		return;
	}

	// Only regenerate if we actually remove a service
	if (TrackedServices.Remove(ServiceID) != 0)
	{
		ServiceID = UE_INVALID_BANDWIDTH_SERVICE_HANDLE;
		GeneratePriorityBucketMap();
	}
}

void FDownlinkBandwidthManager::UnbindToCVarSubscription(FDelegateHandle& Handle)
{ 
	if (Handle.IsValid())
	{
		CVarSubscriptionDelegate.Remove(Handle);
	}
}

void FDownlinkBandwidthManager::SetActivationForService(const UE::BandwidthManager::FServiceHandleID& ServiceID, bool ActiveState)
{
	UE::TScopeLock Lock(Mutex);

	if (ServiceID == UE_INVALID_BANDWIDTH_SERVICE_HANDLE)
	{
		return;
	}

	if (FManagedService* Service = TrackedServices.Find(ServiceID))
	{
		if (Service->bActiveConnection != ActiveState)
		{
			Service->bActiveConnection = ActiveState;
			GeneratePriorityBucketMap();
		}
	}
}

void FDownlinkBandwidthManager::ChangeServicePriority(const UE::BandwidthManager::FServiceHandleID& ServiceID, UE::BandwidthManager::EBandwidthPriority NewPriority, UE::BandwidthManager::EBandwidthDistributionPriorityState PriorityState)
{
	UE::TScopeLock Lock(Mutex);

	if (ServiceID == UE_INVALID_BANDWIDTH_SERVICE_HANDLE)
	{
		return;
	}

	if (FManagedService* Service = TrackedServices.Find(ServiceID))
	{
		bool DidPriorityChange = false;
		switch (PriorityState) 
		{
		case UE::BandwidthManager::EBandwidthDistributionPriorityState::IdleGameplay:
			DidPriorityChange = Service->ConfigData.IdleGameplayPriority != NewPriority;
			Service->ConfigData.IdleGameplayPriority = NewPriority;
			break;
		case UE::BandwidthManager::EBandwidthDistributionPriorityState::InGameplay:
			DidPriorityChange = Service->ConfigData.InGameplayPriority != NewPriority;
			Service->ConfigData.InGameplayPriority = NewPriority;
			break;
		default:
			DidPriorityChange = Service->ConfigData.InGameplayPriority != NewPriority
				|| Service->ConfigData.IdleGameplayPriority != NewPriority;
			Service->ConfigData.IdleGameplayPriority = NewPriority;
			Service->ConfigData.InGameplayPriority = NewPriority;
			break;
		}

		if (DidPriorityChange)
		{
			GeneratePriorityBucketMap();
		}
	}
}

int32 FDownlinkBandwidthManager::GetCurrentMaxAvailableBandwidthInBytes()
{
	int32 AvailableBandwithKbps = FMath::Abs(CapKbps);

	if (CurrentGameplayPriorityState == UE::BandwidthManager::EBandwidthDistributionPriorityState::InGameplay)
	{
		int32 ReducedBandwidth = CapKbps - ReservedGameplayBandwidthKbps;
		AvailableBandwithKbps = (ReducedBandwidth <= 0) ? DLBW::GMinRequiredBandwidthDuringGameplay : ReducedBandwidth;
	}

	//overflow protection for byte conversion
	constexpr int32 IntMax = TNumericLimits<int32>::Max(); 
	return (AvailableBandwithKbps > IntMax / UE::BandwidthManager::KilobitsToBytesConversionValue) ? IntMax : AvailableBandwithKbps * UE::BandwidthManager::KilobitsToBytesConversionValue;
}

void FDownlinkBandwidthManager::SetDistributionGameplayState(UE::BandwidthManager::EBandwidthDistributionPriorityState State)
{
	UE::TScopeLock Lock(Mutex);

	if (CurrentGameplayPriorityState != State)
	{
		CurrentGameplayPriorityState = State;
		GeneratePriorityBucketMap();
	}
}

void FDownlinkBandwidthManager::Tick()
{
	// Called as a background poll to see if redistribution is needed
	DistributeBandwidth();

#if CSV_PROFILER_STATS
	UpdateCsvStats();
#endif

	Log(DLBW::GEnableLogPrinting);
}

#if CSV_PROFILER_STATS
void FDownlinkBandwidthManager::UpdateCsvStats()
{
	if (!FCsvProfiler::Get()->IsCapturing())
	{
		return;
	}

	const FBandwidthAllocatorTelemetry Snapshot = GetTelemetrySnapshot();

	const int32 DeltaDecisions = Snapshot.DecisionCount - CsvPreviousDecisionCount;
	const int32 DeltaReallocations = Snapshot.ReallocationCount - CsvPreviousReallocationCount;
	const int32 DeltaErrors = Snapshot.AllocatorErrors - CsvPreviousAllocatorErrors;
	CsvPreviousDecisionCount = Snapshot.DecisionCount;
	CsvPreviousReallocationCount = Snapshot.ReallocationCount;
	CsvPreviousAllocatorErrors = Snapshot.AllocatorErrors;

	// Allocator state
	CSV_CUSTOM_STAT(BandwidthAllocator, EffectiveBandwidthKbps, Snapshot.EffectiveBandwidthKbps, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, NumRegisteredSystems, Snapshot.NumRegisteredSystems, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, Enabled, Snapshot.bEnabled ? 1 : 0, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, Enforcing, Snapshot.bEnforcing ? 1 : 0, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, InGameplay, Snapshot.bInGameplay ? 1 : 0, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, BandwidthPercentage, Snapshot.BandwidthPercentage, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, MinGameplayReserveKbps, Snapshot.MinGameplayReserveKbps, ECsvCustomStatOp::Set);

	// Per-tick deltas
	CSV_CUSTOM_STAT(BandwidthAllocator, DecisionsPerTick, DeltaDecisions, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, ReallocationsPerTick, DeltaReallocations, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, ErrorsPerTick, DeltaErrors, ECsvCustomStatOp::Set);

	// Measurement data
	CSV_CUSTOM_STAT(BandwidthAllocator, MeasuredBandwidthKbps, Snapshot.MeasuredBandwidthKbps, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, MeasurementAverageKbps, Snapshot.MeasurementAverageKbps, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, MeasurementMinKbps, Snapshot.MeasurementMinKbps, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, MeasurementMaxKbps, Snapshot.MeasurementMaxKbps, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(BandwidthAllocator, MeasurementVariancePct, Snapshot.MeasurementVariancePercent, ECsvCustomStatOp::Set);
}
#endif

void FDownlinkBandwidthManager::GeneratePriorityBucketMap()
{
	UE::TScopeLock Lock(Mutex);

	int32 BandwidthBudgetBytes = GetCurrentMaxAvailableBandwidthInBytes();
	if (BandwidthBudgetBytes <= 0)
	{
		TelemetryAllocatorErrors++;
		UE_LOGF(LogDownlinkBandwidthManager, Warning, "%ls - BandwidthBudgetPerFrame was set to less than 0. Value: %i, CapKbps: %i",
			UE_SOURCE_LOCATION, BandwidthBudgetBytes, CapKbps);
		return;
	}

	// If we are generating a new set of buckets, then we need to remove stale data
	if (PriorityBucketMapping.Num() != 0)
	{
		PriorityBucketMapping.Reset();
	}

	// We need to generate the current priority buckets since services could have changed
	for (TPair<UE::BandwidthManager::FServiceHandleID, FManagedService>& ServicePair : TrackedServices)
	{
		const FManagedService& ServiceRef = ServicePair.Value;
		if (!ServiceRef.bActiveConnection)
		{
			//if not active, then don't distribute bandwidth for it
			continue;
		}

		UE::BandwidthManager::FPriorityBucket& PriorityBucket = PriorityBucketMapping.FindOrAdd(GetPriorityValueForService(ServiceRef));
		PriorityBucket.NumberOfServicesAtPriority++;
	}
	
	// we do not need an exponent larger than 255, so this lets us make sure that value is not too large and within the limits of the formula
	const int32 InitialBandwidthToBudget = BandwidthBudgetBytes;
	uint8 WeightBiasExponent = FMath::Max<uint8>(1, static_cast<uint8>(DLBW::CVarDLBWPriorityWeightExponent.GetValueOnAnyThread()));
	const int32 BucketCount = PriorityBucketMapping.Num();
	int32 BucketNumber = 0;
	double OverallReduction = 0.0;
	for (TPair<UE::BandwidthManager::EBandwidthPriority, UE::BandwidthManager::FPriorityBucket>& PriorityPair : PriorityBucketMapping)
	{
		if (BandwidthBudgetBytes <= 0)
		{
			// we seem to be out of bandwidth
			break;
		}

		UE::BandwidthManager::FPriorityBucket& BucketRef = PriorityPair.Value;
		// This method of distribution allows the highest priority bucket to take the 
		// highest percentage based on a weighted exponent value, then trickle down the
		// remainder while still considering the weight.
		// Example:
		// PriorityList = [High, Medium, Low], WeightBiasExponent = 3
		// This would yield a starting WeightedPercentage = 0.702
		// The next Cycle would be interpolate from 0 - 0.702 and then WeightedPercentage = 0.209
		// The Overall Reduction becomes 0.911, meaning the final bucket gets 1-0.911 or 0.089
		// In Percentages of the total bandwidth that is 70.2, 20.9, and 8.9%
		// This is the total bandwidth each bucket is allowed and that bucket then needs to
		// subdivide its allowance across it's own services

		if (++BucketNumber == BucketCount)
		{
			// Last Item, we want to give out what's left to maximize bandwidth usage
			BucketRef.BandwidthAllocationForPriority = BandwidthBudgetBytes;
		}
		else
		{
			double WeightedPercentage = FMath::InterpEaseOut<double>(0.0, 1.0 - OverallReduction, 1.0f / (float)BucketCount, WeightBiasExponent);
			OverallReduction += WeightedPercentage;

			// This is why we are tracking the remainder by bandwidth and not used percentage
			int32 WeightedBandwidth = static_cast<int32>(static_cast<double>(InitialBandwidthToBudget) * WeightedPercentage);
			BandwidthBudgetBytes -= FMath::Min<int32>(WeightedBandwidth, BandwidthBudgetBytes);
			// ^^^ Prevents any potential negative roll over
			BucketRef.BandwidthAllocationForPriority = WeightedBandwidth;

		}
	}

	// Now that the buckets have been created/changed, we need to distribute bandwidth to services
	DistributeBandwidth();
}

void FDownlinkBandwidthManager::DistributeBandwidth()
{
	UE::TScopeLock Lock(Mutex);

	TelemetryDecisionCount++;

	int32 BandwidthBudgetBytes = GetCurrentMaxAvailableBandwidthInBytes();
	if (BandwidthBudgetBytes <= 0)
	{
		UE_LOGF(LogDownlinkBandwidthManager, Warning, "%ls - BandwidthBudgetPerFrame was set to less than 0. Value: %i, CapKbps: %i", 
			UE_SOURCE_LOCATION, BandwidthBudgetBytes, CapKbps);
		return;
	}

	// Distributes Allocations based on bucket allowance
	for (TPair<UE::BandwidthManager::FServiceHandleID, FManagedService>& ServicePair : TrackedServices)
	{
		if (BandwidthBudgetBytes <= 0)
		{
			// we seem to be out of bandwidth
			break;
		}

		FManagedService& ServiceRef = ServicePair.Value;
		if (!ServiceRef.bActiveConnection)
		{
			//if not active, then don't distribute bandwidth for it
			if (ServiceRef.OnBandwidthUpdate.IsSet() && ServiceRef.AllotedBandwidthKbps > 0)
			{
				ServiceRef.AllotedBandwidthKbps = 0;
				ServiceRef.OnBandwidthUpdate(MoveTemp(ServiceRef.AllotedBandwidthKbps));
			}
			continue;
		}

		if (UE::BandwidthManager::FPriorityBucket* PriorityBucket = PriorityBucketMapping.Find(GetPriorityValueForService(ServiceRef)))
		{
			// Gives us the highest available bandwidth this priority can have based on total service needs
			int32 CurrentServiceAbsoluteBandwidth = PriorityBucket->BandwidthAllocationForPriority / PriorityBucket->NumberOfServicesAtPriority;

			// If reducing BandwidthBudgetBytes by CurrentServiceAbsoluteBandwidth would yield less than 0 (rollover) then use the smallest
			int32 BandwidthToUse = FMath::Min(BandwidthBudgetBytes,CurrentServiceAbsoluteBandwidth) / UE::BandwidthManager::KilobitsToBytesConversionValue; //bytes to Kb
			if (ServiceRef.ConfigData.ComparisonValue.IsValid())
			{
				// Then take the minimum bandwidth between the "need" and available
				int32 ComparisonBandwidth = FMath::Max(0, *ServiceRef.ConfigData.ComparisonValue.Get());
				ServiceRef.AllotedBandwidthKbps = FMath::Min(BandwidthToUse, ComparisonBandwidth);
			}
			else
			{
				ServiceRef.AllotedBandwidthKbps = BandwidthToUse;
			}

			// reduce total amount by amount distributed
			BandwidthBudgetBytes -= ServiceRef.AllotedBandwidthKbps * UE::BandwidthManager::KilobitsToBytesConversionValue;

			if (DLBW::GEnforceDistributionAllocation && ServiceRef.OnBandwidthUpdate.IsSet() && ServiceRef.AllotedBandwidthKbps > 0)
			{
				ServiceRef.OnBandwidthUpdate(MoveTemp(ServiceRef.AllotedBandwidthKbps));
			}
		}
	}

	// Track reallocations: check if any service allocation changed from previous
	for (const TPair<UE::BandwidthManager::FServiceHandleID, FManagedService>& ServicePair : TrackedServices)
	{
		const int32* PreviousAllocation = PreviousAllocations.Find(ServicePair.Key);
		if (PreviousAllocation && *PreviousAllocation != ServicePair.Value.AllotedBandwidthKbps)
		{
			TelemetryReallocationCount++;
			break;
		}
	}

	// Store current allocations for next comparison
	PreviousAllocations.Reset();
	for (const TPair<UE::BandwidthManager::FServiceHandleID, FManagedService>& ServicePair : TrackedServices)
	{
		PreviousAllocations.Add(ServicePair.Key, ServicePair.Value.AllotedBandwidthKbps);
	}
}

UE::BandwidthManager::EBandwidthPriority FDownlinkBandwidthManager::GetPriorityValueForService(const FManagedService& MonitoredService) const
{
	return (CurrentGameplayPriorityState == UE::BandwidthManager::EBandwidthDistributionPriorityState::IdleGameplay) ? MonitoredService.ConfigData.IdleGameplayPriority : MonitoredService.ConfigData.InGameplayPriority;
}

void FDownlinkBandwidthManager::Log(bool bPrint)
{
	constexpr const FStringView CategoryName = TEXT("Downlink Bandwidth Manager");
	constexpr const float CategoryScale = 2.0f;
	constexpr const float EntryScale = 1.25;
	const FColor CategoryColor = FColor::Emerald;
	const FColor EntryColor = FColor::Cyan;
	TArray<FString> LogText;

	auto WriteLogEntry = [bPrint, &LogText, &CategoryName](const FString& Message, float Scale, FColor Color, bool bIsSubHeader = false)
		{ 
			IClientBandwidthGlobalDelegates::AddTextToDebugDisplay(CategoryName, Message, Scale, Color, bIsSubHeader);
			if (bPrint) 
			{ 
				LogText.Add(Message);
			}
		};

	// Need to clear since we are logging on Tick
	IClientBandwidthGlobalDelegates::ClearDebugInfoForTick(CategoryName);
	
	FString PrintMsg = TEXT("FDownlinkBandwidthManager Internal Data:");
	WriteLogEntry(PrintMsg, CategoryScale, CategoryColor, true);

	PrintMsg = FString::Printf(TEXT("Default Bandwidth Cap in Kbps: %i"), static_cast<int>(DLBW::GDefaultBandwidthToAllocate));
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	PrintMsg = FString::Printf(TEXT("Minimum Bandwidth Cap during gameplay in Kbps: %i"), static_cast<int>(DLBW::GMinRequiredBandwidthDuringGameplay));
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	PrintMsg = FString::Printf(TEXT("Current Bandwidth Cap in Kbps: %i"), static_cast<int>(CapKbps));
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	PrintMsg = FString::Printf(TEXT("Current Gameplay Priority State: %ls"),
		(CurrentGameplayPriorityState == UE::BandwidthManager::EBandwidthDistributionPriorityState::IdleGameplay) ? TEXT("Outside Normal Gameplay") : TEXT("In Active Gameplay"));
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	PrintMsg = FString::Printf(TEXT("Tick Polling in seconds: %0.3f"), DLBW::CVarDLBWDistributionPolling.GetValueOnAnyThread());
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	PrintMsg = TEXT("FDownlinkBandwidthManager Prioritization:");
	WriteLogEntry(PrintMsg, CategoryScale, CategoryColor, true);

	PrintMsg = FString::Printf(TEXT("Priority Weight Exponent: %i"), static_cast<int>(DLBW::CVarDLBWPriorityWeightExponent.GetValueOnAnyThread()));
	WriteLogEntry(PrintMsg, EntryScale, EntryColor);

	for (TPair<UE::BandwidthManager::EBandwidthPriority, UE::BandwidthManager::FPriorityBucket>& BucketData : PriorityBucketMapping)
	{
		PrintMsg = FString::Printf(TEXT("Bucket Priority Value: %ls"), *LexToString(BucketData.Key));
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);

		PrintMsg = FString::Printf(TEXT("Bucket Service Count: %u"), static_cast<unsigned int>(BucketData.Value.NumberOfServicesAtPriority));
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);

		PrintMsg = FString::Printf(TEXT("Bandwidth for this Bucket: %i"), static_cast<int>(BucketData.Value.BandwidthAllocationForPriority));
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);
	}

	PrintMsg = TEXT("FDownlinkBandwidthManager Tracked Services:");
	WriteLogEntry(PrintMsg, CategoryScale, CategoryColor, true);

	for (const TPair<UE::BandwidthManager::FServiceHandleID, FManagedService>& ServiceData : TrackedServices)
	{
		PrintMsg = FString::Printf(TEXT("Tracked Service ID: %i, Name: %ls"), static_cast<int>(ServiceData.Key), *ServiceData.Value.DebugName);
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);

		PrintMsg = FString::Printf(TEXT("Service Gameplay Priority Value: %ls"), *LexToString(ServiceData.Value.ConfigData.InGameplayPriority));
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);

		PrintMsg = FString::Printf(TEXT("Service Idle Priority Value: %ls"), *LexToString(ServiceData.Value.ConfigData.IdleGameplayPriority));
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);

		PrintMsg = FString::Printf(TEXT("Is Service Currently Active: %ls"), (ServiceData.Value.bActiveConnection) ? TEXT("True") : TEXT("False"));
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);

		PrintMsg = TEXT("Service Comparison Value: Null");
		if (ServiceData.Value.ConfigData.ComparisonValue.IsValid())
		{
			PrintMsg = FString::Printf(TEXT("Service Comparison Value: %i"), static_cast<int>(*ServiceData.Value.ConfigData.ComparisonValue.Get()));
		}
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);

		PrintMsg = FString::Printf(TEXT("Service Allotted Bandwidth in Kbps: %i"), static_cast<int>(ServiceData.Value.AllotedBandwidthKbps));
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);

		PrintMsg = FString::Printf(TEXT("Service Bound Callback: %ls"), (ServiceData.Value.OnBandwidthUpdate.IsSet()) ? TEXT("Set") : TEXT("Unset"));
		WriteLogEntry(PrintMsg, EntryScale, EntryColor);
	}

	if (bPrint)
	{
		UE_LOGF(LogDownlinkBandwidthManager, Log, "FDownlinkBandwidthManager Logging Breakdown:");

		for (const FString& Text : LogText)
		{
			UE_LOGF(LogDownlinkBandwidthManager, Log, "%ls", *Text);
		}

		UE_LOGF(LogDownlinkBandwidthManager, Log, "FDownlinkBandwidthManager Logging Complete.");
	}
}

int32 FDownlinkBandwidthManager::GetNumRegisteredServices() const
{
	UE::TScopeLock Lock(Mutex);
	return TrackedServices.Num();
}

FBandwidthAllocatorTelemetry FDownlinkBandwidthManager::GetTelemetrySnapshot() const
{
	FBandwidthAllocatorTelemetry Telemetry;

	bool bMeasurementCompleted = false;

	// Copy DLBW-local data under the lock
	{
		UE::TScopeLock Lock(Mutex);

		Telemetry.bEnabled = TrackedServices.Num() > 0;
		Telemetry.EffectiveBandwidthKbps = CapKbps;
		Telemetry.NumRegisteredSystems = TrackedServices.Num();
		Telemetry.DecisionCount = TelemetryDecisionCount;
		Telemetry.ReallocationCount = TelemetryReallocationCount;
		Telemetry.AllocatorErrors = TelemetryAllocatorErrors;
		Telemetry.bMeasurementSuccess = bTelemetryMeasurementSuccess;
		Telemetry.bInGameplay = (CurrentGameplayPriorityState == UE::BandwidthManager::EBandwidthDistributionPriorityState::InGameplay);
		bMeasurementCompleted = bTelemetryMeasurementCompleted;
	}
	// Release DLBW mutex before accessing BWTool to prevent lock ordering inversion
	// (BWTool callback -> SetIncomingBandwidthCap -> DLBW mutex is the reverse path)

	static IConsoleVariable* UseDLBWManagerCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("iax.UseDLBWManager"));
	Telemetry.bEnforcing = UseDLBWManagerCVar ? UseDLBWManagerCVar->GetBool() : false;

	Telemetry.BandwidthPercentage = DLBW::GBandwidthPercentageToUse;
	Telemetry.MinGameplayReserveKbps = DLBW::GMinRequiredBandwidthDuringGameplay;

	// Measurement data from the BandwidthMeasurementTool (accessed without DLBW lock)
	const FBandwidthMeasurementTool& BWTool = FBandwidthMeasurementTool::Get();
	const FBandwidthMeasurementTool::FBandwidthSnapshotData& MeasurementData = BWTool.GetTelemetryData();

	if (bMeasurementCompleted)
	{
		Telemetry.MeasurementResult = Telemetry.bMeasurementSuccess ? TEXT("Success") : TEXT("Failed");
		Telemetry.MeasuredBandwidthKbps = static_cast<int32>(MeasurementData.RecentMeasurementInBps / UE::BandwidthManager::KilobitsToBytesConversionValue);
		Telemetry.MeasurementDurationSec = static_cast<float>(MeasurementData.RecentMeasurementTime);
		Telemetry.MeasurementRequestCount = static_cast<int32>(MeasurementData.FinishRequestCount);
		Telemetry.MeasurementCompletedCount = static_cast<int32>(MeasurementData.CompleteRequestCount);
		Telemetry.MeasurementHistoryCount = static_cast<int32>(MeasurementData.RecentMeasurementCount);
		Telemetry.MeasurementAverageKbps = static_cast<int32>(MeasurementData.RecentAverageInBps / UE::BandwidthManager::KilobitsToBytesConversionValue);

		// Measurement quality stats
		if (const TOptional<FBandwidthMeasurementTool::FMeasurementQualityStats> QualityStats = BWTool.ComputeMeasurementQuality())
		{
			Telemetry.MeasurementVariancePercent = QualityStats->VariancePercent;
			Telemetry.MeasurementMinKbps = static_cast<int32>(QualityStats->MinBps / UE::BandwidthManager::KilobitsToBytesConversionValue);
			Telemetry.MeasurementMaxKbps = static_cast<int32>(QualityStats->MaxBps / UE::BandwidthManager::KilobitsToBytesConversionValue);
		}
	}
	else
	{
		Telemetry.MeasurementResult = TEXT("NotAttempted");
	}

	// Warmup duration from CVar
	Telemetry.MeasurementWarmupSec = BWTool.GetTimeNeededForMeasurement();

	return Telemetry;
}

void FDownlinkBandwidthManager::ResetTelemetryCounters()
{
	UE::TScopeLock Lock(Mutex);

	TelemetryDecisionCount = 0;
	TelemetryReallocationCount = 0;
	TelemetryAllocatorErrors = 0;
	// Note: bTelemetryMeasurementCompleted/Success are not reset - they reflect lifetime state
}


