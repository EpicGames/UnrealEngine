// Copyright Epic Games, Inc. All Rights Reserved.

#include "DownlinkBandwidthManagerConfig.h"
#include "HAL/IConsoleManager.h"


void UDownlinkBandwidthManagerConfig::PostReloadConfig(class FProperty* PropertyThatWasLoaded)
{
	SetConsoleVariablesFromConfigurables();
}

void UDownlinkBandwidthManagerConfig::SetConsoleVariablesFromConfigurables()
{
	RolloutPercentage = FMath::Clamp(RolloutPercentage, 0, 100);

	if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("net.DLBW.RolloutChance")))
	{
		ConsoleVariable->Set(RolloutPercentage, ECVF_SetByCode);
	}

	if (IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(TEXT("net.DLBW.Enforcement")))
	{
		ConsoleVariable->Set(EnforceDistributionAllocation, ECVF_SetByCode);
	}
}
