// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "DownlinkBandwidthManager.h"

// DownlinkBandwidthManager module
// Central service for dividing available downlink bandwidth across systems.
// Provides per-profile and per-socket throttling, driven by INI config,
// so HTTP/streaming subsystems can share bandwidth fairly by priority.

class FDownlinkBandwidthManagerModule : public IModuleInterface
{
	void StartupModule()
	{
		FDownlinkBandwidthManager::Get().Initialize();
	}

	void ShutdownModule()
	{
		FDownlinkBandwidthManager::Get().Shutdown();
	}
};

IMPLEMENT_MODULE(FDownlinkBandwidthManagerModule, DownlinkBandwidthManager)