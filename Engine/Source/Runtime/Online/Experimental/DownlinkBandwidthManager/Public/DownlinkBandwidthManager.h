// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/Ticker.h"
#include "CoreMinimal.h"

#define UE_API DOWNLINKBANDWIDTHMANAGER_API
#define UE_INVALID_BANDWIDTH_SERVICE_HANDLE -1

#include "ProfilingDebugging/CsvProfilerConfig.h"

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogDownlinkBandwidthManager, Log, All);

#if CSV_PROFILER_STATS
#include "ProfilingDebugging/CsvProfiler.h"
CSV_DECLARE_CATEGORY_MODULE_EXTERN(DOWNLINKBANDWIDTHMANAGER_API, BandwidthAllocator);
#endif

// Usage for this callback will receive a parameter in Kbps
using FOnBandwidthUpdate = TUniqueFunction<void(int32&&)>;

namespace UE::BandwidthManager
{
	enum EBandwidthPriority : uint8
	{
		HighPriority = 0,
		MediumPriority = 1,
		LowPriority = 2,
		WhenAvailable = 3
	};

	enum EBandwidthDistributionPriorityState : uint8
	{
		InGameplay,
		IdleGameplay
	};

	typedef int32 FServiceHandleID;

	DECLARE_MULTICAST_DELEGATE(FOnCVarSetSubscription);

	struct FManagedServiceConfigurableData
	{
		FManagedServiceConfigurableData()
			: ComparisonValue(nullptr),
			InGameplayPriority(EBandwidthPriority::WhenAvailable),
			IdleGameplayPriority(EBandwidthPriority::WhenAvailable) {
		};

		FManagedServiceConfigurableData(const FManagedServiceConfigurableData& Copyable)
			: ComparisonValue(Copyable.ComparisonValue),
			InGameplayPriority(Copyable.InGameplayPriority),
			IdleGameplayPriority(Copyable.IdleGameplayPriority) {
		};

		// Shared Pointer for a preferred bandwidth cap, distribution will 
		// prefer the minimum between this and the calculated bandwidth.
		TSharedPtr<int32> ComparisonValue;
		// The priority to use when the client is considered in active gameplay
		EBandwidthPriority InGameplayPriority;
		// The priority to use when the client is considered in less active gameplay
		EBandwidthPriority IdleGameplayPriority;
	};

	struct FPriorityBucket
	{
		FPriorityBucket() {};
		int32 BandwidthAllocationForPriority = 0;
		uint8 NumberOfServicesAtPriority = 0;
	};

	// Conversion value for Kb to bytes
	constexpr int32 KilobitsToBytesConversionValue = 1000 / 8;
}

struct FManagedService
{

	FManagedService(FOnBandwidthUpdate&& CallBack)
		: bActiveConnection(false),
		ConfigData(),
		AllotedBandwidthKbps(0),
		OnBandwidthUpdate(MoveTemp(CallBack)) {
	};

	FManagedService(const FManagedService& Copyable)
		: DebugName(Copyable.DebugName),
		bActiveConnection(Copyable.bActiveConnection),
		ConfigData(Copyable.ConfigData),
		AllotedBandwidthKbps(Copyable.AllotedBandwidthKbps),
		OnBandwidthUpdate(MoveTemp(Copyable.OnBandwidthUpdate)) {};

	FString DebugName;
	bool bActiveConnection;
	UE::BandwidthManager::FManagedServiceConfigurableData ConfigData;
	int32 AllotedBandwidthKbps;
	// callback for subscriber to handle the new allotment that reports report back in Kbps
	FOnBandwidthUpdate&& OnBandwidthUpdate; 
};

// Telemetry snapshot for analytics reporting
struct FBandwidthAllocatorTelemetry
{
	bool bEnabled = false;
	bool bEnforcing = false;
	int32 MeasuredBandwidthKbps = 0;
	float MeasurementDurationSec = 0.f;
	int32 MeasurementRequestCount = 0;
	int32 MeasurementCompletedCount = 0;
	int32 MeasurementHistoryCount = 0;
	int32 MeasurementAverageKbps = 0;
	int32 EffectiveBandwidthKbps = 0;
	float BandwidthPercentage = 0.f;
	int32 MinGameplayReserveKbps = 0;
	int32 NumRegisteredSystems = 0;
	int32 DecisionCount = 0;
	int32 ReallocationCount = 0;
	int32 AllocatorErrors = 0;
	bool bInGameplay = false;
	bool bMeasurementSuccess = false;
	FString MeasurementResult; // "Success", "Failed", "NotAttempted"
	// Measurement quality
	float MeasurementVariancePercent = 0.f;
	int32 MeasurementMinKbps = 0;
	int32 MeasurementMaxKbps = 0;
	float MeasurementWarmupSec = 0.f;
};

class FDownlinkBandwidthManager
{

public:
	UE_NONCOPYABLE(FDownlinkBandwidthManager);

	FDownlinkBandwidthManager();
	~FDownlinkBandwidthManager();
	// --- Singleton access ---
	UE_API static FDownlinkBandwidthManager& Get();

	// Check to see if the probability check for roll out has passed/failed
	UE_API static bool HasLocalUserPassedRolloutCheck();

	// Check to see if the probability check for roll out has passed/failed and Enforcement is enabled
	UE_API static bool HasValueEnforcementPassed();

	// --- Lifecycle (called by module) ---
	void Initialize();
	void Shutdown();

	// Service Initialization and registration with Distribution Manager, FOnBandwidthUpdate will report back in Kbps 
	UE_API UE::BandwidthManager::FServiceHandleID RegisterMonitoredService(
		const FString& ServiceName,
		UE::BandwidthManager::FManagedServiceConfigurableData& ServiceConfigData,
		bool StartActive = false,
		FOnBandwidthUpdate&& Callback = nullptr);

	// Removes service from manager considerations and invalidates ServiceID after
	UE_API void UnregisterMonitoredService(UE::BandwidthManager::FServiceHandleID& ServiceID);

	// Determines if service should be considered for distribution
	UE_API void SetActivationForService(const UE::BandwidthManager::FServiceHandleID& ServiceID, bool ActiveState);

	// Changes the associated Services Priority according to the priority state
	UE_API void ChangeServicePriority(const UE::BandwidthManager::FServiceHandleID& ServiceID, UE::BandwidthManager::EBandwidthPriority NewPriority, UE::BandwidthManager::EBandwidthDistributionPriorityState PriorityState);

	// Ways for the game to communicate back or check to see what priority distribution pattern the manager is in
	UE_API void SetDistributionGameplayState(UE::BandwidthManager::EBandwidthDistributionPriorityState State);
	
	// Sets the bandwidth Cap for the Manager to distribute across all services
	UE_API void SetIncomingBandwidthCap(int32 InKbps);

	// Sets the Gameplay bandwidth for the Manager to reserve during the gameplay state
	void SetGameplayBandwidthCap(int32 InKbps) { ReservedGameplayBandwidthKbps = InKbps; };

	// Returns the number of registered services (for telemetry A/B population flags)
	UE_API int32 GetNumRegisteredServices() const;

	// Returns a snapshot of telemetry state for analytics reporting
	UE_API FBandwidthAllocatorTelemetry GetTelemetrySnapshot() const;

	// Resets accumulated telemetry counters (called after analytics flush)
	UE_API void ResetTelemetryCounters();

	// Allows for logging output of FDownlinkBandwidthManager, enabled via CVarDLBWEnableBandwidthLogging
	void Log(bool bPrint);

	// Binds for Subscription when CVar values get set/changed
	UE_API FDelegateHandle BindToCVarSubscription(TFunction<void()> CallBack);

	// Removes Delegate Callback for CVarSubscriptionDelegate
	UE_API void UnbindToCVarSubscription(FDelegateHandle& Handle);
private:
	// Called once per frame/tick to apply an updated allotment based on active services
	void Tick();

#if CSV_PROFILER_STATS
	// Emits allocator state, per-tick deltas, and measurement data to the CSV profiler
	void UpdateCsvStats();
#endif

	// Source of Bandwidth Assignment
	void DistributeBandwidth();

	// Called when services are altered to maintain PriorityBucketMapping
	void GeneratePriorityBucketMap();

	// Helper function to get priority based on CurrentGameplayPriority
	UE::BandwidthManager::EBandwidthPriority GetPriorityValueForService(const FManagedService& MonitoredService) const;

	// Helper function to get the current available bandwidth to distribute based on gameplay state
	int32 GetCurrentMaxAvailableBandwidthInBytes();

	// Broadcast function for CVarSubscriptionDelegate that can be called without exposure
	friend void BroadcastCVarSubscription();

	// Singleton instance pointer
	static TUniquePtr<FDownlinkBandwidthManager> Instance;

	mutable FTransactionallySafeCriticalSection Mutex;

	// Handle for the polling tick
	FTSTicker::FDelegateHandle TickHandle;

	// The hard cap used for determining bandwidth distribution
	int32 CapKbps = 0;

	// The amount of bandwidth to reserve during gameplay
	int32 ReservedGameplayBandwidthKbps = 10000;

	// Incrementing Handle value for external services to track their FManagedService data within the manager
	int32 NextServiceId = UE_INVALID_BANDWIDTH_SERVICE_HANDLE;

	// Service Data Mapping
	TMap<UE::BandwidthManager::FServiceHandleID, FManagedService> TrackedServices;

	// Mapping of expected priorities to their calculated allotment of bandwidth
	TMap<UE::BandwidthManager::EBandwidthPriority, UE::BandwidthManager::FPriorityBucket> PriorityBucketMapping;

	// The Managers perceived state of gameplay to consider for distribution
	UE::BandwidthManager::EBandwidthDistributionPriorityState CurrentGameplayPriorityState = UE::BandwidthManager::EBandwidthDistributionPriorityState::IdleGameplay;

	// Randomized chance roll at initialization to determine if the manager should rollout for this client's launch
	float RolloutPercentage = 0.0f;

	// Telemetry counters (accumulated between analytics flushes)
	int32 TelemetryDecisionCount = 0;
	int32 TelemetryReallocationCount = 0;
	int32 TelemetryAllocatorErrors = 0;
	bool bTelemetryMeasurementCompleted = false;
	bool bTelemetryMeasurementSuccess = false;

	// Previous per-service bandwidth allocations (in Kbps), used to detect reallocations between ticks for telemetry
	TMap<UE::BandwidthManager::FServiceHandleID, int32> PreviousAllocations;

#if CSV_PROFILER_STATS
	int32 CsvPreviousDecisionCount = 0;
	int32 CsvPreviousReallocationCount = 0;
	int32 CsvPreviousAllocatorErrors = 0;
#endif

	// Delegate to be triggered when rollout CVars are set
	UE::BandwidthManager::FOnCVarSetSubscription CVarSubscriptionDelegate;
};

#undef UE_API
