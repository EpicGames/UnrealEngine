// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

#include "DownlinkBandwidthManagerConfig.generated.h"

#define UE_API DOWNLINKBANDWIDTHMANAGER_API


UCLASS(config = Engine, MinimalAPI)
class UDownlinkBandwidthManagerConfig : public UObject
{
	GENERATED_BODY()
public:
	UDownlinkBandwidthManagerConfig() {};

	UE_API virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;

	void SetConsoleVariablesFromConfigurables();

	// The pass percentage for the user to run the Distribution Manager.
	UPROPERTY(Config)
	int32 RolloutPercentage = 0;

	// Enables the application of the distributor's calculated allocations to subscribed systems if percentage check has passed
	UPROPERTY(Config)
	bool EnforceDistributionAllocation = false;
};
#undef UE_API