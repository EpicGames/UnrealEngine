// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HTTP : ModuleRules
{
	protected virtual bool bPlatformEventLoopEnabledByDefault { get { return true; } }

	protected virtual bool bPlatformSupportToIncreaseMaxRequestsAtRuntime { get { return true; } }

	protected virtual bool bPlatformSupportsSocketTestCommand { get { return Target.IsInPlatformGroup(UnrealPlatformGroup.Android); } }

	protected virtual bool bPlatformSupportsOSProxyAddress
	{
		get
		{
			return Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Android);
		}
	}

	protected virtual bool bPlatformSupportsWinHttp
	{
		get
		{
			return Target.Platform.IsInGroup(UnrealPlatformGroup.Windows);
		}
	}

	protected virtual bool bPlatformSupportsLibCurl
	{
		get
		{
			return (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && !Target.WindowsPlatform.bUseXCurl) ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Android);
		}
	}
	protected virtual bool bPlatformCurlReuseConnectionEnabledByDefault
	{
		get
		{
			return true;
		}
	}

	protected virtual bool bPlatformSupportsXCurl { get { return Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.bUseXCurl; } }
	protected virtual bool bPlatformSupportsCurlMultiSocket { get { return !bPlatformSupportsXCurl; } }
	protected virtual bool bPlatformSupportsVerbConnect { get { return !bPlatformSupportsXCurl; } }

	protected virtual bool bPlatformSupportsCurlMultiPoll { get { return true; } }

	protected virtual bool bPlatformSupportsCurlMultiWait { get { return false; } }
	protected virtual bool bPlatformSupportsCurlQuickExit { get { return !bPlatformSupportsXCurl; } }
	protected virtual bool bPlatformSupportsLocalHttpServer 
	{ 
		get 
		{ 
			return !bPlatformSupportsXCurl && 
				!Target.IsInPlatformGroup(UnrealPlatformGroup.Android) && 
				!Target.IsInPlatformGroup(UnrealPlatformGroup.IOS); 
		} 
	}

	protected virtual int CurlDefaultMaxTotalConnections { get { return 256; } }
	protected virtual int CurlDefaultMaxIdleConnections { get { return 5; } } // Default value of CURLMOPT_MAXCONNECTS is 5

	protected virtual int DefaultMaxConcurrentRequests
	{
		get
		{
			if (bPlatformSupportsLibCurl)
			{
				if (CurlDefaultMaxTotalConnections <= CurlDefaultMaxIdleConnections)
				{
					throw new BuildException($"CurlDefaultMaxTotalConnections {CurlDefaultMaxTotalConnections} must be greater than CurlDefaultMaxIdleConnections {CurlDefaultMaxIdleConnections}");
				}

				return CurlDefaultMaxTotalConnections - CurlDefaultMaxIdleConnections; // Means max active connections
			}

			return 256;
		}
	}

	private bool bPlatformSupportsCurl { get { return bPlatformSupportsLibCurl || bPlatformSupportsXCurl; } }

	protected virtual bool bPlatformRequiresOpenSSL
	{
		get
		{
			return Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Android);
		}
	}

	protected virtual bool bPlatformSupportsUnixSockets
	{
		get
		{
			return (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && !Target.WindowsPlatform.bUseXCurl) ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
		}
	}

	public HTTP(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PublicDefinitions.Add("HTTP_PACKAGE=1");

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
			);

		if (Target.bCompileAgainstApplicationCore && Target.Platform.IsInGroup(UnrealPlatformGroup.IOS)) // ios, tvos and visionos fall into this group
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"BackgroundHttpIOS",
					"BackgroundHTTPFileHash",
				}
			);
		}

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"EventLoop",
			}
			);

		if (bPlatformSupportsCurl)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Sockets",
				}
			);

			if (bPlatformSupportsXCurl)
			{
				PublicDependencyModuleNames.Add("XCurl");
			}
			else if (bPlatformSupportsLibCurl)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "libcurl");

				PublicDefinitions.Add("CURL_ENABLE_DEBUG_CALLBACK=1");
				if (Target.Configuration != UnrealTargetConfiguration.Shipping)
				{
					PublicDefinitions.Add("CURL_ENABLE_NO_TIMEOUTS_OPTION=1");
				}
			}
		}

		PrivateDefinitions.Add("UE_HTTP_PLATFORM_SUPPORTS_OS_PROXY_ADDRESS=" + (bPlatformSupportsOSProxyAddress ? "1" : "0"));
		PrivateDefinitions.Add("UE_HTTP_EVENT_LOOP_ENABLE_CHANCE_BY_DEFAULT=" + (bPlatformEventLoopEnabledByDefault ? "100" : "0"));
		PrivateDefinitions.Add("UE_HTTP_CURL_REUSE_CONNECTION_ENABLED_BY_DEFAULT=" + (bPlatformCurlReuseConnectionEnabledByDefault ? "1" : "0"));
		PrivateDefinitions.Add("UE_HTTP_SUPPORT_TO_INCREASE_MAX_REQUESTS_AT_RUNTIME=" + (bPlatformSupportToIncreaseMaxRequestsAtRuntime ? "1" : "0"));
		PrivateDefinitions.Add("UE_HTTP_SOCKET_TEST_COMMAND_ENABLED=" + (bPlatformSupportsSocketTestCommand ? "1" : "0"));
		PrivateDefinitions.Add("WITH_CURL_LIBCURL =" + (bPlatformSupportsLibCurl ? "1" : "0"));
		PrivateDefinitions.Add("WITH_CURL_XCURL=" + (bPlatformSupportsXCurl ? "1" : "0"));
		PrivateDefinitions.Add("WITH_CURL_MULTIPOLL=" + (bPlatformSupportsCurlMultiPoll ? "1" : "0"));
		PrivateDefinitions.Add("WITH_CURL_MULTIWAIT=" + (bPlatformSupportsCurlMultiWait ? "1" : "0"));
		PrivateDefinitions.Add("WITH_CURL_MULTISOCKET=" + (bPlatformSupportsCurlMultiSocket ? "1" : "0"));
		PrivateDefinitions.Add("WITH_CURL_QUICKEXIT=" + (bPlatformSupportsCurlQuickExit ? "1" : "0"));
		PrivateDefinitions.Add("WITH_CURL= " + ((bPlatformSupportsLibCurl || bPlatformSupportsXCurl) ? "1" : "0"));

		// Use Curl over WinHttp on platforms that support it (until WinHttp client security is in a good place at the least)
		if (bPlatformSupportsWinHttp)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "WinHttp");
			PublicDefinitions.Add("WITH_WINHTTP=1");
		}
		else
		{
			PublicDefinitions.Add("WITH_WINHTTP=0");
		}

		if (bPlatformRequiresOpenSSL)
		{
			PrivateDependencyModuleNames.Add("SSL");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
		}
		else
		{
			PrivateDefinitions.Add("WITH_SSL=0");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Apple))
		{
			PublicFrameworks.Add("Security");
		}

		PrivateDefinitions.Add("UE_HTTP_DEFAULT_MAX_CONCURRENT_REQUESTS=" + DefaultMaxConcurrentRequests);
		PrivateDefinitions.Add("UE_HTTP_CURL_DEFAULT_MAX_TOTAL_CONNECTIONS=" + CurlDefaultMaxTotalConnections);
		PrivateDefinitions.Add("UE_HTTP_CURL_DEFAULT_MAX_IDLE_CONNECTIONS=" + CurlDefaultMaxIdleConnections);

		float PlatformConnectionTimeoutMaxDeviation = 0.5f;
		if (bPlatformSupportsXCurl)
		{
			PlatformConnectionTimeoutMaxDeviation = 4.5f;
		}
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
		{
			PlatformConnectionTimeoutMaxDeviation = 1.5f;
		}
		PublicDefinitions.Add("UE_HTTP_CONNECTION_TIMEOUT_MAX_DEVIATION=" + PlatformConnectionTimeoutMaxDeviation);
		PublicDefinitions.Add("UE_HTTP_ACTIVITY_TIMER_START_AFTER_RECEIVED_DATA=" + ((bPlatformSupportsXCurl || Target.IsInPlatformGroup(UnrealPlatformGroup.Apple)) ? "1" : "0"));
		PublicDefinitions.Add("UE_HTTP_SUPPORT_LOCAL_SERVER=" + (bPlatformSupportsLocalHttpServer ? "1" : "0"));
		PublicDefinitions.Add("UE_HTTP_SUPPORT_UNIX_SOCKET=" + (bPlatformSupportsUnixSockets ? "1" : "0"));
		PublicDefinitions.Add("UE_HTTP_SUPPORT_VERB_CONNECT=" + (bPlatformSupportsVerbConnect ? "1" : "0"));
		PublicDefinitions.Add("UE_HTTP_SUPPORT_OPTION_REUSE_CONNECTION=" + (bPlatformSupportsLibCurl ? "1" : "0"));
	}
}
