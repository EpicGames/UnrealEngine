// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WINHTTP

#include "CoreMinimal.h"
#include "WinHttp/Support/WinHttpTypes.h"

using HINTERNET = void*;
using LPVOID = void*;

class IWinHttpConnection
	: public TSharedFromThis<IWinHttpConnection, ESPMode::ThreadSafe>
{
public:
	virtual ~IWinHttpConnection() = default;
	
	/**
	 * [Call on Any Thread]
	 *
	 * Is this object available to use? If this is false, StartRequest will always fail.
	 *
	 * @return True if the connection is valid to use, false if not
	 */
	virtual bool IsValid() const = 0;

	/**
	 * [Call on Any Thread]
	 *
	 * Get the request URL for this connection
	 *
	 * @return A reference to the request URL for this connection
	 */
	virtual const FString& GetRequestUrl() const = 0;

	/**
	 * [Call on Any Thread in Request Callback]
	 *
	 * Get the request handle for this connection. This is only safe to call in a callback for this connection.
	 *
	 * @return The request handle, or nullptr if not available
	 */
	virtual void* GetHandle() = 0;

	/**
	 * [Call on Main Thread only]
	 *
	 * Start this connection if possible. No callbacks will be fired if StartRequest does not return true.
	 *
	 * @return True if the connection was able to start, false if not
	 */
	virtual bool StartRequest() = 0;

	/**
	 * [Call on Any Thread]
	 *
	 * Cancel this connection and start the teardown of any handles owned by this connection
	 *
	 * @return True if the connection is able to cancel, false if it can't
	 */
	virtual bool CancelRequest() = 0;

	/**
	 * [Call on Any Thread]
	 *
	 * Has this connection reached a final state (success or failure)?
	 *
	 * @return True if the connection has finished, false if it hasn't started or is still processing
	 */
	virtual bool IsComplete() const = 0;

	/**
	 * [Call on Main Thread only]
	 *
	 * Process and call delegates events that are queued for the main thread
	 */
	virtual void PumpMessages() = 0;

	/**
	 * [Call on Any Thread]
	 *
	 * Pump the state machine and advance the request's state
	 */
	virtual void PumpStates() = 0;
};

#endif // WITH_WINHTTP
