// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WINHTTP

#include "CoreMinimal.h"
#include "Containers/StringView.h"
#include "HAL/CriticalSection.h"

#include "WinHttp/Support/WinHttpConnection.h"
#include "WinHttp/Support/WinHttpHandle.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"

#include <atomic>

class FWinHttpSession;
class FWinHttpRequestContextHttp;
class FRequestPayload;

using FStringKeyValueMap = TMap<FString, FString>;

DECLARE_DELEGATE_TwoParams(FWinHttpConnectionHttpOnDataTransferred, int32 /*BytesSent*/, int32 /*BytesReceived*/);
DECLARE_DELEGATE_TwoParams(FWinHttpConnectionHttpOnHeaderReceived, const FString& /*HeaderKey*/, const FString& /*HeaderValue*/);
DECLARE_DELEGATE_OneParam(FWinHttpConnectionHttpOnRequestComplete, EHttpRequestStatus::Type /*CompletionStatus*/);

class FWinHttpConnectionHttp
	: public IWinHttpConnection
{
public:
	static HTTP_API TSharedPtr<FWinHttpConnectionHttp, ESPMode::ThreadSafe> CreateHttpConnection(
		FWinHttpSession& Session,
		const FString& Verb,
		const FString& Url,
		const TMap<FString, FString>& Headers,
		const TSharedPtr<FRequestPayload, ESPMode::ThreadSafe>& Payload);

	HTTP_API virtual ~FWinHttpConnectionHttp();
	FWinHttpConnectionHttp(const FWinHttpConnectionHttp& Other) = delete;
	FWinHttpConnectionHttp(FWinHttpConnectionHttp&& Other) = delete;
	FWinHttpConnectionHttp& operator=(const FWinHttpConnectionHttp& Other) = delete;
	FWinHttpConnectionHttp& operator=(FWinHttpConnectionHttp&& Other) = delete;

	//~ Begin IWinHttpConnection Interface
	HTTP_API virtual bool IsValid() const override;
	HTTP_API virtual const FString& GetRequestUrl() const override;
	HTTP_API virtual void* GetHandle() override;
	HTTP_API virtual bool StartRequest() override;
	HTTP_API virtual bool CancelRequest() override;
	HTTP_API virtual bool IsComplete() const override;
	HTTP_API virtual void PumpMessages() override;
	HTTP_API virtual void PumpStates() override;
	//~ End IWinHttpConnection Interface

	/** Provides the current known response code. */
	EHttpResponseCodes::Type GetResponseCode() const { return ResponseCode; }
	/**
	 * Gets the last chunk that is ready for processing on the game thread.
	 * The caller should either use MoveTemp or call Reset after retrieving the data.
	 */
	TArray<uint8>& GetLastChunk() { return GameThreadChunk; }
	/** Provides the current known response content length. */
	uint64 GetResponseContentLength() const { return ResponseContentLength; }
	/** Provides the current known headers received. */
	const TMap<FString, FString>& GetHeadersReceived() const { return HeadersReceived; }

	/// Request Events

	/**
	 * Bind a handler that is called each tick we receive or send data
	 *
	 * @param Handler The Handler to call when we receive or send data
	 */
	HTTP_API void SetDataTransferredHandler(FWinHttpConnectionHttpOnDataTransferred&& Handler);
	/**
	 * Bind a handler that is called each tick we receive a new header
	 *
	 * @param Handler The Handler to call when we receive a new header
	 */
	HTTP_API void SetHeaderReceivedHandler(FWinHttpConnectionHttpOnHeaderReceived&& Handler);
	/**
	 * Bind a handler that is called when the request completes
	 *
	 * @param Handler The Handler to call when the request completes
	 */
	HTTP_API void SetRequestCompletedHandler(FWinHttpConnectionHttpOnRequestComplete&& Handler);

protected:
	HTTP_API FWinHttpConnectionHttp(
		FWinHttpSession& Session,
		const FString& Url,
		const FString& Verb,
		const bool bIsSecure,
		const FString& Domain,
		const TOptional<uint16> Port,
		const FString& PathAndQuery,
		const TMap<FString, FString>& Headers,
		const TSharedPtr<FRequestPayload, ESPMode::ThreadSafe>& Payload);

	/// Request Setup

	/**
	 * Add to our request's headers. These new headers will be added to the existing headers set, and will
	 * replace any values that already exist with the same header name.
	 *
	 * @param Headers Key/Value Map of headers to add to our request
	 * @return True if we could set the headers, false otherwise
	 */
	HTTP_API bool SetHeaders(const TMap<FString, FString>& Headers);

	/**
	 * Add to our request's headers. Theis new header will be added to the existing headers set, and will
	 * replace any values that already exist with the same header name.
	 *
	 * @param Key Key of header to add to our request
	 * @param Value Value of header to set on our request
	 * @return True if we could set the header, false otherwise
	 */
	HTTP_API bool SetHeader(const FString& Key, const FString& Value);
	/**
	 * Set the payload object of our request. This may only be called before StartRequest is called.
	 *
	 * @param NewPayload the Payload object to set (it should not be modified during the request!)
	 * @return True if we were able to set the payload, false otherwise
	 */
	HTTP_API bool SetPayload(const TSharedRef<FRequestPayload, ESPMode::ThreadSafe>& NewPayload);

	/// Sending Request

	/**
	 * Reset our data sent and fire off a send request again
	 *
	 * @return True if we could start the request, or false otherwise
	 */
	HTTP_API bool SendRequest();
	/**
	 * Increment our bytes-sent counters
	 *
	 * @param AmountSent The amount of bytes to increment our counters by
	 */
	HTTP_API void IncrementSentByteCounts(const uint64 AmountSent);
	/**
	 * Increment our bytes-received counters
	 *
	 * @param AmountReceived The amount of bytes to increment our counters by
	 */
	HTTP_API void IncrementReceivedByteCounts(const uint64 AmountReceived);
	/** 
	 * Check if we have more data to send
	 *
	 * @return True if we have data to send, or false otherwise
	 */
	HTTP_API bool HasRequestBodyToSend() const;
	/**
	 * Gives WinHttp the next check of data.
	 *
	 * @param True if we asked WinHttp to send the data, or false if we could not send the next chunk
	 */
	HTTP_API bool SendAdditionalRequestBody();

	/// Request Response

	/**
	 * Tell WinHttp that we are ready for the response of our request
	 *
	 * @return True if the action completed successfully, false otherwise
	 */
	HTTP_API bool RequestResponse();
	/**
	 * Read our HTTP response headers and store the results
	 *
	 * @return True if the action completed successfully, false otherwise
	 */
	HTTP_API bool ProcessResponseHeaders();
	/**
	 * Ask WinHttp to asynchronously tell us how large the next chunk will be. If it's 0, we're done.
	 *
	 * @return True if the action completed successfully, false otherwise
	 */
	HTTP_API bool RequestNextResponseBodyChunkSize();
	/**
	 * Ask WinHttp to asynchronously write the next chunk directly into our response object
	 *
	 * @return True if the action completed successfully, false otherwise
	 */
	HTTP_API bool RequestNextResponseBodyChunkData();

	/// Request State

	/** Set our request as finished and set state to the specified state. This also releases the WinHttp handles. */
	HTTP_API virtual bool FinishRequest(const EHttpRequestStatus::Type FinalState);

	/// Callback handling

	/** Handles storing the server's IP Address if needed */
	HTTP_API virtual void HandleConnectedToServer(const wchar_t* ServerIP);
	/** Handles validating request now that we have server information (certificates, etc) */
	HTTP_API void HandleSendingRequest();
	/** Handles SOMETHING not sure yet*/
	HTTP_API void HandleWriteComplete(const uint64 NumBytesSent);
	/** Handles completing the request so we may read the headers and body of the response */
	HTTP_API void HandleSendRequestComplete();
	/** Handles reading the response headers */
	HTTP_API virtual void HandleHeadersAvailable();
	/** Handles writing the http response body to our response body */
	HTTP_API void HandleDataAvailable(const uint64 NumBytesAvailable);
	/** Handles querying the next chunk of our http response body */
	HTTP_API void HandleReadComplete(const uint64 NumBytesRead);
	/** Handles shutting down the request when there was an error */
	HTTP_API void HandleRequestError(const uint32 ErrorApiId, const uint32 ErrorCode);
	/** Handles our request object closing, ending this request */
	HTTP_API virtual void HandleHandleClosing();

private:
	/**
	 * Handle receiving HTTP status callbacks from WinHttp
	 *
	 * @param ResourceHandle The handle of the resource which generated this Status
	 * @param InternetStatus The event that is being triggered
	 * @param StatusInformation Pointer to optional data for this event, if there is any
	 * @param StatusInformationLength The length of data available to read in StatusInformation or 0 if not used
	 */
	HTTP_API void HandleHttpStatusCallback(HINTERNET ResourceHandle, EWinHttpCallbackStatus Status, void* StatusInformation, uint32 StatusInformationLength);
	
	/** Mark our callback as a friend so they can call the above status callback function */
	friend void CALLBACK UE_WinHttpStatusHttpCallback(HINTERNET, DWORD_PTR, DWORD, LPVOID, DWORD);

	/** Reset both PayloadBuffer and Payload when we don't need them anymore, releasing the memory */
	HTTP_API void ReleasePayloadData();

protected:
	/** Critical section used to synchronize access to the below properties across multiple threads */
	mutable FCriticalSection SyncObject;

	/// Members for internal request state
	/** Handle to our connection object (do not reuse across requests!) */
	FWinHttpHandle ConnectionHandle;
	/** Handle to our request object (do not reuse across requests!)*/
	FWinHttpHandle RequestHandle;
	/** Keep-alive for this object, to ensure it does not destruct while a request is in progress */
	TSharedPtr<FWinHttpConnectionHttp, ESPMode::ThreadSafe> KeepAlive;

	/** The current request state */
	TOptional<EHttpRequestStatus::Type> FinalState;
	/** Has this request successfully connected to the server? */
	bool bConnectedToServer = false;
	/** Has this request been cancelled? */
	bool bRequestCancelled = false;
	/** If true, we have a pending event that needs to be broadcast on the game thread */
	std::atomic<bool> bHasPendingDelegate{false};

private:
	FString RequestUrl;

	enum class EState : uint8
	{
		/// Initial state

		/** We have not started the request */
		WaitToStart,

		/// Send Request states

		/** Send our request (or resend it due to redirect to a different host) */
		SendRequest,
		/** Waiting for our request headers and/or body to be completely written to the connection */
		WaitForSendComplete,
		/** Send an additional chunk of request body */
		SendAdditionalRequestBody,

		/// Receive Header states

		/** Request the response now that we've finished writing our request */
		RequestResponse,
		/** Wait for the response headers to be received from the server */
		WaitForResponseHeaders,
		/** Read values from our response headers */
		ProcessResponseHeaders,

		/// Receive Body states

		/** Request the size of the next chunk of our response body */
		RequestNextResponseBodyChunkSize,
		/** Wait for a size of the next chunk from the response body */
		WaitForNextResponseBodyChunkSize,
		/** Request the data for the next chunk from the response body to written to our buffer */
		RequestNextResponseBodyChunkData,
		/** Waiting for data from the next chunk to be written to our buffer */
		WaitForNextResponseBodyChunkData,

		/// Final state

		/** We are done */
		Finished
	};

	EState CurrentAction = EState::WaitToStart;

	/// Members for sending request

	/** Payload body*/
	TSharedPtr<FRequestPayload, ESPMode::ThreadSafe> Payload;
	/** Amount of bytes from payload that has been sent successfully */
	uint64 NumBytesSuccessfullySent = 0;
	/** Data from the payload currently trying to be written */
	TArray<uint8> PayloadBuffer;

	/// Members for request response

	/** The response code received from our request */
	EHttpResponseCodes::Type ResponseCode = EHttpResponseCodes::Unknown;
	/** The headers we received from our request */
	TMap<FString, FString> HeadersReceived;
	/** The current chunk being built in the WinHTTP thread. Always access within a critical section. */
	TArray<uint8> CurrentChunk;
	/** The current chunk being built for consumption on the game thread. Only access on the game thread. */
	TArray<uint8> GameThreadChunk;
	/** The content length from the response headers, or 0. */
	uint64 ResponseContentLength = 0;
	/** */
	uint64 BytesWrittenToGameThreadChunk = 0;
	/** The amount of bytes that are available to read from WinHttpReadData now */
	TOptional<uint64> ResponseBytesAvailable;
	
	/// Members for reporting updates to main thread

	/** Called on the game thread during tick if we have sent or received data since last tick */
	FWinHttpConnectionHttpOnDataTransferred OnDataTransferredHandler;
	/** Data to report to the game thread about how many bytes have been sent since last tick (not total received) */
	TOptional<uint64> BytesToReportSent;
	/** Data to report to the game thread about how many bytes have been received since last tick (not total received) */
	TOptional<uint64> BytesToReportReceived;

	/** Called on the game thread during tick if we have received header data since last tick */
	FWinHttpConnectionHttpOnHeaderReceived OnHeaderReceivedHandler;
	/** List of headers we have received that we haven't told the game thread about yet */
	TArray<FString> HeaderKeysToReportReceived;

	/** Called on the game thread if the request has completed since last tick */
	FWinHttpConnectionHttpOnRequestComplete OnRequestCompleteHandler;
};

#endif // WITH_WINHTTP
