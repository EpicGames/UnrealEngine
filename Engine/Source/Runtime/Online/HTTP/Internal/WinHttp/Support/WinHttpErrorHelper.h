// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WINHTTP

#include "CoreMinimal.h"

class FWinHttpErrorHelper
{
public:
	/**
	 * Log that an WinHttpOpen call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpOpenFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpCloseHandle call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpCloseHandleFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpSetTimeouts call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpSetTimeoutsFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpConnect call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpConnectFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpOpenRequest call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpOpenRequestFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpSetOption call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpSetOptionFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpAddRequestHeaders call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpAddRequestHeadersFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpSetStatusCallback call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpSetStatusCallbackFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpSendRequest call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpSendRequestFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpReceiveResponse call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpReceiveResponseFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpQueryHeaders call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpQueryHeadersFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpQueryDataAvailable call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpQueryDataAvailableFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpReadData call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpReadDataFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpWriteData call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static HTTP_API void LogWinHttpWriteDataFailure(const uint32 ErrorCode);

private:
	/** This class should not be instantiated */
	FWinHttpErrorHelper() = delete;
};


#endif // WITH_WINHTTP
