// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WINHTTP

#include "CoreMinimal.h"

using HINTERNET = void*;

class FWinHttpHandle
{
public:
	/**
	 * Construct an invalid object
	 */
	FWinHttpHandle() = default;

	/**
	 * Wrap a new HINTERNET Handle
	 */
	HTTP_API explicit FWinHttpHandle(HINTERNET NewHandle);

	/**
	 * Destroy any currently held object
	 */
	HTTP_API ~FWinHttpHandle();

	// Copy/Move constructors
	FWinHttpHandle(const FWinHttpHandle& Other) = delete;
	HTTP_API FWinHttpHandle(FWinHttpHandle&& Other);
	FWinHttpHandle& operator=(const FWinHttpHandle& Other) = delete;
	HTTP_API FWinHttpHandle& operator=(FWinHttpHandle&& Other);

	/**
	 * Wrap a new handle (destroying any previously held object)
	 *
	 * @param NewHandle The new handle to wrap (must not be wrapped by anything else)
	 * @return A reference to this object that now wraps NewHandle
	 */
	HTTP_API FWinHttpHandle& operator=(HINTERNET NewHandle);

	/**
	 * Destroy our current handle and reset our state to holding nothing
	 */
	HTTP_API void Reset();

	/**
	 * Do we contain a valid handle?
	 *
	 * @return True if we have a valid handle, false otherwise
	 */
	HTTP_API explicit operator bool() const;

	/**
	 * Do we contain a valid handle?
	 *
	 * @return True if we have a valid handle, false otherwise
	 */
	HTTP_API bool IsValid() const;

	/**
	 * Get the underlying handle for use
	 *
	 * @return The HINTERNET handle we're wrapping
	 */
	HTTP_API HINTERNET Get() const;

protected:
	/**
	 * The handle we're wrap (if we have one)
	 */
	HINTERNET Handle = nullptr;
};

#endif // WITH_WINHTTP
