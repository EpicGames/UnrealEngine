// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WINHTTP

#include "CoreMinimal.h"
#include "WinHttp/Support/WinHttpHandle.h"

class IWinHttpConnection;

using HINTERNET = void*;

class FWinHttpSession
{
public:
	/**
	 * Construct a new WinHttp session with the specified security protocols flags
	 */
	HTTP_API FWinHttpSession(const uint32 SecurityProtocolFlags, const bool bForceSecureConnections);

	/**
	 * FWinHttpSession is move-only
	 */
	FWinHttpSession(const FWinHttpSession& Other) = delete;
	FWinHttpSession(FWinHttpSession&& Other) = default;
	FWinHttpSession& operator=(const FWinHttpSession& Other) = delete;
	FWinHttpSession& operator=(FWinHttpSession&& Other) = default;

	/**
	 * Did this session initialize successfully?
	 */
	HTTP_API bool IsValid() const;

	/**
	 * Get the underlying session handle
	 *
	 * @return the HINTERNET for this session
	 */
	HTTP_API HINTERNET Get() const;

	/**
	 * Are we only allowed to make secure connection requests (HTTPS, etc)
	 *
	 * @return True if the platform does not allow for insecure messages
	 */
	HTTP_API bool AreOnlySecureConnectionsAllowed() const;

private:
	/** The handle for our session that we are wrapping */
	FWinHttpHandle SessionHandle;
	/** Should we force connections to be secure? */
	bool bForceSecureConnections = false;
};

#endif // WITH_WINHTTP
