// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WINHTTP

#pragma once

#include "CoreMinimal.h"

#pragma warning(push)
#pragma warning(disable : 28285) // Disable static analysis syntax error in WinHttpSetHeaders macros
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <winhttp.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#pragma warning(pop)

enum class EWinHttpCallbackStatus : uint32
{
	ResolvingName = WINHTTP_CALLBACK_STATUS_RESOLVING_NAME,
	NameResolved = WINHTTP_CALLBACK_STATUS_NAME_RESOLVED,
	ConnectingToServer = WINHTTP_CALLBACK_STATUS_CONNECTING_TO_SERVER,
	ConnectedToServer = WINHTTP_CALLBACK_STATUS_CONNECTED_TO_SERVER,
	SendingRequest = WINHTTP_CALLBACK_STATUS_SENDING_REQUEST,
	RequestSent = WINHTTP_CALLBACK_STATUS_REQUEST_SENT,
	ReceivingResponse = WINHTTP_CALLBACK_STATUS_RECEIVING_RESPONSE,
	ResponseReceived = WINHTTP_CALLBACK_STATUS_RESPONSE_RECEIVED,
	ClosingConnection = WINHTTP_CALLBACK_STATUS_CLOSING_CONNECTION,
	ConnectionClosed = WINHTTP_CALLBACK_STATUS_CONNECTION_CLOSED,
	HandleCreated = WINHTTP_CALLBACK_STATUS_HANDLE_CREATED,
	HandleClosing = WINHTTP_CALLBACK_STATUS_HANDLE_CLOSING,
	DetectingProxy = WINHTTP_CALLBACK_STATUS_DETECTING_PROXY,
	Redirect = WINHTTP_CALLBACK_STATUS_REDIRECT,
	IntermediateResponse = WINHTTP_CALLBACK_STATUS_INTERMEDIATE_RESPONSE,
	SecureFailure = WINHTTP_CALLBACK_STATUS_SECURE_FAILURE,
	HeadersAvailable = WINHTTP_CALLBACK_STATUS_HEADERS_AVAILABLE,
	DataAvailable = WINHTTP_CALLBACK_STATUS_DATA_AVAILABLE,
	ReadComplete = WINHTTP_CALLBACK_STATUS_READ_COMPLETE,
	WriteComplete = WINHTTP_CALLBACK_STATUS_WRITE_COMPLETE,
	RequestError = WINHTTP_CALLBACK_STATUS_REQUEST_ERROR,
	SendRequestComplete = WINHTTP_CALLBACK_STATUS_SENDREQUEST_COMPLETE,
	GetProxyForUrlComplete = WINHTTP_CALLBACK_STATUS_GETPROXYFORURL_COMPLETE,
	CloseComplete = WINHTTP_CALLBACK_STATUS_CLOSE_COMPLETE,
	ShutdownComplete = WINHTTP_CALLBACK_STATUS_SHUTDOWN_COMPLETE,
	SettingsWriteComplete = WINHTTP_CALLBACK_STATUS_SETTINGS_WRITE_COMPLETE,
	SettingsReadComplete = WINHTTP_CALLBACK_STATUS_SETTINGS_READ_COMPLETE
};

HTTP_API const TCHAR* LexToString(const EWinHttpCallbackStatus Status);
HTTP_API bool IsValidStatus(const EWinHttpCallbackStatus Status);

HTTP_API DECLARE_LOG_CATEGORY_EXTERN(LogWinHttp, Log, All);

#endif // WITH_WINHTTP
