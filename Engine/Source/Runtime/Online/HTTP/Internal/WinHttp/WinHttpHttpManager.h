// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WINHTTP

#pragma once

#include "CoreMinimal.h"
#include "HttpManager.h"

class FWinHttpSession;

DECLARE_DELEGATE_OneParam(FWinHttpQuerySessionComplete, FWinHttpSession* /*HttpSessionPtr*/);

class IWinHttpConnection;

class FWinHttpHttpManager : public FTSTickerObjectBase
{
public:
	static HTTP_API FWinHttpHttpManager* GetManager();

	HTTP_API FWinHttpHttpManager();
	HTTP_API virtual ~FWinHttpHttpManager();

	/**
	 * Asynchronously finds an existing WinHttp session for the provided URL, or creates a new one for it.
	 *
	 * @param Url The URL to find or create a WinHttp session for
	 * @param Delegate The delegate that is called with the WinHttp session pointer if successful, or null otherwise
	 */
	virtual void QuerySessionForUrl(const FString& Url, FWinHttpQuerySessionComplete&& Delegate) = 0;

	/**
	 * Validate the provided connection before we start sending our request.
	 *
	 * NOTE: this is called on multiple threads, and should be written in a way that handles this safely!
	 * 
	 * @param Connection the connection to validate
	 * @return True if the request validated successfully, false if not
	 */
	virtual bool ValidateRequestCertificates(IWinHttpConnection& Connection) = 0;

	/**
	 * Release any resources that were pinned for this request.
	 *
	 * NOTE: this is a no-op on Windows builds, but does things on other platforms.
	 *
	 * @param Connection The connection to release resources for
	 */
	virtual void ReleaseRequestResources(IWinHttpConnection& Connection) = 0;
};

#endif // WITH_WINHTTP
