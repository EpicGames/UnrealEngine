// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidPlatformHttp.h"
#include "Curl/CurlHttp.h"
#include "Curl/CurlHttpManager.h"

void FAndroidPlatformHttp::Init()
{
	FCurlHttpManager::InitCurl();
}

class FHttpManager * FAndroidPlatformHttp::CreatePlatformHttpManager()
{
	return new FCurlHttpManager();
}

void FAndroidPlatformHttp::Shutdown()
{
	FCurlHttpManager::ShutdownCurl();
}

IHttpRequest* FAndroidPlatformHttp::ConstructRequest()
{
	return new FCurlHttpRequest();
}

TOptional<FString> FAndroidPlatformHttp::GetOperatingSystemProxyAddress()
{
	FString ProxyAddress;

#if USE_ANDROID_JNI
	extern int32 AndroidThunkCpp_GetMetaDataInt(const FString& Key);
	extern FString AndroidThunkCpp_GetMetaDataString(const FString& Key);

	FString ProxyHost = AndroidThunkCpp_GetMetaDataString(TEXT("unreal.http.proxy.proxyHost"));
	int32 ProxyPort = AndroidThunkCpp_GetMetaDataInt(TEXT("unreal.http.proxy.proxyPort"));

	if (ProxyPort != -1 && !ProxyHost.IsEmpty())
	{
		ProxyAddress = FString::Printf(TEXT("%s:%d"), *ProxyHost, ProxyPort);
	}
#endif

	return ProxyAddress;
}

bool FAndroidPlatformHttp::IsOperatingSystemProxyInformationSupported()
{
	return true;
}

FString FAndroidPlatformHttp::GetConfiguredProxyAddress()
{
	if (TOptional<FString> ConfigProxy = GetProxyAddressFromConfig())
	{
		return ConfigProxy.GetValue();
	}
	if (TOptional<FString> OsProxy = FAndroidPlatformHttp::GetOperatingSystemProxyAddress())
	{
		return OsProxy.GetValue();
	}
	return FString();
}