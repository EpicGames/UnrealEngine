// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleEventLoopHttpThread.h"

#include "Apple/AppleHttp.h" 

bool FAppleEventLoopHttpThread::StartThreadedRequest(FHttpRequestCommon* Request)
{
	FHttpResponsePtr Response = Request->GetResponse();
	auto AppleResponse = StaticCastSharedPtr<FAppleHttpResponse>(Response);
	AppleResponse->SetNewAppleHttpEventDelegate(FNewAppleHttpEventDelegate::CreateLambda([IOAccess = EventLoop->GetIOAccess()]() mutable
	{
		IOAccess.Notify();
	}));
	return FEventLoopHttpThread::StartThreadedRequest(Request);
}

void FAppleEventLoopHttpThread::CompleteThreadedRequest(FHttpRequestCommon* Request)
{
}

void FAppleEventLoopHttpThread::CreateEventLoop()
{	
	UE::EventLoop::TEventLoop<FAppleHTTPIOManager>::FParams EventLoopParams;
	EventLoopParams.IOManagerParams.ProcessRequests = [this]()
	{ 
		TArray<FHttpRequestCommon*> RequestsToCancel;
		TArray<FHttpRequestCommon*> RequestsToComplete;
		Process(RequestsToCancel, RequestsToComplete);
	};
	EventLoop.Emplace(MoveTemp(EventLoopParams));
}

void FAppleEventLoopHttpThread::DestroyEventLoop()
{
	EventLoop.Reset();
}

void FAppleEventLoopHttpThread::UpdateEventLoopConfigs()
{
}

UE::EventLoop::IEventLoop* FAppleEventLoopHttpThread::GetEventLoop()
{
	return EventLoop.IsSet() ? &*EventLoop : nullptr;
}

UE::EventLoop::IEventLoop& FAppleEventLoopHttpThread::GetEventLoopChecked()
{
	return *EventLoop;
}
