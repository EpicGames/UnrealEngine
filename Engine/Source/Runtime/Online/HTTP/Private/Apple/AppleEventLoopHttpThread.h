// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Async/ManualResetEvent.h"
#include "EventLoop/EventLoop.h"
#include "EventLoop/IEventLoopIOManager.h"
#include "Misc/Timespan.h"
#include "Misc/MonotonicTime.h"

#include "EventLoopHttpThread.h"

class FAppleHTTPIOAccess final
{
public:
	explicit FAppleHTTPIOAccess(TSharedRef<UE::FManualResetEvent> Event)
		:SharedEvent(MoveTemp(Event))
	{
	}
	
	void Notify()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FAppleHTTPIOAccess_Notify);
		SharedEvent->Notify();
	}
	
private:
	TSharedRef<UE::FManualResetEvent> SharedEvent;
};

class FAppleHTTPIOManager final: public UE::EventLoop::IIOManager
{
public:
	using FIOAccess = FAppleHTTPIOAccess;

	struct FParams
	{
		TUniqueFunction<void()> ProcessRequests;
	};

	FAppleHTTPIOManager(UE::EventLoop::IEventLoop& EventLoop, FParams&& Params)
		: Event(MakeShared<UE::FManualResetEvent>())
		, IOAccess(Event)
		, Params(MoveTemp(Params))
	{
	}

	virtual ~FAppleHTTPIOManager() = default;

	virtual bool Init() override
	{
		return true;
	}

	virtual void Shutdown() override
	{
	}

	virtual void Notify() override
	{
		Event->Notify();
	}

	virtual void Poll(FTimespan WaitTime) override
	{
		if (Event->WaitFor(UE::FMonotonicTimeSpan::FromSeconds(WaitTime.GetTotalSeconds())))
		{
			Event->Reset();
			Params.ProcessRequests();
		}
	}

	FIOAccess& GetIOAccess()
	{
		return IOAccess;
	}

private:
	TSharedRef<UE::FManualResetEvent> Event;
	FIOAccess IOAccess;
	FParams Params;
};

class FAppleEventLoopHttpThread
	: public FEventLoopHttpThread
{
protected:
	//~ Begin FHttpThread Interface
	virtual bool StartThreadedRequest(FHttpRequestCommon* Request) override;
	virtual void CompleteThreadedRequest(FHttpRequestCommon* Request) override;
	//~ End FHttpThread Interface
protected:
	virtual void CreateEventLoop() override;
	virtual void DestroyEventLoop() override;
	virtual void UpdateEventLoopConfigs() override;

	virtual UE::EventLoop::IEventLoop* GetEventLoop() override;
	virtual UE::EventLoop::IEventLoop& GetEventLoopChecked() override;

	TOptional<UE::EventLoop::TEventLoop<FAppleHTTPIOManager>> EventLoop;
};
