// Copyright Epic Games, Inc. All Rights Reserved.


#include "AppleHttp.h"
#include "Apple/ApplePlatformHttp.h"
#include "AppleHttpObjc.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "Http.h"
#include "HttpManager.h"
#include "HttpModule.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/ScopeAutoreleasePool.h"

#include "Interfaces/IHttpRequest.h"
#include "Misc/EngineVersion.h"
#include "Misc/App.h"
#include "Misc/Base64.h"
#include "Stats/Stats.h"

@implementation FAppleHttpResponseDelegate
@synthesize Response;
@synthesize RequestStatus;
@synthesize FailureReason;
@synthesize BytesWritten;
@synthesize BytesReceived;
@synthesize SourceRequest;

-(int32)GetStatusCode;
{
	if (self.Response == nil)
	{
		return 0;
	}
	else if ([self.Response isKindOfClass: [NSHTTPURLResponse class]])
	{
		return ((NSHTTPURLResponse*)self.Response).statusCode;
	}
	else
	{
		return 200;
	}
}

-(NSDictionary*)GetResponseHeaders;
{
	if (self.Response == nil)
	{
		return nil;
	}
	else if ([self.Response isKindOfClass: [NSHTTPURLResponse class]])
	{
		return ((NSHTTPURLResponse*)self.Response).allHeaderFields;
	}
	else
	{
		return nil;
	}
}

- (FAppleHttpResponseDelegate*)initWithRequest:(FAppleHttpRequest&) Request
{
	self = [super init];
	
	BytesWritten = 0;
	BytesReceived = 0;
	RequestStatus = EHttpRequestStatus::NotStarted;
	FailureReason = EHttpFailureReason::None;
	bAnyHttpActivity = false;
	SourceRequest = StaticCastWeakPtr<FAppleHttpRequest>(TWeakPtr<IHttpRequest>(Request.AsShared()));
	bInitializedWithValidStream = Request.IsInitializedWithValidStream();
	
	return self;
}

- (void)CleanSharedObjects
{
	self.SourceRequest = {};
}

- (void)dealloc
{
	[Response release];
	[super dealloc];
}

- (void) HandleStatusCodeReceived
{
	if (TSharedPtr<FAppleHttpRequest> Request = SourceRequest.Pin())
	{
		int32 StatusCode = [self GetStatusCode];
		Request->HandleStatusCodeReceived(StatusCode);
	}
}

- (void)SetRequestStatus:(EHttpRequestStatus::Type)InRequestStatus
{
	self.RequestStatus = InRequestStatus;
}

- (bool)HandleBodyDataReceived:(void*)Ptr Size:(int64)InSize
{
	if (TSharedPtr<FAppleHttpRequest> Request = SourceRequest.Pin())
	{
		return Request->HandleResponseBodyDataReceived(reinterpret_cast<uint8*>(Ptr), InSize);
	}
	return false;
}

- (void) SaveEffectiveURL:(const FString&) InEffectiveURL
{
	if (TSharedPtr<FAppleHttpRequest> Request = SourceRequest.Pin())
	{
		Request->SetEffectiveURL(InEffectiveURL);
	}
}

- (void) HandleResponseHeadersReceived
{
	if (TSharedPtr<FAppleHttpRequest> Request = SourceRequest.Pin())
	{
		TMap<FString, FString> ResponseHeaders;
		if (NSDictionary* Headers = [self GetResponseHeaders])
		{
			for (NSString* Key in [Headers allKeys])
			{
				FString ConvertedValue([Headers objectForKey:Key]);
				FString ConvertedKey(Key);
				ResponseHeaders.Emplace(MoveTemp(ConvertedKey), MoveTemp(ConvertedValue));
			}
		}

		Request->HandleResponseHeadersReceived(MoveTemp(ResponseHeaders));
	}
}

- (BOOL) DidValidActivityOcurred:(FStringView) Reason
{
	TSharedPtr<FAppleHttpRequest> Request = SourceRequest.Pin();

	if (!Request)
	{
		return FALSE;
	}

	if (!bAnyHttpActivity)
	{
		bAnyHttpActivity = true;

		Request->ConnectTime = FPlatformTime::Seconds() - Request->StartProcessTime;

		Request->StartActivityTimeoutTimer();
	}

	Request->ResetActivityTimeoutTimer(Reason);
	return TRUE;
}

- (void)URLSession:(NSURLSession *)session task:(NSURLSessionTask *)task didSendBodyData:(int64_t)bytesSent totalBytesSent:(int64_t)totalBytesSent totalBytesExpectedToSend:(int64_t)totalBytesExpectedToSend
{
	if (![self DidValidActivityOcurred: TEXTVIEW("Sent body data")])
	{
		return;
	}
	UE_LOGF(LogHttp, VeryVerbose, "URLSession:task:didSendBodyData:totalBytesSent:totalBytesExpectedToSend: totalBytesSent = %lld, totalBytesSent = %lld: %p", totalBytesSent, totalBytesExpectedToSend, self);
	self.BytesWritten = totalBytesSent;
}

- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask didReceiveResponse:(NSURLResponse *)response completionHandler:(void (^)(NSURLSessionResponseDisposition disposition))completionHandler
{
	if (![self DidValidActivityOcurred: TEXTVIEW("Received response")])
	{
		completionHandler(NSURLSessionResponseCancel);
		return;
	}

	self.Response = response;

	[self HandleStatusCodeReceived];

	NSURL* Url = [self.Response URL];
	FString EffectiveURL([Url absoluteString]);
	[self SaveEffectiveURL: EffectiveURL];

	[self HandleResponseHeadersReceived];

	uint64 ExpectedResponseLength = response.expectedContentLength;
	UE_LOGF(LogHttp, VeryVerbose, "URLSession:dataTask:didReceiveResponse:completionHandler: expectedContentLength = %lld", ExpectedResponseLength);
	completionHandler(NSURLSessionResponseAllow);
}

- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask didReceiveData:(NSData *)data
{
	if (![self DidValidActivityOcurred: TEXTVIEW("Received data")])
	{
		return;
	}
	
	__block int64 NewBytesReceived = 0;
	__block bool bSerializeSucceed = false;
	[data enumerateByteRangesUsingBlock:^(const void *bytes, NSRange byteRange, BOOL *stop) {
		NewBytesReceived += byteRange.length;
		bSerializeSucceed = [self HandleBodyDataReceived : const_cast<void*>(bytes) Size : byteRange.length];
		*stop = bSerializeSucceed? NO : YES;
	}];
	
	if (!bSerializeSucceed)
	{
		[dataTask cancel];
	}
	// Keep BytesReceived as a separated value to avoid concurrent accesses to Payload
	self.BytesReceived += NewBytesReceived;
	UE_LOGF(LogHttp, VeryVerbose, "URLSession:dataTask:didReceiveData with %llu bytes. After Append, Payload Length = %llu: %p", NewBytesReceived, self.BytesReceived, self);
	
	NewAppleHttpEventDelegate.ExecuteIfBound();
}

- (void)URLSession:(NSURLSession *)session task:(NSURLSessionTask *)task didCompleteWithError:(nullable NSError *)error
{
	TSharedPtr<FAppleHttpRequest> Request = SourceRequest.Pin();

	if (!Request)
	{
		return;
	}

	self.RequestStatus = EHttpRequestStatus::Failed;
	if (error == nil)
	{
		UE_LOGF(LogHttp, VeryVerbose, "URLSession:task:didCompleteWithError. Http request succeeded: %p", self);
		self.RequestStatus = EHttpRequestStatus::Succeeded;
	}
	else
	{
		self.RequestStatus = EHttpRequestStatus::Failed;
		// Determine if the specific error was failing to connect to the host.
		switch ([error code])
		{
			case NSURLErrorTimedOut:
			case NSURLErrorCannotFindHost:
			case NSURLErrorCannotConnectToHost:
			case NSURLErrorDNSLookupFailed:
				self.FailureReason = EHttpFailureReason::ConnectionError;
				break;
			case NSURLErrorCancelled:
				self.FailureReason = EHttpFailureReason::Cancelled;
				break;
			default:
				self.FailureReason = EHttpFailureReason::Other;
				break;
		}

		UE_CLOGF(self.FailureReason != EHttpFailureReason::Cancelled, LogHttp, Warning, "URLSession:task:didCompleteWithError. Http request failed - %ls %ls: %p",
			   *FString([error localizedDescription]),
			   *FString([[error userInfo] objectForKey:NSURLErrorFailingURLStringErrorKey]),
			   self);

		// Log more details if verbose logging is enabled and this is an SSL error
		if (UE_LOG_ACTIVE(LogHttp, Verbose))
		{
			SecTrustRef PeerTrustInfo = reinterpret_cast<SecTrustRef>([[error userInfo] objectForKey:NSURLErrorFailingURLPeerTrustErrorKey]);
			if (PeerTrustInfo != nullptr)
			{
				SecTrustResultType TrustResult = kSecTrustResultInvalid;
				SecTrustGetTrustResult(PeerTrustInfo, &TrustResult);
				
				FString TrustResultString;
				switch (TrustResult)
				{
#define MAP_TO_RESULTSTRING(Constant) case Constant: TrustResultString = TEXT(#Constant); break;
						MAP_TO_RESULTSTRING(kSecTrustResultInvalid)
						MAP_TO_RESULTSTRING(kSecTrustResultProceed)
						MAP_TO_RESULTSTRING(kSecTrustResultDeny)
						MAP_TO_RESULTSTRING(kSecTrustResultUnspecified)
						MAP_TO_RESULTSTRING(kSecTrustResultRecoverableTrustFailure)
						MAP_TO_RESULTSTRING(kSecTrustResultFatalTrustFailure)
						MAP_TO_RESULTSTRING(kSecTrustResultOtherError)
#undef MAP_TO_RESULTSTRING
					default:
						TrustResultString = TEXT("unknown");
						break;
				}
				UE_LOGF(LogHttp, Verbose, "URLSession:task:didCompleteWithError. SSL trust result: %ls (%d)", *TrustResultString, TrustResult);
			}
		}
	}

	Request->StopActivityTimeoutTimer();
	NewAppleHttpEventDelegate.ExecuteIfBound();
}

- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask willCacheResponse:(NSCachedURLResponse *)proposedResponse completionHandler:(void (^)(NSCachedURLResponse *cachedResponse))completionHandler
{
	// All FAppleHttpRequest use NSURLRequestReloadIgnoringLocalCacheData
	// NSURLRequestReloadIgnoringLocalCacheData disables loading of data from cache, but responses can still be stored in cache
	// Passing nil to this handler disables caching the responses
	completionHandler(nil);
}
@end


/**
 * NSInputStream subclass to send streamed FArchive contents
 */
@interface FNSInputStreamFromArchive : NSInputStream<NSStreamDelegate>
{
	TSharedPtr<FArchive> Archive;
	int64 AlreadySentContent;
	NSStreamStatus StreamStatus;
	id<NSStreamDelegate> Delegate;
}
@end

@implementation FNSInputStreamFromArchive

+(FNSInputStreamFromArchive*)inputStreamWithArchive:(TSharedRef<FArchive>) Archive
{
	FNSInputStreamFromArchive* Ret = [[[FNSInputStreamFromArchive alloc] init] autorelease];
	Ret->Archive = Archive;
	return Ret;
}

- (id)init
{
	self = [super init];
	if (self)
	{
		AlreadySentContent = 0;
		StreamStatus = NSStreamStatusNotOpen;
		// Docs say it is good practice that streams are it's own delegates by default
		Delegate = self;
	}
	
	return self;
}

/** NSStream implementation */
- (void)dealloc
{
	[super dealloc];
}

- (void)open
{
	AlreadySentContent = 0;
	StreamStatus = NSStreamStatusOpen;
}

- (void)close
{
	StreamStatus = NSStreamStatusClosed;
}

- (NSStreamStatus)streamStatus
{
	return StreamStatus;
}

- (NSError *)streamError
{
	return nil;
}

- (id<NSStreamDelegate>)delegate
{
	return Delegate;
}

- (void)setDelegate:(id<NSStreamDelegate>)InDelegate
{
	Delegate = InDelegate ?: self;
}

- (id)propertyForKey:(NSString *)key
{
	return nil;
}

- (BOOL)setProperty:(id)property forKey:(NSString *)key
{
	return NO;
}

- (void)scheduleInRunLoop:(NSRunLoop *)aRunLoop forMode:(NSString *)mode
{
	// There is no need to scheduled anything. Data is always available until end is reached
}

- (void)removeFromRunLoop:(NSRunLoop *)aRunLoop forMode:(NSString *)mode
{
	// There is no need to be descheduled since we didn't schedule
}

/** NSStreamDelegate implementation */
- (void)stream:(NSStream *)stream handleEvent:(NSStreamEvent)eventCode
{
	// Won't update local data
}

/** NSInputStream implementation. Those methods are invoked in a worker thread out of our control */

// Reads up to 'len' bytes into 'buffer'. Returns the actual number of bytes read.
- (NSInteger)read:(uint8_t *)buffer maxLength:(NSUInteger)len
{
	const int64 ContentLength = Archive->TotalSize();
	check(AlreadySentContent <= ContentLength);
	const int64 SizeToSend = ContentLength - AlreadySentContent;
	const int64 SizeToSendThisTime = FMath::Min(SizeToSend, static_cast<int64>(len));
	if (SizeToSendThisTime != 0)
	{
		if (Archive->Tell() != AlreadySentContent)
		{
			Archive->Seek(AlreadySentContent);
		}
		Archive->Serialize((uint8*)buffer, SizeToSendThisTime);
		AlreadySentContent += SizeToSendThisTime;
	}
	return SizeToSendThisTime;
}

// return NO because getting the internal buffer is not appropriate for this subclass
- (BOOL)getBuffer:(uint8_t **)buffer length:(NSUInteger *)len
{
	return NO;
}

// returns YES to always force reads
- (BOOL)hasBytesAvailable
{
	return YES;
}
@end

/****************************************************************************
 * FAppleHttpRequest implementation
 ***************************************************************************/

FAppleHttpRequest::FAppleHttpRequest(NSURLSession* InSession, NSURLSession* InBackgroundSession)
:   Session([InSession retain])
,	BackgroundSession([InBackgroundSession retain])
,   Task(nil)
,	ContentBytesLength(0)
,	LastReportedBytesWritten(0)
,	LastReportedBytesRead(0)
{
	bUsePlatformActivityTimeout = false;
	
	Request = [[NSMutableURLRequest alloc] init];

	// Disable cache to mimic WinInet behavior
	Request.cachePolicy = NSURLRequestReloadIgnoringLocalCacheData;

	// Add default headers
	const TMap<FString, FString>& DefaultHeaders = FHttpModule::Get().GetDefaultHeaders();
	for (TMap<FString, FString>::TConstIterator It(DefaultHeaders); It; ++It)
	{
		SetHeader(It.Key(), It.Value());
	}
}

FAppleHttpRequest::~FAppleHttpRequest()
{
	PostProcess();
	[Request release];
    [Session release];
}

void FAppleHttpRequest::SetHeader(const FString& HeaderName, const FString& HeaderValue)
{
	SCOPED_AUTORELEASE_POOL;
	[Request setValue: HeaderValue.GetNSString() forHTTPHeaderField: HeaderName.GetNSString()];

	FHttpRequestCommon::SetHeader(HeaderName, HeaderValue);
}

void FAppleHttpRequest::AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue)
{
    if (!HeaderName.IsEmpty() && !AdditionalHeaderValue.IsEmpty())
    {
        NSDictionary* Headers = [Request allHTTPHeaderFields];
        NSString* PreviousHeaderValuePtr = [Headers objectForKey: HeaderName.GetNSString()];
        FString PreviousValue(PreviousHeaderValuePtr);
		FString NewValue;
		if (!PreviousValue.IsEmpty())
		{
			NewValue = PreviousValue + TEXT(", ");
		}
		NewValue += AdditionalHeaderValue;

        SetHeader(HeaderName, NewValue);
	}
}

TArray<FString> FAppleHttpRequest::GetAllHeaders() const
{
	SCOPED_AUTORELEASE_POOL;
	NSDictionary* Headers = Request.allHTTPHeaderFields;
	TArray<FString> Result;
	Result.Reserve(Headers.count);
	for (NSString* Key in Headers.allKeys)
	{
		FString ConvertedValue(Headers[Key]);
		FString ConvertedKey(Key);
		Result.Add( FString::Printf( TEXT("%s: %s"), *ConvertedKey, *ConvertedValue ) );
	}
	return Result;
}

const TArray<uint8>& FAppleHttpRequest::GetContent() const
{
	StorageForGetContent.Empty();
	if (StreamedContentSource.IsType<FNoStreamSource>())
	{
		SCOPED_AUTORELEASE_POOL;
		NSData* Body = Request.HTTPBody; // accessing HTTPBody will call retain autorelease on the value, increasing its retain count
		StorageForGetContent.Append((const uint8*)Body.bytes, Body.length);
	}
	else
	{
		UE_LOGF(LogHttp, Warning, "FAppleHttpRequest::GetContent() called on a request that is set up for streaming a file. Return value is an empty buffer");
	}
	return StorageForGetContent;
}

void FAppleHttpRequest::SetContent(const TArray<uint8>& ContentPayload)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FAppleHttpRequest::SetContent() - attempted to set content on a request that is inflight");
		return;
	}
	
	StreamedContentSource.Emplace<FNoStreamSource>();
	Request.HTTPBody = [NSData dataWithBytes:ContentPayload.GetData() length:ContentPayload.Num()];
	ContentBytesLength = ContentPayload.Num();
}

void FAppleHttpRequest::SetContent(TArray<uint8>&& ContentPayload)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FAppleHttpRequest::SetContent() - attempted to set content on a request that is inflight");
		return;
	}

	UE_LOGF(LogHttp, Verbose, "FAppleHttpRequest::SetContent(). Payload size %d", ContentPayload.Num());

	StreamedContentSource.Emplace<FNoStreamSource>();
	// We cannot use NSData dataWithBytesNoCopy:length:freeWhenDone: and keep the data in this instance because we don't have control
	// over the lifetime of the request copy that NSURLSessionTask keeps
	Request.HTTPBody = [NSData dataWithBytes:ContentPayload.GetData() length:ContentPayload.Num()];
	ContentBytesLength = ContentPayload.Num();

	// Clear argument content since client code probably expects that
	ContentPayload.Empty();
}

FString FAppleHttpRequest::GetContentType() const
{
	FString ContentType = GetHeader(TEXT("Content-Type"));
	return ContentType;
}

uint64 FAppleHttpRequest::GetContentLength() const
{
	return ContentBytesLength;
}

void FAppleHttpRequest::SetContentAsString(const FString& ContentString)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FAppleHttpRequest::SetContentAsString() - attempted to set content on a request that is inflight");
		return;
	}

	UE_LOGF(LogHttp, Verbose, "FAppleHttpRequest::SetContentAsString() - %ls", *ContentString);
	FTCHARToUTF8 Converter(*ContentString);

	StreamedContentSource.Emplace<FNoStreamSource>();
	// The extra length computation here is unfortunate, but it's technically not safe to assume the length is the same.
	Request.HTTPBody = [NSData dataWithBytes:(ANSICHAR*)Converter.Get() length:Converter.Length()];
	ContentBytesLength = Converter.Length();
}

bool FAppleHttpRequest::SetContentAsStreamedFile(const FString& Filename)
{
	SCOPED_AUTORELEASE_POOL;

	FString PlatformFilename = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*Filename);

	UE_LOGF(LogHttp, Verbose, "FAppleHttpRequest::SetContentAsStreamedFile() - %ls", *PlatformFilename);
	
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FAppleHttpRequest::SetContentAsStreamedFile() - attempted to set content on a request that is inflight");
		return false;
	}

	Request.HTTPBody = nil;

	if (IPlatformFile::GetPlatformPhysical().FileExists(*PlatformFilename))
	{
		FFileStatData FileStatData = IPlatformFile::GetPlatformPhysical().GetStatData(*PlatformFilename);
		UE_LOGF(LogHttp, VeryVerbose, "FAppleHttpRequest::SetContentAsStreamedFile succeeded in getting the file size - %lld", FileStatData.FileSize);
		StreamedContentSource.Emplace<FString>(PlatformFilename);
		ContentBytesLength = FileStatData.FileSize;
		return true;
	}
	else
	{
		UE_LOGF(LogHttp, Warning, "FAppleHttpRequest::SetContentAsStreamedFile failed to get file size errno: %d: %ls", errno, UTF8_TO_TCHAR(strerror(errno)));
		StreamedContentSource.Emplace<FInvalidStreamSource>();
		ContentBytesLength = 0;
		return false;
	}
}

bool FAppleHttpRequest::SetContentFromStream(TSharedRef<FArchive> Stream)
{
	SCOPED_AUTORELEASE_POOL;
	UE_LOGF(LogHttp, Verbose, "FAppleHttpRequest::SetContentFromStream() - %p", &Stream.Get());

	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FAppleHttpRequest::SetContentFromStream() - attempted to set content on a request that is inflight");
		return false;
	}

	Request.HTTPBody = nil;
	ContentBytesLength = Stream->TotalSize();
	StreamedContentSource.Emplace<TSharedRef<FArchive>>(MoveTemp(Stream));

	return true;
}

FString FAppleHttpRequest::GetVerb() const
{
	FString ConvertedVerb(Request.HTTPMethod);
	return ConvertedVerb;
}

void FAppleHttpRequest::SetVerb(const FString& Verb)
{
	SCOPED_AUTORELEASE_POOL;
	UE_LOGF(LogHttp, Verbose, "FAppleHttpRequest::SetVerb() - %ls", *Verb);
	Request.HTTPMethod = Verb.GetNSString();
}

bool FAppleHttpRequest::ProcessRequest()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAppleHttpRequest_ProcessRequest);

	SCOPED_AUTORELEASE_POOL;

	if (!PreProcess())
	{
		return false;
	}
	
	StartProcessTime = FPlatformTime::Seconds();

	SetStatus(EHttpRequestStatus::Processing);
	SetFailureReason(EHttpFailureReason::None);
	// AppleEventLoop sets a delegate into the response to be able to notify events
	InitResponse();

	// if we specified the option to use ImmediateRequest and the feature is enabled (which means the BackgroundSession should not be nil)
	if (GetOption(HttpRequestOptions::RequestMode) == LexToString(EHttpRequestMode::ImmediateRequest) 
		&& FApplePlatformHttp::IsBackgroundRequestFeatureEnabled() && BackgroundSession != nil)
	{

		FHttpModule::Get().GetHttpManager().AddRequest(SharedThis(this));

		// for background sessions we must use the download/upload APIs instead of the dataTask ones
		// it is a requirement for the background session to handle these requests while in background
		Task = [BackgroundSession uploadTaskWithStreamedRequest:Request];

		UE_LOGF(LogHttp, VeryVerbose, "Starting immediate request %p with task %p", this, Task);

		FApplePlatformHttp::Add([Task retain], SharedThis(this));

		[Task resume];
	}
	else
	{
		FHttpModule::Get().GetHttpManager().AddThreadedRequest(SharedThis(this));
	}
	return true;
}

struct FAppleHttpRequest::FAppleHttpStreamFactory
{
	NSInputStream *operator()(FNoStreamSource)
	{
		return nil;
	}
	
	NSInputStream *operator()(FInvalidStreamSource)
	{
		return nil;
	}

	NSInputStream *operator()(const FString& Filename)
	{
		return [NSInputStream inputStreamWithFileAtPath: Filename.GetNSString()];
	}
	
	NSInputStream *operator()(const TSharedRef<FArchive>& Archive)
	{
		return [FNSInputStreamFromArchive inputStreamWithArchive: Archive];
	}
};

bool FAppleHttpRequest::SetupRequest()
{
	SCOPED_AUTORELEASE_POOL;

	Request.URL = [NSURL URLWithString: URL.GetNSString()];

	// set the content-length and user-agent (it is possible that the OS ignores this value)
	if(GetContentLength() > 0)
	{
		UE_LOGF(LogHttp, VeryVerbose, "Setting content length: %lld", GetContentLength());
		SetHeader(TEXT("Content-Length"), FString::Printf(TEXT("%llu"), GetContentLength()));
	}

	PostProcess();

	LastReportedBytesWritten = 0;
	LastReportedBytesRead = 0;
	ElapsedTime = 0.0f;

	float HttpConnectionTimeout = FHttpModule::Get().GetHttpConnectionTimeout();
	check(HttpConnectionTimeout > 0.0f);
	Request.timeoutInterval = HttpConnectionTimeout;
	
	UE_CLOG(
		HttpConnectionTimeout < GetActivityTimeoutOrDefault(),
		LogHttp,
		Warning, 
		TEXT(
			"HttpConnectionTimeout can't be less than HttpActivityTimeout, otherwise requests may complete "
			"unexpectedly with ConnectionError after %.2f(HttpConnectionTimeout) seconds without activity, "
			"instead of intended %.2f(HttpActivityTimeout) seconds"
		), 
		HttpConnectionTimeout, GetActivityTimeoutOrDefault());

	if (NSInputStream *HttpBodyStream = Visit(FAppleHttpStreamFactory{}, StreamedContentSource))
	{
		Request.HTTPBodyStream = HttpBodyStream;
	}
	else if (!StreamedContentSource.IsType<FNoStreamSource>())
	{
		UE_LOGF(LogHttp, Warning, "Could not create native stream from stream source");
		SetStatus(EHttpRequestStatus::Failed);
		SetFailureReason(EHttpFailureReason::Other);
		return false;
	}
	
	return true;
}

FHttpResponsePtr FAppleHttpRequest::CreateResponse()
{
	return MakeShared<FAppleHttpResponse>(*this);
}

void FAppleHttpRequest::MockResponseData()
{
	TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(ResponseCommon);
	[Response->ResponseDelegate SetRequestStatus: EHttpRequestStatus::Succeeded];
}

void FAppleHttpRequest::FinishRequest()
{
	PostProcess();

	TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(ResponseCommon);
	bool bSucceeded = (Response && Response->GetStatusFromDelegate() == EHttpRequestStatus::Succeeded);

	if (!bSucceeded)
	{
		if (FailureReason == EHttpFailureReason::None) // FailureReason could have been set by FHttpRequestCommon::WillTriggerMockFailure
		{
			EHttpFailureReason Reason = EHttpFailureReason::Other;
			if (Response)
			{
				Reason = Response->GetFailureReasonFromDelegate();
				if (Reason == EHttpFailureReason::Cancelled)
				{
					if (bTimedOut)
					{
						Reason = EHttpFailureReason::TimedOut;
					}
					else if (bActivityTimedOut)
					{
						Reason = EHttpFailureReason::ConnectionError;
					}
				}
			}
			else if (bCanceled)
			{
				Reason = EHttpFailureReason::Cancelled;
			}
			SetFailureReason(Reason);
		}

		if (GetFailureReason() == EHttpFailureReason::ConnectionError)
		{
			ResponseCommon = nullptr;
		}
	}

	OnFinishRequest(bSucceeded);
}

void FAppleHttpRequest::CleanupRequest()
{
	TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(ResponseCommon);
	if (Response != nullptr)
	{
		Response->CleanSharedObjects();
	}

	if(Task != nil)
	{
		if (CompletionStatus == EHttpRequestStatus::Processing)
		{
			[Task cancel];
		}
		[Task release];
		Task = nil;
	}
}

void FAppleHttpRequest::AbortRequest()
{
	if (Task != nil)
	{
		[Task cancel];
	}
	else
	{
		// No Task means SetupRequest was not called, so we were not added to the HttpManager yet
		FinishRequestNotInHttpManager();
	}
}

void FAppleHttpRequest::Tick(float DeltaSeconds)
{
	if (DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnGameThread)
	{
		CheckProgressDelegate();
	}
}

bool FAppleHttpRequest::IsInitializedWithValidStream() const
{
	return bInitializedWithValidStream;
}

void FAppleHttpRequest::HandleResponseHeadersReceived(TMap<FString, FString>&& ResponseHeaders)
{
	FHttpResponsePtr HttpResponse = GetResponse();
	TSharedPtr<FAppleHttpResponse> AppHttpResponse = StaticCastSharedPtr<FAppleHttpResponse>(HttpResponse);
	AppHttpResponse->SetHeaders(MoveTemp(ResponseHeaders));

	if (GetDelegateThreadPolicy() == EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread)
	{
		BroadcastResponseHeadersReceived();
	}
	else if (OnHeaderReceived().IsBound())
	{
		FHttpModule::Get().GetHttpManager().AddGameThreadTask([this, StrongThis = AsShared()]()
		{
			BroadcastResponseHeadersReceived();
		});
	}
}

bool FAppleHttpRequest::HandleResponseBodyDataReceived(uint8* Ptr, uint64 Size)
{
	if (bInitializedWithValidStream)
	{
		return PassReceivedDataToStream(Ptr, Size);
	}
	else
	{
		FHttpResponsePtr HttpResponse = GetResponse();
		TSharedPtr<FAppleHttpResponse> AppleHttpResponse = StaticCastSharedPtr<FAppleHttpResponse>(HttpResponse);
		AppleHttpResponse->AppendToPayload(reinterpret_cast<uint8*>(Ptr), Size);
		return true;
	}
}

void FAppleHttpRequest::CheckProgressDelegate()
{
	TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(ResponseCommon);
	if (Response.IsValid() && (CompletionStatus == EHttpRequestStatus::Processing || Response->GetStatusFromDelegate() == EHttpRequestStatus::Failed))
	{
		const uint64 BytesWritten = Response->GetNumBytesWritten();
		const uint64 BytesRead = Response->GetNumBytesReceived();
		if (BytesWritten != LastReportedBytesWritten || BytesRead != LastReportedBytesRead)
		{
			OnRequestProgress64().ExecuteIfBound(SharedThis(this), BytesWritten, BytesRead);
			LastReportedBytesWritten = BytesWritten;
			LastReportedBytesRead = BytesRead;
		}
	}
}

bool FAppleHttpRequest::StartThreadedRequest()
{
	if (bCanceled)
	{
		UE_LOGF(LogHttp, Verbose, "StartThreadedRequest ignored because request has been canceled. %ls url=%ls", *GetVerb(), *GetURL());
		return false;
	}

	if (Task != nil)
	{
		UE_LOGF(LogHttp, Verbose, "StartThreadedRequest ignored because task was already in progress. %ls url=%ls", *GetVerb(), *GetURL());
		return false;
	}

	Task = [Session dataTaskWithRequest: Request];

	if (Task == nil)
	{
		UE_LOGF(LogHttp, Warning, "StartThreadedRequest failed. Could not initialize NSURLSessionTask.");
		SetStatus(EHttpRequestStatus::Failed);
		SetFailureReason(EHttpFailureReason::ConnectionError);
		return false;
	}

	// Both Task and Response keep a strong reference to the delegate
	TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(ResponseCommon);
	Task.delegate = Response->ResponseDelegate;

	[[Task retain] resume];
	
	return FHttpRequestCommon::StartThreadedRequest();
}

bool FAppleHttpRequest::IsThreadedRequestComplete()
{
	TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(ResponseCommon);
	return (Response.IsValid() && Response->IsReady());
}

void FAppleHttpRequest::TickThreadedRequest(float DeltaSeconds)
{
	ElapsedTime += DeltaSeconds;

	if (DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread)
	{
		CheckProgressDelegate();
	}
}

/****************************************************************************
 * FAppleHttpResponse implementation
 **************************************************************************/

FAppleHttpResponse::FAppleHttpResponse(FAppleHttpRequest& InRequest)
	: FHttpResponseCommon(InRequest)
{
	ResponseDelegate = [[FAppleHttpResponseDelegate alloc] initWithRequest: InRequest];
	UE_LOGF(LogHttp, VeryVerbose, "FAppleHttpResponse::FAppleHttpResponse(). Request: %p ResponseDelegate: %p", &InRequest, ResponseDelegate);
}

FAppleHttpResponse::~FAppleHttpResponse()
{
	[ResponseDelegate release];
	ResponseDelegate = nil;
}

void FAppleHttpResponse::SetNewAppleHttpEventDelegate(FNewAppleHttpEventDelegate&& Delegate)
{	
	ResponseDelegate->NewAppleHttpEventDelegate = MoveTemp(Delegate);
}

void FAppleHttpResponse::SetHeaders(TMap<FString, FString>&& InHeaders)
{
	Headers = MoveTemp(InHeaders);
}

FAppleHttpResponseDelegate* FAppleHttpResponse::GetResponseDelegate() const
{
	return ResponseDelegate;
}

void FAppleHttpResponse::CleanSharedObjects()
{
	[ResponseDelegate CleanSharedObjects];
}

TArray<FString> FAppleHttpResponse::GetAllHeaders() const
{
	TArray<FString> Result;
	SCOPED_AUTORELEASE_POOL;
	if (NSDictionary* Headers = [ResponseDelegate GetResponseHeaders])
	{
		Result.Reserve([Headers count]);
		for (NSString* Key in [Headers allKeys])
		{
			FString ConvertedValue([Headers objectForKey:Key]);
			FString ConvertedKey(Key);
			Result.Add( FString::Printf( TEXT("%s: %s"), *ConvertedKey, *ConvertedValue ) );
		}
	}
	return Result;
}

FString FAppleHttpResponse::GetContentType() const
{
	return GetHeader( TEXT( "Content-Type" ) );
}

uint64 FAppleHttpResponse::GetContentLength() const
{
	return ResponseDelegate.Response.expectedContentLength;
}

bool FAppleHttpResponse::IsReady() const
{
	return EHttpRequestStatus::IsFinished(ResponseDelegate.RequestStatus);
}

EHttpRequestStatus::Type FAppleHttpResponse::GetStatusFromDelegate() const
{
	return ResponseDelegate.RequestStatus;
}

EHttpFailureReason FAppleHttpResponse::GetFailureReasonFromDelegate() const
{
	return ResponseDelegate.FailureReason;
}

const uint64 FAppleHttpResponse::GetNumBytesReceived() const
{
	return ResponseDelegate.BytesReceived;
}

const uint64 FAppleHttpResponse::GetNumBytesWritten() const
{
	return ResponseDelegate.BytesWritten;
}
