// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/HttpResponseCommon.h"
#include "GenericPlatform/HttpRequestCommon.h"
#include "PlatformHttp.h"
#include "HttpPackage.h"
#include "Misc/TVariant.h"

/**
 * Delegate invoked when in progress Task completes. It is invoked in an out of our control thread
 *
 */
DECLARE_DELEGATE(FNewAppleHttpEventDelegate);

@class NSMutableURLRequest;

/**
 * Apple implementation of an Http request
 */
class FAppleHttpRequest : public FHttpRequestCommon
{
public:
	// implementation friends
	friend class FAppleHttpResponse;

	//~ Begin IHttpBase Interface
	virtual TArray<FString> GetAllHeaders() const override;	
	virtual FString GetContentType() const override;
	virtual uint64 GetContentLength() const override;
	virtual const TArray<uint8>& GetContent() const override;
	//~ End IHttpBase Interface

	//~ Begin IHttpRequest Interface
	virtual FString GetVerb() const override;
	virtual void SetVerb(const FString& Verb) override;
	virtual void SetContent(const TArray<uint8>& ContentPayload) override;
	virtual void SetContent(TArray<uint8>&& ContentPayload) override;
	virtual void SetContentAsString(const FString& ContentString) override;
    virtual bool SetContentAsStreamedFile(const FString& Filename) override;
	virtual bool SetContentFromStream(TSharedRef<FArchive> Stream) override;
	virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) override;
	virtual void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) override;
	virtual bool ProcessRequest() override;
	virtual void Tick(float DeltaSeconds) override;
	//~ End IHttpRequest Interface

	//~ Begin IHttpRequestThreaded Interface
	virtual bool StartThreadedRequest() override;
	virtual void FinishRequest() override;
	virtual bool IsThreadedRequestComplete() override;
	virtual void TickThreadedRequest(float DeltaSeconds) override;
	//~ End IHttpRequestThreaded Interface

	/**
	 * Constructor
	 *
	 * @param InSession - NSURLSession session used to create NSURLSessionTask to retrieve the response
	 * @param InBackgroundSession - NSURLSession session used to create NSURLSessionTask to retrieve the response in the background
	 */
	FAppleHttpRequest(NSURLSession* InSession, NSURLSession* InBackgroundSession);

	/**
	 * Destructor. Clean up any connection/request handles
	 */
	virtual ~FAppleHttpRequest();

PACKAGE_SCOPE:
	using FHttpRequestCommon::StartProcessTime;
	using FHttpRequestCommon::ConnectTime;
	using FHttpRequestCommon::BroadcastResponseHeadersReceived;
	using FHttpRequestCommon::StartActivityTimeoutTimer;
	using FHttpRequestCommon::ResetActivityTimeoutTimer;
	using FHttpRequestCommon::StopActivityTimeoutTimer;
	using FHttpRequestCommon::HandleStatusCodeReceived;
	using FHttpRequestCommon::SetEffectiveURL;
	using FHttpRequestCommon::PassReceivedDataToStream;

	bool IsInitializedWithValidStream() const;

	void HandleResponseHeadersReceived(TMap<FString, FString>&& ResponseHeaders);
	bool HandleResponseBodyDataReceived(uint8* Ptr, uint64 Size);

private:
	/**
	 * Trigger the request progress delegate if progress has changed
	 */
	void CheckProgressDelegate();

	/**
	 * Create the session connection and initiate the web request
	 *
	 * @return true if the request was started
	 */
	virtual bool SetupRequest() override;

	virtual FHttpResponsePtr CreateResponse() override;
	virtual void MockResponseData() override;

	virtual void AbortRequest() override;

	/**
	 * Close session/request handles and unregister callbacks
	 */
	virtual void CleanupRequest() override;

private:
	/** This is the NSMutableURLRequest, all our Apple functionality will deal with this. */
	NSMutableURLRequest* Request;

	/** This is the session our request belongs to (depending on the RequestMode option) */
	NSURLSession* Session;
	/** This is the background session our request belongs to (depending on the RequestMode option) */
	NSURLSession* BackgroundSession;

	/** This is the Task associated to the sessionin charge of our request */
	NSURLSessionTask* Task;
    
	struct FAppleHttpStreamFactory;
	struct FNoStreamSource{};
	struct FInvalidStreamSource{};
	/** Source to create stream from
		FNoStreamedSource: No streamed data
		FString: Filename set from SetContentAsStreamedFile
		TSharedRef<FArchive>: Stream set from SetContentFromStream
	 */
	TVariant<FNoStreamSource, FInvalidStreamSource, FString, TSharedRef<FArchive>> StreamedContentSource;

	/** The request payload length in bytes. This must be tracked separately for a file stream */
	uint64 ContentBytesLength;

	/** Array used to retrieve back content set on the ObjC request when calling GetContent*/
	mutable TArray<uint8> StorageForGetContent;

	/** Last reported bytes written */
	int32 LastReportedBytesWritten;

	/** Last reported bytesread */
	int32 LastReportedBytesRead;
};

@class FAppleHttpResponseDelegate;

/**
 * Apple implementation of an Http response
 */
class FAppleHttpResponse : public FHttpResponseCommon
{
public:
	//~ Begin IHttpBase Interface
	virtual TArray<FString> GetAllHeaders() const override;	
	virtual FString GetContentType() const override;
	virtual uint64 GetContentLength() const override;
	//~ End IHttpBase Interface

	/**
	 * Check whether a response is ready or not.
	 */
	bool IsReady() const;

	/**
	 * Get the number of bytes received so far
	 */
	const uint64 GetNumBytesReceived() const;

	/**
	* Get the number of bytes sent so far
	*/
	const uint64 GetNumBytesWritten() const;

	/**
	 * Cleans internal shared objects between request and response
	 */
	void CleanSharedObjects();

	/**
	 * Sets delegate invoked when  URLSession:dataTask:didReceiveData or URLSession:task:didCompleteWithError: are triggered
	 * Should be set right before task is started 
	*/
	void SetNewAppleHttpEventDelegate(FNewAppleHttpEventDelegate&& Delegate);

	void SetHeaders(TMap<FString, FString>&& InHeaders);

	FAppleHttpResponseDelegate* GetResponseDelegate() const;

	using FHttpResponseCommon::AppendToPayload;

	/**
	 * Constructor
	 *
	 * @param InRequest - original request that created this response
	 */
	FAppleHttpResponse(FAppleHttpRequest& InRequest);

	/**
	 * Destructor
	 */
	virtual ~FAppleHttpResponse();

private:
	// implementation friends
	friend class FAppleHttpRequest;

	/**
	 * Get status from the internal delegate
	 */
	EHttpRequestStatus::Type GetStatusFromDelegate() const;

	/**
	 * Get reason of failure from the internal delegate
	 */
	EHttpFailureReason GetFailureReasonFromDelegate() const;

	// Delegate implementation. Keeps the response state and data
	FAppleHttpResponseDelegate* ResponseDelegate;
};
