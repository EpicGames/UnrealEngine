// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleHttpBackgroundSession.h"
#include "Interfaces/IHttpRequest.h"

#if PLATFORM_IOS

#include "Interfaces/IHttpBase.h"
#include "Http.h"
#include "AppleHttp.h"
#include "AppleHttpObjc.h"
#include "HttpManager.h"

#include "BackgroundURLSessionHandler.h"

@implementation FApplePlatformHttpBackgroundSessionDelegate

- (void)URLSession:(NSURLSession *)session task:(NSURLSessionTask *)task didSendBodyData:(int64_t)bytesSent totalBytesSent:(int64_t)totalBytesSent totalBytesExpectedToSend:(int64_t)totalBytesExpectedToSend 
{
	TSharedPtr<FAppleHttpRequest> Request;
	if (FApplePlatformHttp::Find(task, Request) && Request.IsValid())
	{
		TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(Request->GetResponse());

		[Response->GetResponseDelegate() URLSession:session task:task didSendBodyData:bytesSent totalBytesSent:totalBytesSent totalBytesExpectedToSend:totalBytesExpectedToSend];
	}
}

- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask didReceiveResponse:(NSURLResponse *)response completionHandler:(void (^)(NSURLSessionResponseDisposition))completionHandler 
{
	TSharedPtr<FAppleHttpRequest> Request;
	if (FApplePlatformHttp::Find((NSURLSessionTask*)dataTask, Request) && Request.IsValid())
	{
		TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(Request->GetResponse());

		[Response->GetResponseDelegate() URLSession:session dataTask:dataTask didReceiveResponse:response completionHandler:completionHandler];

		return;
	}
	
	completionHandler(NSURLSessionResponseAllow);
}

- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask didReceiveData:(NSData *)data 
{
	TSharedPtr<FAppleHttpRequest> Request;
	if (FApplePlatformHttp::Find((NSURLSessionTask*)dataTask, Request) && Request.IsValid())
	{
		TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(Request->GetResponse());

		[Response->GetResponseDelegate() URLSession:session dataTask:dataTask didReceiveData:data];
	}
}

- (void)URLSession:(NSURLSession *)session task:(NSURLSessionTask *)task didCompleteWithError:(NSError *)error 
{
	// here we gonna remove the task and request from the map since they will be released
	TSharedPtr<FAppleHttpRequest> Request;
	if (FApplePlatformHttp::RemoveAndCopyValue(task, Request) && Request.IsValid())
	{
		TSharedPtr<FAppleHttpResponse> Response = StaticCastSharedPtr<FAppleHttpResponse>(Request->GetResponse());

		[Response->GetResponseDelegate() URLSession:session task:task didCompleteWithError:error];

		FHttpManager& HttpManager = FHttpModule::Get().GetHttpManager();

		HttpManager.RemoveRequest(Request->AsShared());
		HttpManager.BroadcastHttpRequestCompleted(Request->AsShared());
	}
}

- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask willCacheResponse:(NSCachedURLResponse *)proposedResponse completionHandler:(void (^)(NSCachedURLResponse * _Nullable))completionHandler 
{
	// All FAppleHttpRequest use NSURLRequestReloadIgnoringLocalCacheData
	// NSURLRequestReloadIgnoringLocalCacheData disables loading of data from cache, but responses can still be stored in cache
	// Passing nil to this handler disables caching the responses
	completionHandler(nil);
}

- (void)URLSessionDidFinishEventsForBackgroundURLSession:(NSURLSession *)session 
{
	NSString* Id = session.configuration.identifier;

	if ([FBackgroundURLSessionHandler::BackgroundSessionEventCompleteDelegateMap objectForKey:Id] == nil)
	{
		return;
	}
	
	void(^CompletionHandler)() = [[FBackgroundURLSessionHandler::BackgroundSessionEventCompleteDelegateMap objectForKey:session.configuration.identifier] retain];
	[FBackgroundURLSessionHandler::BackgroundSessionEventCompleteDelegateMap removeObjectForKey:Id];

	[[NSOperationQueue mainQueue] addOperationWithBlock:^
	{
		CompletionHandler();
		[CompletionHandler release];
	}];
}

@end

#endif // PLATFORM_IOS
