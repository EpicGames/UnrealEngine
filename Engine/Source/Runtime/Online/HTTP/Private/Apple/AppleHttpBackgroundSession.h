// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_IOS

#import <Foundation/Foundation.h>

@interface FApplePlatformHttpBackgroundSessionDelegate : NSObject<NSURLSessionDataDelegate>

- (void)URLSession:(NSURLSession* _Nonnull)session task:(NSURLSessionTask* _Nonnull)task didSendBodyData:(int64_t)bytesSent totalBytesSent:(int64_t)totalBytesSent totalBytesExpectedToSend:(int64_t)totalBytesExpectedToSend;
- (void)URLSession:(NSURLSession* _Nonnull)session dataTask:(NSURLSessionDataTask* _Nonnull)dataTask didReceiveResponse:(NSURLResponse* _Nullable)response completionHandler:(void (^ _Nonnull)(NSURLSessionResponseDisposition))completionHandler;
- (void)URLSession:(NSURLSession* _Nonnull)session dataTask:(NSURLSessionDataTask* _Nonnull)dataTask didReceiveData:(NSData* _Nonnull)data;
- (void)URLSession:(NSURLSession* _Nonnull)session task:(NSURLSessionTask* _Nonnull)task didCompleteWithError:(NSError* _Nullable)error;
- (void)URLSession:(NSURLSession* _Nonnull)session dataTask:(NSURLSessionDataTask* _Nonnull)dataTask willCacheResponse:(NSCachedURLResponse* _Nullable)proposedResponse completionHandler:(void (^ _Nonnull)(NSCachedURLResponse * _Nullable))completionHandler;
- (void)URLSessionDidFinishEventsForBackgroundURLSession:(NSURLSession* _Nonnull)session;

@end

#endif // PLATFORM_IOS