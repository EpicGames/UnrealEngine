// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/AppleHttpManager.h"

#include "Apple/AppleEventLoopHttpThread.h"
#include "HAL/IConsoleManager.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CommandLine.h"
#include "HttpModule.h"
#include "Http.h"

FHttpThreadBase* FAppleHttpManager::CreateHttpThread()
{
	if (bUseEventLoop)
	{
		UE_LOGF(LogHttp, Log, "CreateHttpThread using FAppleEventLoopHttpThread");
		return new FAppleEventLoopHttpThread();
	}

	UE_LOGF(LogHttp, Log, "CreateHttpThread using FLegacyHttpThread");
	return new FLegacyHttpThread();
}
