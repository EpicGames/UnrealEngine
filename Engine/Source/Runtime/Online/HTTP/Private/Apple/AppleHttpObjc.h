// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IHttpBase.h"
#include "AppleHttp.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <Foundation/Foundation.h>

/**
 * Class to hold data from delegate implementation notifications.
 */

@interface FAppleHttpResponseDelegate : NSObject<NSURLSessionDataDelegate>
{
	// Flag to indicate the request was initialized with stream. In that case even if stream was set to 
	// null later on internally, the request itself won't cache received data anymore
	@public BOOL bInitializedWithValidStream;

	/** Have we received any data? */
	BOOL bAnyHttpActivity;

	/** Delegate invoked after processing URLSession:dataTask:didReceiveData or URLSession:task:didCompleteWithError:*/
	@public FNewAppleHttpEventDelegate NewAppleHttpEventDelegate;
}

/** A handle for the response */
@property(retain) NSURLResponse* Response;
/** The total number of bytes written out during the request/response */
@property uint64 BytesWritten;
/** The total number of bytes received out during the request/response */
@property uint64 BytesReceived;
/** Request status */
@property EHttpRequestStatus::Type RequestStatus;
/** Reason of failure */
@property EHttpFailureReason FailureReason;
/** Associated request. Cleared when canceled */
@property TWeakPtr<FAppleHttpRequest> SourceRequest;

/** NSURLSessionDataDelegate delegate methods. Those are called from a thread controlled by the NSURLSession */

/** Sent periodically to notify the delegate of upload progress. */
- (void)URLSession:(NSURLSession *)session task:(NSURLSessionTask *)task didSendBodyData:(int64_t)bytesSent totalBytesSent:(int64_t)totalBytesSent totalBytesExpectedToSend:(int64_t)totalBytesExpectedToSend;
/** The task has received a response and no further messages will be received until the completion block is called. */
- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask didReceiveResponse:(NSURLResponse *)response completionHandler:(void (^)(NSURLSessionResponseDisposition disposition))completionHandler;
/** Sent when data is available for the delegate to consume. Data may be discontiguous */
- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask didReceiveData:(NSData *)data;
/** Sent as the last message related to a specific task.  A nil Error implies that no error occurred and this task is complete. */
- (void)URLSession:(NSURLSession *)session task:(NSURLSessionTask *)task didCompleteWithError:(nullable NSError *)error;
/** Asks the delegate if it needs to store responses in the cache. */
- (void)URLSession:(NSURLSession *)session dataTask:(NSURLSessionDataTask *)dataTask willCacheResponse:(NSCachedURLResponse *)proposedResponse completionHandler:(void (^)(NSCachedURLResponse *cachedResponse))completionHandler;
@end