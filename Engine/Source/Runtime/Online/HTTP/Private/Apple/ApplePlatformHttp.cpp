// Copyright Epic Games, Inc. All Rights Reserved.

#include "Apple/ApplePlatformHttp.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include "HAL/IConsoleManager.h"
#include "Http.h"
#include "AppleHttpManager.h"
#include "AppleHttpBackgroundSession.h"
#include "AppleHttp.h"
#include "Apple/CFRef.h"
#include "Misc/Base64.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeRWLock.h"

#include "CommonCrypto/CommonDigest.h"

#if WITH_SSL
#include "Ssl.h"

static FString AppleGetCertificateSummary(SecCertificateRef Cert)
{
	TCFRef<CFStringRef> Summary = SecCertificateCopySubjectSummary(Cert);
	return FString(Summary);
}

static bool ValidateCertificatePublicKeyPinningApple(NSURLAuthenticationChallenge *challenge)
{
	// CC gives the actual key, but strips the ASN.1 header... which means
	// we can't calulate a proper SPKI hash without reconstructing it. sigh.
	static const unsigned char rsa2048Asn1Header[] =
	{
		0x30, 0x82, 0x01, 0x22, 0x30, 0x0d, 0x06, 0x09,
		0x2a, 0x86, 0x48, 0x86, 0xf7, 0x0d, 0x01, 0x01,
		0x01, 0x05, 0x00, 0x03, 0x82, 0x01, 0x0f, 0x00
	};
	static const unsigned char rsa4096Asn1Header[] =
	{
		0x30, 0x82, 0x02, 0x22, 0x30, 0x0d, 0x06, 0x09,
		0x2a, 0x86, 0x48, 0x86, 0xf7, 0x0d, 0x01, 0x01,
		0x01, 0x05, 0x00, 0x03, 0x82, 0x02, 0x0f, 0x00
	};
	static const unsigned char ecdsaSecp256r1Asn1Header[] =
	{
		0x30, 0x59, 0x30, 0x13, 0x06, 0x07, 0x2a, 0x86,
		0x48, 0xce, 0x3d, 0x02, 0x01, 0x06, 0x08, 0x2a,
		0x86, 0x48, 0xce, 0x3d, 0x03, 0x01, 0x07, 0x03,
		0x42, 0x00
	};
	static const unsigned char ecdsaSecp384r1Asn1Header[] =
	{
		0x30, 0x76, 0x30, 0x10, 0x06, 0x07, 0x2a, 0x86,
		0x48, 0xce, 0x3d, 0x02, 0x01, 0x06, 0x05, 0x2b,
		0x81, 0x04, 0x00, 0x22, 0x03, 0x62, 0x00
	};

	SecTrustRef RemoteTrust = challenge.protectionSpace.serverTrust;
	FString RemoteHost = FString(UTF8_TO_TCHAR([challenge.protectionSpace.host UTF8String]));
	if ((RemoteTrust == NULL) || (RemoteHost.IsEmpty()))
	{
		UE_LOGF(LogHttp, Error, "failed certificate pinning validation: could not parse parameters during certificate pinning evaluation");
		return false;
	}

	if (!SecTrustEvaluateWithError(RemoteTrust, nil))
	{
		UE_LOGF(LogHttp, Error, "failed certificate pinning validation: default certificate trust evaluation failed for domain '%ls'", *RemoteHost);
		return false;
	}
	// look at all certs in the remote chain and calculate the SHA256 hash of their DER-encoded SPKI
	// the chain starts with the server's cert itself, so walk backwards to optimize for roots first
	TArray<TArray<uint8, TFixedAllocator<ISslCertificateManager::PUBLIC_KEY_DIGEST_SIZE>>> CertDigests;
	
	TCFRef<CFArrayRef> Certificates = SecTrustCopyCertificateChain(RemoteTrust);
	if (Certificates == nil)
	{
		UE_LOGF(LogHttp, Error, "No certificate could be copied in the certificate chain used to evaluate trust.");
		return false;
	}
	
	CFIndex CertificateCount = CFArrayGetCount(Certificates);
	for (int i = 0; i < CertificateCount; ++i)
	{
		SecCertificateRef Cert = (SecCertificateRef)CFArrayGetValueAtIndex(Certificates, i);

		TCFRef<CFStringRef> Summary = SecCertificateCopySubjectSummary(Cert);
		UE_LOGF(LogHttp, VeryVerbose, "inspecting certificate. Summary: %ls ", *AppleGetCertificateSummary(Cert));

		TCFRef<SecKeyRef> CertPubKey = SecCertificateCopyKey(Cert);

		TCFRef<CFDataRef> CertPubKeyData = SecKeyCopyExternalRepresentation(CertPubKey, NULL);
		if (!CertPubKeyData)
		{
			UE_LOGF(LogHttp, Warning, "could not extract public key from certificate %i for domain '%ls'; skipping!", i, *RemoteHost);
			continue;
		}
		
		// we got the key. now we have to figure out what type of key it is; thanks, CommonCrypto.
		TCFRef<CFDictionaryRef> CertPubKeyAttr = SecKeyCopyAttributes(CertPubKey);
		NSString *CertPubKeyType = static_cast<NSString *>(CFDictionaryGetValue(CertPubKeyAttr, kSecAttrKeyType));
		NSNumber *CertPubKeySize = static_cast<NSNumber *>(CFDictionaryGetValue(CertPubKeyAttr, kSecAttrKeySizeInBits));
		char *CertPubKeyASN1Header;
		uint8_t CertPubKeyASN1HeaderSize = 0;
		if ([CertPubKeyType isEqualToString: (NSString *)kSecAttrKeyTypeRSA])
		{
			switch ([CertPubKeySize integerValue])
			{
				case 2048:
					UE_LOGF(LogHttp, VeryVerbose, "found 2048 bit RSA pubkey");
					CertPubKeyASN1Header = (char *)rsa2048Asn1Header;
					CertPubKeyASN1HeaderSize = sizeof(rsa2048Asn1Header);
					break;
				case 4096:
					UE_LOGF(LogHttp, VeryVerbose, "found 4096 bit RSA pubkey");
					CertPubKeyASN1Header = (char *)rsa4096Asn1Header;
					CertPubKeyASN1HeaderSize = sizeof(rsa4096Asn1Header);
					break;
				default:
					UE_LOGF(LogHttp, Log, "unsupported RSA key length %i for certificate %i for domain '%ls'; skipping!", [CertPubKeySize integerValue], i, *RemoteHost);
					continue;
			}
		}
		else if ([CertPubKeyType isEqualToString: (NSString *)kSecAttrKeyTypeECSECPrimeRandom])
		{
			switch ([CertPubKeySize integerValue])
			{
				case 256:
					UE_LOGF(LogHttp, VeryVerbose, "found 256 bit ECDSA pubkey");
					CertPubKeyASN1Header = (char *)ecdsaSecp256r1Asn1Header;
					CertPubKeyASN1HeaderSize = sizeof(ecdsaSecp256r1Asn1Header);
					break;
				case 384:
					UE_LOGF(LogHttp, VeryVerbose, "found 384 bit ECDSA pubkey");
					CertPubKeyASN1Header = (char *)ecdsaSecp384r1Asn1Header;
					CertPubKeyASN1HeaderSize = sizeof(ecdsaSecp384r1Asn1Header);
					break;
				default:
					UE_LOGF(LogHttp, Log, "unsupported ECDSA key length %i for certificate %i for domain '%ls'; skipping!", [CertPubKeySize integerValue], i, *RemoteHost);
					continue;
			}
		}
		else {
			UE_LOGF(LogHttp, Log, "unsupported key type (not RSA or ECDSA) for certificate %i for domain '%ls'; skipping!", i, *RemoteHost);
			continue;
		}
		
		UE_LOGF(LogHttp, VeryVerbose, "constructed key header: [%d] %ls", CertPubKeyASN1HeaderSize, *FString::FromHexBlob((const uint8*)CertPubKeyASN1Header, CertPubKeyASN1HeaderSize));
		UE_LOGF(LogHttp, VeryVerbose, "current pubkey: [%d] %ls", [(NSData*)CertPubKeyData length], *FString::FromHexBlob((const uint8*)[(NSData*)CertPubKeyData bytes], [(NSData*)CertPubKeyData length]));
		
		// smash 'em together to get a proper key with an ASN.1 header
		NSMutableData *ReconstructedPubKey = [NSMutableData data];
		[ReconstructedPubKey appendBytes:CertPubKeyASN1Header length:CertPubKeyASN1HeaderSize];
		[ReconstructedPubKey appendData:CertPubKeyData];
		UE_LOGF(LogHttp, VeryVerbose, "reconstructed key: [%d] %ls", [ReconstructedPubKey length], *FString::FromHexBlob((const uint8*)[ReconstructedPubKey bytes], [ReconstructedPubKey length]));
		
		TArray<uint8, TFixedAllocator<ISslCertificateManager::PUBLIC_KEY_DIGEST_SIZE>> CertCalcDigest;
		CertCalcDigest.AddUninitialized(CC_SHA256_DIGEST_LENGTH);
		if (!CC_SHA256([ReconstructedPubKey bytes], (CC_LONG)[ReconstructedPubKey length], CertCalcDigest.GetData()))
		{
			UE_LOGF(LogHttp, Warning, "could not calculate SHA256 digest of public key %d for domain '%ls'; skipping!", i, *RemoteHost);
		}
		else
		{
			CertDigests.Add(CertCalcDigest);
			UE_LOGF(LogHttp, Verbose, "added SHA256 digest to list for evaluation: domain: '%ls' Base64 digest: %ls", *RemoteHost, *FBase64::Encode(CertCalcDigest.GetData(), CertCalcDigest.Num()));
			UE_LOGF(LogHttp, VeryVerbose, "Certificate digest binary content: [%d] '%ls'", CertCalcDigest.Num(), *FString::FromHexBlob(CertCalcDigest.GetData(), CertCalcDigest.Num()));
		}
	}
	
	//finally, see if any of the pubkeys in the chain match any of our pinned pubkey hashes
	if (CertDigests.IsEmpty() || !FSslModule::Get().GetCertificateManager().VerifySslCertificates(CertDigests, RemoteHost))
	{
		// we could not validate any of the provided certs in chain with the pinned hashes for this host
		UE_LOGF(LogHttp, Error, "failed certificate pinning validation: no SPKI hashes in request matched pinned hashes for domain '%ls' (was provided %d certificates in request)", *RemoteHost, CertDigests.Num());
		return false;
	}
	else
	{
		UE_LOGF(LogHttp, Verbose, "certificate public key pinning either succeeded or is disabled; continuing with auth");
		return true;
	}
}

/** NSURLSessionDelegate implementation in charge of validating certificate pinning */
@interface FApplePlatformHttpSessionDelegate: NSObject<NSURLSessionDelegate>
- (void)URLSession:(NSURLSession *)session didReceiveChallenge:(NSURLAuthenticationChallenge *)challenge completionHandler:(void (^)(NSURLSessionAuthChallengeDisposition disposition, NSURLCredential *credential))completionHandler;
@end

@implementation FApplePlatformHttpSessionDelegate

- (void)URLSession:(NSURLSession *)session didReceiveChallenge:(NSURLAuthenticationChallenge *)challenge completionHandler:(void (^)(NSURLSessionAuthChallengeDisposition disposition, NSURLCredential *credential))completionHandler
{
	if (!ensure(ISslCertificateManager::PUBLIC_KEY_DIGEST_SIZE == CC_SHA256_DIGEST_LENGTH))
	{
		UE_LOGF(LogHttp, Error, "failed certificate pinning validation: SslCertificateManager is using non-SHA256 SPKI hashes [expected %d bytes, got %d bytes]", CC_SHA256_DIGEST_LENGTH, ISslCertificateManager::PUBLIC_KEY_DIGEST_SIZE);
		completionHandler(NSURLSessionAuthChallengeCancelAuthenticationChallenge, nil);
		return;
	}
	
	// we only care about challenges to the received certificate chain
	if ([challenge.protectionSpace.authenticationMethod isEqualToString:NSURLAuthenticationMethodServerTrust] == NO)
	{
		UE_LOGF(LogHttp, Verbose, "challenge was not a server trust; continuing with auth");
		completionHandler(NSURLSessionAuthChallengePerformDefaultHandling, nil);
		return;
	}
	
	if (ValidateCertificatePublicKeyPinningApple(challenge))
	{
		completionHandler(NSURLSessionAuthChallengeUseCredential, [NSURLCredential credentialForTrust: challenge.protectionSpace.serverTrust]);
	}
	else
	{
		completionHandler(NSURLSessionAuthChallengeCancelAuthenticationChallenge, nil);
	}
}

@end

#endif

static TAutoConsoleVariable<bool> CVarEnableBackgroundRequests(
	TEXT("AppleHttp.EnableBackgroundRequests"),
	true,
	TEXT("If true, will use the Background URLSession to schedule tasks that can run while the app is in the background. "
			   "If false, will use the current flow scheduling the request to be executed."),
	ECVF_Default);

FRWLock FApplePlatformHttp::TaskRequestMapLock;
TMap<NSURLSessionTask*, TSharedPtr<FAppleHttpRequest>> FApplePlatformHttp::TaskRequestMap;

void FApplePlatformHttp::Init()
{
#if WITH_SSL
	// Load SSL module during HTTP module's StatupModule() to make sure module manager figures out the dependencies correctly
	// and doesn't unload SSL before unloading HTTP module at exit
	FSslModule::Get();
#endif

	// Lazy init, call InitWithNSUrlSession when need to create request, so session config can be set before creating session
}

void FApplePlatformHttp::Shutdown()
{
	ShutdownWithNSUrlSession();
}

void FApplePlatformHttp::InitWithNSUrlSession()
{
	NSURLSessionConfiguration* Config = [NSURLSessionConfiguration defaultSessionConfiguration];

	// Disable cache to mimic WinInet behavior
	Config.requestCachePolicy = NSURLRequestReloadIgnoringLocalCacheData;

#if WITH_SSL
	// Load SSL module during HTTP module's StatupModule() to make sure module manager figures out the dependencies correctly
	// and doesn't unload SSL before unloading HTTP module at exit
	FSslModule::Get();

	FApplePlatformHttpSessionDelegate* Delegate = [[FApplePlatformHttpSessionDelegate alloc] init];
	
	Session = [NSURLSession sessionWithConfiguration: Config delegate: Delegate delegateQueue: nil];
#else
	Session = [NSURLSession sessionWithConfiguration: Config];
#endif
	
    [Session retain];

#if PLATFORM_IOS // won't work on Mac, tvOS and visionOS
	if (IsBackgroundRequestFeatureEnabled()) // can be disabled by cvar
	{
		Config = [NSURLSessionConfiguration backgroundSessionConfigurationWithIdentifier:@"com.unrealengine.backgroundhttpsession"];
		
		// Disable cache to mimic WinInet behavior
		Config.requestCachePolicy = NSURLRequestReloadIgnoringLocalCacheData;
		// Set discretionary to false so the OS will send the request ASAP (with fewer considerations on battery level, power consumption, and such)
		Config.discretionary = NO;
		Config.sessionSendsLaunchEvents = YES;

		FApplePlatformHttpBackgroundSessionDelegate* Delegate = [[FApplePlatformHttpBackgroundSessionDelegate alloc] init];
		BackgroundSession = [NSURLSession sessionWithConfiguration:Config delegate:Delegate delegateQueue:nil];

		[BackgroundSession retain];
	}
#endif // PLATFORM_IOS
}

void FApplePlatformHttp::ShutdownWithNSUrlSession()
{
	[Session invalidateAndCancel];
	[Session release];
	Session = nil;

	[BackgroundSession invalidateAndCancel];
	[BackgroundSession release];
	BackgroundSession = nil;
}

FHttpManager* FApplePlatformHttp::CreatePlatformHttpManager()
{
	return new FAppleHttpManager();
}

IHttpRequest* FApplePlatformHttp::ConstructRequest()
{
	if (Session == nil)
	{
		InitWithNSUrlSession();
	}

	// To avoid breaking the ConstructRequest contract I'll be passing both sessions here
	// If the background feature is disabled, BackgroundSession will be nil and I can check that inside the FAppleHttpRequest
	FAppleHttpRequest* Request = new FAppleHttpRequest(Session, BackgroundSession);

	return Request;
}

bool FApplePlatformHttp::IsBackgroundRequestFeatureEnabled()
{
	return CVarEnableBackgroundRequests.GetValueOnAnyThread();
}

TSharedPtr<FAppleHttpRequest>& FApplePlatformHttp::Add(NSURLSessionTask* Task, const TSharedPtr<FAppleHttpRequest>& Request)
{
	FRWScopeLock ScopeLock(TaskRequestMapLock, FRWScopeLockType::SLT_Write);

	return TaskRequestMap.Add(Task, CopyTemp(Request));
}

bool FApplePlatformHttp::Find(NSURLSessionTask* Task, TSharedPtr<FAppleHttpRequest>& Request)
{
	FRWScopeLock ScopeLock(TaskRequestMapLock, FRWScopeLockType::SLT_ReadOnly);

	if (TSharedPtr<FAppleHttpRequest>* Found = TaskRequestMap.Find(Task))
	{
		Request = *Found;
		return true;
	}
	
	return false;
}

bool FApplePlatformHttp::RemoveAndCopyValue(NSURLSessionTask* Task, TSharedPtr<FAppleHttpRequest>& Request)
{
	FRWScopeLock ScopeLock(TaskRequestMapLock, FRWScopeLockType::SLT_Write);

	return TaskRequestMap.RemoveAndCopyValue(Task, Request);
}