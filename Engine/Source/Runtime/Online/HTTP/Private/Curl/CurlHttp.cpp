// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_CURL

#include "Curl/CurlHttp.h"
#include "Stats/Stats.h"
#include "Misc/App.h"
#include "HttpConstants.h"
#include "HttpModule.h"
#include "Http.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#include "Curl/CurlHttpManager.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"
#include "Internationalization/Regex.h"

#if WITH_SSL
#include "Ssl.h"
#include <openssl/ssl.h>
#endif

TAutoConsoleVariable<bool> CVarHttpCurlAllowHTTP2(
	TEXT("http.CurlAllowHTTP2"),
	false,
	TEXT("Whether to allow HTTP/2 for curl requests"),
	ECVF_Default
);

TAutoConsoleVariable<bool> CVarHttpAlwaysSetContentLengthEnabled(
	TEXT("http.AlwaysSetContentLengthEnabled"),
	false,
	TEXT("Always set Content-Length header even if there is no content to upload when do the http request"),
	ECVF_SaveForNextBoot
);

extern TAutoConsoleVariable<bool> CVarHttpReuseConnectionEnabled;

TAutoConsoleVariable<bool> CVarHttpCurlResetRequestCompletedOnRetryEnabled(
	TEXT("http.CurlResetRequestCompletedOnRetryEnabled"),
	true,
	TEXT("When enabled, bCurlRequestCompleted is reset to false in ClearInCaseOfRetry so the request can be reused for retrying."),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<bool> CVarHttpCurlDeferEasyHandleLifetime(
	TEXT("http.CurlDeferEasyHandleLifetime"),
	true,
	TEXT("When enabled, the curl easy handle is created in SetupRequest and destroyed in CleanupRequest rather than in the constructor/destructor. This prevents the handle from outliving the curl library during shutdown."),
	ECVF_SaveForNextBoot
);

#if WITH_SSL
static int SslCertVerify(int PreverifyOk, X509_STORE_CTX* Context)
{
	if (PreverifyOk == 1)
	{
		SSL* Handle = static_cast<SSL*>(X509_STORE_CTX_get_ex_data(Context, SSL_get_ex_data_X509_STORE_CTX_idx()));
		check(Handle);

		SSL_CTX* SslContext = SSL_get_SSL_CTX(Handle);
		check(SslContext);

		FCurlHttpRequest* Request = static_cast<FCurlHttpRequest*>(SSL_CTX_get_app_data(SslContext));
		check(Request);

		const FString Domain = FPlatformHttp::GetUrlDomain(Request->GetURL());

		if (!FSslModule::Get().GetCertificateManager().VerifySslCertificates(Context, Domain))
		{
			PreverifyOk = 0;
		}
	}

	return PreverifyOk;
}

static CURLcode sslctx_function(CURL * curl, void * sslctx, void * parm)
{
	SSL_CTX* Context = static_cast<SSL_CTX*>(sslctx);
	const ISslCertificateManager& CertificateManager = FSslModule::Get().GetCertificateManager();

	CertificateManager.AddCertificatesToSslContext(Context);
	if (FCurlHttpManager::CurlRequestOptions.bVerifyPeer)
	{
		FCurlHttpRequest* Request = static_cast<FCurlHttpRequest*>(parm);
		SSL_CTX_set_verify(Context, SSL_CTX_get_verify_mode(Context), SslCertVerify);
		SSL_CTX_set_app_data(Context, Request);
	}

	/* all set to go */
	return CURLE_OK;
}
#endif //#if WITH_SSL

FCurlHttpRequest::FCurlHttpRequest()
	: EasyHandle(nullptr)
	, HeaderList(nullptr)
	, Verb(TEXT("GET"))
	, bCurlRequestCompleted(false)
	, bRedirected(false)
	, CurlAddToMultiResult(CURLM_OK)
	, CurlCompletionResult(CURLE_OK)
	, bAnyHttpActivity(false)
	, BytesSent(0)
	, TotalBytesSent(0)
	, TotalBytesRead(0)
	, LastReportedBytesRead(0)
	, LastReportedBytesSent(0)
	, LeastRecentlyCachedInfoMessageIndex(0)
{
	InfoMessageCache.AddDefaulted(NumberOfInfoMessagesToCache);

	if (!CVarHttpCurlDeferEasyHandleLifetime.GetValueOnAnyThread())
	{
		checkf(FCurlHttpManager::IsInit(), TEXT("Curl request was created while the library is shutdown"));
		InitEasyHandle();
	}

	// Add default headers
	const TMap<FString, FString>& DefaultHeaders = FHttpModule::Get().GetDefaultHeaders();
	for (TMap<FString, FString>::TConstIterator It(DefaultHeaders); It; ++It)
	{
		SetHeader(It.Key(), It.Value());
	}

	bUsePlatformActivityTimeout = false;
}

FCurlHttpRequest::~FCurlHttpRequest()
{
	if (!CVarHttpCurlDeferEasyHandleLifetime.GetValueOnAnyThread())
	{
		DestroyEasyHandle();
	}
}

bool FCurlHttpRequest::InitEasyHandle()
{
	check(!EasyHandle);

	EasyHandle = curl_easy_init();
	if (!EasyHandle)
	{
		UE_LOGF(LogHttp, Error, "%p: Failed to initialize curl easy handle", this);
		return false;
	}

	// Always setup the debug function to allow for activity to be tracked
	curl_easy_setopt(EasyHandle, CURLOPT_DEBUGDATA, this);
	curl_easy_setopt(EasyHandle, CURLOPT_DEBUGFUNCTION, StaticDebugCallback);
	curl_easy_setopt(EasyHandle, CURLOPT_VERBOSE, 1L);

	curl_easy_setopt(EasyHandle, CURLOPT_BUFFERSIZE, FCurlHttpManager::CurlRequestOptions.BufferSize);

	curl_easy_setopt(EasyHandle, CURLOPT_USE_SSL, CURLUSESSL_ALL);

	// HTTP2 is linked in for newer libcurl builds and the library will use it by default.
	// There have been issues found with it use in production on long lived servers with heavy HTTP usage, for
	// that reason we're disabling its use by default in the general purpose curl request wrapper and only
	// allowing use of HTTP2 from other curl wrappers like the DerivedDataCache one.
	// Note that CURL_HTTP_VERSION_1_1 was the default for libcurl version before 7.62.0
	if (CVarHttpCurlAllowHTTP2.GetValueOnAnyThread())
	{
		curl_easy_setopt(EasyHandle, CURLOPT_HTTP_VERSION, CURL_HTTP_VERSION_2TLS);
	}
	else
	{
		curl_easy_setopt(EasyHandle, CURLOPT_HTTP_VERSION, CURL_HTTP_VERSION_1_1);
	}

	// set certificate verification (disable to allow self-signed certificates)
	if (FCurlHttpManager::CurlRequestOptions.bVerifyPeer)
	{
		curl_easy_setopt(EasyHandle, CURLOPT_SSL_VERIFYPEER, 1L);
	}
	else
	{
		curl_easy_setopt(EasyHandle, CURLOPT_SSL_VERIFYPEER, 0L);
	}

	// allow http redirects to be followed
	curl_easy_setopt(EasyHandle, CURLOPT_FOLLOWLOCATION, 1L);

	// required for all multi-threaded handles
	curl_easy_setopt(EasyHandle, CURLOPT_NOSIGNAL, 1L);

	// associate with this just in case
	curl_easy_setopt(EasyHandle, CURLOPT_PRIVATE, this);

	const FString& ProxyAddress = FHttpModule::Get().GetProxyAddress();
	if (!ProxyAddress.IsEmpty())
	{
		// guaranteed to be valid at this point
		curl_easy_setopt(EasyHandle, CURLOPT_PROXY, TCHAR_TO_ANSI(*ProxyAddress));
	}

	const FString& HttpNoProxy = FHttpModule::Get().GetHttpNoProxy();
	if (!HttpNoProxy.IsEmpty())
	{
		curl_easy_setopt(EasyHandle, CURLOPT_NOPROXY, TCHAR_TO_ANSI(*HttpNoProxy));
	}

	if (!CVarHttpReuseConnectionEnabled.GetValueOnAnyThread())
	{
		curl_easy_setopt(EasyHandle, CURLOPT_FORBID_REUSE, 1L);
	}

#if PLATFORM_LINUX && !WITH_SSL
	static const char* const CertBundlePath = []() -> const char* {
		static const char * KnownBundlePaths[] =
		{
			"/etc/pki/tls/certs/ca-bundle.crt",
			"/etc/ssl/certs/ca-certificates.crt",
			"/etc/ssl/ca-bundle.pem"
		};

		for (const char* BundlePath : KnownBundlePaths)
		{
			FString FileName(BundlePath);
			UE_LOGF(LogHttp, Log, " Libcurl: checking if '%ls' exists", *FileName);

			if (FPaths::FileExists(FileName))
			{
				return BundlePath;
			}
		}

		return nullptr;
	}();

	// set CURLOPT_CAINFO to a bundle we know exists as the default may not be present
	curl_easy_setopt(EasyHandle, CURLOPT_CAINFO, CertBundlePath);
#endif

	curl_easy_setopt(EasyHandle, CURLOPT_SSLCERTTYPE, "PEM");
#if WITH_SSL
	// unset CURLOPT_CAINFO as certs will be added via sslctx_function
	curl_easy_setopt(EasyHandle, CURLOPT_CAINFO, nullptr);
	curl_easy_setopt(EasyHandle, CURLOPT_SSL_CTX_FUNCTION, *sslctx_function);
	curl_easy_setopt(EasyHandle, CURLOPT_SSL_CTX_DATA, this);
#endif // #if WITH_SSL

	return true;
}

void FCurlHttpRequest::DestroyEasyHandle()
{
	if (FCurlHttpManager::IsInit())
	{
		if (EasyHandle)
		{
			// cleanup the handle first (that order is used in howtos)
			curl_easy_cleanup(EasyHandle);
			EasyHandle = nullptr;
		}
 
		// destroy headers list
		if (HeaderList)
		{
			curl_slist_free_all(HeaderList);
			HeaderList = nullptr;
		}
	}
	else
	{
		// Curl library has already been shut down (e.g. user code holds a reference to the request shared ptr).
		// Curl handles cannot be safely cleaned up; memory is leaked. Flush all HTTP requests before shutting
		// down the HTTP module to avoid this.
		UE_LOGF(LogHttp, Warning, "Curl library has already been shut down. Curl handles will be leaked. Flush all HTTP requests before shutting down the HTTP module to avoid this.");
	}
}

FString FCurlHttpRequest::CombineHeaderKeyValue(const FString& HeaderKey, const FString& HeaderValue)
{
	FString Combined;
	const TCHAR Separator[] = TEXT(": ");
	constexpr const int32 SeparatorLength = UE_ARRAY_COUNT(Separator) - 1;
	Combined.Reserve(HeaderKey.Len() + SeparatorLength + HeaderValue.Len());
	Combined.Append(HeaderKey);
	Combined.AppendChars(Separator, SeparatorLength);
	Combined.Append(HeaderValue);
	return Combined;
}

TArray<FString> FCurlHttpRequest::GetAllHeaders() const
{
	TArray<FString> Result;
	Result.Reserve(RequestHeaders.Num());
	for (const TPair<FString, FString>& It : RequestHeaders)
	{
		Result.Emplace(CombineHeaderKeyValue(It.Key, It.Value));
	}
	return Result;
}

FString FCurlHttpRequest::GetContentType() const
{
	return GetHeader(TEXT( "Content-Type" ));
}

uint64 FCurlHttpRequest::GetContentLength() const
{
	return RequestPayload.IsValid() ? RequestPayload->GetContentLength() : 0;
}

const TArray<uint8>& FCurlHttpRequest::GetContent() const
{
	static const TArray<uint8> EmptyContent;
	return RequestPayload.IsValid() ? RequestPayload->GetContent() : EmptyContent;
}

void FCurlHttpRequest::SetVerb(const FString& InVerb)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FCurlHttpRequest::SetVerb() - attempted to set verb on a request that is inflight");
		return;
	}

	Verb = InVerb.ToUpper();
}

void FCurlHttpRequest::SetOption(const FName Option, const FString& OptionValue)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FCurlHttpRequest::SetOption() - attempted to set option on a request that is inflight");
		return;
	}

	FHttpRequestCommon::SetOption(Option, OptionValue);
}

void FCurlHttpRequest::SetContent(const TArray<uint8>& ContentPayload)
{
	SetContent(CopyTemp(ContentPayload));
}

void FCurlHttpRequest::SetContent(TArray<uint8>&& ContentPayload)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FCurlHttpRequest::SetContent() - attempted to set content on a request that is inflight");
		return;
	}

	RequestPayload = MakeUnique<FRequestPayloadInMemory>(MoveTemp(ContentPayload));
	bIsRequestPayloadSeekable = true;
}

void FCurlHttpRequest::SetContentAsString(const FString& ContentString)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FCurlHttpRequest::SetContentAsString() - attempted to set content on a request that is inflight");
		return;
	}

	uint64 Utf8Length = FPlatformString::ConvertedLength<UTF8CHAR>(*ContentString, ContentString.Len());
	TArray<uint8> Buffer;
	Buffer.SetNumUninitialized(Utf8Length);
	FPlatformString::Convert((UTF8CHAR*)Buffer.GetData(), Buffer.Num(), *ContentString, ContentString.Len());
	RequestPayload = MakeUnique<FRequestPayloadInMemory>(MoveTemp(Buffer));
	bIsRequestPayloadSeekable = true;
}

bool FCurlHttpRequest::SetContentAsStreamedFile(const FString& Filename)
{
	if (!SetContentAsStreamedFileDefaultImpl(Filename))
	{
		return false;
	}

	bIsRequestPayloadSeekable = false;
	return true;
}

bool FCurlHttpRequest::SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream)
{
	UE_LOGF(LogHttp, Verbose, "FCurlHttpRequest::SetContentFromStream() - %ls", *Stream->GetArchiveName());

	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FCurlHttpRequest::SetContentFromStream() - attempted to set content on a request that is inflight");
		return false;
	}

	RequestPayload = MakeUnique<FRequestPayloadInFileStream>(Stream);
	bIsRequestPayloadSeekable = false;
	return true;
}

void FCurlHttpRequest::AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FCurlHttpRequest::AppendToHeader() - attempted to append to header on a request that is inflight");
		return;
	}

	if (!HeaderName.IsEmpty() && !AdditionalHeaderValue.IsEmpty())
	{
		FString* PreviousValue = RequestHeaders.Find(HeaderName);
		FString NewValue;
		if (PreviousValue != nullptr && !PreviousValue->IsEmpty())
		{
			NewValue = (*PreviousValue) + TEXT(", ");
		}
		NewValue += AdditionalHeaderValue;

		SetHeader(HeaderName, NewValue);
	}
}

FString FCurlHttpRequest::GetVerb() const
{
	return Verb;
}

size_t FCurlHttpRequest::StaticUploadCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StaticUploadCallback);
	check(Ptr);
	check(UserData);

	// dispatch
	FCurlHttpRequest* Request = reinterpret_cast<FCurlHttpRequest*>(UserData);
	return Request->UploadCallback(Ptr, SizeInBlocks, BlockSizeInBytes);
}

int FCurlHttpRequest::StaticSeekCallback(void* UserData, curl_off_t Offset, int Origin)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StaticSeekCallback);
	check(UserData);

	// dispatch
	FCurlHttpRequest* Request = reinterpret_cast<FCurlHttpRequest*>(UserData);
	return Request->SeekCallback(Offset, Origin);
}

size_t FCurlHttpRequest::StaticReceiveResponseHeaderCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StaticReceiveResponseHeaderCallback);
	check(Ptr);
	check(UserData);

	// dispatch
	FCurlHttpRequest* Request = reinterpret_cast<FCurlHttpRequest*>(UserData);
	return Request->ReceiveResponseHeaderCallback(Ptr, SizeInBlocks, BlockSizeInBytes);	
}

size_t FCurlHttpRequest::StaticReceiveResponseBodyCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StaticReceiveResponseBodyCallback);
	check(Ptr);
	check(UserData);

	// dispatch
	FCurlHttpRequest* Request = reinterpret_cast<FCurlHttpRequest*>(UserData);

	size_t Result = Request->ReceiveResponseBodyCallback(Ptr, SizeInBlocks, BlockSizeInBytes);	

	if (Request->DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread)
	{
		Request->CheckProgressDelegate();
	}

	return Result;
}

size_t FCurlHttpRequest::StaticDebugCallback(CURL * Handle, curl_infotype DebugInfoType, char * DebugInfo, size_t DebugInfoSize, void* UserData)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StaticDebugCallback);
	check(Handle);
	check(UserData);

	// dispatch
	FCurlHttpRequest* Request = reinterpret_cast<FCurlHttpRequest*>(UserData);
	return Request->DebugCallback(Handle, DebugInfoType, DebugInfo, DebugInfoSize);
}

size_t FCurlHttpRequest::ReceiveResponseHeaderCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_ReceiveResponseHeaderCallback);

	InitResponse();

	OnAnyActivityOccur(TEXTVIEW("Received header"));

	uint32 HeaderSize = SizeInBlocks * BlockSizeInBytes;
	if (HeaderSize > 0 && HeaderSize <= CURL_MAX_HTTP_HEADER)
	{
		TArray<char> AnsiHeader;
		AnsiHeader.AddUninitialized(HeaderSize + 1);

		FMemory::Memcpy(AnsiHeader.GetData(), Ptr, HeaderSize);
		AnsiHeader[HeaderSize] = 0;

		FString Header(ANSI_TO_TCHAR(AnsiHeader.GetData()));
		// kill \n\r
		Header = Header.Replace(TEXT("\n"), TEXT(""));
		Header = Header.Replace(TEXT("\r"), TEXT(""));

		FString HeaderKey, HeaderValue;
		if (Header.Split(TEXT(":"), &HeaderKey, &HeaderValue))
		{
			HeaderValue.TrimStartInline();
			if (!HeaderKey.IsEmpty() && !HeaderValue.IsEmpty() && !bRedirected)
			{
				TSharedPtr<FCurlHttpResponse> Response = StaticCastSharedPtr<FCurlHttpResponse>(ResponseCommon);

				//Store the content length so OnRequestProgress64() delegates have something to work with
				if (HeaderKey == TEXT("Content-Length"))
				{
					Response->ContentLength = FCString::Atoi64(*HeaderValue);
				}

				const constexpr FStringView Seperator(TEXTVIEW(", "));

				FString NewValue;
				FString* PreviousValue = Response->Headers.Find(HeaderKey);
				if (PreviousValue != nullptr && !PreviousValue->IsEmpty())
				{
					NewValue.Reserve(PreviousValue->Len() + Seperator.Len() + HeaderValue.Len());
					NewValue = *PreviousValue;
					NewValue += Seperator;
				}
				NewValue += HeaderValue;
				Response->Headers.Add(HeaderKey, MoveTemp(NewValue));

				if (DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread)
				{
					OnHeaderReceived().ExecuteIfBound(SharedThis(this), HeaderKey, HeaderValue);
				}
				else if (OnHeaderReceived().IsBound())
				{
					NewlyReceivedHeaders.Enqueue(TPair<FString, FString>(MoveTemp(HeaderKey), MoveTemp(HeaderValue)));
				}
			}
		}
		else
		{
			if (Header.IsEmpty())
			{
				char* EffectiveUrlPtr = nullptr;
				if (curl_easy_getinfo(EasyHandle, CURLINFO_EFFECTIVE_URL, &EffectiveUrlPtr) == CURLE_OK)
				{
					if (EffectiveUrlPtr)
					{
						SetEffectiveURL(FString(EffectiveUrlPtr));
					}
				}
			}
			else
			{
				long HttpCode = 0;
				if (CURLE_OK == curl_easy_getinfo(EasyHandle, CURLINFO_RESPONSE_CODE, &HttpCode))
				{
					bRedirected = (HttpCode >= 300 && HttpCode < 400) && (HttpCode != 304);

					if (!bRedirected)
					{
						TriggerStatusCodeReceivedDelegate(HttpCode);
					}
				}
			}
		}
		return HeaderSize;
	}
	else
	{
		UE_LOGF(LogHttp, Warning, "%p: Could not process response header for request - header size (%d) is invalid.", this, HeaderSize);
	}

	return 0;
}

size_t FCurlHttpRequest::ReceiveResponseBodyCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_ReceiveResponseBodyCallback);
	LLM_SCOPE(ELLMTag::Networking);

	InitResponse();

	OnAnyActivityOccur(TEXTVIEW("Received body"));

	// Number of bytes actually taken care of. If that amount differs from the amount passed to your 
	// callback function, it will signal an error condition to the library. This will cause the transfer 
	// to get aborted and the libcurl function used will return CURLE_WRITE_ERROR.
	size_t NumberOfBytesProcessed = 0;
	const size_t SizeToDownload = SizeInBlocks * BlockSizeInBytes;
	const int64 TotalRead = TotalBytesRead.load();
	const size_t CurrentBytesRead = TotalRead > 0 ? TotalRead : 0;
	TSharedPtr<FCurlHttpResponse> Response = StaticCastSharedPtr<FCurlHttpResponse>(ResponseCommon);
	UE_LOGF(LogHttp, Verbose, "%p: ReceiveResponseBodyCallback: %llu bytes out of %llu received. (SizeInBlocks=%llu, BlockSizeInBytes=%llu, TotalBytesRead=%llu, Response->GetContentLength()=%llu, SizeToDownload=%llu (<-this will get returned from the callback))",
		this, CurrentBytesRead + SizeToDownload, Response->GetContentLength(),
		SizeInBlocks, BlockSizeInBytes, CurrentBytesRead, Response->GetContentLength(), SizeToDownload);

	// note that we can be passed 0 bytes if file transmitted has 0 length
	if (SizeToDownload == 0)
	{
		return NumberOfBytesProcessed;
	}

	if (bInitializedWithValidStream)
	{
		if (PassReceivedDataToStream(Ptr, SizeToDownload))
		{
			NumberOfBytesProcessed = SizeToDownload;
		}
		else if (bCanceled)
		{
			// If it's because of cancellation, set processed size as well so curl don't raise a warning caused by CURLE_WRITE_ERROR 
			// The transfer will be stopped by cancel flow anyway
			NumberOfBytesProcessed = SizeToDownload;
		}
	}
	else
	{
		const size_t NewTotalSize = CurrentBytesRead + SizeToDownload;
		const uint64 ContentLength = Response->GetContentLength();
		if (NewTotalSize > TNumericLimits<int32>::Max())
		{
			UE_LOGF(LogHttp, Warning, "%p: ReceiveResponseBodyCallback: Payload size exceeds int32 limits: %llu bytes, consider using SetResponseBodyReceiveStream instead", this, NewTotalSize);
			SetFailureReason(EHttpFailureReason::ResponseTooLarge);
			return 0;
		}

		// First time chunk added, reserve the capacity, if we know the content length, avoiding relocations
		if (Response->Payload.IsEmpty() && ContentLength > 0 && ContentLength <= TNumericLimits<int32>::Max())
		{
			Response->Payload.Reserve(static_cast<int32>(ContentLength));
		}

		// Grow the array if needed
		if (NewTotalSize > static_cast<size_t>(Response->Payload.Num()))
		{
			const size_t PayloadMax = Response->Payload.Max();
			if (ContentLength == 0 && NewTotalSize > PayloadMax)
			{
				constexpr size_t SizeMultiplier = 2;
				const size_t NewReserve = FMath::Max<size_t>(NewTotalSize, PayloadMax * SizeMultiplier);
				if (NewReserve > TNumericLimits<int32>::Max())
				{
					UE_LOGF(LogHttp, Warning, "%p: ReceiveResponseBodyCallback: Payload size exceeds int32 limits: TotalSize = %llu bytes, NewTotalSize = %llu, consider using SetResponseBodyReceiveStream instead", this, NewTotalSize, NewReserve);
					SetFailureReason(EHttpFailureReason::ResponseTooLarge);
					return 0;
				}
				Response->Payload.Reserve(NewReserve);
			}

			Response->Payload.SetNumUninitialized(static_cast<int32>(NewTotalSize));
		}

		// Finally, copy the data
		uint8* Dest = Response->Payload.GetData() + CurrentBytesRead;
		FMemory::Memcpy(Dest, Ptr, SizeToDownload);

		NumberOfBytesProcessed = SizeToDownload;
	}

	TotalBytesRead += NumberOfBytesProcessed;

	return NumberOfBytesProcessed;
}

size_t FCurlHttpRequest::UploadCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes)
{
	OnAnyActivityOccur(TEXTVIEW("Upload callback"));

	size_t MaxBufferSize = SizeInBlocks * BlockSizeInBytes;
	size_t SizeAlreadySent = BytesSent.load();
	size_t SizeSentThisTime = RequestPayload->FillOutputBuffer(Ptr, MaxBufferSize, SizeAlreadySent);
	BytesSent += SizeSentThisTime;
	TotalBytesSent += SizeSentThisTime;

	UE_LOGF(LogHttp, Verbose, "%p: UploadCallback: %llu bytes out of %llu sent (%llu bytes total sent). (SizeInBlocks=%llu, BlockSizeInBytes=%llu, SizeToSendThisTime=%llu (<-this will get returned from the callback))",
		this, BytesSent.load(), RequestPayload->GetContentLength(), TotalBytesSent.load(), SizeInBlocks, BlockSizeInBytes, SizeSentThisTime);

	return SizeSentThisTime;
}

int FCurlHttpRequest::SeekCallback(curl_off_t Offset, int Origin)
{
	// Only support seeking to the very beginning
	if (bIsRequestPayloadSeekable && Origin == SEEK_SET && Offset == 0)
	{
		UE_LOGF(LogHttp, Log, "%p: SeekCallback: Resetting to the beginning. We had uploaded %llu bytes", this, BytesSent.load());
		BytesSent.store(0);
		bIsRequestPayloadSeekable = false; // Do not attempt to re-seek
		return CURL_SEEKFUNC_OK;
	}
	UE_LOGF(LogHttp, Warning, "%p: SeekCallback: Failed to seek to Offset=%lld, Origin=%d %ls", 
		this,
		(int64)(Offset),
		Origin, 
		bIsRequestPayloadSeekable ? TEXT("not implemented") : TEXT("seek disabled"));
	return CURL_SEEKFUNC_CANTSEEK;
}

size_t FCurlHttpRequest::DebugCallback(CURL * Handle, curl_infotype DebugInfoType, char * DebugInfo, size_t DebugInfoSize)
{
	check(Handle == EasyHandle);	// make sure it's us
#if CURL_ENABLE_DEBUG_CALLBACK
	switch(DebugInfoType)
	{
		case CURLINFO_TEXT:
			{
				// in this case DebugInfo is a C string (see http://curl.haxx.se/libcurl/c/debug.html)
				// C string is not null terminated:  https://curl.haxx.se/libcurl/c/CURLOPT_DEBUGFUNCTION.html

				// Truncate at 1023 characters. This is just an arbitrary number based on a buffer size seen in
				// the libcurl code.
				DebugInfoSize = FMath::Min(DebugInfoSize, (size_t)1023);

				// Calculate the actual length of the string due to incorrect use of snprintf() in lib/vtls/openssl.c.
				char* FoundNulPtr = (char*)memchr(DebugInfo, 0, DebugInfoSize);
				int CalculatedSize = FoundNulPtr != nullptr ? FoundNulPtr - DebugInfo : DebugInfoSize;

				FString DebugText(CalculatedSize, static_cast<const ANSICHAR*>(DebugInfo));

				DebugText.ReplaceInline(TEXT("\n"), TEXT(""), ESearchCase::CaseSensitive);
				DebugText.ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
				UE_LOGF(LogHttp, VeryVerbose, "%p: '%ls'", this, *DebugText);
				const FScopeLock CacheLock(&InfoMessageCacheCriticalSection);
				if (InfoMessageCache.Num() > 0)
				{
					InfoMessageCache[LeastRecentlyCachedInfoMessageIndex] = MoveTemp(DebugText);
					LeastRecentlyCachedInfoMessageIndex = (LeastRecentlyCachedInfoMessageIndex + 1) % InfoMessageCache.Num();
				}
			}
			break;

		case CURLINFO_HEADER_IN:
			UE_LOG(LogHttp, VeryVerbose, TEXT("%p: Received header (%" SIZE_T_FMT " bytes)"), this, DebugInfoSize);
			break;

		case CURLINFO_HEADER_OUT:
			{
				if (UE_LOG_ACTIVE(LogHttp, VeryVerbose))
				{
					// C string is not null terminated:  https://curl.haxx.se/libcurl/c/CURLOPT_DEBUGFUNCTION.html

					// Scan for \r\n\r\n.  According to some code in tool_cb_dbg.c, special processing is needed for
					// CURLINFO_HEADER_OUT blocks when containing both headers and data (which may be binary).
					//
					// Truncate at 1023 characters. This is just an arbitrary number based on a buffer size seen in
					// the libcurl code.
					int RecalculatedSize = FMath::Min(DebugInfoSize, (size_t)1023);
					for (int Index = 0; Index <= RecalculatedSize - 4; ++Index)
					{
						if (DebugInfo[Index] == '\r' && DebugInfo[Index + 1] == '\n'
								&& DebugInfo[Index + 2] == '\r' && DebugInfo[Index + 3] == '\n')
						{
							RecalculatedSize = Index;
							break;
						}
					}

					// As lib/http.c states that CURLINFO_HEADER_OUT may contain binary data, only print it if
					// the header data is readable.
					bool bIsPrintable = true;
					for (int Index = 0; Index < RecalculatedSize; ++Index)
					{
						unsigned char Ch = DebugInfo[Index];
						if (!isprint(Ch) && !isspace(Ch))
						{
							bIsPrintable = false;
							break;
						}
					}

					if (bIsPrintable)
					{
						FString DebugText(RecalculatedSize, static_cast<const ANSICHAR*>(DebugInfo));

						DebugText.ReplaceInline(TEXT("\n"), TEXT(""), ESearchCase::CaseSensitive);
						DebugText.ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
						UE_LOGF(LogHttp, VeryVerbose, "%p: Sent header (%d bytes) - %ls", this, RecalculatedSize, *DebugText);
					}
					else
					{
						UE_LOGF(LogHttp, VeryVerbose, "%p: Sent header (%d bytes) - contains binary data", this, RecalculatedSize);
					}
					}
			}
			break;

		case CURLINFO_DATA_IN:
			UE_LOG(LogHttp, VeryVerbose, TEXT("%p: Received data (%" SIZE_T_FMT " bytes)"), this, DebugInfoSize);
			break;

		case CURLINFO_DATA_OUT:
			UE_LOG(LogHttp, VeryVerbose, TEXT("%p: Sent data (%" SIZE_T_FMT " bytes)"), this, DebugInfoSize);
			break;

		case CURLINFO_SSL_DATA_IN:
			UE_LOG(LogHttp, VeryVerbose, TEXT("%p: Received SSL data (%" SIZE_T_FMT " bytes)"), this, DebugInfoSize);
			break;

		case CURLINFO_SSL_DATA_OUT:
			UE_LOG(LogHttp, VeryVerbose, TEXT("%p: Sent SSL data (%" SIZE_T_FMT " bytes)"), this, DebugInfoSize);
			break;

		default:
			UE_LOG(LogHttp, VeryVerbose, TEXT("%p: DebugCallback: Unknown DebugInfoType=%d (DebugInfoSize: %" SIZE_T_FMT " bytes)"), this, (int32)DebugInfoType, DebugInfoSize);
			break;
	}
#endif // CURL_ENABLE_DEBUG_CALLBACK

	switch (DebugInfoType)
	{
	case CURLINFO_HEADER_IN: 
		OnAnyActivityOccur(TEXTVIEW("Header in"));
		break;
	case CURLINFO_HEADER_OUT: 
		// Unlike libCurl, currently there is an issue in xCurl that it triggers CURLINFO_HEADER_OUT even if can't 
		// connect. Had to disable this code, make sure not to treat that event as connected/activity happened
#if !WITH_CURL_XCURL
		OnAnyActivityOccur(TEXTVIEW("Header out"));
#endif
		break;
	case CURLINFO_DATA_IN: 
		OnAnyActivityOccur(TEXTVIEW("Data in"));
		break;
	case CURLINFO_DATA_OUT:
		OnAnyActivityOccur(TEXTVIEW("Data out"));
		break;
	case CURLINFO_SSL_DATA_IN:
		OnAnyActivityOccur(TEXTVIEW("Ssl data in"));
		break;
	case CURLINFO_SSL_DATA_OUT:
		OnAnyActivityOccur(TEXTVIEW("Ssl data out"));
		break;
	default:
		break;
	}

	return 0;
}

void FCurlHttpRequest::OnAnyActivityOccur(FStringView Reason)
{
	if (!bAnyHttpActivity)
	{
		bAnyHttpActivity = true;

#if WITH_CURL_XCURL
		ConnectTime = FPlatformTime::Seconds() - StartProcessTime;
#else
		curl_easy_getinfo(EasyHandle, CURLINFO_CONNECT_TIME, &ConnectTime);
#endif

		StartActivityTimeoutTimer();
	}

	ResetActivityTimeoutTimer(Reason);
}

bool FCurlHttpRequest::SetupRequest()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_SetupRequest);

	if (CVarHttpCurlDeferEasyHandleLifetime.GetValueOnAnyThread())
	{
		checkf(FCurlHttpManager::IsInit(), TEXT("Curl request was created while the library is shutdown"));

		if (!InitEasyHandle())
		{
			return false;
		}
	}

	check(EasyHandle);

	if (!RequestPayload.IsValid())
	{
		RequestPayload = MakeUnique<FRequestPayloadInMemory>(TArray<uint8>());
		bIsRequestPayloadSeekable = true;
	}

	if (!OpenRequestPayloadDefaultImpl())
	{
		return false;
	}

	bCurlRequestCompleted = false;
	CurlAddToMultiResult = CURLM_OK;
	LastReportedBytesSent = 0;

	UE_LOGF(LogHttp, Verbose, "%p: Custom headers are %ls", this, RequestHeaders.Num() ? TEXT("present") : TEXT("NOT present"));
	UE_LOGF(LogHttp, Verbose, "%p: Payload size=%llu", this, RequestPayload->GetContentLength());

	// content-length should be present http://www.w3.org/Protocols/rfc2616/rfc2616-sec4.html#sec4.4
	if (GetHeader(TEXT("Content-Length")).IsEmpty())
	{
		if (RequestPayload->GetContentLength() != 0 || CVarHttpAlwaysSetContentLengthEnabled.GetValueOnAnyThread())
		{
			SetHeader(TEXT("Content-Length"), FString::Printf(TEXT("%llu"), RequestPayload->GetContentLength()));
		}
	}

	// Remove "Expect: 100-continue" since this is supposed to cause problematic behavior on Amazon ELB (and WinInet doesn't send that either)
	// (also see http://www.iandennismiller.com/posts/curl-http1-1-100-continue-and-multipartform-data-post.html , http://the-stickman.com/web-development/php-and-curl-disabling-100-continue-header/ )
	if (GetHeader(TEXT("Expect")).IsEmpty())
	{
		SetHeader(TEXT("Expect"), TEXT(""));
	}

	return true;
}

bool FCurlHttpRequest::StartThreadedRequest()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StartThreadedRequest);
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StartThreadedRequest_SLIST_FREE_HEADERS);
		curl_slist_free_all(HeaderList);
		HeaderList = nullptr;
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StartThreadedRequest_EASY_SETOPT);

		curl_easy_setopt(EasyHandle, CURLOPT_URL, TCHAR_TO_ANSI(*GetURL()));

		SetupOptionUnixSocketPath();
		SetupOptionHttpVersion();
		SetupOptionEnableCookies();

		if (!FCurlHttpManager::CurlRequestOptions.LocalHostAddr.IsEmpty())
		{
			// Set the local address to use for making these requests
			CURLcode ErrCode = curl_easy_setopt(EasyHandle, CURLOPT_INTERFACE, TCHAR_TO_ANSI(*FCurlHttpManager::CurlRequestOptions.LocalHostAddr));
		}

		bool bUseReadFunction = false;

		// set up verb (note that Verb is expected to be uppercase only)
		if (Verb == TEXT("POST"))
		{
			// If we don't pass any other Content-Type, RequestPayload is assumed to be URL-encoded by this time
			// In the case of using a streamed file, you must explicitly set the Content-Type, because RequestPayload->IsURLEncoded returns false.
			check(!GetHeader(TEXT("Content-Type")).IsEmpty() || RequestPayload->IsURLEncoded());
			curl_easy_setopt(EasyHandle, CURLOPT_POST, 1L);
			curl_easy_setopt(EasyHandle, CURLOPT_POSTFIELDS, NULL);
#if WITH_CURL_XCURL
			checkf(RequestPayload->GetContentLength() <= TNumericLimits<int32>::Max(), TEXT("xCurl 2206.4.0.0 doesn't support uploading file with length more than 32 bits"));
			curl_easy_setopt(EasyHandle, CURLOPT_INFILESIZE, RequestPayload->GetContentLength());
#else
			curl_easy_setopt(EasyHandle, CURLOPT_POSTFIELDSIZE, RequestPayload->GetContentLength());
#endif
			bUseReadFunction = true;
		}
		else if (Verb == TEXT("PUT") || Verb == TEXT("PATCH"))
		{
			curl_easy_setopt(EasyHandle, CURLOPT_UPLOAD, 1L);
#if WITH_CURL_XCURL
			checkf(RequestPayload->GetContentLength() <= TNumericLimits<int32>::Max(), TEXT("xCurl 2206.4.0.0 doesn't support uploading file with length more than 32 Bits"));
#endif
			curl_easy_setopt(EasyHandle, CURLOPT_INFILESIZE, RequestPayload->GetContentLength());
			if (Verb != TEXT("PUT"))
			{
				curl_easy_setopt(EasyHandle, CURLOPT_CUSTOMREQUEST, TCHAR_TO_UTF8(*Verb));
			}

			bUseReadFunction = true;
		}
		else if (Verb == TEXT("GET"))
		{
			// technically might not be needed unless we reuse the handles
			curl_easy_setopt(EasyHandle, CURLOPT_HTTPGET, 1L);
		}
		else if (Verb == TEXT("HEAD"))
		{
			curl_easy_setopt(EasyHandle, CURLOPT_NOBODY, 1L);
		}
		else if (Verb == TEXT("DELETE"))
		{
			// If we don't pass any other Content-Type, RequestPayload is assumed to be URL-encoded by this time
			// (if we pass, don't check here and trust the request)
			check(!GetHeader(TEXT("Content-Type")).IsEmpty() || RequestPayload->IsURLEncoded());

			curl_easy_setopt(EasyHandle, CURLOPT_POST, 1L);
			curl_easy_setopt(EasyHandle, CURLOPT_CUSTOMREQUEST, "DELETE");
			curl_easy_setopt(EasyHandle, CURLOPT_POSTFIELDSIZE, RequestPayload->GetContentLength());
			bUseReadFunction = true;
		}
		else
		{
			curl_easy_setopt(EasyHandle, CURLOPT_CUSTOMREQUEST, TCHAR_TO_ANSI(*Verb));
		}

		if (bUseReadFunction)
		{
			BytesSent.store(0);
			TotalBytesSent.store(0);
			curl_easy_setopt(EasyHandle, CURLOPT_READDATA, this);
			curl_easy_setopt(EasyHandle, CURLOPT_READFUNCTION, StaticUploadCallback);
		}

		// set up header function to receive response headers
		curl_easy_setopt(EasyHandle, CURLOPT_HEADERDATA, this);
		curl_easy_setopt(EasyHandle, CURLOPT_HEADERFUNCTION, StaticReceiveResponseHeaderCallback);

		// set up write function to receive response payload
		curl_easy_setopt(EasyHandle, CURLOPT_WRITEDATA, this);
		curl_easy_setopt(EasyHandle, CURLOPT_WRITEFUNCTION, StaticReceiveResponseBodyCallback);

		// set up headers

		// Empty string here tells Curl to list all supported encodings, allowing servers to send compressed content.
		if (FCurlHttpManager::CurlRequestOptions.bAcceptCompressedContent)
		{
#if WITH_CURL_XCURL
			// xCurl doesn't seem to support passing empty string to enable all supported built-in compressions
			curl_easy_setopt(EasyHandle, CURLOPT_ACCEPT_ENCODING, "gzip");
#else
			curl_easy_setopt(EasyHandle, CURLOPT_ACCEPT_ENCODING, "");
#endif
		}

		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StartThreadedRequest_SLIST_APPEND_HEADERS);

			for (const TPair<FString, FString>& HeaderPair : RequestHeaders)
			{
				FString FullHeader = FString::Format(TEXT("{0}: {1}"), { HeaderPair.Key, HeaderPair.Value });
				curl_slist* NewHeaderList = curl_slist_append(HeaderList, TCHAR_TO_UTF8(ToCStr(FullHeader)));
				if (!NewHeaderList)
				{
					UE_LOGF(LogHttp, Warning, "Failed to append header '%ls'", *HeaderPair.Key);
					continue;
				}

				HeaderList = NewHeaderList;
			}
		}

		if (HeaderList)
		{
			curl_easy_setopt(EasyHandle, CURLOPT_HTTPHEADER, HeaderList);
		}

		// Set connection timeout in seconds
		int32 HttpConnectionTimeout = FHttpModule::Get().GetHttpConnectionTimeout();
		check(HttpConnectionTimeout > 0);
		curl_easy_setopt(EasyHandle, CURLOPT_CONNECTTIMEOUT, HttpConnectionTimeout);

		if (FCurlHttpManager::CurlRequestOptions.bAllowSeekFunction && bIsRequestPayloadSeekable)
		{
			curl_easy_setopt(EasyHandle, CURLOPT_SEEKDATA, this);
			curl_easy_setopt(EasyHandle, CURLOPT_SEEKFUNCTION, StaticSeekCallback);
		}
	}

#if !WITH_CURL_XCURL
	{
		//Tracking the locking in the CURLOPT_SHARE branch of the curl_easy_setopt implementation
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_StartThreadedRequest_EASY_CURLOPT_SHARE);

		curl_easy_setopt(EasyHandle, CURLOPT_SHARE, FCurlHttpManager::GShareHandle);
	}
#endif

#if WITH_CURL_QUICKEXIT
	// Avoid hanging and waiting for threaded resolver
	curl_easy_setopt(EasyHandle, CURLOPT_QUICK_EXIT, 1L);
#endif

	ElapsedTime = 0.0f;
	
	return FHttpRequestCommon::StartThreadedRequest();
}

void FCurlHttpRequest::SetupOptionUnixSocketPath()
{
#if UE_HTTP_SUPPORT_UNIX_SOCKET
		FString UnixSocketPath = GetOption(HttpRequestOptions::UnixSocketPath);
		if (UnixSocketPath.Len() > 0)
		{
			curl_easy_setopt(EasyHandle, CURLOPT_UNIX_SOCKET_PATH, TCHAR_TO_ANSI(*UnixSocketPath));
		}
#endif //UE_HTTP_SUPPORT_UNIX_SOCKET
}

void FCurlHttpRequest::SetupOptionHttpVersion()
{
	const FString HttpVersion = GetOption(HttpRequestOptions::HttpVersion);
	if (!HttpVersion.IsEmpty())
	{
		if (HttpVersion == FHttpConstants::VERSION_2TLS)
		{
			curl_easy_setopt(EasyHandle, CURLOPT_HTTP_VERSION, CURL_HTTP_VERSION_2TLS);
		}
		else if (HttpVersion == FHttpConstants::VERSION_1_1)
		{
			curl_easy_setopt(EasyHandle, CURLOPT_HTTP_VERSION, CURL_HTTP_VERSION_1_1);
		}
		else
		{
			UE_LOGF(LogHttp, Warning, "Unsupported http version %ls", *HttpVersion);
		}
	}
}

void FCurlHttpRequest::SetupOptionEnableCookies()
{
	const FString EnableCookies = GetOption(HttpRequestOptions::EnableCookies);
	if (EnableCookies.ToBool())
	{
		// Enable the cookie engine without reading any initial cookies.
		curl_easy_setopt(EasyHandle, CURLOPT_COOKIEFILE, "");
	}
}

void FCurlHttpRequest::CleanupRequestHttpThread()
{
	curl_easy_setopt(EasyHandle, CURLOPT_SHARE, nullptr);
}

bool FCurlHttpRequest::ProcessRequest()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_ProcessRequest);

	// Clear out response. If this is a re-used request, Response could point to a stale response until SetupRequestHttpThread is called
	ResponseCommon = nullptr;
	LastReportedBytesRead = 0;

	if (!PreProcess())
	{
		return false;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_CurlHttpAddThreadedRequest);
	// Mark as in-flight to prevent overlapped requests using the same object
	SetStatus(EHttpRequestStatus::Processing);

	// Add to global list while being processed so that the ref counted request does not get deleted
	FHttpModule::Get().GetHttpManager().AddThreadedRequest(SharedThis(this));

#if WITH_CURL_XCURL
	StartProcessTime = FPlatformTime::Seconds();
#endif

	UE_LOGF(LogHttp, Verbose, "%p: request (easy handle:%p) has been added to threaded queue for processing", this, EasyHandle);
	return true;
}

void FCurlHttpRequest::ClearInCaseOfRetry()
{
	FHttpRequestCommon::ClearInCaseOfRetry();

	// Clear out response. If this is a re-used request, Response could point to a stale response until SetupRequestHttpThread is called
	LastReportedBytesRead = 0;
	TotalBytesRead = 0;
	bAnyHttpActivity = false;
	if (CVarHttpCurlResetRequestCompletedOnRetryEnabled.GetValueOnAnyThread())
	{
		bCurlRequestCompleted = false;
	}

	// Clear the info cache log so we don't output messages from previous requests when reusing/retrying a request
	{
		const FScopeLock CacheLock(&InfoMessageCacheCriticalSection);
		for (FString& Line : InfoMessageCache)
		{
			Line.Reset();
		}
	}
}

bool FCurlHttpRequest::IsThreadedRequestComplete()
{
	if (bCurlRequestCompleted && ElapsedTime >= FHttpModule::Get().GetHttpDelayTime())
	{
		return true;
	}

	if (CurlAddToMultiResult != CURLM_OK)
	{
		return true;
	}

	return false;
}

void FCurlHttpRequest::TickThreadedRequest(float DeltaSeconds)
{
	ElapsedTime += DeltaSeconds;
}

void FCurlHttpRequest::AbortRequest()
{
	if (bCurlRequestCompleted)
	{
		return;
	}

	FHttpManager& HttpManager = FHttpModule::Get().GetHttpManager();
	if (HttpManager.IsValidRequest(this))
	{
		HttpManager.CancelThreadedRequest(SharedThis(this));
	}
	else
	{
		FinishRequestNotInHttpManager();
	}
}

void FCurlHttpRequest::Tick(float DeltaSeconds)
{
	if (DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnGameThread)
	{
		CheckProgressDelegate();
		BroadcastNewlyReceivedHeaders();
	}
}

void FCurlHttpRequest::CheckProgressDelegate()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_CheckProgressDelegate);
	const uint64 CurrentBytesRead = TotalBytesRead.load();
	const uint64 CurrentBytesSent = BytesSent.load();

	const bool bProcessing = CompletionStatus == EHttpRequestStatus::Processing;
	const bool bBytesSentChanged = (CurrentBytesSent != LastReportedBytesSent);
	const bool bBytesReceivedChanged = CurrentBytesRead != LastReportedBytesRead;
	const bool bProgressChanged = bBytesSentChanged || bBytesReceivedChanged;
	if (bProcessing && bProgressChanged)
	{
		LastReportedBytesSent = CurrentBytesSent;
		LastReportedBytesRead = CurrentBytesRead;
		// Update response progress
		OnRequestProgress64().ExecuteIfBound(SharedThis(this), LastReportedBytesSent, LastReportedBytesRead);
	}
}

void FCurlHttpRequest::BroadcastNewlyReceivedHeaders()
{
	// Process the headers received on the HTTP thread and merge them into the response's list of headers and then broadcast the new headers
	TPair<FString, FString> NewHeader;
	while (NewlyReceivedHeaders.Dequeue(NewHeader))
	{
		OnHeaderReceived().ExecuteIfBound(SharedThis(this), NewHeader.Key, NewHeader.Value);
	}
}

void FCurlHttpRequest::MarkAsCompleted(CURLcode InCurlCompletionResult)
{
	CurlCompletionResult = InCurlCompletionResult;
	bCurlRequestCompleted = true;

	if (TSharedPtr<FCurlHttpResponse> Response = StaticCastSharedPtr<FCurlHttpResponse>(ResponseCommon))
	{
		// get the information
		long HttpCode = 0;
		if (CURLE_OK == curl_easy_getinfo(EasyHandle, CURLINFO_RESPONSE_CODE, &HttpCode))
		{
			Response->SetResponseCode(HttpCode);
		}
	}

	StopActivityTimeoutTimer();
}

void FCurlHttpRequest::FinishRequest()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpRequest_FinishRequest);

	PostProcess();
	
	CheckProgressDelegate();

	TSharedPtr<FCurlHttpResponse> Response = StaticCastSharedPtr<FCurlHttpResponse>(ResponseCommon);

	// if completed, get more info
	if (bCurlRequestCompleted)
	{
		if (Response.IsValid())
		{
			Response->bSucceeded = (CURLE_OK == CurlCompletionResult);

			// If content length wasn't received through response header 
			if (Response->ContentLength == 0)
			{
				Response->ContentLength = TotalBytesRead;
			}

			if (Response->GetResponseCode() <= 0 && GetURL().StartsWith(TEXT("Http"), ESearchCase::IgnoreCase))
			{
				UE_LOGF(LogHttp, Warning, "%p: invalid HTTP response code received. URL: %ls, HTTP code: %d, content length: %llu, actual payload size: %llu",
					this, *GetURL(), Response->GetResponseCode(), Response->ContentLength, TotalBytesRead.load());
				Response->bSucceeded = false;
			}
		}
	}
	
	// if just finished, mark as stopped async processing
	if (Response.IsValid())
	{
		// Broadcast any headers we haven't broadcast yet
		// If using EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread, we don't add to NewlyReceivedHeaders and will have already broadcast
		if (DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnGameThread)
		{
			BroadcastNewlyReceivedHeaders();
		}
	}

	bool bSucceeded = Response.IsValid() && Response->bSucceeded;
	if (!bSucceeded)
	{
		LogCurlErrors();

		if (bCurlRequestCompleted)
		{
			switch (CurlCompletionResult)
			{
			case CURLE_COULDNT_CONNECT:
			case CURLE_OPERATION_TIMEDOUT:
			case CURLE_COULDNT_RESOLVE_PROXY:
			case CURLE_COULDNT_RESOLVE_HOST:
			case CURLE_SSL_CONNECT_ERROR:
#if WITH_CURL_XCURL
			case CURLE_SEND_ERROR:
#endif
				// report these as connection errors (safe to retry)
				SetFailureReason(EHttpFailureReason::ConnectionError);
				break;
			default:
				break;
			}
		}
	}

	OnFinishRequest(bSucceeded);
}

void FCurlHttpRequest::LogCurlErrors()
{
	if (CurlAddToMultiResult != CURLM_OK)
	{
		UE_LOGF(LogHttp, Warning, "%p: request failed, libcurl multi error: %d (%ls)", this, (int32)CurlAddToMultiResult, ANSI_TO_TCHAR(curl_multi_strerror(CurlAddToMultiResult)));
	}
	else if (CurlCompletionResult != CURLE_OK)
	{
		UE_LOGF(LogHttp, Warning, "%p: request failed, libcurl error: %d (%ls)", this, (int32)CurlCompletionResult, ANSI_TO_TCHAR(curl_easy_strerror(CurlCompletionResult)));
	}

	const bool bAborted = (bCanceled || bTimedOut || bActivityTimedOut);
	if (!bAborted)
	{
		const FScopeLock CacheLock(&InfoMessageCacheCriticalSection);
		for (int32 i = 0; i < InfoMessageCache.Num(); ++i)
		{
			int RelativeIndex = (LeastRecentlyCachedInfoMessageIndex + i) % InfoMessageCache.Num();
			if (!InfoMessageCache[RelativeIndex].IsEmpty())
			{
				UE_LOGF(LogHttp, Warning, "%p: libcurl info message cache %d (%ls)", this, RelativeIndex, *(InfoMessageCache[RelativeIndex]));
			}
		}
	}
}

void FCurlHttpRequest::CleanupRequest()
{
	CloseRequestPayloadDefaultImpl();

	if (CVarHttpCurlDeferEasyHandleLifetime.GetValueOnAnyThread())
	{
		DestroyEasyHandle();
	}
}

FHttpResponsePtr FCurlHttpRequest::CreateResponse()
{
	return MakeShared<FCurlHttpResponse>(*this);
}

void FCurlHttpRequest::MockResponseData()
{
	CurlCompletionResult = CURLE_OK;
	bCurlRequestCompleted = true;

	TSharedPtr<FCurlHttpResponse> Response = StaticCastSharedPtr<FCurlHttpResponse>(ResponseCommon);
	Response->bSucceeded = true;
}

// FCurlHttpRequest

FCurlHttpResponse::FCurlHttpResponse(const FCurlHttpRequest& InRequest)
	: FHttpResponseCommon(InRequest)
	, ContentLength(0)
	, bSucceeded(0)
{
}

TArray<FString> FCurlHttpResponse::GetAllHeaders() const
{
	TArray<FString> Result;
	if (!bIsReady)
	{
		UE_LOGF(LogHttp, Warning, "Can't get cached headers. Response still processing. %ls", *GetURL());
	}
	else
	{
		Result.Reserve(Headers.Num());
		for (const TPair<FString, FString>& It : Headers)
		{
			Result.Emplace(FCurlHttpRequest::CombineHeaderKeyValue(It.Key, It.Value));
		}
	}
	return Result;
}

FString FCurlHttpResponse::GetContentType() const
{
	return GetHeader(TEXT("Content-Type"));
}

uint64 FCurlHttpResponse::GetContentLength() const
{
	return ContentLength;
}

#endif //WITH_CURL