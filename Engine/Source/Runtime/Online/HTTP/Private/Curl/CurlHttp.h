// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/SpscQueue.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/ThreadSafeCounter.h"
#include "GenericPlatform/HttpRequestCommon.h"
#include "GenericPlatform/HttpResponseCommon.h"

class FCurlHttpResponse;

#if WITH_CURL
#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif

#ifdef PLATFORM_CURL_INCLUDE
	#include PLATFORM_CURL_INCLUDE
#else
	#include "curl/curl.h"
#endif

#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#if !defined(CURL_ENABLE_DEBUG_CALLBACK)
	#define CURL_ENABLE_DEBUG_CALLBACK 0
#endif

#if !defined(CURL_ENABLE_NO_TIMEOUTS_OPTION)
	#define CURL_ENABLE_NO_TIMEOUTS_OPTION 0
#endif

/**
 * Curl implementation of an HTTP request
 */
class FCurlHttpRequest : public FHttpRequestCommon
{
public:

	// implementation friends
	friend class FCurlHttpResponse;

	//~ Begin IHttpBase Interface
	virtual TArray<FString> GetAllHeaders() const override;
	virtual FString GetContentType() const override;
	virtual uint64 GetContentLength() const override;
	virtual const TArray<uint8>& GetContent() const override;
	//~ End IHttpBase Interface

	//~ Begin IHttpRequest Interface
	virtual FString GetVerb() const override;
	virtual void SetVerb(const FString& InVerb) override;
	virtual void SetOption(const FName Option, const FString& OptionValue) override;
	virtual void SetContent(const TArray<uint8>& ContentPayload) override;
	virtual void SetContent(TArray<uint8>&& ContentPayload) override;
	virtual void SetContentAsString(const FString& ContentString) override;
	virtual bool SetContentAsStreamedFile(const FString& Filename) override;
	virtual bool SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) override;
	virtual void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) override;
	virtual bool ProcessRequest() override;
	virtual void Tick(float DeltaSeconds) override;
	//~ End IHttpRequest Interface

	//~ Begin IHttpRequestThreaded Interface
	virtual bool StartThreadedRequest() override;
	virtual void FinishRequest() override;
	virtual bool IsThreadedRequestComplete() override;
	virtual void TickThreadedRequest(float DeltaSeconds) override;
	//~ End IHttpRequestThreaded Interface

	/**
	 * Perform the http-thread cleanup of the request
	 */
	void  CleanupRequestHttpThread();

	/**
	 * Returns libcurl's easy handle - needed for HTTP manager.
	 *
	 * @return libcurl's easy handle
	 */
	inline CURL * GetEasyHandle() const
	{
		return EasyHandle;
	}

	/**
	 * Marks request as completed (set by HTTP manager).
	 *
	 * Note that this method is intended to be lightweight,
	 * more processing will be done in Tick()
	 *
	 * @param CurlCompletionResult Operation result code as returned by libcurl
	 */
	void MarkAsCompleted(CURLcode InCurlCompletionResult);
	
	/** 
	 * Set the result for adding the easy handle to curl multi
	 */
	void SetAddToCurlMultiResult(CURLMcode Result)
	{
		CurlAddToMultiResult = Result;
	}

	/**
	 * Constructor
	 */
	FCurlHttpRequest();

	/**
	 * Destructor. Clean up any connection/request handles
	 */
	virtual ~FCurlHttpRequest();

protected:
	/**
	 * Perform the game-thread setup of the request
	 *
	 * @return true if the request was successfully setup
	 */
	virtual bool SetupRequest() override;

private:

	/**
	 * Initialize the curl easy handle and apply all static options.
	 * Called either from the constructor (legacy path) or SetupRequest (deferred path).
	 *
	 * @return true if the handle was successfully created
	 */
	bool InitEasyHandle();

	/**
	 * Destroy the curl easy handle and free the header list.
	 * Called either from the destructor (legacy path) or CleanupRequest (deferred path).
	 */
	void DestroyEasyHandle();

	/**
	 * Static callback to be used as read function (CURLOPT_READFUNCTION), will dispatch the call to proper instance
	 *
	 * @param Ptr buffer to copy data to (allocated and managed by libcurl)
	 * @param SizeInBlocks size of above buffer, in 'blocks'
	 * @param BlockSizeInBytes size of a single block
	 * @param UserData data we associated with request (will be a pointer to FCurlHttpRequest instance)
	 * @return number of bytes actually written to buffer, or CURL_READFUNC_ABORT to abort the operation
	 */
	static size_t StaticUploadCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);

	/**
	 * Method called when libcurl wants us to supply more data (see CURLOPT_READFUNCTION)
	 *
	 * @param Ptr buffer to copy data to (allocated and managed by libcurl)
	 * @param SizeInBlocks size of above buffer, in 'blocks'
	 * @param BlockSizeInBytes size of a single block
	 * @return number of bytes actually written to buffer, or CURL_READFUNC_ABORT to abort the operation
	 */
	size_t UploadCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes);

	/**
	 * Static callback to be used as seek function (CURLOPT_SEEKFUNCTION), will dispatch the call to proper instance
	 *
	 * @param UserData data we associated with request (will be a pointer to FCurlHttpRequest instance)
	 * @param Offset offset from Origin to seek to
	 * @param Origin where to seek to. Can be SEEK_SET, SEEK_CUR, or SEEK_END
	 * @return CURL_SEEKFUNC_OK if the seek was successful, CURL_SEEKFUNC_FAIL if the request should be failed due to inability to seek, or CURL_SEEKFUNC_CANTSEEK to allow curl to try to workaround the inability to seek
	 */
	static int StaticSeekCallback(void* UserData, curl_off_t Offset, int Origin);

	/**
	 * Method called when libcurl wants us to seek to a position in the stream (see CURLOPT_SEEKFUNCTION)
	 *
	 * @param Offset offset from Origin to seek to
	 * @param Origin where to seek to. Can be SEEK_SET, SEEK_CUR, or SEEK_END
	 * @return CURL_SEEKFUNC_OK if the seek was successful, CURL_SEEKFUNC_FAIL if the request should be failed due to inability to seek, or CURL_SEEKFUNC_CANTSEEK to allow curl to try to workaround the inability to seek
	 */
	int SeekCallback(curl_off_t Offset, int Origin);

	/**
	 * Static callback to be used as header function (CURLOPT_HEADERFUNCTION), will dispatch the call to proper instance
	 *
	 * @param Ptr buffer to copy data to (allocated and managed by libcurl)
	 * @param SizeInBlocks size of above buffer, in 'blocks'
	 * @param BlockSizeInBytes size of a single block
	 * @param UserData data we associated with request (will be a pointer to FCurlHttpRequest instance)
	 * @return number of bytes actually processed, error is triggered if it does not match number of bytes passed
	 */
	static size_t StaticReceiveResponseHeaderCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);

	/**
	 * Method called when libcurl wants us to receive response header (see CURLOPT_HEADERFUNCTION). Headers will be passed
	 * line by line (i.e. this callback will be called with a full line), not necessarily zero-terminated. This callback will
	 * be also passed any intermediate headers, not only final response's ones.
	 *
	 * @param Ptr buffer to copy data to (allocated and managed by libcurl)
	 * @param SizeInBlocks size of above buffer, in 'blocks'
	 * @param BlockSizeInBytes size of a single block
	 * @return number of bytes actually processed, error is triggered if it does not match number of bytes passed
	 */
	size_t ReceiveResponseHeaderCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes);

	/**
	 * Static callback to be used as write function (CURLOPT_WRITEFUNCTION), will dispatch the call to proper instance
	 *
	 * @param Ptr buffer to copy data to (allocated and managed by libcurl)
	 * @param SizeInBlocks size of above buffer, in 'blocks'
	 * @param BlockSizeInBytes size of a single block
	 * @param UserData data we associated with request (will be a pointer to FCurlHttpRequest instance)
	 * @return number of bytes actually processed, error is triggered if it does not match number of bytes passed
	 */
	static size_t StaticReceiveResponseBodyCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);

	/**
	 * Method called when libcurl wants us to receive response body (see CURLOPT_WRITEFUNCTION)
	 *
	 * @param Ptr buffer to copy data to (allocated and managed by libcurl)
	 * @param SizeInBlocks size of above buffer, in 'blocks'
	 * @param BlockSizeInBytes size of a single block
	 * @return number of bytes actually processed, error is triggered if it does not match number of bytes passed
	 */
	size_t ReceiveResponseBodyCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes);

	/**
	 * Static callback to be used as debug function (CURLOPT_DEBUGFUNCTION), will dispatch the call to proper instance
	 *
	 * @param Handle handle to which the debug information applies
	 * @param DebugInfoType type of information (CURLINFO_*) 
	 * @param DebugInfo debug information itself (may NOT be text, may NOT be zero-terminated)
	 * @param DebugInfoSize exact size of debug information
	 * @param UserData data we associated with request (will be a pointer to FCurlHttpRequest instance)
	 * @return must return 0
	 */
	static size_t StaticDebugCallback(CURL * Handle, curl_infotype DebugInfoType, char * DebugInfo, size_t DebugInfoSize, void* UserData);

	/**
	 * Method called with debug information about libcurl activities (see CURLOPT_DEBUGFUNCTION)
	 *
	 * @param Handle handle to which the debug information applies
	 * @param DebugInfoType type of information (CURLINFO_*) 
	 * @param DebugInfo debug information itself (may NOT be text, may NOT be zero-terminated)
	 * @param DebugInfoSize exact size of debug information
	 * @return must return 0
	 */
	size_t DebugCallback(CURL * Handle, curl_infotype DebugInfoType, char * DebugInfo, size_t DebugInfoSize);

	virtual void AbortRequest() override;

	/**
	 * Trigger the request progress delegate if progress has changed
	 */
	void CheckProgressDelegate();

	/** Broadcast newly received headers */
	void BroadcastNewlyReceivedHeaders();

	/** Combine a header's key/value in the format "Key: Value" */
	static FString CombineHeaderKeyValue(const FString& HeaderKey, const FString& HeaderValue);

	virtual void CleanupRequest() override;

	void OnAnyActivityOccur(FStringView Reason);

	virtual void ClearInCaseOfRetry() override;

	virtual FHttpResponsePtr CreateResponse() override;
	virtual void MockResponseData() override;

	void SetupOptionUnixSocketPath();
	void SetupOptionHttpVersion();
	void SetupOptionEnableCookies();

	void LogCurlErrors();

private:

	/** Pointer to an easy handle specific to this request */
	CURL *			EasyHandle;	
	/** List of custom headers to be passed to CURL */
	curl_slist *	HeaderList;
	/** Cached verb */
	FString			Verb;
	/** Set to true when request has been completed */
	std::atomic<bool> bCurlRequestCompleted;
	/** Set to true when request has "30* Multiple Choices" (e.g. 301 Moved Permanently, 302 temporary redirect, 308 Permanent Redirect, etc.) */
	bool			bRedirected;
	/** Set to true if request failed to be added to curl multi */
	CURLMcode		CurlAddToMultiResult;
	/** Operation result code as returned by libcurl */
	CURLcode		CurlCompletionResult;
	/** Is the request payload seekable? */
	bool bIsRequestPayloadSeekable = false;
	/** Have we had any HTTP activity with the host? Sending headers, SSL handshake, etc */
	bool bAnyHttpActivity;
	/** Newly received headers we need to inform listeners about */
	TSpscQueue<TPair<FString, FString>> NewlyReceivedHeaders;
	/** Number of bytes sent already */
	std::atomic<int64> BytesSent;
	/** Total number of bytes sent already (includes data re-sent by seek attempts) */
	std::atomic<int64> TotalBytesSent;
	/** Caches how many bytes of the response we've read so far */
	std::atomic<int64> TotalBytesRead;
	/** Last bytes read reported to progress delegate */
	uint64 LastReportedBytesRead;
	/** Last bytes sent reported to progress delegate */
	uint64 LastReportedBytesSent;
	/** Number of info channel messages to cache */
	static const constexpr int32 NumberOfInfoMessagesToCache = 50;
	/** Index of least recently cached message */
	int32 LeastRecentlyCachedInfoMessageIndex;
	/** Critical section for accessing InfoMessageCache */
	FCriticalSection InfoMessageCacheCriticalSection;
	/** Cache of info messages from libcurl */
	TArray<FString, TFixedAllocator<NumberOfInfoMessagesToCache>> InfoMessageCache;
};

/**
 * Curl implementation of an HTTP response
 */
class FCurlHttpResponse : public FHttpResponseCommon
{
public:
	// implementation friends
	friend class FCurlHttpRequest;

	//~ Begin IHttpBase Interface
	virtual TArray<FString> GetAllHeaders() const override;	
	virtual FString GetContentType() const override;
	virtual uint64 GetContentLength() const override;
	//~ End IHttpBase Interface

	/**
	 * Constructor
	 *
	 * @param InRequest - original request that created this response
	 */
	FCurlHttpResponse(const FCurlHttpRequest& InRequest);

private:
	/** Cached content length from completed response */
	uint64 ContentLength;
	/** True if the response was successfully received/processed */
	int32 volatile bSucceeded;
};

#endif //WITH_CURL
