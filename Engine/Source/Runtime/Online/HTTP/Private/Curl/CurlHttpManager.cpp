// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curl/CurlHttpManager.h"

#if WITH_CURL

#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/LocalTimestampDirectoryVisitor.h"
#include "Misc/Paths.h"
#include "Misc/Fork.h"

#include "Curl/CurlHttpThread.h"
#include "Curl/CurlMultiPollEventLoopHttpThread.h"
#include "Curl/CurlMultiWaitEventLoopHttpThread.h"
#include "Curl/CurlSocketEventLoopHttpThread.h"
#include "Curl/CurlHttp.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/IConsoleManager.h"
#include "HttpModule.h"
#include "ThirdParty/UECurl.h"

#if WITH_SSL
#include "Modules/ModuleManager.h"
#include "Ssl.h"
#include <openssl/crypto.h>
#endif

#include "SocketSubsystem.h"
#include "IPAddress.h"

#include "Http.h"

#ifndef DISABLE_UNVERIFIED_CERTIFICATE_LOADING
#define DISABLE_UNVERIFIED_CERTIFICATE_LOADING 0
#endif

// at least from LIBCURL_VERSION_NUM >= 0x080f00 -> 8.15
#if !defined(LIBCURL_PLATFORM_NO_DEBUG_MODE) && defined(LIBCURL_EXTERNAL_ASSERT_HANDLER)
extern "C"
{
	typedef void (*curl_assert_callback)(const char* expr, const char* file, int32 line);
	extern curl_assert_callback curl_global_assert_handler;
}

namespace CurlPrivate
{
static void UECurlAssertHandler(const char* InExpr, const char* InFile, int32 InLine)
{
	UE_LOGF(LogHttp, Error, "CURL ASSERTION FAILED: %s at %s:%d", InExpr, InFile, InLine);
}

static void InitCurlAssertHandler()
{
	curl_global_assert_handler = &UECurlAssertHandler;
}
}
#endif // !LIBCURL_PLATFORM_NO_DEBUG_MODE && LIBCURL_EXTERNAL_ASSERT_HANDLER

CURLM* FCurlHttpManager::GMultiHandle = nullptr;
#if !WITH_CURL_XCURL
CURLSH* FCurlHttpManager::GShareHandle = nullptr;
#endif

FCurlHttpManager::FCurlRequestOptions FCurlHttpManager::CurlRequestOptions;

extern TAutoConsoleVariable<int32> CVarHttpMaxConcurrentRequests;

TAutoConsoleVariable<int32> CVarHttpCurlMaxTotalConnections(
	TEXT("HTTP.Curl.MaxTotalConnections"),
	UE_HTTP_CURL_DEFAULT_MAX_TOTAL_CONNECTIONS,
	TEXT("The max number of total http connections"),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<int32> CVarHttpCurlMaxIdleConnections(
	TEXT("HTTP.Curl.MaxIdleConnections"),
	UE_HTTP_CURL_DEFAULT_MAX_IDLE_CONNECTIONS,
	TEXT("The max number of idle http connections"),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<bool> CVarHttpReuseConnectionEnabled(
	TEXT("http.CurlReuseConnectionEnabled"),
	UE_HTTP_CURL_REUSE_CONNECTION_ENABLED_BY_DEFAULT,
	TEXT("Whether to reuse connection by default"),
	ECVF_SaveForNextBoot
);

// Seeded once during InitCurl() from the [HTTP] HttpMaxConnectionsPerServer config value if > 0.
TAutoConsoleVariable<int32> CVarHttpCurlMaxHostConnections(
	TEXT("HTTP.Curl.MaxHostConnections"),
	0,
	TEXT("The max number of connections per host (CURLMOPT_MAX_HOST_CONNECTIONS). 0 means unlimited. Can be changed at runtime."),
	ECVF_Default
);

namespace UE::Curl
{

static FCurlInitializeData InitializeData;

/** Set the max number of connections per host at runtime. Must be called from the HTTP thread. */
static void SetMaxHostConnections(int32 InMaxHostConnections)
{
	if (!FCurlHttpManager::IsInit())
	{
		UE_LOGF(LogHttp, Warning, "SetMaxHostConnections: curl not initialized, cannot change limit");
		return;
	}

	// Normalize negative values to 0 (unlimited)
	const int32 EffectiveMaxHostConnections = FMath::Max(InMaxHostConnections, 0);

	const CURLMcode SetOptResult = curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_MAX_HOST_CONNECTIONS, static_cast<long>(EffectiveMaxHostConnections));
	if (SetOptResult != CURLM_OK)
	{
		FUTF8ToTCHAR Converter(curl_multi_strerror(SetOptResult));
		UE_LOGF(LogHttp, Warning, "SetMaxHostConnections: Failed to set to %d, error %d ('%ls')", EffectiveMaxHostConnections, (int32)SetOptResult, Converter.Get());
		return;
	}

	UE_LOGF(LogHttp, Log, "SetMaxHostConnections: Changed from %d to %d", FCurlHttpManager::CurlRequestOptions.MaxHostConnections, EffectiveMaxHostConnections);
	FCurlHttpManager::CurlRequestOptions.MaxHostConnections = EffectiveMaxHostConnections;
}

/** Set the max idle connections (CURLMOPT_MAXCONNECTS) based on reuse enabled CVar. Must be called from the HTTP thread. */
static void SetMaxConnects()
{
	if (!FCurlHttpManager::IsInit())
	{
		return;
	}

	const bool bReuseEnabled = CVarHttpReuseConnectionEnabled.GetValueOnAnyThread();
	const long MaxConnects = static_cast<long>(bReuseEnabled ? CVarHttpCurlMaxIdleConnections.GetValueOnAnyThread() : 0);

	const CURLMcode SetOptResult = curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_MAXCONNECTS, MaxConnects);
	if (SetOptResult != CURLM_OK)
	{
		FUTF8ToTCHAR Converter(curl_multi_strerror(SetOptResult));
		UE_LOGF(LogHttp, Warning, "SetMaxConnects: Failed to set to %ld, error %d ('%ls')", MaxConnects, (int32)SetOptResult, Converter.Get());
	}
}

}

bool FCurlHttpManager::IsInit()
{
	return GMultiHandle != nullptr;
}

void FCurlHttpManager::InitCurl()
{
	if (IsInit())
	{
		UE_LOGF(LogInit, Warning, "Already initialized multi handle");
		return;
	}

#if !defined(LIBCURL_PLATFORM_NO_DEBUG_MODE) && defined(LIBCURL_EXTERNAL_ASSERT_HANDLER)
	CurlPrivate::InitCurlAssertHandler();
#endif // !LIBCURL_PLATFORM_NO_DEBUG_MODE && LIBCURL_EXTERNAL_ASSERT_HANDLER

#if WITH_SSL
	// Make sure SSL is loaded so that we can use the shared cert pool, and to globally initialize OpenSSL if possible
	FSslModule& SslModule = FModuleManager::LoadModuleChecked<FSslModule>("SSL");
#endif

	CURLcode InitResult = (CURLcode) UE::Curl::ConditionalInitialize(UE::Curl::InitializeData);
	if (InitResult == 0)
	{
		curl_version_info_data * VersionInfo = curl_version_info(CURLVERSION_NOW);
		if (VersionInfo)
		{
			UE_LOGF(LogInit, Log, "Using libcurl %ls", ANSI_TO_TCHAR(VersionInfo->version));
			UE_LOGF(LogInit, Log, " - built for %ls", ANSI_TO_TCHAR(VersionInfo->host));

			if (VersionInfo->features & CURL_VERSION_SSL)
			{
				UE_LOGF(LogInit, Log, " - supports SSL with %ls", ANSI_TO_TCHAR(VersionInfo->ssl_version));
			}
			else
			{
				// No SSL
				UE_LOGF(LogInit, Log, " - NO SSL SUPPORT!");
			}

			if (VersionInfo->features & CURL_VERSION_LIBZ)
			{
				UE_LOGF(LogInit, Log, " - supports HTTP deflate (compression) using libz %ls", ANSI_TO_TCHAR(VersionInfo->libz_version));
			}

			UE_LOGF(LogInit, Log, " - other features:");

#define PrintCurlFeature(Feature)	\
			if (VersionInfo->features & Feature) \
			{ \
			UE_LOGF(LogInit, Log, "     %ls", TEXT(#Feature));	\
			}

			PrintCurlFeature(CURL_VERSION_SSL);
			PrintCurlFeature(CURL_VERSION_LIBZ);

			PrintCurlFeature(CURL_VERSION_DEBUG);
			PrintCurlFeature(CURL_VERSION_IPV6);
			PrintCurlFeature(CURL_VERSION_ASYNCHDNS);
			PrintCurlFeature(CURL_VERSION_LARGEFILE);
			PrintCurlFeature(CURL_VERSION_IDN);
			PrintCurlFeature(CURL_VERSION_CONV);
			PrintCurlFeature(CURL_VERSION_TLSAUTH_SRP);
			PrintCurlFeature(CURL_VERSION_HTTP2); 
#undef PrintCurlFeature
		}

		GMultiHandle = curl_multi_init();
		if (NULL == GMultiHandle)
		{
			UE_LOGF(LogInit, Fatal, "Could not initialize create libcurl multi handle! HTTP transfers will not function properly.");
		}

#if !WITH_CURL_XCURL
		int32 MaxTotalConnections = 0;
		// Support of customization by game settings
		if (GConfig->GetInt(TEXT("HTTP.Curl"), TEXT("MaxTotalConnections"), MaxTotalConnections, GEngineIni) && MaxTotalConnections > 0)
		{
			// If CVarHttpCurlMaxTotalConnections was set by ECVF_SetByHotfix through ECVF_SaveForNextBoot, this call will be an none operation
			CVarHttpCurlMaxTotalConnections.AsVariable()->Set(MaxTotalConnections, ECVF_SetByGameSetting);
		}

		CURLMcode SetOptResult = curl_multi_setopt(GMultiHandle, CURLMOPT_MAX_TOTAL_CONNECTIONS, static_cast<long>(CVarHttpCurlMaxTotalConnections.GetValueOnAnyThread()));
		if (SetOptResult != CURLM_OK)
		{
			UE_LOGF(LogInit, Warning, "Failed to set libcurl max total connections options (%d), error %d ('%ls')",
				CVarHttpCurlMaxTotalConnections.GetValueOnAnyThread(), static_cast<int32>(SetOptResult), StringCast<TCHAR>(curl_multi_strerror(SetOptResult)).Get());
		}

		// Support of customization by game settings
		bool bCurlReuseConnectionEnabledFromConfig = true;
		if (GConfig->GetBool(TEXT("http"), TEXT("CurlReuseConnectionEnabled"), bCurlReuseConnectionEnabledFromConfig, GEngineIni))
		{
			// If CVarHttpReuseConnectionEnabled was set by ECVF_SetByHotfix through ECVF_SaveForNextBoot, this call will be a none operation
			CVarHttpReuseConnectionEnabled.AsVariable()->Set(bCurlReuseConnectionEnabledFromConfig, ECVF_SetByGameSetting);
		}

		// Support of customization by game settings
		int32 MaxIdleConnections = 0;
		if (GConfig->GetInt(TEXT("HTTP.Curl"), TEXT("MaxConnects"), MaxIdleConnections, GEngineIni) && MaxIdleConnections >= 0)
		{
			// If CVarHttpCurlMaxIdleConnections was set by ECVF_SetByHotfix through ECVF_SaveForNextBoot, this call will be an none operation
			CVarHttpCurlMaxIdleConnections.AsVariable()->Set(MaxIdleConnections, ECVF_SetByGameSetting);
		}

		UE::Curl::SetMaxConnects();

		GShareHandle = curl_share_init();
		if (NULL != GShareHandle)
		{
			curl_share_setopt(GShareHandle, CURLSHOPT_SHARE, CURL_LOCK_DATA_COOKIE);
			curl_share_setopt(GShareHandle, CURLSHOPT_SHARE, CURL_LOCK_DATA_DNS);
			curl_share_setopt(GShareHandle, CURLSHOPT_SHARE, CURL_LOCK_DATA_SSL_SESSION);
		}
		else
		{
			UE_LOGF(LogInit, Fatal, "Could not initialize libcurl share handle!");
		}
#endif // !WITH_CURL_XCURL
	}
	else
	{
		UE_LOGF(LogInit, Fatal, "Could not initialize libcurl (result=%d), HTTP transfers will not function properly.", (int32)InitResult);
	}

#if WITH_SSL
	// Set default verify peer value based on availability of certificates
	CurlRequestOptions.bVerifyPeer = SslModule.GetCertificateManager().HasCertificatesAvailable();
#endif

	bool bVerifyPeer = true;
#if DISABLE_UNVERIFIED_CERTIFICATE_LOADING
	CurlRequestOptions.bVerifyPeer = bVerifyPeer;
#else
	if (GConfig->GetBool(TEXT("/Script/Engine.NetworkSettings"), TEXT("n.VerifyPeer"), bVerifyPeer, GEngineIni))
	{
		CurlRequestOptions.bVerifyPeer = bVerifyPeer;
	}
#endif

	bool bAcceptCompressedContent = true;
	if (GConfig->GetBool(TEXT("HTTP"), TEXT("AcceptCompressedContent"), bAcceptCompressedContent, GEngineIni))
	{
		CurlRequestOptions.bAcceptCompressedContent = bAcceptCompressedContent;
	}

	int32 ConfigBufferSize = 0;
	if (GConfig->GetInt(TEXT("HTTP.Curl"), TEXT("BufferSize"), ConfigBufferSize, GEngineIni) && ConfigBufferSize > 0)
	{
		CurlRequestOptions.BufferSize = ConfigBufferSize;
	}

	GConfig->GetBool(TEXT("HTTP.Curl"), TEXT("bAllowSeekFunction"), CurlRequestOptions.bAllowSeekFunction, GEngineIni);


	// Seed the CVar from the [HTTP] config value
	const int32 ConfigMaxHostConnections = FHttpModule::Get().GetHttpMaxConnectionsPerServer();
	if (ConfigMaxHostConnections > 0)
	{
		CVarHttpCurlMaxHostConnections.AsVariable()->Set(ConfigMaxHostConnections, ECVF_SetByGameSetting);
	}
	CurlRequestOptions.MaxHostConnections = FMath::Max(CVarHttpCurlMaxHostConnections.GetValueOnAnyThread(), 0);

	if (CurlRequestOptions.MaxHostConnections > 0)
	{
		const CURLMcode SetOptResult = curl_multi_setopt(GMultiHandle, CURLMOPT_MAX_HOST_CONNECTIONS, static_cast<long>(CurlRequestOptions.MaxHostConnections));
		if (SetOptResult != CURLM_OK)
		{
			FUTF8ToTCHAR Converter(curl_multi_strerror(SetOptResult));
			UE_LOGF(LogInit, Warning, "Failed to set max host connections options (%d), error %d ('%ls')",
				CurlRequestOptions.MaxHostConnections, (int32)SetOptResult, Converter.Get());
			CurlRequestOptions.MaxHostConnections = 0;
		}
	}
	else
	{
		CurlRequestOptions.MaxHostConnections = 0;
	}


	TCHAR Home[256] = TEXT("");
	if (FParse::Value(FCommandLine::Get(), TEXT("MULTIHOMEHTTP="), Home, UE_ARRAY_COUNT(Home)))
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		if (SocketSubsystem && SocketSubsystem->GetAddressFromString(Home).IsValid())
		{
			CurlRequestOptions.LocalHostAddr = FString(Home);
		}
	}

	// print for visibility
	CurlRequestOptions.Log();

	// Bridge game-thread CVar changes to the HTTP thread. When game code calls Set() on this CVar,
	// the delegate captures the new value and dispatches it via AddHttpThreadTask so that
	// curl_multi_setopt is called on the correct thread.
	CVarHttpCurlMaxHostConnections.AsVariable()->SetOnChangedCallback(
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
		{
			const int32 NewValue = Variable->GetInt();
			FHttpModule::Get().GetHttpManager().AddHttpThreadTask([NewValue]()
			{
				UE::Curl::SetMaxHostConnections(NewValue);
			});
		})
	);

	CVarHttpReuseConnectionEnabled.AsVariable()->SetOnChangedCallback(
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
		{
			FHttpModule::Get().GetHttpManager().AddHttpThreadTask([]()
			{
				UE::Curl::SetMaxConnects();
			});
		})
	);

	CVarHttpCurlMaxIdleConnections.AsVariable()->SetOnChangedCallback(
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
		{
			FHttpModule::Get().GetHttpManager().AddHttpThreadTask([]()
			{
				UE::Curl::SetMaxConnects();
			});
		})
	);
}

void FCurlHttpManager::FCurlRequestOptions::Log()
{
	UE_LOGF(LogInit, Log, " CurlRequestOptions (configurable via config and command line):");
		UE_LOGF(LogInit, Log, " - bVerifyPeer = %ls  - Libcurl will %lsverify peer certificate",
		bVerifyPeer ? TEXT("true") : TEXT("false"),
		bVerifyPeer ? TEXT("") : TEXT("NOT ")
		);

	const FString& ProxyAddress = FHttpModule::Get().GetProxyAddress();
	const bool bUseHttpProxy = !ProxyAddress.IsEmpty();
	UE_LOGF(LogInit, Log, " - bUseHttpProxy = %ls  - Libcurl will %lsuse HTTP proxy",
		bUseHttpProxy ? TEXT("true") : TEXT("false"),
		bUseHttpProxy ? TEXT("") : TEXT("NOT ")
		);	
	if (bUseHttpProxy)
	{
		UE_LOGF(LogInit, Log, " - HttpProxyAddress = '%ls'", *ProxyAddress);
	}

	UE_LOGF(LogInit, Log, " - CVarHttpReuseConnectionEnabled = %ls  - Libcurl will %lsreuse connections",
		CVarHttpReuseConnectionEnabled.GetValueOnAnyThread() ? TEXT("true") : TEXT("false"),
		CVarHttpReuseConnectionEnabled.GetValueOnAnyThread() ? TEXT("") : TEXT("NOT ")
		);

	UE_LOGF(LogInit, Log, " - MaxHostConnections = %d  - Libcurl will %lslimit the number of connections to a host",
		MaxHostConnections,
		(MaxHostConnections == 0) ? TEXT("NOT ") : TEXT("")
		);

	UE_LOGF(LogInit, Log, " - LocalHostAddr = %ls", LocalHostAddr.IsEmpty() ? TEXT("Default") : *LocalHostAddr);

	UE_LOGF(LogInit, Log, " - BufferSize = %d", CurlRequestOptions.BufferSize);
}


void FCurlHttpManager::ShutdownCurl()
{
#if !WITH_CURL_XCURL
	if (GShareHandle != nullptr)
	{
		CURLSHcode ShareCleanupCode = curl_share_cleanup(GShareHandle);
		UE_CLOGF(ShareCleanupCode != CURLSHE_OK, LogHttp, Warning, "curl_share_cleanup failed. ReturnValue=[%d]", static_cast<int32>(ShareCleanupCode));
		GShareHandle = nullptr;
	}
#endif

	CVarHttpCurlMaxHostConnections.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate());
	CVarHttpReuseConnectionEnabled.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate());
	CVarHttpCurlMaxIdleConnections.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate());

	if (GMultiHandle != nullptr)
	{
		CURLMcode MutliCleanupCode = curl_multi_cleanup(GMultiHandle);
		UE_CLOGF(MutliCleanupCode != CURLM_OK, LogHttp, Warning, "curl_multi_cleanup failed. ReturnValue=[%d]", static_cast<int32>(MutliCleanupCode));
		GMultiHandle = nullptr;
	}

	UE::Curl::ConditionalShutdown(UE::Curl::InitializeData);
}

void FCurlHttpManager::OnBeforeFork()
{
	FHttpManager::OnBeforeFork();

	Thread->StopThread();
	ShutdownCurl();
}

void FCurlHttpManager::OnAfterFork()
{
	InitCurl();

	if (FForkProcessHelper::IsForkedChildProcess() == false || FForkProcessHelper::SupportsMultithreadingPostFork() == false)
	{
		// Since this will create a fake thread its safe to create it immediately here
		Thread->StartThread();
	}

	FHttpManager::OnAfterFork();
}

void FCurlHttpManager::OnEndFramePostFork()
{
	if (FForkProcessHelper::SupportsMultithreadingPostFork())
	{
		// We forked and the frame is done, time to start the autonomous thread
		check(FForkProcessHelper::IsForkedMultithreadInstance());
		Thread->StartThread();
	}

	FHttpManager::OnEndFramePostFork();
}

void FCurlHttpManager::UpdateConfigs()
{
	// Update configs - update settings that are safe to update after initialize 
	FHttpManager::UpdateConfigs();

	{
		bool bAcceptCompressedContent = true;
		if (GConfig->GetBool(TEXT("HTTP"), TEXT("AcceptCompressedContent"), bAcceptCompressedContent, GEngineIni))
		{
			if (CurlRequestOptions.bAcceptCompressedContent != bAcceptCompressedContent)
			{
				UE_LOGF(LogHttp, Log, "AcceptCompressedContent changed from %ls to %ls", *LexToString(CurlRequestOptions.bAcceptCompressedContent), *LexToString(bAcceptCompressedContent));
				CurlRequestOptions.bAcceptCompressedContent = bAcceptCompressedContent;
			}
		}
	}

	{
		int32 ConfigBufferSize = 0;
		if (GConfig->GetInt(TEXT("HTTP.Curl"), TEXT("BufferSize"), ConfigBufferSize, GEngineIni) && ConfigBufferSize > 0)
		{
			if (CurlRequestOptions.BufferSize != ConfigBufferSize)
			{
				UE_LOGF(LogHttp, Log, "BufferSize changed from %d to %d", CurlRequestOptions.BufferSize, ConfigBufferSize);
				CurlRequestOptions.BufferSize = ConfigBufferSize;
			}
		}
	}

	{
		bool bConfigAllowSeekFunction = false;
		if (GConfig->GetBool(TEXT("HTTP.Curl"), TEXT("bAllowSeekFunction"), bConfigAllowSeekFunction, GEngineIni))
		{
			if (CurlRequestOptions.bAllowSeekFunction != bConfigAllowSeekFunction)
			{
				UE_LOGF(LogHttp, Log, "bAllowSeekFunction changed from %ls to %ls", *LexToString(CurlRequestOptions.bAllowSeekFunction), *LexToString(bConfigAllowSeekFunction));
				CurlRequestOptions.bAllowSeekFunction = bConfigAllowSeekFunction;
			}
		}
	}
}

FHttpThreadBase* FCurlHttpManager::CreateHttpThread()
{
	if (bUseEventLoop)
	{
#if WITH_CURL_MULTIPOLL
		UE_LOGF(LogInit, Log, "CreateHttpThread using FCurlMultiPollEventLoopHttpThread");
		return new FCurlMultiPollEventLoopHttpThread();

#elif WITH_CURL_MULTISOCKET
		UE_LOGF(LogInit, Log, "CreateHttpThread using FCurlSocketEventLoopHttpThread");
		return new FCurlSocketEventLoopHttpThread();

#elif WITH_CURL_MULTIWAIT
		UE_LOGF(LogInit, Log, "CreateHttpThread using FCurlMultiWaitEventLoopHttpThread");
		return new FCurlMultiWaitEventLoopHttpThread();
#endif
	}

	UE_LOGF(LogInit, Log, "CreateHttpThread using FCurlHttpThread");
	return new FCurlHttpThread();
}

bool FCurlHttpManager::SupportsDynamicProxy() const
{
	return true;
}
#endif //WITH_CURL
