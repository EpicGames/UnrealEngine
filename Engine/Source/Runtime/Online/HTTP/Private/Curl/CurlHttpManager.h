// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HttpManager.h"

class FHttpThread;

#if WITH_CURL

#if !WITH_CURL_XCURL
typedef void CURLSH;
#endif

#if defined(CURL_NO_OLDIES)
typedef struct Curl_multi CURLM;
#else
typedef void CURLM;
#endif

class FCurlHttpManager : public FHttpManager
{
public:
	static void InitCurl();
	static void ShutdownCurl();
	static bool IsInit();
#if !WITH_CURL_XCURL
	static CURLSH* GShareHandle;
#endif
	static CURLM * GMultiHandle;

	static struct FCurlRequestOptions
	{
		FCurlRequestOptions()
			:	bVerifyPeer(true)
			,	bAcceptCompressedContent(true)
			,	MaxHostConnections(0)
			,	BufferSize(64*1024)
		{}

		/** Prints out the options to the log */
		void Log();

		/** Whether or not should verify peer certificate (disable to allow self-signed certs) */
		bool bVerifyPeer;

		/** Allow servers to send compressed content.  Can have a very small cpu cost, and huge bandwidth and response time savings from correctly configured servers. */
		bool bAcceptCompressedContent;

		/** The maximum number of connections to a particular host */
		int32 MaxHostConnections;

		/** Local address to use when making request, respects MULTIHOME command line option */
		FString LocalHostAddr;

		/** Receive buffer size */
		int32 BufferSize;

		/** Do we allow seeking? */
		bool bAllowSeekFunction = false;
	}
	CurlRequestOptions;

	//~ Begin HttpManager Interface
	virtual void OnBeforeFork() override;
	virtual void OnAfterFork() override;
	virtual void OnEndFramePostFork() override;
	virtual void UpdateConfigs() override;

public:
	virtual bool SupportsDynamicProxy() const override;
protected:
	virtual FHttpThreadBase* CreateHttpThread() override;
	//~ End HttpManager Interface
};

#endif //WITH_CURL 
