// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curl/CurlHttpThread.h"
#include "Stats/Stats.h"
#include "Http.h"
#include "Curl/CurlHttp.h"
#include "Curl/CurlHttpManager.h"

#if WITH_CURL

FCurlHttpThread::FCurlHttpThread()
{
}

void FCurlHttpThread::HttpThreadTick(float DeltaSeconds)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpThread_HttpThreadTick);
	check(FCurlHttpManager::IsInit());

	if (RunningThreadedRequests.Num() > 0)
	{
		int RunningRequests = -1;
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpThread_HttpThreadTick_Perform);
			curl_multi_perform(FCurlHttpManager::GMultiHandle, &RunningRequests);
		}

		// read more info if number of requests changed or if there's zero running
		// (note that some requests might have never be "running" from libcurl's point of view)
		if (RunningRequests == 0 || RunningRequests != RunningThreadedRequests.Num())
		{
			for (;;)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlHttpThread_HttpThreadTick_Loop);
				int MsgsStillInQueue = 0;	// may use that to impose some upper limit we may spend in that loop
				CURLMsg * Message = curl_multi_info_read(FCurlHttpManager::GMultiHandle, &MsgsStillInQueue);

				if (Message == NULL)
				{
					break;
				}

				// find out which requests have completed
				if (Message->msg == CURLMSG_DONE)
				{
					CURL* CompletedHandle = Message->easy_handle;
					curl_multi_remove_handle(FCurlHttpManager::GMultiHandle, CompletedHandle);

					FHttpRequestCommon** Request = HandlesToRequests.Find(CompletedHandle);
					if (Request)
					{
						FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(*Request);
						CurlRequest->MarkAsCompleted(Message->data.result);

						UE_LOGF(LogHttp, Verbose, "Request %p (easy handle:%p) has completed (code:%d) and has been marked as such", CurlRequest, CompletedHandle, (int32)Message->data.result);

						HandlesToRequests.Remove(CompletedHandle);
					}
					else
					{
						UE_LOGF(LogHttp, Warning, "Could not find mapping for completed request (easy handle: %p)", CompletedHandle);
					}
				}
			}
		}
	}

	FLegacyHttpThread::HttpThreadTick(DeltaSeconds);
}

bool FCurlHttpThread::StartThreadedRequest(FHttpRequestCommon* Request)
{
	FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(Request);
	CURL* EasyHandle = CurlRequest->GetEasyHandle();
	ensure(!HandlesToRequests.Contains(EasyHandle));

	if (!FLegacyHttpThread::StartThreadedRequest(Request))
	{
		UE_LOGF(LogHttp, Warning, "Could not set libcurl options for easy handle, processing HTTP request failed. Increase verbosity for additional information.");
		return false;
	}

	CURLMcode AddResult = curl_multi_add_handle(FCurlHttpManager::GMultiHandle, EasyHandle);
	CurlRequest->SetAddToCurlMultiResult(AddResult);

	if (AddResult != CURLM_OK)
	{
		UE_LOGF(LogHttp, Warning, "Failed to add easy handle %p to multi handle with code %d", EasyHandle, (int)AddResult);
		return false;
	}

	HandlesToRequests.Add(EasyHandle, Request);

	return true;
}

void FCurlHttpThread::CompleteThreadedRequest(FHttpRequestCommon* Request)
{
	FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(Request);
	CURL* EasyHandle = CurlRequest->GetEasyHandle();

	if (HandlesToRequests.Find(EasyHandle))
	{
		curl_multi_remove_handle(FCurlHttpManager::GMultiHandle, EasyHandle);
		HandlesToRequests.Remove(EasyHandle);
	}

	CurlRequest->CleanupRequestHttpThread();
}

#endif
