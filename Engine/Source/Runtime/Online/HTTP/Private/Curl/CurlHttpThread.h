// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_CURL

#include "HttpThread.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif
#ifdef PLATFORM_CURL_INCLUDE
#include PLATFORM_CURL_INCLUDE
#else
#include "curl/curl.h"
#endif
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#endif //WITH_CURL

class FHttpRequestCommon;

#if WITH_CURL

class FCurlHttpThread
	: public FLegacyHttpThread
{
public:
	
	FCurlHttpThread();

protected:
	//~ Begin FHttpThread Interface
	virtual void HttpThreadTick(float DeltaSeconds) override;
	virtual bool StartThreadedRequest(FHttpRequestCommon* Request) override;
	virtual void CompleteThreadedRequest(FHttpRequestCommon* Request) override;
	//~ End FHttpThread Interface
protected:

	/** Mapping of libcurl easy handles to HTTP requests */
	TMap<CURL*, FHttpRequestCommon*> HandlesToRequests;
};


#endif //WITH_CURL
