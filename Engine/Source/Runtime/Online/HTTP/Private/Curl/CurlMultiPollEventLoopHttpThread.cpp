// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curl/CurlMultiPollEventLoopHttpThread.h"
#include "Stats/Stats.h"
#include "Http.h"
#include "Curl/CurlHttp.h"
#include "Curl/CurlHttpManager.h"

#include "EventLoop/EventLoop.h"

#if WITH_CURL
#if WITH_CURL_MULTIPOLL

FCurlMultiPollIOManagerIOAccess::FCurlMultiPollIOManagerIOAccess(FCurlMultiPollIOManager& InIOManager)
	: IOManager(InIOManager)
{
}

FCurlMultiPollIOManager::FCurlMultiPollIOManager(UE::EventLoop::IEventLoop& InEventLoop, FParams&& InParams)
	: IOAccess(*this)
	, EventLoop(InEventLoop)
	, Params(MoveTemp(InParams))
{
}

bool FCurlMultiPollIOManager::Init()
{
	check(Params.ProcessCurlRequests);
	return true;
}

void FCurlMultiPollIOManager::Shutdown()
{
}

void FCurlMultiPollIOManager::Notify()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiPollIOManager_Notify);
	CheckMultiCodeOk(curl_multi_wakeup(Params.MultiHandle));
}

void FCurlMultiPollIOManager::Poll(FTimespan WaitTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiPollIOManager_Poll);

	int RunningRequests = -1;
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiPollIOManager_Poll_MultiPerform);
		CheckMultiCodeOk(curl_multi_perform(Params.MultiHandle, &RunningRequests));
	}

	Params.ProcessCurlRequests();

	// Poll for IO.
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiPollIOManager_Poll_MultiPoll);
		const int64 TimeoutMs = FMath::RoundToPositiveInfinity(WaitTime.GetTotalMilliseconds());
		CheckMultiCodeOk(curl_multi_poll(Params.MultiHandle, nullptr, 0, TimeoutMs, nullptr));
	}
}

inline void FCurlMultiPollIOManager::CheckMultiCodeOk(const CURLMcode Code)
{
	// GMultiHandle may be invalid when the process is forked and curl_multi_init is called again.
	checkf(Code == CURLM_OK, TEXT("Error in curl_multi operation: %hs"), curl_multi_strerror(Code));
}

FCurlMultiPollEventLoopHttpThread::FCurlMultiPollEventLoopHttpThread()
{
}

bool FCurlMultiPollEventLoopHttpThread::StartThreadedRequest(FHttpRequestCommon* Request)
{
	FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(Request);
	CURL* EasyHandle = CurlRequest->GetEasyHandle();
	ensure(!HandlesToRequests.Contains(EasyHandle));

	if (!FEventLoopHttpThread::StartThreadedRequest(Request))
	{
		UE_LOGF(LogHttp, Warning, "Could not set libcurl options for easy handle, processing HTTP request failed. Increase verbosity for additional information.");
		return false;
	}

	CURLMcode AddResult = curl_multi_add_handle(FCurlHttpManager::GMultiHandle, EasyHandle);
	CurlRequest->SetAddToCurlMultiResult(AddResult);

	if (AddResult != CURLM_OK)
	{
		UE_LOGF(LogHttp, Warning, "Failed to add easy handle %p to multi handle with code %d", EasyHandle, (int)AddResult);
		return false;
	}

	HandlesToRequests.Add(EasyHandle, Request);

	return true;
}

void FCurlMultiPollEventLoopHttpThread::CompleteThreadedRequest(FHttpRequestCommon* Request)
{
	FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(Request);
	CURL* EasyHandle = CurlRequest->GetEasyHandle();

	if (HandlesToRequests.Find(EasyHandle))
	{
		curl_multi_remove_handle(FCurlHttpManager::GMultiHandle, EasyHandle);
		HandlesToRequests.Remove(EasyHandle);
	}

	CurlRequest->CleanupRequestHttpThread();
}

void FCurlMultiPollEventLoopHttpThread::CreateEventLoop()
{
	UE::EventLoop::TEventLoop<FCurlMultiPollIOManager>::FParams EventLoopParams;
	EventLoopParams.IOManagerParams.MultiHandle = FCurlHttpManager::GMultiHandle;
	EventLoopParams.IOManagerParams.ProcessCurlRequests = [this](){ ProcessCurlRequests(); };
	EventLoop.Emplace(MoveTemp(EventLoopParams));
}

void FCurlMultiPollEventLoopHttpThread::DestroyEventLoop()
{
	EventLoop.Reset();
}

void FCurlMultiPollEventLoopHttpThread::UpdateEventLoopConfigs()
{
}

UE::EventLoop::IEventLoop* FCurlMultiPollEventLoopHttpThread::GetEventLoop()
{
	return EventLoop.IsSet() ? &*EventLoop : nullptr;
}

UE::EventLoop::IEventLoop& FCurlMultiPollEventLoopHttpThread::GetEventLoopChecked()
{
	return *EventLoop;
}

void FCurlMultiPollEventLoopHttpThread::ProcessCurlRequests()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiPollEventLoopHttpThread_ProcessCurlRequests);

	bool CompletedRequest = false;

	for (;;)
	{
		int MsgsStillInQueue = 0;	// may use that to impose some upper limit we may spend in that loop
		CURLMsg* Message = curl_multi_info_read(FCurlHttpManager::GMultiHandle, &MsgsStillInQueue);

		if (Message == NULL)
		{
			break;
		}

		if (Message->msg == CURLMSG_DONE)
		{
			CURL* CompletedHandle = Message->easy_handle;
			curl_multi_remove_handle(FCurlHttpManager::GMultiHandle, CompletedHandle);

			FHttpRequestCommon** Request = HandlesToRequests.Find(CompletedHandle);
			if (Request)
			{
				FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(*Request);
				CurlRequest->MarkAsCompleted(Message->data.result);

				UE_LOGF(LogHttp, Verbose, "Request %p (easy handle:%p) has completed (code:%d) and has been marked as such", CurlRequest, CompletedHandle, (int32)Message->data.result);

				HandlesToRequests.Remove(CompletedHandle);
				CompletedRequest = true;
			}
			else
			{
				UE_LOGF(LogHttp, Warning, "Could not find mapping for completed request (easy handle: %p)", CompletedHandle);
			}
		}
	}

	// If any requests completed, immediately process requests to handle completion event.
	if (CompletedRequest)
	{
		TArray<FHttpRequestCommon*> RequestsToCancel;
		TArray<FHttpRequestCommon*> RequestsToComplete;
		Process(RequestsToCancel, RequestsToComplete);
	}
}

#endif // WITH_CURL_MULTIPOLL
#endif // WITH_CURL
