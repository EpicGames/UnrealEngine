// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if WITH_CURL
#if WITH_CURL_MULTIPOLL

#include "EventLoopHttpThread.h"
#include "EventLoop/EventLoop.h"
#include "Templates/Function.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif
#ifdef PLATFORM_CURL_INCLUDE
#include PLATFORM_CURL_INCLUDE
#else
#include "curl/curl.h"
#endif
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

class FHttpRequestCommon;
class FCurlMultiPollIOManager;

class FCurlMultiPollIOManagerIOAccess final : public FNoncopyable
{
public:
	FCurlMultiPollIOManagerIOAccess(FCurlMultiPollIOManager& InIOManager);

	// Nothing needed for now.

private:
	FCurlMultiPollIOManager& IOManager;
};

class FCurlMultiPollIOManager final : public UE::EventLoop::IIOManager
{
public:
	using FIOAccess = FCurlMultiPollIOManagerIOAccess;

	struct FParams
	{
		CURLM* MultiHandle = nullptr;
		TUniqueFunction<void()> ProcessCurlRequests;
	};

	FCurlMultiPollIOManager(UE::EventLoop::IEventLoop& EventLoop, FParams&& Params);
	virtual ~FCurlMultiPollIOManager() = default;
	virtual bool Init() override;
	virtual void Shutdown() override;
	virtual void Notify() override;
	virtual void Poll(FTimespan WaitTime) override;

	FIOAccess& GetIOAccess()
	{
		return IOAccess;
	}

private:
	void CheckMultiCodeOk(const CURLMcode Code);

	FIOAccess IOAccess;
	UE::EventLoop::IEventLoop& EventLoop;
	FParams Params;
};

class FCurlMultiPollEventLoopHttpThread
	: public FEventLoopHttpThread
{
public:

	FCurlMultiPollEventLoopHttpThread();

protected:
	//~ Begin FHttpThread Interface
	virtual bool StartThreadedRequest(FHttpRequestCommon* Request) override;
	virtual void CompleteThreadedRequest(FHttpRequestCommon* Request) override;
	//~ End FHttpThread Interface
protected:
	virtual void CreateEventLoop() override;
	virtual void DestroyEventLoop() override;
	virtual void UpdateEventLoopConfigs() override;
	virtual UE::EventLoop::IEventLoop* GetEventLoop() override;
	virtual UE::EventLoop::IEventLoop& GetEventLoopChecked() override;

	void ProcessCurlRequests();

	TOptional<UE::EventLoop::TEventLoop<FCurlMultiPollIOManager>> EventLoop;

	/** Mapping of libcurl easy handles to HTTP requests */
	TMap<CURL*, FHttpRequestCommon*> HandlesToRequests;
};

#endif // WITH_CURL_MULTIPOLL
#endif // WITH_CURL
