// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curl/CurlMultiWaitEventLoopHttpThread.h"
#include "Stats/Stats.h"
#include "Http.h"
#include "Curl/CurlHttp.h"
#include "Curl/CurlHttpManager.h"

#include "EventLoop/EventLoop.h"

#if WITH_CURL
#if WITH_CURL_MULTIWAIT

FCurlMultiWaitIOManagerIOAccess::FCurlMultiWaitIOManagerIOAccess(FCurlMultiWaitIOManager& InIOManager)
	: IOManager(InIOManager)
{
}

FCurlMultiWaitIOManager::FCurlMultiWaitIOManager(UE::EventLoop::IEventLoop& InEventLoop, FParams&& InParams)
	: IOAccess(*this)
	, EventLoop(InEventLoop)
	, Params(MoveTemp(InParams))
	, EmptySequentialWaitCount(0)
{
}

bool FCurlMultiWaitIOManager::Init()
{
	if (!ensure(EventLoopNotifier.Init()))
	{
		UE_LOGF(LogHttp, Warning, "Socket notifier init failed, multi-wait eventloop performance may degrade!");
	}

	check(Params.ProcessCurlRequests);
	return true;
}

void FCurlMultiWaitIOManager::Shutdown()
{
	EventLoopNotifier.Shutdown();
}

void FCurlMultiWaitIOManager::Notify()
{
	EventLoopNotifier.Notify();
}

void FCurlMultiWaitIOManager::Poll(FTimespan WaitTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiWaitIOManager_Poll);

	int RunningRequests = -1;
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiWaitIOManager_Poll_MultiPerform);
		CheckMultiCodeOk(curl_multi_perform(Params.MultiHandle, &RunningRequests));
	}

	Params.ProcessCurlRequests();

	int32 FileDescriptorRead = EventLoopNotifier.GetFileDescriptorRead();
	if (FileDescriptorRead != INVALID_SOCKET)
	{
		struct curl_waitfd EarlyWakeUpWait;
		EarlyWakeUpWait.fd = FileDescriptorRead;
		EarlyWakeUpWait.events = CURL_WAIT_POLLIN; /* wait for input */
		EarlyWakeUpWait.revents = 0; /* clear it */

		const int64 TimeoutMs = FMath::RoundToPositiveInfinity(WaitTime.GetTotalMilliseconds());
		int NumFDs = 0;

		CheckMultiCodeOk(curl_multi_wait(Params.MultiHandle, &EarlyWakeUpWait, 1, TimeoutMs, &NumFDs));
		if (EarlyWakeUpWait.revents > 0)
		{
			EventLoopNotifier.Clear();
		}
	}
	else
	{
		// The maximum wait time affects how quickly new requests are started / canceled.
		const FTimespan MaxWaitTime = FTimespan::FromMilliseconds(10);
		FTimespan CurrentWaitTime = FMath::Min(WaitTime, MaxWaitTime);

		// A call to curl_multi_wait will return immediately with no FDs set when a timer is fired or
		// when there are no sockets being handled. When it returns early the first time assume that a
		// timer has fired. On the second iteration assume that no FDs are set and instead sleep for
		// the duration.
		if (EmptySequentialWaitCount > 1)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiWaitIOManager_Poll_Sleep);
			FPlatformProcess::SleepNoStats(CurrentWaitTime.GetTotalSeconds());
			EmptySequentialWaitCount = 0;
		}
		else
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiWaitIOManager_Poll_MultiWait);
			const int64 TimeoutMs = FMath::RoundToPositiveInfinity(WaitTime.GetTotalMilliseconds());
			int NumFDs = 0;
			CheckMultiCodeOk(curl_multi_wait(Params.MultiHandle, nullptr, 0, TimeoutMs, &NumFDs));

			// Track how many sequential calls had no FDs set.
			EmptySequentialWaitCount = (NumFDs != 0) ? 0 : ++EmptySequentialWaitCount;
		}
	}
}

inline void FCurlMultiWaitIOManager::CheckMultiCodeOk(const CURLMcode Code)
{
	// GMultiHandle may be invalid when the process is forked and curl_multi_init is called again.
	checkf(Code == CURLM_OK, TEXT("Error in curl_multi operation: %hs"), curl_multi_strerror(Code));
}

FCurlMultiWaitEventLoopHttpThread::FCurlMultiWaitEventLoopHttpThread()
{
}

bool FCurlMultiWaitEventLoopHttpThread::StartThreadedRequest(FHttpRequestCommon* Request)
{
	FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(Request);
	CURL* EasyHandle = CurlRequest->GetEasyHandle();
	ensure(!HandlesToRequests.Contains(EasyHandle));

	if (!FEventLoopHttpThread::StartThreadedRequest(Request))
	{
		UE_LOGF(LogHttp, Warning, "Could not set libcurl options for easy handle, processing HTTP request failed. Increase verbosity for additional information.");
		return false;
	}

	CURLMcode AddResult = curl_multi_add_handle(FCurlHttpManager::GMultiHandle, EasyHandle);
	CurlRequest->SetAddToCurlMultiResult(AddResult);

	if (AddResult != CURLM_OK)
	{
		UE_LOGF(LogHttp, Warning, "Failed to add easy handle %p to multi handle with code %d", EasyHandle, (int)AddResult);
		return false;
	}

	HandlesToRequests.Add(EasyHandle, Request);

	return true;
}

void FCurlMultiWaitEventLoopHttpThread::CompleteThreadedRequest(FHttpRequestCommon* Request)
{
	FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(Request);
	CURL* EasyHandle = CurlRequest->GetEasyHandle();

	if (HandlesToRequests.Remove(EasyHandle) > 0)
	{
		curl_multi_remove_handle(FCurlHttpManager::GMultiHandle, EasyHandle);
	}

	CurlRequest->CleanupRequestHttpThread();
}

void FCurlMultiWaitEventLoopHttpThread::CreateEventLoop()
{
	UE::EventLoop::TEventLoop<FCurlMultiWaitIOManager>::FParams EventLoopParams;
	EventLoopParams.IOManagerParams.MultiHandle = FCurlHttpManager::GMultiHandle;
	EventLoopParams.IOManagerParams.ProcessCurlRequests = [this](){ ProcessCurlRequests(); };
	EventLoop.Emplace(MoveTemp(EventLoopParams));
}

void FCurlMultiWaitEventLoopHttpThread::DestroyEventLoop()
{
	EventLoop.Reset();
}

void FCurlMultiWaitEventLoopHttpThread::UpdateEventLoopConfigs()
{
}

UE::EventLoop::IEventLoop* FCurlMultiWaitEventLoopHttpThread::GetEventLoop()
{
	return EventLoop.IsSet() ? &*EventLoop : nullptr;
}

UE::EventLoop::IEventLoop& FCurlMultiWaitEventLoopHttpThread::GetEventLoopChecked()
{
	return *EventLoop;
}

void FCurlMultiWaitEventLoopHttpThread::ProcessCurlRequests()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlMultiWaitEventLoopHttpThread_ProcessCurlRequests);

	bool CompletedRequest = false;

	for (;;)
	{
		int MsgsStillInQueue = 0;	// may use that to impose some upper limit we may spend in that loop
		CURLMsg* Message = curl_multi_info_read(FCurlHttpManager::GMultiHandle, &MsgsStillInQueue);

		if (Message == NULL)
		{
			break;
		}

		if (Message->msg == CURLMSG_DONE)
		{
			CURL* CompletedHandle = Message->easy_handle;
			curl_multi_remove_handle(FCurlHttpManager::GMultiHandle, CompletedHandle);

			FHttpRequestCommon** Request = HandlesToRequests.Find(CompletedHandle);
			if (Request)
			{
				FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(*Request);
				CurlRequest->MarkAsCompleted(Message->data.result);

				UE_LOGF(LogHttp, Verbose, "Request %p (easy handle:%p) has completed (code:%d) and has been marked as such", CurlRequest, CompletedHandle, (int32)Message->data.result);

				HandlesToRequests.Remove(CompletedHandle);
				CompletedRequest = true;
			}
			else
			{
				UE_LOGF(LogHttp, Warning, "Could not find mapping for completed request (easy handle: %p)", CompletedHandle);
			}
		}
	}

	// If any requests completed, immediately process requests to handle completion event.
	if (CompletedRequest)
	{
		TArray<FHttpRequestCommon*> RequestsToCancel;
		TArray<FHttpRequestCommon*> RequestsToComplete;
		Process(RequestsToCancel, RequestsToComplete);
	}
}

#endif // WITH_CURL_MULTIWAIT
#endif // WITH_CURL
