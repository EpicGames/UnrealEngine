// Copyright Epic Games, Inc. All Rights Reserved.

#include "Curl/CurlSocketEventLoopHttpThread.h"
#include "Stats/Stats.h"
#include "Http.h"
#include "Curl/CurlHttp.h"
#include "Curl/CurlHttpManager.h"

#include "EventLoop/EventLoop.h"

#if WITH_CURL
#if WITH_CURL_MULTISOCKET

namespace
{

UE::EventLoop::EIOFlags TranslateCurlPollEventFlags(int EventFlags)
{
	UE::EventLoop::EIOFlags RequestFlags = UE::EventLoop::EIOFlags::None;

	if ((EventFlags & CURL_POLL_IN) > 0)
	{
		RequestFlags |= UE::EventLoop::EIOFlags::Read;
	}

	if ((EventFlags & CURL_POLL_OUT) > 0)
	{
		RequestFlags |= UE::EventLoop::EIOFlags::Write;
	}

	return RequestFlags;
}

int TranslateCurlSocketActionFlags(UE::EventLoop::EIOFlags EventFlags)
{
	int OutFlags = 0;

	if (EnumHasAnyFlags(EventFlags, UE::EventLoop::EIOFlags::Read))
	{
		OutFlags |= CURL_CSELECT_IN;
	}

	if (EnumHasAnyFlags(EventFlags, UE::EventLoop::EIOFlags::Write))
	{
		OutFlags |= CURL_CSELECT_OUT;
	}

	return OutFlags;
}

} // anonymous

FCurlSocketEventLoopHttpThread::FCurlSocketEventLoopHttpThread()
{
}

void FCurlSocketEventLoopHttpThread::HttpThreadTick(float DeltaSeconds)
{
	FEventLoopHttpThread::HttpThreadTick(DeltaSeconds);
}

bool FCurlSocketEventLoopHttpThread::StartThreadedRequest(FHttpRequestCommon* Request)
{
	FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(Request);
	CURL* EasyHandle = CurlRequest->GetEasyHandle();
	ensure(!HandlesToRequests.Contains(EasyHandle));

	if (!FEventLoopHttpThread::StartThreadedRequest(Request))
	{
		UE_LOGF(LogHttp, Warning, "Could not set libcurl options for easy handle, processing HTTP request failed. Increase verbosity for additional information.");
		return false;
	}

	CURLMcode AddResult = curl_multi_add_handle(FCurlHttpManager::GMultiHandle, EasyHandle);
	CurlRequest->SetAddToCurlMultiResult(AddResult);

	if (AddResult != CURLM_OK)
	{
		UE_LOGF(LogHttp, Warning, "Failed to add easy handle %p to multi handle with code %d", EasyHandle, (int)AddResult);
		return false;
	}

	HandlesToRequests.Add(EasyHandle, Request);

	return true;
}

void FCurlSocketEventLoopHttpThread::CompleteThreadedRequest(FHttpRequestCommon* Request)
{
	FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(Request);
	CURL* EasyHandle = CurlRequest->GetEasyHandle();

	if (HandlesToRequests.Find(EasyHandle))
	{
		curl_multi_remove_handle(FCurlHttpManager::GMultiHandle, EasyHandle);
		HandlesToRequests.Remove(EasyHandle);
	}

	CurlRequest->CleanupRequestHttpThread();
}

void FCurlSocketEventLoopHttpThread::CreateEventLoop()
{
	UE::EventLoop::TEventLoop<UE::EventLoop::FIOManagerBSDSocket>::FParams EventLoopParams;
	EventLoop.Emplace(MoveTemp(EventLoopParams));

	check(FCurlHttpManager::GMultiHandle);
	curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_SOCKETFUNCTION, &FCurlSocketEventLoopHttpThread::CurlSocketCallback);
	curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_SOCKETDATA, this);
	curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_TIMERFUNCTION, &FCurlSocketEventLoopHttpThread::CurlTimerCallback);
	curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_TIMERDATA, this);
}

void FCurlSocketEventLoopHttpThread::DestroyEventLoop()
{
	check(FCurlHttpManager::GMultiHandle);
	curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_SOCKETFUNCTION, nullptr);
	curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_SOCKETDATA, nullptr);
	curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_TIMERFUNCTION, nullptr);
	curl_multi_setopt(FCurlHttpManager::GMultiHandle, CURLMOPT_TIMERDATA, nullptr);

	EventLoop.Reset();
}

void FCurlSocketEventLoopHttpThread::UpdateEventLoopConfigs()
{
}

UE::EventLoop::IEventLoop* FCurlSocketEventLoopHttpThread::GetEventLoop()
{
	return EventLoop.IsSet() ? &*EventLoop : nullptr;
}

UE::EventLoop::IEventLoop& FCurlSocketEventLoopHttpThread::GetEventLoopChecked()
{
	return *EventLoop;
}

int FCurlSocketEventLoopHttpThread::CurlSocketCallback(CURL* CurlE, curl_socket_t Socket, int EventFlags, void* UserData, void* SocketData)
{
	return reinterpret_cast<FCurlSocketEventLoopHttpThread*>(UserData)->HandleCurlSocketCallback(CurlE, Socket, EventFlags, SocketData);
}

int FCurlSocketEventLoopHttpThread::HandleCurlSocketCallback(CURL* CurlE, curl_socket_t Socket, int EventFlags, void* SocketData)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlSocketEventLoopHttpThread_HandleCurlSocketCallback);

	FCurlSocketData* CurlSocketData = reinterpret_cast<FCurlSocketData*>(SocketData);

	UE_LOGF(LogHttp, VeryVerbose, "[FCurlSocketEventLoopHttpThread::HandleCurlSocketCallback] Socket: %p, EventFlags: 0x%08X, SocketData: %p", reinterpret_cast<void*>(Socket), EventFlags, SocketData);

	switch (EventFlags)
	{
	case CURL_POLL_IN:
	case CURL_POLL_OUT:
	case CURL_POLL_INOUT:
	{
		if (!CurlSocketData)
		{
			CurlSocketData = new FCurlSocketData;
			CurlSocketData->Socket = Socket;
			curl_multi_assign(FCurlHttpManager::GMultiHandle, Socket, (void*)CurlSocketData);
		}

		if (CurlSocketData->IORequestHandle.IsValid())
		{
			EventLoop->GetIOAccess().DestroyIORequest(CurlSocketData->IORequestHandle);
		}

		UE::EventLoop::FIORequestBSDSocket Request;
		Request.Socket = Socket;
		Request.Flags = TranslateCurlPollEventFlags(EventFlags);
		Request.Callback = [this](SOCKET Socket, UE::EventLoop::ESocketIoRequestStatus Status, UE::EventLoop::EIOFlags SignaledFlags)
		{
			ProcessCurlSocketEvent(Socket, Status, SignaledFlags);
		};

		CurlSocketData->IORequestHandle = EventLoop->GetIOAccess().CreateSocketIORequest(MoveTemp(Request));
		break;
	}

	case CURL_POLL_REMOVE:
		if (CurlSocketData)
		{
			EventLoop->GetIOAccess().DestroyIORequest(CurlSocketData->IORequestHandle);
			delete CurlSocketData;
			curl_multi_assign(FCurlHttpManager::GMultiHandle, Socket, NULL);
		}
		break;

	default:
		checkNoEntry();
	}

	return 0;
}

int FCurlSocketEventLoopHttpThread::CurlTimerCallback(CURLM* CurlM, long TimeoutMS, void* UserData)
{
	return reinterpret_cast<FCurlSocketEventLoopHttpThread*>(UserData)->HandleCurlTimerCallback(CurlM, TimeoutMS);
}

int FCurlSocketEventLoopHttpThread::HandleCurlTimerCallback(CURLM* CurlM, long TimeoutMS)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlSocketEventLoopHttpThread_HandleCurlTimerCallback);
	UE_LOGF(LogHttp, VeryVerbose, "[FCurlSocketEventLoopHttpThread::HandleCurlTimerCallback] Timeout: %ld", TimeoutMS);

	// Start by canceling any previous timer.
	EventLoop->ClearTimer(RequestTimeoutTimer);

	if (TimeoutMS >= 0)
	{
		UE_LOGF(LogHttp, VeryVerbose, "[FCurlSocketEventLoopHttpThread::HandleCurlTimerCallback] Set timer for: %ld ms", TimeoutMS);

		RequestTimeoutTimer = EventLoop->SetTimer([this]()
		{
			ProcessCurlSocketActions(CURL_SOCKET_TIMEOUT, 0);
		},
		FTimespan::FromMilliseconds(TimeoutMS));
	}

	return 0;
}

void FCurlSocketEventLoopHttpThread::ProcessCurlSocketActions(curl_socket_t Socket, int EventFlags)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlSocketEventLoopHttpThread_ProcessCurlSocketActions);

	UE_LOGF(LogHttp, VeryVerbose, "[FCurlSocketEventLoopHttpThread::ProcessCurlSocketActions] Socket: %p CurlEvents: 0x%08X", reinterpret_cast<void*>(Socket), EventFlags);

	int running_handles;
	curl_multi_socket_action(FCurlHttpManager::GMultiHandle, Socket, EventFlags, &running_handles);
	ProcessCurlRequests();
}

void FCurlSocketEventLoopHttpThread::ProcessCurlSocketEvent(curl_socket_t Socket, UE::EventLoop::ESocketIoRequestStatus Status, UE::EventLoop::EIOFlags Flags)
{
	if (Status != UE::EventLoop::ESocketIoRequestStatus::Ok)
	{
		UE_LOGF(LogHttp, Warning, "Socket event request failed. Socket %p, Status: %ls", reinterpret_cast<void*>(Socket), *LexToString(Status));
		ProcessCurlSocketActions(CURL_SOCKET_TIMEOUT, 0);
	}
	else
	{
		ProcessCurlSocketActions(Socket, TranslateCurlSocketActionFlags(Flags));
	}
}

void FCurlSocketEventLoopHttpThread::ProcessCurlRequests()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurlSocketEventLoopHttpThread_ProcessCurlRequests);

	bool CompletedRequest = false;

	for (;;)
	{
		int MsgsStillInQueue = 0;	// may use that to impose some upper limit we may spend in that loop
		CURLMsg* Message = curl_multi_info_read(FCurlHttpManager::GMultiHandle, &MsgsStillInQueue);

		if (Message == NULL)
		{
			break;
		}

		if (Message->msg == CURLMSG_DONE)
		{
			CURL* CompletedHandle = Message->easy_handle;
			curl_multi_remove_handle(FCurlHttpManager::GMultiHandle, CompletedHandle);

			FHttpRequestCommon** Request = HandlesToRequests.Find(CompletedHandle);
			if (Request)
			{
				FCurlHttpRequest* CurlRequest = static_cast<FCurlHttpRequest*>(*Request);
				CurlRequest->MarkAsCompleted(Message->data.result);

				UE_LOGF(LogHttp, Verbose, "Request %p (easy handle:%p) has completed (code:%d) and has been marked as such", CurlRequest, CompletedHandle, (int32)Message->data.result);

				HandlesToRequests.Remove(CompletedHandle);
				CompletedRequest = true;
			}
			else
			{
				UE_LOGF(LogHttp, Warning, "Could not find mapping for completed request (easy handle: %p)", CompletedHandle);
			}
		}
	}

	// If any requests completed, immediately process requests to handle completion event.
	if (CompletedRequest)
	{
		TArray<FHttpRequestCommon*> RequestsToCancel;
		TArray<FHttpRequestCommon*> RequestsToComplete;
		Process(RequestsToCancel, RequestsToComplete);
	}
}

#endif // WITH_CURL_MULTISOCKET
#endif // WITH_CURL
