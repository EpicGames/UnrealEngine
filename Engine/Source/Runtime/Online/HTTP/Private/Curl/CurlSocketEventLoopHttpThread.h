// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if WITH_CURL
#if WITH_CURL_MULTISOCKET

#include "EventLoopHttpThread.h"
#include "EventLoop/BSDSocket/EventLoopIOManagerBSDSocket.h"
#include "EventLoop/EventLoop.h"
#include "Templates/Function.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif
#ifdef PLATFORM_CURL_INCLUDE
#include PLATFORM_CURL_INCLUDE
#else
#include "curl/curl.h"
#endif
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

class FHttpRequestCommon;

class FCurlSocketEventLoopHttpThread
	: public FEventLoopHttpThread
{
public:

	FCurlSocketEventLoopHttpThread();

protected:
	//~ Begin FHttpThread Interface
	virtual void HttpThreadTick(float DeltaSeconds) override;
	virtual bool StartThreadedRequest(FHttpRequestCommon* Request) override;
	virtual void CompleteThreadedRequest(FHttpRequestCommon* Request) override;
	//~ End FHttpThread Interface
protected:
	virtual void CreateEventLoop() override;
	virtual void DestroyEventLoop() override;
	virtual void UpdateEventLoopConfigs() override;
	virtual UE::EventLoop::IEventLoop* GetEventLoop() override;
	virtual UE::EventLoop::IEventLoop& GetEventLoopChecked() override;

	static int CurlSocketCallback(CURL* CurlE, curl_socket_t Socket, int EventFlags, void* UserData, void* SocketData);
	int HandleCurlSocketCallback(CURL* CurlE, curl_socket_t Socket, int EventFlags, void* SocketData);

	static int CurlTimerCallback(CURLM* CurlM, long TimeoutMS, void* UserData);
	int HandleCurlTimerCallback(CURLM* CurlM, long TimeoutMS);

	void ProcessCurlSocketActions(curl_socket_t Socket, int EventFlags);
	void ProcessCurlSocketEvent(curl_socket_t Socket, UE::EventLoop::ESocketIoRequestStatus Status, UE::EventLoop::EIOFlags Flags);
	void ProcessCurlRequests();

	struct FCurlSocketData
	{
		curl_socket_t Socket;
		UE::EventLoop::FIORequestHandle IORequestHandle;
	};

	/** Mapping of libcurl easy handles to HTTP requests */
	TMap<CURL*, FHttpRequestCommon*> HandlesToRequests;

	UE::EventLoop::FTimerHandle RequestTimeoutTimer;

	TOptional<UE::EventLoop::TEventLoop<UE::EventLoop::FIOManagerBSDSocket>> EventLoop;
};

#endif // WITH_CURL_MULTISOCKET
#endif // WITH_CURL
