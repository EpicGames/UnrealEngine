// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventLoopHttpThread.h"
#include "EventLoop/EventLoop.h"
#include "EventLoop/EventLoopIOManagerNull.h"
#include "GenericPlatform/HttpRequestCommon.h"
#include "Http.h"
#include "PlatformHttp.h"
#include "Stats/Stats.h"

class FHttpTaskTimerHandleEventLoop : public IHttpTaskTimerHandle
{
public:
	FHttpTaskTimerHandleEventLoop(UE::EventLoop::FTimerHandle InHandle)
		: Handle(InHandle)
	{
	}

	virtual void RemoveTaskFrom(FHttpThreadBase* HttpThreadBase) 
	{ 
		HttpThreadBase->RemoveTimerHandle(Handle);
	}

private:
	UE::EventLoop::FTimerHandle Handle;
};


FEventLoopHttpThread::FEventLoopHttpThread()
{
	FPlatformHttp::AddDefaultUserAgentProjectComment(TEXT("http-eventloop"));
}

FEventLoopHttpThread::~FEventLoopHttpThread()
{
}

void FEventLoopHttpThread::StartThread()
{
	CreateEventLoop();
	ResetTickTimer();
	FHttpThreadBase::StartThread();
}

void FEventLoopHttpThread::StopThread()
{
	UE::EventLoop::IEventLoop* EventLoop = GetEventLoop();
	if (EventLoop)
	{
		UE::EventLoop::TEventLoop<UE::EventLoop::FIOManagerNull> ShutdownEventLoop;
		verify(ShutdownEventLoop.Init());

		// Request shutdown for the http event loop, which will trigger shutdown of the
		// ShutdownEventLoop on completion.
		EventLoop->RequestShutdown([&ShutdownEventLoop]() {
			ShutdownEventLoop.RequestShutdown();
		});

		// Set timer for shutdown duration warning.
		const FTimespan ShutdownWarnTime = FTimespan::FromSeconds(10);
		const double ShutdownStartTime = FPlatformTime::Seconds();
		ShutdownEventLoop.SetTimer([ShutdownStartTime]()
		{
			UE_LOGF(LogHttp, Warning, "Still waiting for event loop shutdown. Elapsed time: %f seconds", FPlatformTime::Seconds() - ShutdownStartTime);
		},
		ShutdownWarnTime,
		true /* repeat */);

		if (NeedsSingleThreadTick())
		{
			// Set timer to poll http event loop.
			const FTimespan HttpEventLoopTickFrequency = FTimespan::FromMilliseconds(1);
			ShutdownEventLoop.SetTimer([EventLoop]()
			{
				// Run the http event loop waiting 0ms for IO.
				EventLoop->RunOnce(FTimespan::Zero());
			},
			HttpEventLoopTickFrequency,
			true /* repeat */);
		}

		// Wait for loop to shutdown.
		ShutdownEventLoop.Run();
	}

	// Stop the thread before destroying the event loop to give the loop a chance to return from 'Run'.
	FHttpThreadBase::StopThread();

	if (EventLoop)
	{
		DestroyEventLoop();
	}
}

void FEventLoopHttpThread::UpdateConfigs()
{
	ResetTickTimer();
	UpdateEventLoopConfigs();
	FHttpThreadBase::UpdateConfigs();
}

void FEventLoopHttpThread::AddRequest(FHttpRequestCommon* Request)
{
	FHttpThreadBase::AddRequest(Request);

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FEventLoopHttpThread_AddRequest);
	if (UE::EventLoop::IEventLoop* EventLoop = GetEventLoop())
	{
		// Force a wakeup to process new tasks.
		EventLoop->PostAsyncTask([this]()
		{
			TArray<FHttpRequestCommon*> RequestsToCancel;
			TArray<FHttpRequestCommon*> RequestsToComplete;
			Process(RequestsToCancel, RequestsToComplete);
		});
	}
}

void FEventLoopHttpThread::CancelRequest(FHttpRequestCommon* Request)
{
	FHttpThreadBase::CancelRequest(Request);

	if (UE::EventLoop::IEventLoop* EventLoop = GetEventLoop())
	{
		// Force a wakeup to process new tasks.
		EventLoop->PostAsyncTask([this]()
		{
			TArray<FHttpRequestCommon*> RequestsToCancel;
			TArray<FHttpRequestCommon*> RequestsToComplete;
			Process(RequestsToCancel, RequestsToComplete);
		});
	}
}

void FEventLoopHttpThread::GetCompletedRequests(TArray<FHttpRequestCommon*>& OutCompletedRequests)
{
	FHttpThreadBase::GetCompletedRequests(OutCompletedRequests);
}

void FEventLoopHttpThread::Tick()
{
	FHttpThreadBase::Tick();

	if (ensure(NeedsSingleThreadTick()))
	{
		// Run the http event loop waiting 0ms for IO.
		GetEventLoopChecked().RunOnce(FTimespan::Zero());
	}
}

bool FEventLoopHttpThread::Init()
{
	if (!GetEventLoopChecked().Init())
	{
		UE_LOGF(LogHttp, Error, "Failed to initialize the event loop.");
		return false;
	}

	return FHttpThreadBase::Init();
}

uint32 FEventLoopHttpThread::Run()
{
	GetEventLoopChecked().Run();
	return 0;
}

void FEventLoopHttpThread::ResetTickTimer()
{
	UE::EventLoop::IEventLoop& EventLoop = GetEventLoopChecked();
	EventLoop.ClearTimer(RequestTickTimer);

	RequestTickTimer = EventLoop.SetTimer([this]()
	{
		TArray<FHttpRequestCommon*> RequestsToCancel;
		TArray<FHttpRequestCommon*> RequestsToComplete;
		Process(RequestsToCancel, RequestsToComplete);
	},
	FTimespan::FromSeconds(FHttpModule::Get().GetHttpEventLoopThreadTickIntervalInSeconds()),
	true /* repeat */);
}

TSharedPtr<IHttpTaskTimerHandle> FEventLoopHttpThread::AddHttpThreadTask(TFunction<void()>&& Task, float InDelay)
{
	UE::EventLoop::IEventLoop& EventLoop = GetEventLoopChecked();
	return MakeShared<FHttpTaskTimerHandleEventLoop>(EventLoop.SetTimer(Task, FTimespan::FromSeconds(InDelay)));
}

void FEventLoopHttpThread::RemoveTimerHandle(FTSTicker::FDelegateHandle DelegateHandle)
{
	checkNoEntry();
}

void FEventLoopHttpThread::RemoveTimerHandle(UE::EventLoop::FTimerHandle EventLoopTimerHandle)
{
	UE::EventLoop::IEventLoop& EventLoop = GetEventLoopChecked();
	EventLoop.ClearTimer(EventLoopTimerHandle);
}
