// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HttpThread.h"
#include "EventLoop/IEventLoop.h"

class FEventLoopHttpThread : public FHttpThreadBase
{
public:

	FEventLoopHttpThread();
	virtual ~FEventLoopHttpThread();

	virtual void StartThread() override final;
	virtual void StopThread() override final;
	virtual void UpdateConfigs() override final;
	virtual void AddRequest(FHttpRequestCommon* Request) override final;
	virtual void CancelRequest(FHttpRequestCommon* Request) override final;
	virtual void GetCompletedRequests(TArray<FHttpRequestCommon*>& OutCompletedRequests) override final;

	//~ Begin FSingleThreadRunnable Interface
	// Cannot be overridden to ensure identical behavior with the threaded tick
	virtual void Tick() override final;
	//~ End FSingleThreadRunnable Interface

protected:
	virtual void CreateEventLoop() = 0;
	virtual void DestroyEventLoop() = 0;
	virtual void UpdateEventLoopConfigs() = 0;
	virtual UE::EventLoop::IEventLoop* GetEventLoop() = 0;
	virtual UE::EventLoop::IEventLoop& GetEventLoopChecked() = 0;

	//~ Begin FRunnable Interface
	virtual bool Init() override;
	// Cannot be overridden to ensure identical behavior with the single threaded tick
	virtual uint32 Run() override final;
	//~ End FRunnable Interface

	void ResetTickTimer();

	virtual TSharedPtr<IHttpTaskTimerHandle> AddHttpThreadTask(TFunction<void()>&& Task, float InDelay) override;

	virtual void RemoveTimerHandle(FTSTicker::FDelegateHandle DelegateHandle) override;

	virtual void RemoveTimerHandle(UE::EventLoop::FTimerHandle EventLoopTimerHandle) override;

protected:
	UE::EventLoop::FTimerHandle RequestTickTimer;
};
