// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformHttp.h"

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "GenericPlatform/HttpRequestImpl.h"
#include "HAL/CriticalSection.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/EngineVersion.h"
#include "Misc/App.h"
#include "Misc/ScopeRWLock.h"
#include "String/BytesToHex.h"

TAutoConsoleVariable<bool> CVarDefaultUserAgentCommentsEnabled(
	TEXT("http.DefaultUserAgentCommentsEnabled"),
	true,
	TEXT("Whether comments are supported in the defualt user agent string"),
	ECVF_SaveForNextBoot
);

namespace
{
	/**
	 * Utility to split stringviews based on a substring.  Returns everything before SplitString as 'Left' and everything after as
	 * 'Right'. If SplitString is not found, OutLeft and OutRight are not modified and the function will return false.
	 *
	 * @param SourceString The string to serach for SplitString, that will be split into a left-half and a right-half
	 * @param SplitString The string to find inside of SouceString
	 * @param OutLeft If SplitString is found, this will point at the string of characters BEFORE SplitString in SourceString
	 * @param OutRight If SplitString is found, this will point at the string of characters AFTER SplitString in SourceString
	 */
	bool Split(const FStringView SourceString, const FStringView SplitString, FStringView& OutLeft, FStringView& OutRight)
	{
		if (SourceString.IsEmpty() || SplitString.IsEmpty() || SourceString.Len() < SplitString.Len())
		{
			return false;
		}
		
		for (int32 SourceIndex = 0; SourceIndex + SplitString.Len() <= SourceString.Len(); ++SourceIndex)
		{
			// Keep track if we matched or not
			bool bWasMatch = true;
			
			// Try to find matching substring
			for (int32 SplitIndex = 0; SplitIndex < SplitString.Len(); ++SplitIndex)
			{
				if (SourceString[SourceIndex + SplitIndex] != SplitString[SplitIndex])
				{
					bWasMatch = false;
					break;
				}
			}

			// If we didn't mismatch, we found our index to split on
			if (bWasMatch)
			{
				OutLeft = SourceString.Left(SourceIndex);
				OutRight = SourceString.RightChop(SourceIndex + SplitString.Len());
				return true;
			}
		}

		return false;
	}

	/**
	 * Utility to split stringviews based on a single character 'SplitChar'. Returns everything before SplitChar as 'Left' and everything after as
	 * 'Right'. If SplitChar is not found, OutLeft and OutRight are not modified and the function will return false.
	 *
	 * @param SourceString The string to serach for SplitChar, that will be split into a left-half and a right-half
	 * @param SplitChar The character to find inside of SouceString
	 * @param OutLeft If SplitChar is found, this will point at the string of characters BEFORE SplitChar in SourceString
	 * @param OutRight If SplitChar is found, this will point at the string of characters AFTER SplitChar in SourceString
	 */
	bool Split(const FStringView SourceString, const TCHAR SplitChar, FStringView& Left, FStringView& Right)
	{
		int32 Position = INDEX_NONE;
		if (SourceString.FindChar(SplitChar, Position) && Position != INDEX_NONE)
		{
			Left = SourceString.Left(Position);
			Right = SourceString.RightChop(Position + 1);
			return true;
		}

		return false;
	}
}

/**
 * A generic http request
 */
class FGenericPlatformHttpRequest : public FHttpRequestImpl
{
public:
	// IHttpBase
	virtual const FString& GetURL() const override { return URL; }
	virtual FString GetURLParameter(const FString& ParameterName) const override { return TEXT(""); }
	virtual FString GetHeader(const FString& HeaderName) const override { return TEXT(""); }
	virtual TArray<FString> GetAllHeaders() const override { return TArray<FString>(); }
	virtual FString GetContentType() const override { return TEXT(""); }
	virtual uint64 GetContentLength() const override { return 0; }
	virtual const TArray<uint8>& GetContent() const override { static TArray<uint8> Temp; return Temp; }

	// IHttpRequest
	virtual FString GetVerb() const override { return TEXT(""); }
	virtual void SetVerb(const FString& Verb) override {}
	virtual void SetURL(const FString& InURL) override { URL = InURL; }
	virtual FString GetOption(const FName Option) const override { return TEXT(""); }
	virtual void SetOption(const FName Option, const FString& OptionValue) override {}
	virtual void SetContent(const TArray<uint8>& ContentPayload) override {}
	virtual void SetContent(TArray<uint8>&& ContentPayload) override {}
	virtual void SetContentAsString(const FString& ContentString) override {}
	virtual bool SetContentAsStreamedFile(const FString& Filename) override { return false; }
	virtual bool SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) override { return false; }
	virtual bool SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) override { return false; }
	virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) override {}
	virtual void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) override {}
	virtual bool ProcessRequest() override { return false; }
	virtual void CancelRequest() override {}
	virtual EHttpRequestStatus::Type GetStatus() const override { return EHttpRequestStatus::NotStarted; }
	virtual EHttpFailureReason GetFailureReason() const override { return EHttpFailureReason::None; }
	virtual const FString& GetEffectiveURL() const override { return EffectiveURL; }
	virtual const FHttpResponsePtr GetResponse() const override { return nullptr; }
	virtual void Tick(float DeltaSeconds) override {}
	virtual float GetElapsedTime() const override { return 0.0f; }
	virtual void SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InDelegateThreadPolicy) override {}
	virtual EHttpRequestDelegateThreadPolicy GetDelegateThreadPolicy() const override { return EHttpRequestDelegateThreadPolicy::CompleteOnGameThread; }
	HTTP_API virtual void SetPriority(EHttpRequestPriority InPriority) override {}
	HTTP_API virtual EHttpRequestPriority GetPriority() const override { return EHttpRequestPriority::Normal; }
	virtual void SetTimeout(float InTimeoutSecs) override {}
	virtual void ClearTimeout() override {}
	virtual void ResetTimeoutStatus() override {}
	virtual TOptional<float> GetTimeout() const override { return TOptional<float>(); }
	virtual void SetActivityTimeout(float InTimeoutSecs) override {}
	virtual void ProcessRequestUntilComplete() override {}

private:
	FString URL;
	FString EffectiveURL;
};

FDefaultUserAgentBuilder::FDefaultUserAgentBuilder()
	: ProjectName(FApp::GetProjectName())
	, ProjectVersion(FApp::GetBuildVersion())
	, ProjectComments()
#ifdef OVERRIDE_HEADER_PLATFORM_NAME
#define _TEXTIFY(T) #T
	, PlatformName(TEXT(_TEXTIFY(OVERRIDE_HEADER_PLATFORM_NAME)))
	, PlatformVersion(TEXT(_TEXTIFY(OVERRIDE_HEADER_PLATFORM_VERSION)))
#undef _TEXTIFY
#else
	, PlatformName(FString(FPlatformProperties::IniPlatformName()))
	, PlatformVersion(FPlatformMisc::GetOSVersion())
#endif
	, PlatformComments()
	, AgentVersion(1)
{
}

FString FDefaultUserAgentBuilder::BuildUserAgentString(const TSet<FString>* AllowedProjectCommentsFilter, const TSet<FString>* AllowedPlatformCommentsFilter) const
{
	// strip/escape slashes and whitespace from components
	return FString::Printf(TEXT("%s/%s%s %s/%s%s"),
		*FGenericPlatformHttp::EscapeUserAgentString(ProjectName),
		*FGenericPlatformHttp::EscapeUserAgentString(ProjectVersion),
		*BuildCommentString(ProjectComments, AllowedProjectCommentsFilter),
		*FGenericPlatformHttp::EscapeUserAgentString(PlatformName),
		*FGenericPlatformHttp::EscapeUserAgentString(PlatformVersion),
		*BuildCommentString(PlatformComments, AllowedPlatformCommentsFilter));
}

void FDefaultUserAgentBuilder::SetProjectName(const FString& InProjectName)
{
	ProjectName = InProjectName;
	++AgentVersion;
}

void FDefaultUserAgentBuilder::SetProjectVersion(const FString& InProjectVersion)
{
	ProjectVersion = InProjectVersion;
	++AgentVersion;
}

void FDefaultUserAgentBuilder::AddProjectComment(const FString& InComment)
{
	ProjectComments.AddUnique(InComment);
	++AgentVersion;
}

void FDefaultUserAgentBuilder::SetPlatformName(const FString& InPlatformName)
{
	PlatformName = InPlatformName;
	++AgentVersion;
}

void FDefaultUserAgentBuilder::SetPlatformVersion(const FString& InPlatformVersion)
{
	PlatformVersion = InPlatformVersion;
	++AgentVersion;
}

void FDefaultUserAgentBuilder::AddPlatformComment(const FString& InComment)
{
	PlatformComments.AddUnique(InComment);
	++AgentVersion;
}

uint32 FDefaultUserAgentBuilder::GetAgentVersion() const
{
	return AgentVersion;
}

FString FDefaultUserAgentBuilder::BuildCommentString(const TArray<FString>& Comments, const TSet<FString>* AllowedCommentsFilter)
{
	TArray<FString> FilteredComments;
	Algo::TransformIf(Comments, FilteredComments,
		[AllowedCommentsFilter](const FString& Comment) -> bool
		{
			return !AllowedCommentsFilter || AllowedCommentsFilter->Contains(Comment);
		},
		[](const FString& UnsanitizedComment)
		{
			FString SanitizedComment = UnsanitizedComment;
			SanitizedComment.ReplaceInline(TEXT("("), TEXT(""));
			SanitizedComment.ReplaceInline(TEXT(")"), TEXT(""));
			SanitizedComment.ReplaceInline(TEXT(";"), TEXT(""));
			return SanitizedComment;
		}
	);

	return !FilteredComments.IsEmpty() ? FString::Printf(TEXT(" (%s)"), *FString::Join(FilteredComments, TEXT("; "))) : FString();
}

class FUserAgentImpl final
{
public:
	static FUserAgentImpl& Get()
	{
		static FUserAgentImpl Instance;
		return Instance;
	}

	FUserAgentImpl()
		: RWLock()
		, bCachedCommentsEnabled(false)
	{
		CachedUserAgent = BuildUserAgentString();
		CVarDefaultUserAgentCommentsEnabled->OnChangedDelegate().AddRaw(this, &FUserAgentImpl::OnCommentsEnabledChanged);
	}

	void AddProjectComment(const FString& Comment)
	{
		FRWScopeLock WriteLock(RWLock, SLT_Write);
		Builder.AddProjectComment(Comment);
		CachedUserAgent = BuildUserAgentString();
	}

	void AddPlatformComment(const FString& Comment)
	{
		FRWScopeLock WriteLock(RWLock, SLT_Write);
		Builder.AddPlatformComment(Comment);
		CachedUserAgent = BuildUserAgentString();
	}

	FString GetUserAgent()
	{
		FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);
		return CachedUserAgent;
	}

	FDefaultUserAgentBuilder GetBuilder()
	{
		FRWScopeLock ReadLock(RWLock, SLT_ReadOnly);
		return Builder;
	}

	uint32 GetAgentVersion()
	{
		return Builder.GetAgentVersion();
	}

private:
	FString BuildUserAgentString()
	{
		if (CVarDefaultUserAgentCommentsEnabled.GetValueOnAnyThread())
		{
			return Builder.BuildUserAgentString();
		}
		else
		{
			// Build the user agent string without any comments.
			TSet<FString> AllowedCommentsFilter;
			return Builder.BuildUserAgentString(&AllowedCommentsFilter, &AllowedCommentsFilter);
		}
	}

	void OnCommentsEnabledChanged(IConsoleVariable*)
	{
		FRWScopeLock WriteLock(RWLock, SLT_Write);
		CachedUserAgent = BuildUserAgentString();
	}

	FRWLock RWLock;
	FDefaultUserAgentBuilder Builder;
	FString CachedUserAgent;
	bool bCachedCommentsEnabled;
};

void FGenericPlatformHttp::Init()
{
	// Call during init to instantiate the user agent cache.
	FUserAgentImpl::Get();
}

void FGenericPlatformHttp::Shutdown()
{
}

IHttpRequest* FGenericPlatformHttp::ConstructRequest()
{
	return new FGenericPlatformHttpRequest();
}

static bool IsAllowedChar(UTF8CHAR LookupChar)
{
	static char AllowedChars[] = "ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789-_.~";
	static bool bTableFilled = false;
	static bool AllowedTable[256] = { false };

	if (!bTableFilled)
	{
		for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(AllowedChars) - 1; ++Idx)	// -1 to avoid trailing 0
		{
			uint8 AllowedCharIdx = static_cast<uint8>(AllowedChars[Idx]);
			check(AllowedCharIdx < UE_ARRAY_COUNT(AllowedTable));
			AllowedTable[AllowedCharIdx] = true;
		}

		bTableFilled = true;
	}

	return AllowedTable[LookupChar];
}

FString FGenericPlatformHttp::UrlEncode(const FStringView UnencodedString)
{
	FString EncodedString;
	EncodedString.Reserve(UnencodedString.Len()); // This is a minimum bound. Some characters might be outside the ascii set and require %hh encoding. Some characters might be multi-byte and require several %hh%hh%hh

	UTF8CHAR Utf8ConvertedChar[4] = {};
	TCHAR HexChars[3] = { TCHAR('%') };

	for (const TCHAR& InChar : UnencodedString)
	{
		verify(FPlatformString::Convert(Utf8ConvertedChar, sizeof(Utf8ConvertedChar), &InChar, 1));
		for (int32 ByteIdx = 0; ByteIdx < sizeof(Utf8ConvertedChar); ++ByteIdx)
		{
			UTF8CHAR ByteToEncode = Utf8ConvertedChar[ByteIdx];
			Utf8ConvertedChar[ByteIdx] = UTF8CHAR('\0');
			if (ByteToEncode == '\0')
			{
				break;
			}
			else if (IsAllowedChar(ByteToEncode))
			{
				// We use InChar here as it is the same value ByteToEncode would convert back to anyway
				// Note this relies on the fact that IsAllowedChar is only possible to be true for single-byte UTF8 characters.
				EncodedString.AppendChar(InChar);
			}
			else
			{
				UE::String::BytesToHex(MakeArrayView((uint8*)&ByteToEncode, 1), &HexChars[1]);
				EncodedString.AppendChars(HexChars, 3);
			}
		}
	}

	return EncodedString;
}

FString FGenericPlatformHttp::UrlDecode(const FStringView EncodedString)
{
	FTCHARToUTF8 Converter(EncodedString.GetData(), EncodedString.Len());
	const UTF8CHAR* UTF8Data = (UTF8CHAR*)Converter.Get();	
	
	TArray<UTF8CHAR> Data;
	Data.Reserve(EncodedString.Len());

	for (int32 CharIdx = 0; CharIdx < Converter.Length();)
	{
		if (UTF8Data[CharIdx] == '%')
		{
			int32 Value = 0;
			if (CharIdx < Converter.Length() - 1 && UTF8Data[CharIdx + 1] == 'u')
			{
				if (CharIdx <= Converter.Length() - 6)
				{
					// Treat all %uXXXX as code point
					Value = FParse::HexDigit(static_cast<TCHAR>(UTF8Data[CharIdx + 2])) << 12;
					Value += FParse::HexDigit(static_cast<TCHAR>(UTF8Data[CharIdx + 3])) << 8;
					Value += FParse::HexDigit(static_cast<TCHAR>(UTF8Data[CharIdx + 4])) << 4;
					Value += FParse::HexDigit(static_cast<TCHAR>(UTF8Data[CharIdx + 5]));
					CharIdx += 6;

					UTF8CHAR Buffer[8] = {};
					UTF8CHAR* BufferPtr = Buffer;
					const int32 Len = UE_ARRAY_COUNT(Buffer);
					const UTF8CHAR* BufferEnd = FPlatformString::Convert(BufferPtr, Len, (UTF32CHAR*)&Value, 1);
					check(BufferEnd);

					Data.Append(Buffer, (int32)(BufferEnd - BufferPtr));
				}
				else
				{
					// Not enough in the buffer for valid decoding, skip it
					CharIdx++;
					continue;
				}
			}
			else if(CharIdx <= Converter.Length() - 3)
			{
				// Treat all %XX as straight byte
				Value = FParse::HexDigit(static_cast<TCHAR>(UTF8Data[CharIdx + 1])) << 4;
				Value += FParse::HexDigit(static_cast<TCHAR>(UTF8Data[CharIdx + 2]));
				CharIdx += 3;
				Data.Add((UTF8CHAR)(Value));
			}
			else
			{
				// Not enough in the buffer for valid decoding, skip it
				CharIdx++;
				continue;
			}
		}
		else
		{
			// Non escaped characters
			Data.Add(UTF8Data[CharIdx]);
			CharIdx++;
		}
	}

	Data.Add(UTF8TEXT('\0'));
	return FString(UTF8_TO_TCHAR(Data.GetData()));
}

FString FGenericPlatformHttp::HtmlEncode(const FStringView UnencodedString)
{
	FString EncodedString;
	EncodedString.Reserve(UnencodedString.Len());

	for ( int32 Index = 0; Index != UnencodedString.Len(); ++Index )
	{
		switch ( UnencodedString[Index] )
		{
			case '&':  EncodedString.Append(TEXT("&amp;"));					break;
			case '\"': EncodedString.Append(TEXT("&quot;"));				break;
			case '\'': EncodedString.Append(TEXT("&apos;"));				break;
			case '<':  EncodedString.Append(TEXT("&lt;"));					break;
			case '>':  EncodedString.Append(TEXT("&gt;"));					break;
			default:   EncodedString.AppendChar(UnencodedString[Index]);	break;
		}
	}

	return EncodedString;
}

FString FGenericPlatformHttp::GetUrlDomainAndPort(const FStringView Url)
{
	FStringView Protocol;
	FStringView DomainAndPort;
	// split the http protocol portion from domain
	if (!Split(Url, TEXT("://"), Protocol, DomainAndPort))
	{
		DomainAndPort = Url;
	}
	// strip off everything but the domain and port portion
	for (int32 Index = 0; Index < DomainAndPort.Len(); ++Index)
	{
		TCHAR Character = DomainAndPort[Index];
		if (Character == TEXT('/') || Character == TEXT('?') || Character == TEXT('#'))
		{
			DomainAndPort.LeftInline(Index);
			break;
		}
	}

	return FString(DomainAndPort);
}

FString FGenericPlatformHttp::GetUrlDomain(const FStringView Url)
{
	FStringView Protocol;
	FStringView Domain;
	// split the http protocol portion from domain
	if (!Split(Url, TEXT("://"), Protocol, Domain))
	{
		Domain = Url;
	}
	// strip off everything but the domain portion
	for (int32 Index = 0; Index < Domain.Len(); ++Index)
	{
		TCHAR Character = Domain[Index];
		if (Character == TEXT('/') || Character == TEXT('?') || Character == TEXT(':') || Character == TEXT('#'))
		{
			Domain.LeftInline(Index);
			break;
		}
	}

	return FString(Domain);
}

FString FGenericPlatformHttp::GetUrlBase(const FStringView Url)
{
	FStringView UrlBase = Url;
	for (int32 Index = Url.Find(TEXT("://")) + 3; Index < Url.Len(); ++Index)
	{
		const TCHAR Character = Url[Index];
		if (Character == TEXT('/') || Character == TEXT('?') || Character == TEXT('#'))
		{
			UrlBase.LeftInline(Index);
			break;
		}
	}

	return FString(UrlBase);
}

FString FGenericPlatformHttp::GetMimeType(const FString& FilePath)
{
	const FString FileExtension = FPaths::GetExtension(FilePath, true);

	static TMap<FString, FString> ExtensionMimeTypeMap;
	if ( ExtensionMimeTypeMap.Num() == 0 )
	{
		// Web
		ExtensionMimeTypeMap.Add(TEXT(".html"), TEXT("text/html"));
		ExtensionMimeTypeMap.Add(TEXT(".css"), TEXT("text/css"));
		ExtensionMimeTypeMap.Add(TEXT(".js"), TEXT("application/x-javascript"));

		// Video
		ExtensionMimeTypeMap.Add(TEXT(".avi"), TEXT("video/msvideo, video/avi, video/x-msvideo"));
		ExtensionMimeTypeMap.Add(TEXT(".mpeg"), TEXT("video/mpeg"));

		// Image
		ExtensionMimeTypeMap.Add(TEXT(".bmp"), TEXT("image/bmp"));
		ExtensionMimeTypeMap.Add(TEXT(".gif"), TEXT("image/gif"));
		ExtensionMimeTypeMap.Add(TEXT(".jpg"), TEXT("image/jpeg"));
		ExtensionMimeTypeMap.Add(TEXT(".jpeg"), TEXT("image/jpeg"));
		ExtensionMimeTypeMap.Add(TEXT(".png"), TEXT("image/png"));
		ExtensionMimeTypeMap.Add(TEXT(".svg"), TEXT("image/svg+xml"));
		ExtensionMimeTypeMap.Add(TEXT(".tiff"), TEXT("image/tiff"));

		// Audio
		ExtensionMimeTypeMap.Add(TEXT(".midi"), TEXT("audio/x-midi"));
		ExtensionMimeTypeMap.Add(TEXT(".mp3"), TEXT("audio/mpeg"));
		ExtensionMimeTypeMap.Add(TEXT(".ogg"), TEXT("audio/vorbis, application/ogg"));
		ExtensionMimeTypeMap.Add(TEXT(".wav"), TEXT("audio/wav, audio/x-wav"));

		// Documents
		ExtensionMimeTypeMap.Add(TEXT(".xml"), TEXT("application/xml"));
		ExtensionMimeTypeMap.Add(TEXT(".txt"), TEXT("text/plain"));
		ExtensionMimeTypeMap.Add(TEXT(".log"), TEXT("text/plain"));
		ExtensionMimeTypeMap.Add(TEXT(".tsv"), TEXT("text/tab-separated-values"));
		ExtensionMimeTypeMap.Add(TEXT(".csv"), TEXT("text/csv"));
		ExtensionMimeTypeMap.Add(TEXT(".json"), TEXT("application/json"));
		
		// Compressed
		ExtensionMimeTypeMap.Add(TEXT(".zip"), TEXT("application/zip, application/x-compressed-zip"));
	}

	if ( FString* FoundMimeType = ExtensionMimeTypeMap.Find(FileExtension) )
	{
		return *FoundMimeType;
	}

	return TEXT("application/unknown");
}

FString FGenericPlatformHttp::GetDefaultUserAgent()
{
	return FUserAgentImpl::Get().GetUserAgent();
}

void FGenericPlatformHttp::AddDefaultUserAgentProjectComment(const FString& Comment)
{
	FUserAgentImpl::Get().AddProjectComment(Comment);
}

void FGenericPlatformHttp::AddDefaultUserAgentPlatformComment(const FString& Comment)
{
	FUserAgentImpl::Get().AddPlatformComment(Comment);
}

uint32 FGenericPlatformHttp::GetDefaultUserAgentVersion()
{
	return FUserAgentImpl::Get().GetAgentVersion();
}

FDefaultUserAgentBuilder FGenericPlatformHttp::GetDefaultUserAgentBuilder()
{
	return FUserAgentImpl::Get().GetBuilder();
}

FString FGenericPlatformHttp::EscapeUserAgentString(const FString& UnescapedString)
{
	if (UnescapedString.Contains(" ") || UnescapedString.Contains("/"))
	{
		FString EscapedString = UnescapedString;
		EscapedString.ReplaceInline(TEXT(" "), TEXT(""));
		EscapedString.ReplaceInline(TEXT("/"), TEXT("+"));
		return EscapedString;
	}
	else
	{
		return UnescapedString;
	}
}

TOptional<FString> FGenericPlatformHttp::GetProxyAddressFromConfig()
{
	FString ProxyAddress;
	if (FParse::Value(FCommandLine::Get(), TEXT("httpproxy="), ProxyAddress))
	{
		return ProxyAddress;
	}
	if (GConfig->GetString(TEXT("HTTP"), TEXT("HttpProxyAddress"), ProxyAddress, GEngineIni))
	{
		return ProxyAddress;
	}
	return TOptional<FString>();
}

FString FGenericPlatformHttp::GetConfiguredProxyAddress()
{
	if (TOptional<FString> ConfigProxy = GetProxyAddressFromConfig())
	{
		return ConfigProxy.GetValue();
	}
	if (TOptional<FString> OsProxy = GetOperatingSystemProxyAddress())
	{
		return OsProxy.GetValue();
	}
	return FString();
}

TOptional<FString> FGenericPlatformHttp::GetOperatingSystemProxyAddress()
{
#if UE_HTTP_PLATFORM_SUPPORTS_OS_PROXY_ADDRESS
	checkf(false, TEXT("Platform supports OS proxy address but did not override GetOperatingSystemProxyAddress()"));
#endif
	return TOptional<FString>();
}

bool FGenericPlatformHttp::IsOperatingSystemProxyInformationSupported()
{
	return false;
}

TOptional<FString> FGenericPlatformHttp::GetUrlParameter(const FStringView Url, const FStringView ParameterName)
{
	TOptional<FString> ParameterValue;

	// Find first argument (starts with '?') and split string from then on
	FStringView Endpoint;
	FStringView ParamsAndFragment;
	if (Split(Url, TEXT('?'), Endpoint, ParamsAndFragment))
	{
		// Split out any URL fragment, if we have one
		FStringView AllParams;
		FStringView Fragment;
		if (!Split(ParamsAndFragment, TEXT('#'), AllParams, Fragment))
		{
			// If we have no #, we have no fragment
			AllParams = ParamsAndFragment;
		}

		// Parse our params for each &key=value pair and 
		FStringView ParamsLeftToRead = AllParams;
		do
		{
			// Get the next param into CurrentParam and store the remainder back into AllParams
			FStringView CurrentParam;
			FStringView Remainders;
			if (Split(ParamsLeftToRead, TEXT('&'), CurrentParam, Remainders))
			{
				// Store the new leftover params to parse
				ParamsLeftToRead = Remainders;
			}
			else
			{
				// If there are no more params, the remainder of ParamsLeftToRead is our param to parse
				CurrentParam = ParamsLeftToRead;
				ParamsLeftToRead.Reset();
			}

			// Get the CurrentParam's Key and Value
			FStringView ParamKey;
			FStringView ParamValue;
			if (!Split(CurrentParam, TEXT('='), ParamKey, ParamValue))
			{
				// If the param doesn't have a value, the whole value is the key
				ParamKey = CurrentParam;
			}

			// Check if this is the correct param
			if (ParamKey == ParameterName)
			{
				// Copy Parameter data if it's set, otherwise empty string is fine
				if (!ParamValue.IsEmpty())
				{
					ParameterValue = FGenericPlatformHttp::UrlDecode(ParamValue);
				}
				else
				{
					ParameterValue.Emplace();
				}

				// We're done, break out of our loop
				break;
			}
		}
		while (!ParamsLeftToRead.IsEmpty());
	}

	return ParameterValue;
}

TOptional<uint16> FGenericPlatformHttp::GetUrlPort(const FStringView Url)
{
	TOptional<uint16> Port;

	FStringView Scheme;
	FStringView UrlWithoutScheme;
	if (!Split(Url, TEXT("://"), Scheme, UrlWithoutScheme))
	{
		UrlWithoutScheme = Url;
	}

	TOptional<int32> PortStringStartIndex;
	for (int32 Index = 0; Index < UrlWithoutScheme.Len(); ++Index)
	{
		const TCHAR Character = UrlWithoutScheme[Index];
		if (!PortStringStartIndex.IsSet())
		{
			if (Character == ':')
			{
				if (Index + 1 < UrlWithoutScheme.Len())
				{
					PortStringStartIndex = Index + 1;
				}
				// Start of Port
				continue;
			}
			else if (Character == '/')
			{
				// Port not found
				break;
			}

			// Keep looping
		}
		else if (PortStringStartIndex.IsSet())
		{
			// Break when we find a non-numeric character or our string is ending
			const bool bIsDigit = FChar::IsDigit(Character);
			const bool bIsLastCharacter = Index + 1 == UrlWithoutScheme.Len();
			if (!bIsDigit || bIsLastCharacter)
			{
				// Include the current value if we're at the end of the string and we haven't hit a non-digit
				if (bIsDigit && bIsLastCharacter)
				{
					++Index;
				}

				// Extract our port
				const FStringView PortString = UrlWithoutScheme.Mid(PortStringStartIndex.GetValue(), Index - PortStringStartIndex.GetValue());
				if (!PortString.IsEmpty())
				{
					// Copy string for null-terminator (engine doesn't have a Atoi that allows specifying the length :( )
					const FString AllocatedPortString(PortString);
					LexFromString(Port.Emplace(), *AllocatedPortString);
				}
				break;
			}

			// Keep Looping
		}
	}

	return Port;
}

FString FGenericPlatformHttp::GetUrlPath(const FStringView Url, const bool bIncludeQueryString, const bool bIncludeFragment)
{
	// If bIncludeQueryString is false, bIncludeFragment must be false as well
	check(bIncludeQueryString || !bIncludeFragment);

	FString PathString("/");

	// Remove scheme
	FStringView Scheme;
	FStringView DomainAndPortAndPathAndQueryAndFragement;
	if (!Split(Url, TEXT("://"), Scheme, DomainAndPortAndPathAndQueryAndFragement))
	{
		DomainAndPortAndPathAndQueryAndFragement = Url;
	}

	// Find Domain End
	FStringView PortAndPathAndQueryAndFragment;
	for (int32 Index = 0; Index < DomainAndPortAndPathAndQueryAndFragement.Len(); ++Index)
	{
		const TCHAR Character = DomainAndPortAndPathAndQueryAndFragement[Index];
		if (Character == TEXT(':') || Character == TEXT('/') || Character == TEXT('?') || Character == TEXT('#'))
		{
			PortAndPathAndQueryAndFragment = DomainAndPortAndPathAndQueryAndFragement.RightChop(Index);
			break;
		}
	}

	// Check if we ran out of characters
	if (PortAndPathAndQueryAndFragment.IsEmpty())
	{
		return PathString;
	}
	 
	// Find PortEnd
	FStringView PathAndQueryAndFragment;
	if (PortAndPathAndQueryAndFragment.StartsWith(TEXT(':')))
	{
		for (int32 Index = 1; Index < PortAndPathAndQueryAndFragment.Len(); ++Index)
		{
			// Port is finished when we hit a non-digit (ideally a /, but in practice sometimes this is (incorrectly) missing)
			const TCHAR Character = PortAndPathAndQueryAndFragment[Index];
			if (!FChar::IsDigit(Character))
			{
				PathAndQueryAndFragment = PortAndPathAndQueryAndFragment.RightChop(Index);
				break;
			}
		}
	}
	else
	{
		PathAndQueryAndFragment = PortAndPathAndQueryAndFragment;
	}

	// Remove any leading /
	if (PathAndQueryAndFragment.StartsWith(TEXT('/')))
	{
		PathAndQueryAndFragment.RightChopInline(1);
	}

	// Check if we ran out of characters
	if (PathAndQueryAndFragment.IsEmpty())
	{
		return PathString;
	}

	// Early out if possible
	if (bIncludeQueryString)
	{
		if (bIncludeFragment)
		{
			// They want all of the rest of the string
			PathString += PathAndQueryAndFragment;
		}
		else
		{
			// They only want the query portion
			FStringView PathAndQuery;

			int32 FragmentStartIndex = INDEX_NONE;
			if (PathAndQueryAndFragment.FindChar(TEXT('#'), FragmentStartIndex))
			{
				// Remove the fragment from the string
				PathAndQuery = PathAndQueryAndFragment.Left(FragmentStartIndex);
			}
			else
			{ 
				// There is no fragement on our string
				PathAndQuery = PathAndQueryAndFragment;
			}
			
			PathString += PathAndQuery;
		}
	}
	else
	{
		// Find end of path
		FStringView Path;
		for (int32 Index = 0; Index < PathAndQueryAndFragment.Len(); ++Index)
		{
			// The path is over when either the querystring starts or the fragment starts
			const TCHAR Character = PathAndQueryAndFragment[Index];
			if (Character == TEXT('?') || Character == TEXT('#'))
			{
				Path = PathAndQueryAndFragment.Left(Index);
				break;
			}
			else if (Index + 1 == PathAndQueryAndFragment.Len())
			{
				// If we didn't find anything by now, we don't have a query or fragment
				Path = PathAndQueryAndFragment;
			}
		}

		PathString += Path;
	}

	return PathString;
}


TOptional<bool> FGenericPlatformHttp::IsSecureProtocol(const FStringView Url)
{
	TOptional<bool> bIsSecure;

	FStringView Scheme;
	FStringView UrlWithoutScheme;
	if (Split(Url, TEXT("://"), Scheme, UrlWithoutScheme))
	{
		if (Scheme == TEXT("https") || Scheme == TEXT("wss"))
		{
			bIsSecure = true;
		}
		else if (Scheme == TEXT("http") || Scheme == TEXT("ws"))
		{
			bIsSecure = false;
		}
	}

	return bIsSecure;
}
