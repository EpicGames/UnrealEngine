// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/HttpRequestCommon.h"
#include "GenericPlatform/HttpResponseCommon.h"
#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "Http.h"
#include "HttpConstants.h"
#include "HttpManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Stats/Stats.h"

TAutoConsoleVariable<bool> CVarHttpLogJsonContentOnly(
	TEXT("http.LogJsonContentOnly"),
	true,
	TEXT("When log request/response payload, log json content only"),
	ECVF_SaveForNextBoot
);

FHttpRequestCommon::FHttpRequestCommon()
	: RequestStartTimeAbsoluteSeconds(FPlatformTime::Seconds())
	, ActivityTimeoutAt(0.0)
{
}

FString FHttpRequestCommon::GetURLParameter(const FString& ParameterName) const
{
	FString ReturnValue;
	if (TOptional<FString> OptionalParameterValue = FGenericPlatformHttp::GetUrlParameter(GetURL(), ParameterName))
	{
		ReturnValue = MoveTemp(OptionalParameterValue.GetValue());
	}
	return ReturnValue;
}

void FHttpRequestCommon::SetHeader(const FString& HeaderName, const FString& HeaderValue)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FHttpRequestCommon::SetHeader() - attempted to set header on a request that is inflight");
		return;
	}

	RequestHeaders.Add(HeaderName, HeaderValue);
}

FString FHttpRequestCommon::GetHeader(const FString& HeaderName) const
{
	FString Result;

	const FString* Header = RequestHeaders.Find(HeaderName);
	if (Header != NULL)
	{
		Result = *Header;
	}
	
	return Result;
}

EHttpRequestStatus::Type FHttpRequestCommon::GetStatus() const
{
	return CompletionStatus;
}

const FString& FHttpRequestCommon::GetEffectiveURL() const
{
	return EffectiveURL;
}

EHttpFailureReason FHttpRequestCommon::GetFailureReason() const
{
	return FailureReason;
}

bool FHttpRequestCommon::PreCheck() const
{
#if !UE_HTTP_SUPPORT_VERB_CONNECT
	checkf(!GetVerb().Equals(FString("CONNECT"), ESearchCase::IgnoreCase), TEXT("CONNECT verb is not supported on this platform."));
#endif

	// Disabled http request processing
	if (!FHttpModule::Get().IsHttpEnabled())
	{
		UE_LOGF(LogHttp, Verbose, "Http disabled. Skipping request. url=%ls", *GetURL());
		return false;
	}

	// Prevent overlapped requests using the same instance
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "ProcessRequest failed. Still processing last request.");
		return false;
	}

	// Nothing to do without a valid URL
	if (GetURL().IsEmpty())
	{
		UE_LOGF(LogHttp, Warning, "ProcessRequest failed. No URL was specified.");
		return false;
	}

	if (GetVerb().IsEmpty())
	{
		UE_LOGF(LogHttp, Warning, "ProcessRequest failed. No Verb was specified.");
		return false;
	}

	if (!FHttpModule::Get().GetHttpManager().IsDomainAllowed(GetURL()))
	{
		UE_LOGF(LogHttp, Warning, "ProcessRequest failed. URL '%ls' is not using an allowed domain.", *GetURL());
		return false;
	}

	if (bTimedOut)
	{
		UE_LOGF(LogHttp, Warning, "ProcessRequest failed. Request with URL '%ls' already timed out.", *GetURL());
		return false;
	}

	return true;
}

bool FHttpRequestCommon::TriggerMockResponse()
{
	TOptional<FHttpManager::FMockResponse> MockResponse = FHttpModule::Get().GetHttpManager().GetMockResponse(GetURL(), GetVerb());
	if (MockResponse.IsSet())
	{
		if (MockResponse.GetValue().StatusCode == EHttpResponseCodes::Unknown)
		{
			int32 HttpConnectionTimeout = FHttpModule::Get().GetHttpConnectionTimeout();
			TSharedPtr<FHttpRequestCommon> RequestPtr(SharedThis(this));
			FHttpModule::Get().GetHttpManager().AddHttpThreadTask([RequestPtr]() mutable {
				RequestPtr->SetFailureReason(EHttpFailureReason::ConnectionError);
				RequestPtr->FinishRequestNotInHttpManager();
				RequestPtr.Reset();
			}, HttpConnectionTimeout);

			// Connect timeout mocking will trigger FinishRequest after a delay, still make sure total timeout 
			// works when mocking connect timeout
			StartTotalTimeoutTimer();
		}
		else
		{
			InitResponse();
			ResponseCommon->SetResponseCode(MockResponse.GetValue().StatusCode);
			MockResponseData();
			const FUtf8String& ResponsePayload = MockResponse.GetValue().ResponsePayload;
			if (!ResponsePayload.IsEmpty())
			{
				ResponseCommon->AppendToPayload(reinterpret_cast<const uint8*>(*ResponsePayload), ResponsePayload.Len());
			}
			ResponseCommon->Headers = MockResponse.GetValue().ResponseHeaders;
			FinishRequestNotInHttpManager();
		}

		return true;
	}

	return false;
}

void FHttpRequestCommon::InitResponse()
{
	if (!ResponseCommon)
	{
		FHttpResponsePtr Response = CreateResponse();
		ResponseCommon = StaticCastSharedPtr<FHttpResponseCommon>(Response);
	}
}

void FHttpRequestCommon::PopulateUserAgentHeader()
{
	if (GetHeader(TEXT("User-Agent")).IsEmpty())
	{
		SetHeader(TEXT("User-Agent"), FPlatformHttp::GetDefaultUserAgent());
	}
}

bool FHttpRequestCommon::PreProcess()
{
	ClearInCaseOfRetry();

	if (!PreCheck())
	{
		FinishRequestNotInHttpManager();
		return false;
	}

	if (TriggerMockResponse())
	{
		return false;
	}

	PopulateUserAgentHeader();

	if (!SetupRequest())
	{
		FinishRequestNotInHttpManager();
		return false;
	}

	StartTotalTimeoutTimer();

	UE_LOGF(LogHttp, Verbose, "%p: Verb='%ls' URL='%ls'", this, *GetVerb(), *GetURL());

	return true;
}

void FHttpRequestCommon::PostProcess()
{
	CleanupRequest();
}

void FHttpRequestCommon::ClearInCaseOfRetry()
{
	bActivityTimedOut = false;
	FailureReason = EHttpFailureReason::None;
	bCanceled = false;
	EffectiveURL = GetURL();
	ResponseCommon.Reset();
}

void FHttpRequestCommon::FinishRequestNotInHttpManager()
{
	if (IsInGameThread())
	{
		if (DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnGameThread)
		{
			FinishRequest();
		}
		else
		{
			FHttpModule::Get().GetHttpManager().AddHttpThreadTask([StrongThis = StaticCastSharedRef<FHttpRequestCommon>(AsShared())]()
			{
				StrongThis->FinishRequest();
			});
		}
	}
	else
	{
		if (DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread)
		{
			FinishRequest();
		}
		else
		{
			FHttpModule::Get().GetHttpManager().AddGameThreadTask([StrongThis = StaticCastSharedRef<FHttpRequestCommon>(AsShared())]()
			{
				StrongThis->FinishRequest();
			});
		}
	}
}

void FHttpRequestCommon::SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InDelegateThreadPolicy)
{ 
	DelegateThreadPolicy = InDelegateThreadPolicy; 
}

EHttpRequestDelegateThreadPolicy FHttpRequestCommon::GetDelegateThreadPolicy() const
{ 
	return DelegateThreadPolicy; 
}

FString FHttpRequestCommon::GetOption(const FName Option) const
{
	const FString* OptionValue = Options.Find(Option);
	if (OptionValue)
	{
		return *OptionValue;
	}
	return TEXT("");
}

void FHttpRequestCommon::SetOption(const FName Option, const FString& OptionValue)
{
	Options.Add(Option, OptionValue);
}

void FHttpRequestCommon::HandleRequestSucceed()
{
	SetStatus(EHttpRequestStatus::Succeeded);

	LogResponse(ResponseCommon);

	FHttpModule::Get().GetHttpManager().RecordStatTimeToConnect(ConnectTime);
	FHttpModule::Get().GetHttpManager().RecordStatRequestSucceeded();
}

void FHttpRequestCommon::HandleRequestFailed()
{
	if (FailureReason == EHttpFailureReason::None) // Failure reason was not set by platform, will set it here
	{
		if (bCanceled)
		{
			SetFailureReason(EHttpFailureReason::Cancelled);
		}
		else if (bTimedOut)
		{
			SetFailureReason(EHttpFailureReason::TimedOut);
		}
		else if (!bUsePlatformActivityTimeout && bActivityTimedOut)
		{
			SetFailureReason(EHttpFailureReason::ConnectionError);
		}
		else
		{
			SetFailureReason(EHttpFailureReason::Other);
		}
	}

	SetStatus(EHttpRequestStatus::Failed);

	FHttpModule::Get().GetHttpManager().RecordStatRequestFailed(FailureReason);

	LogFailure();
}

#define UE_HTTP_LOG_AS_WARNING_IF(Condition, Format, ...) \
	if (Condition) \
	{ \
		UE_LOG(LogHttp, Warning, Format, ##__VA_ARGS__); \
	} \
	else \
	{ \
		UE_LOG(LogHttp, Verbose, Format, ##__VA_ARGS__); \
	}

void FHttpRequestCommon::LogFailure() const
{
	const bool bAborted = (bCanceled || bTimedOut || bActivityTimedOut);
	UE_HTTP_LOG_AS_WARNING_IF(!bAborted && !FHttpModule::Get().GetHttpManager().ShouldDisableFailedLog(GetURL()), TEXT("%p %s %s completed with reason '%s' after %.2fs"), this, *GetVerb(), *GetURL(), LexToString(GetFailureReason()), ElapsedTime);
}

void FHttpRequestCommon::SetStatus(EHttpRequestStatus::Type InCompletionStatus)
{
	CompletionStatus = InCompletionStatus;

	if (ResponseCommon)
	{
		ResponseCommon->SetRequestStatus(InCompletionStatus);
	}
}

void FHttpRequestCommon::SetFailureReason(EHttpFailureReason InFailureReason)
{
	UE_CLOGF(FailureReason != EHttpFailureReason::None, LogHttp, Warning, "FailureReason had been set to %ls, now setting to %ls", LexToString(FailureReason), LexToString(InFailureReason));
	FailureReason = InFailureReason;

	if (ResponseCommon)
	{
		ResponseCommon->SetRequestFailureReason(InFailureReason);
	}
}

void FHttpRequestCommon::SetTimeout(float InTimeoutSecs)
{
	TimeoutSecs = InTimeoutSecs;
}

void FHttpRequestCommon::ClearTimeout()
{
	TimeoutSecs.Reset();
	ResetTimeoutStatus();
}

void FHttpRequestCommon::ResetTimeoutStatus()
{
	StopTotalTimeoutTimer();
	bTimedOut = false;
}

TOptional<float> FHttpRequestCommon::GetTimeout() const
{
	return TimeoutSecs;
}

float FHttpRequestCommon::GetTimeoutOrDefault() const
{
	return GetTimeout().Get(FHttpModule::Get().GetHttpTotalTimeout());
}

void FHttpRequestCommon::SetActivityTimeout(float InTimeoutSecs)
{
	ActivityTimeoutSecs = InTimeoutSecs;
}

const FHttpResponsePtr FHttpRequestCommon::GetResponse() const
{
	return ResponseCommon;
}

void FHttpRequestCommon::CancelRequest()
{
	bool bWasCanceled = bCanceled.exchange(true);
	if (bWasCanceled)
	{
		return;
	}

	StopActivityTimeoutTimer();

	StopPassingReceivedData();

	UE_LOGF(LogHttp, Verbose, "HTTP request canceled. URL=%ls", *GetURL());

	FHttpModule::Get().GetHttpManager().AddHttpThreadTask([StrongThis = StaticCastSharedRef<FHttpRequestCommon>(AsShared())]()
	{
		// Run AbortRequest in HTTP thread to avoid potential concurrency issue
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpRequestCommon_AbortRequest);
		StrongThis->AbortRequest();
	});
}

void FHttpRequestCommon::StartActivityTimeoutTimer()
{
	const FScopeLock CacheLock(&HttpTaskTimerHandleCriticalSection);

	if (bUsePlatformActivityTimeout)
	{
		return;
	}

#if !UE_BUILD_SHIPPING
	static const bool bNoTimeouts = FParse::Param(FCommandLine::Get(), TEXT("NoTimeouts"));
	if (bNoTimeouts)
	{
		return;
	}
#endif

	if (bActivityTimedOut)
	{
		return;
	}

	float HttpActivityTimeout = GetActivityTimeoutOrDefault();
	if (HttpActivityTimeout == 0)
	{
		return;
	}

	StartActivityTimeoutTimerBy(HttpActivityTimeout);

	ResetActivityTimeoutTimer(TEXTVIEW("Connected"));
}

void FHttpRequestCommon::StartActivityTimeoutTimerBy(double DelayToTrigger)
{
	if (ActivityTimeoutHttpTaskTimerHandle != nullptr)
	{
		UE_LOGF(LogHttp, Warning, "Request %p already started activity timeout timer", this);
		return;
	}

	TWeakPtr<FHttpRequestCommon> RequestWeakPtr(SharedThis(this));
	ActivityTimeoutHttpTaskTimerHandle = FHttpModule::Get().GetHttpManager().AddHttpThreadTask([RequestWeakPtr]() {
		if (TSharedPtr<FHttpRequestCommon> RequestPtr = RequestWeakPtr.Pin())
		{
			RequestPtr->OnActivityTimeoutTimerTaskTrigger();
		}
	}, DelayToTrigger + 0.05);
}

void FHttpRequestCommon::OnActivityTimeoutTimerTaskTrigger()
{
	const FScopeLock CacheLock(&HttpTaskTimerHandleCriticalSection);

	ActivityTimeoutHttpTaskTimerHandle.Reset();

	if (EHttpRequestStatus::IsFinished(GetStatus()))
	{
		UE_LOGF(LogHttp, Warning, "Request %p had finished when activity timeout timer trigger at [%ls]", this, *FDateTime::Now().ToString(TEXT("%H:%M:%S:%s")));
		return;
	}

	if (FPlatformTime::Seconds() < ActivityTimeoutAt)
	{
		// Check back later
		UE_LOGF(LogHttp, VeryVerbose, "Request %p check response timeout at [%ls], will check again in %.5f seconds", this, *FDateTime::Now().ToString(TEXT("%H:%M:%S:%s")), ActivityTimeoutAt - FPlatformTime::Seconds());
		StartActivityTimeoutTimerBy(ActivityTimeoutAt - FPlatformTime::Seconds());
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpRequestCommon_AbortRequest);
	bActivityTimedOut = true;
	AbortRequest();
	UE_LOGF(LogHttp, Log, "Request [%ls] timed out at [%ls] because of no responding for %0.2f seconds", *GetURL(), *FDateTime::Now().ToString(TEXT("%H:%M:%S:%s")), GetActivityTimeoutOrDefault());
}

void FHttpRequestCommon::ResetActivityTimeoutTimer(FStringView Reason)
{
	const FScopeLock CacheLock(&HttpTaskTimerHandleCriticalSection);

	if (bUsePlatformActivityTimeout)
	{
		return;
	}

	if (!ActivityTimeoutHttpTaskTimerHandle)
	{
		return;
	}

	ActivityTimeoutAt = FPlatformTime::Seconds() + GetActivityTimeoutOrDefault();
	UE_LOGF(LogHttp, VeryVerbose, "Request [%p] reset response timeout timer at %ls: %ls", this, *FDateTime::Now().ToString(TEXT("%H:%M:%S:%s")), Reason.GetData());
}

void FHttpRequestCommon::StopActivityTimeoutTimer()
{
	const FScopeLock CacheLock(&HttpTaskTimerHandleCriticalSection);

	if (bUsePlatformActivityTimeout)
	{
		return;
	}

	if (!ActivityTimeoutHttpTaskTimerHandle)
	{
		return;
	}

	FHttpModule::Get().GetHttpManager().RemoveHttpThreadTask(ActivityTimeoutHttpTaskTimerHandle);
	ActivityTimeoutHttpTaskTimerHandle.Reset();
}

void FHttpRequestCommon::StartTotalTimeoutTimer()
{
	const FScopeLock CacheLock(&HttpTaskTimerHandleCriticalSection);

#if !UE_BUILD_SHIPPING
	static const bool bNoTimeouts = FParse::Param(FCommandLine::Get(), TEXT("NoTimeouts"));
	if (bNoTimeouts)
	{
		return;
	}
#endif

	float TimeoutOrDefault = GetTimeoutOrDefault();
	if (TimeoutOrDefault == 0)
	{
		return;
	}

	if (bTimedOut)
	{
		return;
	}

	// Timeout include retries, so if it's already started before, check this to prevent from adding timer multiple times
	if (TotalTimeoutHttpTaskTimerHandle)
	{
		return;
	}

	TWeakPtr<IHttpRequest> RequestWeakPtr(AsShared());
	TotalTimeoutHttpTaskTimerHandle = FHttpModule::Get().GetHttpManager().AddHttpThreadTask([RequestWeakPtr]() {
		if (TSharedPtr<IHttpRequest> RequestPtr = RequestWeakPtr.Pin())
		{
			TSharedPtr<FHttpRequestCommon> RequestCommonPtr = StaticCastSharedPtr<FHttpRequestCommon>(RequestPtr);
			RequestCommonPtr->OnTotalTimeoutTimerTaskTrigger();
		}
	}, TimeoutOrDefault);
}

void FHttpRequestCommon::OnTotalTimeoutTimerTaskTrigger()
{
	const FScopeLock CacheLock(&HttpTaskTimerHandleCriticalSection);
	bTimedOut = true;

	if (EHttpRequestStatus::IsFinished(GetStatus()))
	{
		return;
	}

	StopActivityTimeoutTimer();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpRequestCommon_AbortRequest);
	UE_LOGF(LogHttp, Warning, "HTTP request timed out after %0.2f seconds URL=%ls", GetTimeoutOrDefault(), *GetURL());

	AbortRequest();
}

void FHttpRequestCommon::StopTotalTimeoutTimer()
{
	const FScopeLock CacheLock(&HttpTaskTimerHandleCriticalSection);

	if (TotalTimeoutHttpTaskTimerHandle)
	{
		FHttpModule::Get().GetHttpManager().RemoveHttpThreadTask(TotalTimeoutHttpTaskTimerHandle);
		TotalTimeoutHttpTaskTimerHandle.Reset();
	}
}

void FHttpRequestCommon::Shutdown()
{
	FHttpRequestImpl::Shutdown();

	StopPassingReceivedData();
	StopActivityTimeoutTimer();
	StopTotalTimeoutTimer();
}

void FHttpRequestCommon::ProcessRequestUntilComplete()
{
	checkf(!OnProcessRequestComplete().IsBound(), TEXT("OnProcessRequestComplete is not supported for sync call"));

	SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);

	FEvent* Event = FPlatformProcess::GetSynchEventFromPool(true);
	OnProcessRequestComplete().BindLambda([Event](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded) {
		Event->Trigger();
	});
	ProcessRequest();
	Event->Wait();
	FPlatformProcess::ReturnSynchEventToPool(Event);
}

void FHttpRequestCommon::HandleStatusCodeReceived(int32 StatusCode)
{
	if (ResponseCommon)
	{
		ResponseCommon->SetResponseCode(StatusCode);
	}
	TriggerStatusCodeReceivedDelegate(StatusCode);
}

void FHttpRequestCommon::TriggerStatusCodeReceivedDelegate(int32 StatusCode)
{
	if (DelegateThreadPolicy == EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread)
	{
		OnStatusCodeReceived().ExecuteIfBound(SharedThis(this), StatusCode);
	}
	else if (OnStatusCodeReceived().IsBound())
	{
		FHttpModule::Get().GetHttpManager().AddGameThreadTask([StrongThis = AsShared(), StatusCode]()
		{
			StrongThis->OnStatusCodeReceived().ExecuteIfBound(StrongThis, StatusCode);
		});
	}
}

void FHttpRequestCommon::SetEffectiveURL(const FString& InEffectiveURL)
{
	EffectiveURL = InEffectiveURL;

	if (ResponseCommon)
	{
		ResponseCommon->SetEffectiveURL(EffectiveURL);
	}
}

bool FHttpRequestCommon::SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream)
{
	const FScopeLock StreamLock(&ResponseBodyReceiveStreamCriticalSection);

	ResponseBodyReceiveStream = Stream;
	bInitializedWithValidStream = true;
	return true;
}

float FHttpRequestCommon::GetElapsedTime() const
{
	return ElapsedTime;
}

bool FHttpRequestCommon::StartThreadedRequest()
{
	LogRequest();
	return true;
}

void FHttpRequestCommon::StartWaitingInQueue()
{
	TimeStartedWaitingInQueue = FPlatformTime::Seconds();
}

double FHttpRequestCommon::GetTimeStartedWaitingInQueue() const
{
	check(TimeStartedWaitingInQueue != 0.0);
	return TimeStartedWaitingInQueue;
}

void FHttpRequestCommon::SetURL(const FString& InURL)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FHttpRequestCommon::SetURL() - attempted to set url on a request that is inflight");
		return;
	}

	URL = InURL;
}

const FString& FHttpRequestCommon::GetURL() const
{
	return URL;
}

void FHttpRequestCommon::SetPriority(EHttpRequestPriority InPriority)
{
	Priority = InPriority;
}

EHttpRequestPriority FHttpRequestCommon::GetPriority() const
{
	return Priority;
}

bool FHttpRequestCommon::PassReceivedDataToStream(void* Ptr, int64 Length)
{
	const FScopeLock StreamLock(&ResponseBodyReceiveStreamCriticalSection);

	if (!ResponseBodyReceiveStream)
	{
		return false;
	}

	ResponseBodyReceiveStream->Serialize(Ptr, Length);

	return !ResponseBodyReceiveStream->GetError();
}

void FHttpRequestCommon::StopPassingReceivedData()
{
	const FScopeLock StreamLock(&ResponseBodyReceiveStreamCriticalSection);

	ResponseBodyReceiveStream = nullptr;
}


float FHttpRequestCommon::GetActivityTimeoutOrDefault() const
{
	return ActivityTimeoutSecs.Get(FHttpModule::Get().GetHttpActivityTimeout());
}

bool FHttpRequestCommon::SetContentAsStreamedFileDefaultImpl(const FString& Filename)
{
	UE_LOGF(LogHttp, Verbose, "FHttpRequestCommon::SetContentAsStreamedFileDefaultImpl() - %ls", *Filename);

	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		UE_LOGF(LogHttp, Warning, "FHttpRequestCommon::SetContentAsStreamedFileDefaultImpl() - attempted to set content on a request that is inflight");
		return false;
	}

	RequestPayload = MakeUnique<FRequestPayloadInFileStream>(*Filename);
	return true;
}

bool FHttpRequestCommon::OpenRequestPayloadDefaultImpl()
{
	if (!RequestPayload)
	{
		return true;
	}

	if (!RequestPayload->Open())
	{
		return false;
	}

	if ((GetVerb().IsEmpty() || GetVerb().Equals(TEXT("GET"), ESearchCase::IgnoreCase)) && RequestPayload->GetContentLength() > 0)
	{
		UE_LOGF(LogHttp, Warning, "An HTTP Get request cannot contain a payload.");
		return false;
	}

	return true;
}

void FHttpRequestCommon::CloseRequestPayloadDefaultImpl()
{
	if (RequestPayload.IsValid())
	{
		RequestPayload->Close();
	}
}

void FHttpRequestCommon::LogRequest() const
{
	bool bShouldLogDetails = FHttpModule::Get().GetHttpManager().ShouldInspect(GetURL());
	UE_HTTP_LOG_AS_WARNING_IF(bShouldLogDetails, TEXT("%p %s %s started from http thread"), this, *GetVerb(), *GetURL());

	LogHeaders(RequestHeaders, bShouldLogDetails, false/*bIsResponse*/);
	LogContent(bShouldLogDetails, *this, false/*bIsResponse*/);
}

void FHttpRequestCommon::LogResponse(const TSharedPtr<FHttpResponseCommon>& InResponse) const
{
	bool bShouldLogDetails = FHttpModule::Get().GetHttpManager().ShouldInspect(GetURL());
	UE_HTTP_LOG_AS_WARNING_IF(bShouldLogDetails, TEXT("%p %s %s completed with code %d after %.2fs"), this, *GetVerb(), *GetURL(), InResponse->GetResponseCode(), ElapsedTime);

	LogHeaders(InResponse->Headers, bShouldLogDetails, true/*bIsResponse*/);
	LogContent(bShouldLogDetails, *InResponse, true/*bIsResponse*/);
}

void FHttpRequestCommon::LogHeaders(const TMap<FString, FString>& Headers, bool bShouldLogAsWarning, bool bIsResponse) const
{
	for (const TPair<FString, FString>& HeaderPair : Headers)
	{
		bool bIsSensitiveData = 
			HeaderPair.Key.Contains(FHttpConstants::HEADER_AUTHORIZATION, ESearchCase::IgnoreCase) || 
			HeaderPair.Key.Contains(TEXT("Set-Cookie"), ESearchCase::IgnoreCase);

		UE_HTTP_LOG_AS_WARNING_IF(bShouldLogAsWarning, TEXT("%p %s Header %s:%s"), 
			this,
			bIsResponse ? TEXT("Response") : TEXT("Request"),
			*HeaderPair.Key,
			bIsSensitiveData ? TEXT("[REDACTED]") : *HeaderPair.Value);
	}
}

void FHttpRequestCommon::LogContent(bool bShouldLogDetails, const IHttpBase& HttpBase, bool bIsResponse) const
{
	if (!bShouldLogDetails)
	{
		return;
	}

	if (CVarHttpLogJsonContentOnly.GetValueOnAnyThread())
	{
		// Use Contains instead of Compare, for request/response header like "Content-Type: application/json;charset=utf-8"
		bool bIsContentTypeJson = HttpBase.GetHeader(FHttpConstants::HEADER_CONTENT_TYPE).Contains(FHttpConstants::MEDIATYPE_JSON, ESearchCase::IgnoreCase);
		if (!bIsContentTypeJson)
			return;
	}

	const TArray<uint8>& Content = HttpBase.GetContent();
	if (Content.IsEmpty())
	{
		return;
	}

	UE_LOGFMT(LogHttp, Warning, "{0} Content({1}):", bIsResponse ? TEXT("Response") : TEXT("Request"), FText::AsMemory(HttpBase.GetContentLength(), EMemoryUnitStandard::IEC).ToString());

	FUtf8StringView ResponseStringView(reinterpret_cast<const UTF8CHAR*>(Content.GetData()), Content.Num());
	int32 StartPos = 0;
	int32 EndPos = 0;
	// The response payload could exceed the maximum length supported by UE_LOG/UE_LOGFMT, so log it line by line if there are multiple lines
	while (StartPos < ResponseStringView.Len())
	{
		EndPos = ResponseStringView.Find("\n", StartPos);
		if (EndPos == INDEX_NONE)
		{
			EndPos = ResponseStringView.Len();
		}

		int32 LengthToLog = EndPos - StartPos;
		while (LengthToLog > 0 && ResponseStringView[StartPos + LengthToLog - 1] == '\r')
		{
			LengthToLog = LengthToLog - 1;
		}

		if (LengthToLog > 0)
		{
			FUtf8StringView Line(&ResponseStringView[StartPos], LengthToLog);
			UE_LOGFMT(LogHttp, Warning, "{Line}", Line);
		}

		StartPos = EndPos + 1;
	}
}

void FHttpRequestCommon::OnFinishRequest(bool bSucceeded)
{
	// TODO: Move more code from impl into this common function

	if (ResponseCommon)
	{
		ResponseCommon->bIsReady = true;
	}

	if (bSucceeded)
	{
		HandleRequestSucceed();
	}
	else
	{
		HandleRequestFailed();
	}

	OnProcessRequestComplete().ExecuteIfBound(SharedThis(this), ResponseCommon, bSucceeded);
}
