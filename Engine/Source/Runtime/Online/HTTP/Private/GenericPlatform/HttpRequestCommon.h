// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/HttpRequestImpl.h"
#include "GenericPlatform/HttpRequestPayload.h"

class FHttpResponseCommon;
class IHttpTaskTimerHandle;

/**
 * Contains implementation of some common functions that don't vary between implementations of different platforms
 */
class FHttpRequestCommon : public FHttpRequestImpl
{
public:
	FHttpRequestCommon();

	// IHttpBase
	HTTP_API virtual FString GetURLParameter(const FString& ParameterName) const override;

	// IHttpRequest
	HTTP_API virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) override;
	HTTP_API virtual FString GetHeader(const FString& HeaderName) const override;
	HTTP_API virtual EHttpRequestStatus::Type GetStatus() const override;
	HTTP_API virtual const FString& GetEffectiveURL() const override;
	HTTP_API virtual EHttpFailureReason GetFailureReason() const override;
	HTTP_API virtual void SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InDelegateThreadPolicy) override;
	HTTP_API virtual EHttpRequestDelegateThreadPolicy GetDelegateThreadPolicy() const override;
	HTTP_API virtual FString GetOption(const FName Option) const override;
	HTTP_API virtual void SetOption(const FName Option, const FString & OptionValue) override;

	HTTP_API virtual void SetTimeout(float InTimeoutSecs) override;
	HTTP_API virtual void ClearTimeout() override;
	HTTP_API virtual void ResetTimeoutStatus() override;
	HTTP_API virtual TOptional<float> GetTimeout() const override;
	HTTP_API float GetTimeoutOrDefault() const;

	HTTP_API virtual void SetActivityTimeout(float InTimeoutSecs) override;

	HTTP_API virtual const FHttpResponsePtr GetResponse() const override;

	// Can be called on game thread or http thread depend on the delegate thread policy
	virtual void FinishRequest() = 0;

	HTTP_API virtual void CancelRequest() override;

	HTTP_API virtual void Shutdown() override;

	HTTP_API virtual void ProcessRequestUntilComplete() override;

	HTTP_API virtual bool SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) override;

	HTTP_API virtual float GetElapsedTime() const override;

	virtual bool IsThreadedRequestComplete() = 0;
	virtual bool StartThreadedRequest();
	virtual void TickThreadedRequest(float DeltaSeconds) = 0;

	HTTP_API void StartWaitingInQueue();
	HTTP_API double GetTimeStartedWaitingInQueue() const;

	HTTP_API virtual void SetURL(const FString& InURL) override;
	HTTP_API virtual const FString& GetURL() const override;

	HTTP_API virtual void SetPriority(EHttpRequestPriority InPriority) override;
	HTTP_API virtual EHttpRequestPriority GetPriority() const override;

	HTTP_API virtual void ClearInCaseOfRetry();

	HTTP_API void SetStatus(EHttpRequestStatus::Type InCompletionStatus);
	HTTP_API void SetFailureReason(EHttpFailureReason InFailureReason);

protected:
	/**
	 * Check if this request is valid or allowed, before actually process the request
	 */
	HTTP_API bool PreProcess();
	HTTP_API void PostProcess();
	HTTP_API void PopulateUserAgentHeader();
	virtual bool SetupRequest() = 0;
	HTTP_API bool PreCheck() const;

	/**
	 * Finish the request when it's not in http manager
	 */
	HTTP_API void FinishRequestNotInHttpManager();

	HTTP_API void HandleRequestSucceed();
	HTTP_API void HandleRequestFailed();

	HTTP_API void StartActivityTimeoutTimer();
	HTTP_API void StartActivityTimeoutTimerBy(double DelayToTrigger);
	HTTP_API void ResetActivityTimeoutTimer(FStringView Reason);
	HTTP_API void OnActivityTimeoutTimerTaskTrigger();
	HTTP_API void StopActivityTimeoutTimer();
	HTTP_API void StartTotalTimeoutTimer();
	HTTP_API void StopTotalTimeoutTimer();
	HTTP_API void OnTotalTimeoutTimerTaskTrigger();

	virtual void AbortRequest() = 0;

	virtual void CleanupRequest() = 0;

	HTTP_API void HandleStatusCodeReceived(int32 StatusCode);
	HTTP_API void TriggerStatusCodeReceivedDelegate(int32 StatusCode);

	HTTP_API void SetEffectiveURL(const FString& InEffectiveURL);

	HTTP_API bool PassReceivedDataToStream(void* Ptr, int64 Length);
	HTTP_API void StopPassingReceivedData();

	HTTP_API float GetActivityTimeoutOrDefault() const;

	HTTP_API bool SetContentAsStreamedFileDefaultImpl(const FString& Filename);
	HTTP_API bool OpenRequestPayloadDefaultImpl();
	HTTP_API void CloseRequestPayloadDefaultImpl();

	HTTP_API void LogFailure() const;
	HTTP_API void LogRequest() const;
	HTTP_API void LogResponse(const TSharedPtr<FHttpResponseCommon>& InResponse) const;
	HTTP_API void LogHeaders(const TMap<FString, FString>& Headers, bool bShouldLogAsWarning, bool bIsResponse) const;
	HTTP_API void LogContent(bool bShouldLogDetails, const IHttpBase& HttpBase, bool bIsResponse) const;
	HTTP_API bool TriggerMockResponse();

	HTTP_API void InitResponse();

	virtual FHttpResponsePtr CreateResponse() = 0;
	// The function for each impl to make sure when mocking response code, the internal state is valid for finishing request successfully
	virtual void MockResponseData() = 0;

	void OnFinishRequest(bool bSucceeded);

protected:
	/** Current status of request being processed */
	EHttpRequestStatus::Type CompletionStatus = EHttpRequestStatus::NotStarted;

	/** Reason of failure of the HTTP request */
	EHttpFailureReason FailureReason = EHttpFailureReason::None;

	/** Thread policy about which thread to complete this request */
	EHttpRequestDelegateThreadPolicy DelegateThreadPolicy = EHttpRequestDelegateThreadPolicy::CompleteOnGameThread;

	/** Timeout in seconds for the entire HTTP request to complete */
	TOptional<float> TimeoutSecs;

	/** Timeout in seconds for the HTTP request activity timeout */
	TOptional<float> ActivityTimeoutSecs;

	/** Indicate the request is timed out, it should quit and fail with EHttpFailureReason::TimedOut */
	std::atomic<bool> bTimedOut = false;
	/** Indicate the request is activity timed out, it should quit and fail with EHttpFailureReason::ConnectionError */
	std::atomic<bool> bActivityTimedOut = false;
	/** Indicate the request is cancelled, it should quit and fail with EHttpFailureReason::Cancelled */
	std::atomic<bool> bCanceled = false;

	/** TODO: Move this feature into CurlHttp */
	bool bUsePlatformActivityTimeout = true;

	/** Record when this request started */
	double RequestStartTimeAbsoluteSeconds;

	/** Record when this request will activity timeout */
	double ActivityTimeoutAt;

	/** Holder the timer handle, if the request get destroyed before triggering the timeout, use this to remove the timer */
	TSharedPtr<IHttpTaskTimerHandle> TotalTimeoutHttpTaskTimerHandle;
	/** Holder the timer handle, if the request get destroyed before triggering the timeout, use this to remove the timer */
	TSharedPtr<IHttpTaskTimerHandle> ActivityTimeoutHttpTaskTimerHandle;
	/** Critical section for accessing HttpTaskTimerHandle */
	FCriticalSection HttpTaskTimerHandleCriticalSection;

	/** Record when the request start to process */
	double StartProcessTime = 0.0;

	/** Record how long it take to connect to the endpoint */
	double ConnectTime = -1.0;

	FString URL;

	/** Cache the effective URL. When redirected, it will be different with original URL */
	FString EffectiveURL;

	/** The response object which we will use to pair with this request */
	TSharedPtr<FHttpResponseCommon> ResponseCommon;

	/** The stream to receive response body */
	TSharedPtr<FArchive> ResponseBodyReceiveStream;

	/** Critical section for accessing ResponseBodyReceiveStream */
	FCriticalSection ResponseBodyReceiveStreamCriticalSection;

	// Flag to indicate the request was initialized with stream. In that case even if stream was set to 
	// null later on internally, the request itself won't cache received data anymore
	std::atomic<bool> bInitializedWithValidStream = false;

	/** Mapping of header section to values. */
	TMap<FString, FString> RequestHeaders;

	/** Payload to use with the request. Typically for POST, PUT, or PATCH */
	TUniquePtr<FRequestPayload> RequestPayload;

	/** Options for this request. */
	TMap<const FName, FString> Options;

	/** Total elapsed time in seconds since the start of the request */
	float ElapsedTime = 0.0f;

	/** Record the time started to wait in the queue */
	double TimeStartedWaitingInQueue = 0.0;

	EHttpRequestPriority Priority = EHttpRequestPriority::Normal;
};
