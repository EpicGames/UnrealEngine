// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/HttpRequestImpl.h"
#include "Stats/Stats.h"
#include "Http.h"

FHttpRequestCompleteDelegate& FHttpRequestImpl::OnProcessRequestComplete()
{
	return RequestCompleteDelegate;
}

FHttpRequestProgressDelegate64& FHttpRequestImpl::OnRequestProgress64() 
{
	return RequestProgressDelegate64;
}

FHttpRequestStatusCodeReceivedDelegate& FHttpRequestImpl::OnStatusCodeReceived()
{
	return StatusCodeReceivedDelegate;
}

FHttpRequestHeaderReceivedDelegate& FHttpRequestImpl::OnHeaderReceived()
{
	return HeaderReceivedDelegate;
}

FHttpRequestWillRetryDelegate& FHttpRequestImpl::OnRequestWillRetry()
{
	return OnRequestWillRetryDelegate;
}

void FHttpRequestImpl::Shutdown()
{
	OnProcessRequestComplete().Unbind();
	OnRequestProgress64().Unbind();
	OnStatusCodeReceived().Unbind();
	OnHeaderReceived().Unbind();
}

void FHttpRequestImpl::BroadcastResponseHeadersReceived()
{
	if (OnHeaderReceived().IsBound())
	{
		const FHttpResponsePtr Response = GetResponse();
		if (Response.IsValid())
		{
			const FHttpRequestPtr ThisPtr(SharedThis(this));
			const TArray<FString> AllHeaders = Response->GetAllHeaders();
			for (const FString& Header : AllHeaders)
			{
				FString HeaderName;
				FString HeaderValue;
				if (Header.Split(TEXT(":"), &HeaderName, &HeaderValue))
				{
					HeaderValue.TrimStartInline();

					QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpRequestImpl_BroadcastResponseHeadersReceived_OnHeaderReceived);
					OnHeaderReceived().ExecuteIfBound(ThisPtr, HeaderName, HeaderValue);
				}
			}
		}
	}
}
