// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/HttpRequestPayload.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Http.h"

bool FGenericPlatformHttp::IsURLEncoded(const TArray<uint8>& Payload)
{
	static char AllowedChars[] = "ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789-_.~";
	static bool bTableFilled = false;
	static bool AllowedTable[256] = { false };

	if (!bTableFilled)
	{
		for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(AllowedChars) - 1; ++Idx)	// -1 to avoid trailing 0
		{
			uint8 AllowedCharIdx = static_cast<uint8>(AllowedChars[Idx]);
			check(AllowedCharIdx < UE_ARRAY_COUNT(AllowedTable));
			AllowedTable[AllowedCharIdx] = true;
		}

		bTableFilled = true;
	}

	const int32 Num = Payload.Num();
	for (int32 Idx = 0; Idx < Num; ++Idx)
	{
		if (!AllowedTable[Payload[Idx]])
			return false;
	}

	return true;
}

FRequestPayloadInFileStream::FRequestPayloadInFileStream(const FString& InFilename)
	: Filename(InFilename)
{
}

FRequestPayloadInFileStream::FRequestPayloadInFileStream(TSharedRef<FArchive> InFile, bool bInCloseWhenComplete) 
	: File(InFile)
	, bCloseWhenComplete(bInCloseWhenComplete)
{
}

FRequestPayloadInFileStream::~FRequestPayloadInFileStream()
{
}

uint64 FRequestPayloadInFileStream::GetContentLength() const
{
	return File ? File->TotalSize() : 0;
}

const TArray<uint8>& FRequestPayloadInFileStream::GetContent() const
{
	ensureMsgf(false, TEXT("GetContent() on a streaming request payload is not allowed"));
	static const TArray<uint8> NotSupported;
	return NotSupported;
}

bool FRequestPayloadInFileStream::IsURLEncoded() const
{
	// Assume that files are not URL encoded, because they probably aren't.
	// This implies that POST requests with streamed files will need the caller to set a Content-Type.
	return false;
}

size_t FRequestPayloadInFileStream::FillOutputBuffer(void* OutputBuffer, size_t MaxOutputBufferSize, size_t SizeAlreadySent)
{
	return FillOutputBuffer(TArrayView<uint8>(static_cast<uint8*>(OutputBuffer), MaxOutputBufferSize), SizeAlreadySent);
}

size_t FRequestPayloadInFileStream::FillOutputBuffer(TArrayView<uint8> OutputBuffer, size_t SizeAlreadySent)
{
	if (!File)
	{
		return 0;
	}

	const size_t ContentLength = GetContentLength();
	check(SizeAlreadySent <= ContentLength);
	const size_t SizeToSend = ContentLength - SizeAlreadySent;
	const size_t SizeToSendThisTime = FMath::Min(SizeToSend, static_cast<size_t>(OutputBuffer.Num()));
	if (SizeToSendThisTime != 0)
	{
		if (File->Tell() != SizeAlreadySent)
		{
			File->Seek(SizeAlreadySent);
		}
		File->Serialize(OutputBuffer.GetData(), static_cast<int64>(SizeToSendThisTime));
	}
	return SizeToSendThisTime;
}

bool FRequestPayloadInFileStream::Open()
{
	if (!File && !Filename.IsEmpty())
	{
		FArchive* RawFile = IFileManager::Get().CreateFileReader(*Filename);
		if (!RawFile)
		{
			UE_LOGF(LogHttp, Warning, "FRequestPayloadInFileStream::Open Failed to open %ls for reading", *Filename);
			return false;
		}

		File = MakeShareable(RawFile);
		bCloseWhenComplete = true;
	}

	return File != nullptr;
}

void FRequestPayloadInFileStream::Close()
{
	if (bCloseWhenComplete && File)
	{
		File->Close();
		File.Reset();
	}
}

FRequestPayloadInMemory::FRequestPayloadInMemory(const TArray<uint8>& Array) : Buffer(Array)
{
}

FRequestPayloadInMemory::FRequestPayloadInMemory(TArray<uint8>&& Array) : Buffer(MoveTemp(Array))
{
}

FRequestPayloadInMemory::~FRequestPayloadInMemory()
{
}

uint64 FRequestPayloadInMemory::GetContentLength() const
{
	return Buffer.Num();
}

const TArray<uint8>& FRequestPayloadInMemory::GetContent() const
{
	return Buffer;
}

bool FRequestPayloadInMemory::IsURLEncoded() const
{
	return FGenericPlatformHttp::IsURLEncoded(Buffer);
}

size_t FRequestPayloadInMemory::FillOutputBuffer(void* OutputBuffer, size_t MaxOutputBufferSize, size_t SizeAlreadySent)
{
	return FillOutputBuffer(TArrayView<uint8>(static_cast<uint8*>(OutputBuffer), MaxOutputBufferSize), SizeAlreadySent);
}

size_t FRequestPayloadInMemory::FillOutputBuffer(TArrayView<uint8> OutputBuffer, size_t SizeAlreadySent)
{
	const size_t ContentLength = static_cast<size_t>(Buffer.Num());
	check(SizeAlreadySent <= ContentLength);
	const size_t SizeToSend = ContentLength - SizeAlreadySent;
	const size_t SizeToSendThisTime = FMath::Min(SizeToSend, static_cast<size_t>(OutputBuffer.Num()));
	if (SizeToSendThisTime != 0)
	{
		FMemory::Memcpy(OutputBuffer.GetData(), Buffer.GetData() + SizeAlreadySent, SizeToSendThisTime);
	}
	return SizeToSendThisTime;
}

bool FRequestPayloadInMemory::Open()
{
	return true;
}

void FRequestPayloadInMemory::Close()
{
}
