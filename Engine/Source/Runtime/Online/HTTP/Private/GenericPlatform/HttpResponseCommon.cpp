// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/HttpResponseCommon.h"
#include "GenericPlatform/HttpRequestCommon.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "Http.h"

FHttpResponseCommon::FHttpResponseCommon(const FHttpRequestCommon& HttpRequest)
	: URL(HttpRequest.GetURL())
	, EffectiveURL(HttpRequest.GetEffectiveURL())
	, CompletionStatus(HttpRequest.GetStatus())
	, FailureReason(HttpRequest.GetFailureReason())
{
}

FString FHttpResponseCommon::GetURLParameter(const FString& ParameterName) const
{
	FString ReturnValue;
	if (TOptional<FString> OptionalParameterValue = FGenericPlatformHttp::GetUrlParameter(URL, ParameterName))
	{
		ReturnValue = MoveTemp(OptionalParameterValue.GetValue());
	}
	return ReturnValue;
}

const FString& FHttpResponseCommon::GetURL() const
{
	return URL;
}

const FString& FHttpResponseCommon::GetEffectiveURL() const
{
	return EffectiveURL;
}

void FHttpResponseCommon::SetRequestStatus(EHttpRequestStatus::Type InCompletionStatus)
{
	CompletionStatus = InCompletionStatus;
}

EHttpRequestStatus::Type FHttpResponseCommon::GetStatus() const
{
	return CompletionStatus;
}

void FHttpResponseCommon::SetRequestFailureReason(EHttpFailureReason InFailureReason)
{
	FailureReason = InFailureReason;
}

EHttpFailureReason FHttpResponseCommon::GetFailureReason() const
{
	return FailureReason;
}

void FHttpResponseCommon::SetEffectiveURL(const FString& InEffectiveURL)
{
	EffectiveURL = InEffectiveURL;
}

int32 FHttpResponseCommon::GetResponseCode() const
{
	return ResponseCode;
}

void FHttpResponseCommon::SetResponseCode(int32 InResponseCode)
{
	ResponseCode = InResponseCode;
}

FUtf8StringView FHttpResponseCommon::GetContentAsUtf8StringView() const
{
	return FUtf8StringView(reinterpret_cast<const UTF8CHAR*>(Payload.GetData()), Payload.Num());
}

FString FHttpResponseCommon::GetHeader(const FString& HeaderName) const
{
	FString Result;
	if (!bIsReady)
	{
		UE_LOGF(LogHttp, Warning, "Can't get cached header [%ls]. Response still processing. %ls", *HeaderName, *GetURL());
	}
	else
	{
		const FString* Header = Headers.Find(HeaderName);
		if (Header != NULL)
		{
			Result = *Header;
		}
	}
	return Result;
}

const TArray<uint8>& FHttpResponseCommon::GetContent() const
{
	if (!bIsReady)
	{
		UE_LOGF(LogHttp, Warning, "Payload is incomplete. Response still processing for %ls", *GetURL());
	}
	return Payload;
}

TArray<uint8> FHttpResponseCommon::TakeContent()
{
	if (!bIsReady)
	{
		UE_LOGF(LogHttp, Warning, "Payload is incomplete. Response still processing for %ls", *GetURL());
	}
	return MoveTemp(Payload);
}

FString FHttpResponseCommon::GetContentAsString() const
{
	// Content is NOT null-terminated; we need to specify lengths here
	FUTF8ToTCHAR TCHARData(reinterpret_cast<const ANSICHAR*>(Payload.GetData()), Payload.Num());
	return FString(TCHARData.Length(), TCHARData.Get());
}

void FHttpResponseCommon::AppendToPayload(const uint8* Ptr, int64 Size)
{
	Payload.Append(Ptr, Size);
}

