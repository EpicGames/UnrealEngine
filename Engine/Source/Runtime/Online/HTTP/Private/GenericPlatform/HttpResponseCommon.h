// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IHttpResponse.h"

class FHttpRequestCommon;

/**
 * Contains implementation of some common functions that don't vary between implementations of different platforms
 */
class FHttpResponseCommon : public IHttpResponse
{
	friend FHttpRequestCommon;

public:
	FHttpResponseCommon(const FHttpRequestCommon& HttpRequest);

	// IHttpBase
	virtual FString GetURLParameter(const FString& ParameterName) const override;
	virtual const FString& GetURL() const override;
	virtual const FString& GetEffectiveURL() const override;
	virtual EHttpRequestStatus::Type GetStatus() const override;
	virtual EHttpFailureReason GetFailureReason() const override;
	virtual int32 GetResponseCode() const override;
	virtual FUtf8StringView GetContentAsUtf8StringView() const override;
	virtual FString GetHeader(const FString& HeaderName) const override;
	virtual const TArray<uint8>& GetContent() const override;
	virtual TArray<uint8> TakeContent() override;
	virtual FString GetContentAsString() const override;

protected:
	void SetRequestStatus(EHttpRequestStatus::Type InCompletionStatus);
	void SetRequestFailureReason(EHttpFailureReason InFailureReason);
	void SetEffectiveURL(const FString& InEffectiveURL);
	void SetResponseCode(int32 InResponseCode);
	void AppendToPayload(const uint8* Ptr, int64 Size);

	FString URL;
	FString EffectiveURL;
	EHttpRequestStatus::Type CompletionStatus;
	EHttpFailureReason FailureReason;
	int32 ResponseCode = EHttpResponseCodes::Unknown;
	TArray<uint8> Payload;
	TMap<FString, FString> Headers;
	bool bIsReady = false;
};
