// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpConstants.h"

const TCHAR* const FHttpConstants::AUTH_BASIC{ TEXT("Basic") };
const TCHAR* const FHttpConstants::AUTH_BEARER{ TEXT("Bearer") };

const TCHAR* const FHttpConstants::VERB_DELETE{ TEXT("DELETE") };
const TCHAR* const FHttpConstants::VERB_GET{ TEXT("GET") };
const TCHAR* const FHttpConstants::VERB_HEAD{ TEXT("HEAD") };
const TCHAR* const FHttpConstants::VERB_PATCH{ TEXT("PATCH") };
const TCHAR* const FHttpConstants::VERB_POST{ TEXT("POST") };
const TCHAR* const FHttpConstants::VERB_PUT{ TEXT("PUT") };

const TCHAR* const FHttpConstants::HEADER_ACCEPT{ TEXT("Accept") };
const TCHAR* const FHttpConstants::HEADER_ACCEPT_ENCODING{ TEXT("Accept-Encoding") };
const TCHAR* const FHttpConstants::HEADER_AUTHORIZATION{ TEXT("Authorization") };
const TCHAR* const FHttpConstants::HEADER_CONTENT_LENGTH{ TEXT("Content-Length") };
const TCHAR* const FHttpConstants::HEADER_CONTENT_TYPE{ TEXT("Content-Type") };
const TCHAR* const FHttpConstants::HEADER_DATE{ TEXT("Date") };
const TCHAR* const FHttpConstants::HEADER_USER_AGENT{ TEXT("User-Agent") };

const TCHAR* const FHttpConstants::MEDIATYPE_JSON{ TEXT("application/json") };
const TCHAR* const FHttpConstants::MEDIATYPE_FORM_URLENCODED{ TEXT("application/x-www-form-urlencoded") };
const TCHAR* const FHttpConstants::MEDIATYPE_OCTET_STREAM{ TEXT("application/octet-stream") };

const TCHAR* const FHttpConstants::MEDIAENCODING_GZIP{ TEXT("gzip") };


const TCHAR* const FHttpConstants::VERSION_2TLS{ TEXT("2TLS") };
const TCHAR* const FHttpConstants::VERSION_1_1{ TEXT("1_1") };

