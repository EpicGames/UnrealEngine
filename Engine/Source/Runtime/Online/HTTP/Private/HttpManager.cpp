// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpManager.h"
#include "HttpModule.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"
#include "Http.h"
#include "Misc/App.h"
#include "Misc/Guid.h"
#include "Misc/Fork.h"
#include "HttpThread.h"
#include "GenericPlatform/HttpRequestCommon.h"
#include "Logging/StructuredLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include "Stats/Stats.h"
#include "Containers/BackgroundableTicker.h"

TAutoConsoleVariable<int32> CVarHttpEventLoopEnableChance(
	TEXT("http.CurlEventLoopEnableChance"),
	UE_HTTP_EVENT_LOOP_ENABLE_CHANCE_BY_DEFAULT,
	TEXT("Enable chance of event loop, from 0 to 100"),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<FString> CVarHttpUrlPatternsToInspect(
	TEXT("http.UrlPatternsToInspect"),
	TEXT(""),
	TEXT("List of url patterns to log headers and json content: \"epicgames.com unrealengine.com ...\""),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<FString> CVarHttpUrlPatternsToDisableFailedLog(
	TEXT("http.UrlPatternsToDisableFailedLog"),
	TEXT(""),
	TEXT("List of url patterns to disable failed log: \"epicgames.com unrealengine.com ...\"")
);

TAutoConsoleVariable<FString> CVarHttpUrlPatternsToMockResponse(
	TEXT("http.UrlPatternsToMockResponse"),
	TEXT(""),
	TEXT("List of url patterns to mock failure with response code, 0 indicates ConnectionError: \"epicgames.com->0 unrealengine.com->503 ...\""),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<bool> CVarHttpRemoveRequestUsingHttpThreadPolicyOnHttpThread(
	TEXT("http.RemoveRequestUsingHttpThreadPolicyOnHttpThread"),
	true,
	TEXT("When enabled, if http request use CompleteOnHttpThread, once complete it will be removed from http manager on http thread"),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<bool> CVarHttpSkipShutdownFlush(
	TEXT("http.SkipShutdownFlush"),
	false,
	TEXT("When enabled, shutdown does not call flush to avoid waiting past allowed shutdown timing")
);

TAutoConsoleVariable<bool> CVarHttpFlushTriggerFatalLogDuringCompleteDelegateEnabled(
	TEXT("http.FlushTriggerFatalLogDuringCompleteDelegateEnabled"),
	true,
	TEXT("When enabled, trigger fatal log if flush during http complete delegate")
);

// FHttpManager

const TCHAR* LexToString(const EHttpFlushReason& FlushReason)
{
	switch (FlushReason)
	{
	case EHttpFlushReason::Default:		return TEXT("Default");
	case EHttpFlushReason::Shutdown:	return TEXT("Shutdown");
	case EHttpFlushReason::FullFlush:	return TEXT("FullFlush");
	}

	checkNoEntry();
	return TEXT("Invalid");
}

namespace
{
	bool ShouldOutputHttpWarnings()
	{
		return !IsRunningCommandlet() && !FApp::IsUnattended();
	}
}

CSV_DEFINE_CATEGORY(HttpManager, true);
CSV_DEFINE_STAT(HttpManager, RequestsInQueue);
CSV_DEFINE_STAT(HttpManager, MaxRequestsInQueue);
CSV_DEFINE_STAT(HttpManager, RequestsInFlight);
CSV_DEFINE_STAT(HttpManager, MaxRequestsInFlight);
CSV_DEFINE_STAT(HttpManager, MaxTimeToWaitInQueue);
CSV_DEFINE_STAT(HttpManager, DownloadedMB);
CSV_DEFINE_STAT(HttpManager, BandwidthMbps);
CSV_DEFINE_STAT(HttpManager, DurationMsAvg);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FHttpManager::FHttpManager()
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
	, Thread(nullptr)
	, CorrelationIdMethod(FHttpManager::GetDefaultCorrelationIdMethod())
{
	bFlushing = false;
}

FHttpManager::~FHttpManager()
{
	if (Thread)
	{
		Thread->StopThread();
		delete Thread;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FHttpManager::Initialize()
{
	if (!Thread)
	{
		bUseEventLoop = (FMath::RandRange(0, 99) < CVarHttpEventLoopEnableChance.GetValueOnGameThread());

		// Also support to change it through runtime args.
		// Can't set cvar CVarHttpEventLoopEnableChance through runtime args or .ini files because http module initialized too early
		FParse::Bool(FCommandLine::Get(), TEXT("useeventloop="), bUseEventLoop);

		if (CVarHttpEventLoopEnableChance.GetValueOnGameThread() < 100 || !bUseEventLoop)
		{
			UE_LOGF(LogHttp, Error, "Legacy http thread has been deprecated, please always use event loop thread instead.");
		}

		Thread = CreateHttpThread();
		Thread->StartThread();
	}

	UpdateConfigs();

	UpdateUrlPatternsToInspect(CVarHttpUrlPatternsToInspect.AsVariable());
	CVarHttpUrlPatternsToInspect.AsVariable()->OnChangedDelegate().AddRaw(this, &FHttpManager::UpdateUrlPatternsToInspect);

	UpdateUrlPatternsToDisableFailedLog(CVarHttpUrlPatternsToDisableFailedLog.AsVariable());
	CVarHttpUrlPatternsToDisableFailedLog.AsVariable()->OnChangedDelegate().AddRaw(this, &FHttpManager::UpdateUrlPatternsToDisableFailedLog);

	UpdateUrlPatternsToMockResponse(CVarHttpUrlPatternsToMockResponse.AsVariable());
	CVarHttpUrlPatternsToMockResponse.AsVariable()->OnChangedDelegate().AddRaw(this, &FHttpManager::UpdateUrlPatternsToMockResponse);
}

void FHttpManager::Shutdown()
{
	CVarHttpUrlPatternsToInspect.AsVariable()->OnChangedDelegate().Clear();
	CVarHttpUrlPatternsToDisableFailedLog.AsVariable()->OnChangedDelegate().Clear();
	CVarHttpUrlPatternsToMockResponse.AsVariable()->OnChangedDelegate().Clear();

	{
		FScopeLock ScopeLock(&RequestLock);

		// Don't emit these tracking logs in commandlet runs. Build system traps warnings during cook, and these are not truly fatal, but useful for tracking down shutdown issues.
		UE_CLOGF(ShouldOutputHttpWarnings() && Requests.Num(), LogHttp, Warning, "[FHttpManager::Shutdown] Unbinding delegates for %d outstanding Http Requests:", Requests.Num());

		// Clear delegates since they may point to deleted instances
		for (TArray<FHttpRequestRef>::TIterator It(Requests); It; ++It)
		{
			TSharedPtr<IHttpRequest> Request = *It;
			StaticCastSharedPtr<FHttpRequestImpl>(Request)->Shutdown();

			// Don't emit these tracking logs in commandlet runs. Build system traps warnings during cook, and these are not truly fatal, but useful for tracking down shutdown issues.
			UE_CLOGF(ShouldOutputHttpWarnings(), LogHttp, Warning, "	verb=[%ls] url=[%ls] refs=[%d] status=%ls", *Request->GetVerb(), *Request->GetURL(), Request.GetSharedReferenceCount(), EHttpRequestStatus::ToString(Request->GetStatus()));
		}
	}

	// Clear general delegates since they may point to deleted instances
	RequestAddedDelegate.Unbind();
	RequestCompletedDelegate.Unbind();

	// Flush all requests
	if (!CVarHttpSkipShutdownFlush.GetValueOnAnyThread())
	{
		Flush(EHttpFlushReason::Shutdown);
	}
}

bool FHttpManager::HasAnyBoundDelegate() const
{
	FScopeLock ScopeLock(&RequestLock);

	for (TArray<FHttpRequestRef>::TConstIterator It(Requests); It; ++It)
	{
		const FHttpRequestRef& Request = *It;
		if (Request->OnProcessRequestComplete().IsBound())
		{
			return true;
		}
	}

	if (RequestAddedDelegate.IsBound())
	{
		return true;
	}

	if (RequestCompletedDelegate.IsBound())
	{
		return true;
	}

	return false;
}

void FHttpManager::ReloadFlushTimeLimits()
{
	FlushTimeLimitsMap.Reset();

	//Save int values of Default and FullFlush?
	for (EHttpFlushReason Reason : TEnumRange<EHttpFlushReason>())
	{
		double SoftLimitSeconds = 2.0;
		double HardLimitSeconds = 4.0;

		// We default the time limits to generous values, keeping the Hard limits always greater than the soft ones, and -1 for the unlimited
		switch (Reason)
		{
		case EHttpFlushReason::Default:
			GConfig->GetDouble(TEXT("HTTP"), TEXT("FlushSoftTimeLimitDefault"), SoftLimitSeconds, GEngineIni);
			GConfig->GetDouble(TEXT("HTTP"), TEXT("FlushHardTimeLimitDefault"), HardLimitSeconds, GEngineIni);
			break;
		case EHttpFlushReason::Shutdown:
			GConfig->GetDouble(TEXT("HTTP"), TEXT("FlushSoftTimeLimitShutdown"), SoftLimitSeconds, GEngineIni);
			GConfig->GetDouble(TEXT("HTTP"), TEXT("FlushHardTimeLimitShutdown"), HardLimitSeconds, GEngineIni);
			
			if ((HardLimitSeconds >= 0) && ((SoftLimitSeconds < 0) || (SoftLimitSeconds >= HardLimitSeconds)))
			{
				UE_CLOGF(!IsRunningCommandlet(), LogHttp, Warning, "Soft limit[%.02f] is higher than the hard limit set[%.02f] in file [%ls]. Please change the soft limit to a value lower than the hard limit for Flush to work correctly. - 1 is unlimited and therefore the highest possible value.", static_cast<float>(SoftLimitSeconds), static_cast<float>(HardLimitSeconds), *GEngineIni);
				// we need to be absolutely sure that SoftLimitSeconds is always strictly less than HardLimitSeconds so remaining requests (if any) can be canceled before exiting
				if (HardLimitSeconds > 0.0)
				{
					SoftLimitSeconds = HardLimitSeconds / 2.0;	// clamping SoftLimitSeconds to a reasonable value
				}
				else
				{
					// HardLimitSeconds should never be 0.0 while shutting down otherwise we can't cancel the remaining requests
					HardLimitSeconds = 0.05;	// using a non zero value 
					SoftLimitSeconds = 0.0;		// cancelling request immediately
				}
			}

			break;
		case EHttpFlushReason::FullFlush:
			SoftLimitSeconds = -1.0;
			HardLimitSeconds = -1.0;
			GConfig->GetDouble(TEXT("HTTP"), TEXT("FlushSoftTimeLimitFullFlush"), SoftLimitSeconds, GEngineIni);
			GConfig->GetDouble(TEXT("HTTP"), TEXT("FlushHardTimeLimitFullFlush"), HardLimitSeconds, GEngineIni);
			break;
		}

		FHttpFlushTimeLimit TimeLimit(SoftLimitSeconds, HardLimitSeconds);

		FlushTimeLimitsMap.Add(Reason, TimeLimit);
	}
}

void FHttpManager::SetCorrelationIdMethod(TFunction<FString()> InCorrelationIdMethod)
{
	check(InCorrelationIdMethod);
	CorrelationIdMethod = MoveTemp(InCorrelationIdMethod);
}

FString FHttpManager::CreateCorrelationId() const
{
	return CorrelationIdMethod();
}

#if !UE_BUILD_SHIPPING
namespace HttpManagerInternal
{

static void CheckNoDuplicatedProtocol(FStringView UrlStringView)
{
	const FStringView Delimiter = TEXT("://");
	const int32 FoundIndex = UE::String::FindFirst(UrlStringView, Delimiter);
	if (FoundIndex != INDEX_NONE)
	{
		FStringView Domain = UrlStringView.RightChop(FoundIndex + Delimiter.Len());
		UE_CLOGF(Domain.Contains(Delimiter), LogHttp, Warning, "Duplicated delimiter '://' found in url %.*ls, is that intended?", UrlStringView.Len(), UrlStringView.GetData());
	}
}

}
#endif // !UE_BUILD_SHIPPING

bool FHttpManager::IsDomainAllowed(const FString& Url) const
{
#if !UE_BUILD_SHIPPING
	HttpManagerInternal::CheckNoDuplicatedProtocol(Url);
#endif // !UE_BUILD_SHIPPING

	return URLRequestFilter.IsRequestAllowed(Url);
}

/*static*/
TFunction<FString()> FHttpManager::GetDefaultCorrelationIdMethod()
{
	return []{ return FGuid::NewGuid().ToString(); };
}

void FHttpManager::OnBeforeFork()
{
	Flush(EHttpFlushReason::Default);
}

void FHttpManager::OnAfterFork()
{

}

void FHttpManager::OnEndFramePostFork()
{
	// nothing
}


void FHttpManager::UpdateConfigs()
{
	URLRequestFilter.UpdateConfig(TEXT("Online.HttpManager"), GEngineIni);

	ReloadFlushTimeLimits();

	if (Thread)
	{
		Thread->UpdateConfigs();
	}
}

void FHttpManager::AddGameThreadTask(TFunction<void()>&& Task, float Delay)
{
	if (Task)
	{
		GameThreadTicker.AddTicker(FTickerDelegate::CreateLambda([Task](float DeltaTime) {
			Task();
			return false;
		}), Delay);
	}
}

TSharedPtr<IHttpTaskTimerHandle> FHttpManager::AddHttpThreadTask(TFunction<void()>&& Task, float InDelay)
{
	check(Thread);
	return Thread->AddHttpThreadTask(MoveTemp(Task), InDelay);
}

void FHttpManager::RemoveHttpThreadTask(TSharedPtr<IHttpTaskTimerHandle> HttpTaskTimerHandle)
{
	check(Thread);
	HttpTaskTimerHandle->RemoveTaskFrom(Thread);
}

FHttpThreadBase* FHttpManager::CreateHttpThread()
{
	return new FLegacyHttpThread();
}

void FHttpManager::Flush(EHttpFlushReason FlushReason)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpManager_Flush);

	if (IsCurrentThreadCompletingRequest())
	{
		UE_CLOGFMT(CVarHttpFlushTriggerFatalLogDuringCompleteDelegateEnabled.GetValueOnAnyThread(), LogHttp, Fatal, "It's not supported to call Flush during http request complete delegate!");
		return;
	}

	checkf(FlushReason != EHttpFlushReason::Shutdown || !HasAnyBoundDelegate(), TEXT("Use Shutdown() instead of Flush(EHttpFlushReason::Shutdown) directly."));
	
	// This variable is set to indicate that flush is happening.
	// While flushing is in progress, the RequestLock is held and threads are blocked when trying to submit new requests.
	bFlushing = true;

	double FlushTimeSoftLimitSeconds = FlushTimeLimitsMap[FlushReason].SoftLimitSeconds;
	double FlushTimeHardLimitSeconds = FlushTimeLimitsMap[FlushReason].HardLimitSeconds;

	// this specifies how long to sleep between calls to tick.
	// The smaller the value, the more quickly we may find out that all requests have completed, but the more work may be done in the meantime.
	float SecondsToSleepForOutstandingThreadedRequests = 0.5f;
	GConfig->GetFloat(TEXT("HTTP"), TEXT("RequestCleanupDelaySec"), SecondsToSleepForOutstandingThreadedRequests, GEngineIni);

	UE_CLOGF(!IsRunningCommandlet(), LogHttp, Verbose, "[FHttpManager::Flush] FlushReason [%ls] FlushTimeSoftLimitSeconds [%.3fs] FlushTimeHardLimitSeconds [%.3fs] SecondsToSleepForOutstandingThreadedRequests [%.3fs]", LexToString(FlushReason), FlushTimeSoftLimitSeconds, FlushTimeHardLimitSeconds, SecondsToSleepForOutstandingThreadedRequests);

	uint32 RequestsNum = 0;

	{
		FScopeLock ScopeLock(&RequestLock);
		RequestsNum = Requests.Num();
	}

	UE_CLOGF(!IsRunningCommandlet() && RequestsNum, LogHttp, Verbose, "[FHttpManager::Flush] Cleanup starts for %d outstanding Http Requests.", RequestsNum);

	double BeginWaitTime = FPlatformTime::Seconds();
	double LastFlushTickTime = BeginWaitTime;
	double StallWarnTime = BeginWaitTime + 0.5;
	double AppTime = FPlatformTime::Seconds();


	// For a duration equal to FlushTimeHardLimitSeconds, we wait for ongoing http requests to complete
	while (RequestsNum > 0 && (FlushTimeHardLimitSeconds < 0 || (AppTime - BeginWaitTime < FlushTimeHardLimitSeconds)))
	{
		SCOPED_ENTER_BACKGROUND_EVENT(STAT_FHttpManager_Flush_Iteration);

		// If time equal to FlushTimeSoftLimitSeconds has passed and there's still ongoing http requests, we cancel them (setting FlushTimeSoftLimitSeconds to 0 does this immediately)
		if (FlushTimeSoftLimitSeconds >= 0 && (AppTime - BeginWaitTime >= FlushTimeSoftLimitSeconds))
		{
			// Don't emit these tracking logs in commandlet runs. Build system traps warnings during cook, and these are not truly fatal, but useful for tracking down shutdown issues.
			UE_CLOGF(ShouldOutputHttpWarnings(), LogHttp, Warning, "[FHttpManager::Flush] FlushTimeSoftLimitSeconds [%.3fs] exceeded. Cancelling %d outstanding HTTP requests:", FlushTimeSoftLimitSeconds, RequestsNum);

			{
				TArray<FHttpRequestRef> RequestsToCancel;

				{
					FScopeLock ScopeLock(&RequestLock);
					RequestsToCancel = Requests;
				}

				for (TArray<FHttpRequestRef>::TIterator It(RequestsToCancel); It; ++It)
				{
					FHttpRequestRef& Request = *It;

					// Don't emit these tracking logs in commandlet runs. Build system traps warnings during cook, and these are not truly fatal, but useful for tracking down shutdown issues.
					UE_CLOGF(ShouldOutputHttpWarnings(), LogHttp, Warning, "	verb=[%ls] url=[%ls] refs=[%d] status=%ls", *Request->GetVerb(), *Request->GetURL(), Request.GetSharedReferenceCount(), EHttpRequestStatus::ToString(Request->GetStatus()));

					FScopedEnterBackgroundEvent(*Request->GetURL());

					Request->CancelRequest();
				}
			}
		}

		// Process ongoing Http Requests
		FlushTick(AppTime - LastFlushTickTime);
		LastFlushTickTime = AppTime;

		{
			FScopeLock ScopeLock(&RequestLock);
			RequestsNum = Requests.Num();
		}

		// Process threaded Http Requests
		if (RequestsNum > 0)
		{
			if (Thread)
			{
				if (Thread->NeedsSingleThreadTick())
				{
					if (AppTime >= StallWarnTime)
					{
						// Don't emit these tracking logs in commandlet runs. Build system traps warnings during cook, and these are not truly fatal, but useful for tracking down shutdown issues.
						UE_CLOGF(ShouldOutputHttpWarnings(), LogHttp, Warning, "	Ticking HTTPThread for %d outstanding Http requests.", RequestsNum);
						StallWarnTime = AppTime + 0.5;
					}
					Thread->Tick();
				}
				else
				{
					// Don't emit these tracking logs in commandlet runs. Build system traps warnings during cook, and these are not truly fatal, but useful for tracking down shutdown issues.
					UE_CLOGF(ShouldOutputHttpWarnings(), LogHttp, Warning, "	Sleeping %.3fs to wait for %d outstanding Http Requests.", SecondsToSleepForOutstandingThreadedRequests, RequestsNum);
					FPlatformProcess::Sleep(SecondsToSleepForOutstandingThreadedRequests);
				}
			}
		}

		AppTime = FPlatformTime::Seconds();
	}

	UE_CLOGF(!IsRunningCommandlet(), LogHttp, Verbose, "[FHttpManager::Flush] Cleanup ended after %.3fs. %d outstanding Http Requests.", AppTime - BeginWaitTime, RequestsNum);

	// Don't emit these tracking logs in commandlet runs. Build system traps warnings during cook, and these are not truly fatal, but useful for tracking down shutdown issues.
	if (RequestsNum > 0 && (FlushTimeHardLimitSeconds > 0 && (AppTime - BeginWaitTime > FlushTimeHardLimitSeconds)) && ShouldOutputHttpWarnings())
	{
		UE_LOGF(LogHttp, Warning, "[FHttpManager::Flush] FlushTimeHardLimitSeconds [%.3fs] exceeded. The following requests are being abandoned without being flushed:", FlushTimeHardLimitSeconds);

		FScopeLock ScopeLock(&RequestLock);

		for (TArray<FHttpRequestRef>::TIterator It(Requests); It; ++It)
		{
			FHttpRequestRef& Request = *It;
			//List the outstanding requests that are being abandoned without being canceled.
			UE_LOGF(LogHttp, Warning, "	verb=[%ls] url=[%ls] refs=[%d] status=%ls", *Request->GetVerb(), *Request->GetURL(), Request.GetSharedReferenceCount(), EHttpRequestStatus::ToString(Request->GetStatus()));
		}
	}

	bFlushing = false;
}

bool FHttpManager::Tick(float DeltaSeconds)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpManager_Tick);

	// Normally Tick() should only be called from game thread. But it's still possible Tick() be called 
	// from off-game thread when quit in purpose like GPU OOM, to flush remain HTTP analysis requests

	// Run GameThread tasks
	{
		FScopeLock ScopeLock(&GameThreadTickerLock);
		GameThreadTicker.Tick(DeltaSeconds);
	}

	if (Thread)
	{
		{
			// Tick each active request
			FScopeLock ScopeLock(&RequestLock);
			for (const FHttpRequestRef& Request : Requests)
			{
				Request->Tick(DeltaSeconds);
			}
		}

		TArray<FHttpRequestCommon*> CompletedThreadedRequests;

		{
			// Thread->GetCompletedRequests doesn't support multi-thread access
			FScopeLock ScopeLock(&CompletedRequestLock);
			Thread->GetCompletedRequests(CompletedThreadedRequests);
		}

		// Finish and remove any completed requests
		for (FHttpRequestCommon* CompletedRequest : CompletedThreadedRequests)
		{
			FHttpRequestRef CompletedRequestRef = CompletedRequest->AsShared();

			if (!CVarHttpRemoveRequestUsingHttpThreadPolicyOnHttpThread.GetValueOnAnyThread() || CompletedRequest->GetDelegateThreadPolicy() == EHttpRequestDelegateThreadPolicy::CompleteOnGameThread)
			{
				FScopeLock ScopeLock(&RequestLock);
				Requests.Remove(CompletedRequestRef);
			}

			if (CompletedRequest->GetDelegateThreadPolicy() == EHttpRequestDelegateThreadPolicy::CompleteOnGameThread)
			{
				MarkCurrentThreadCompletingRequest(true);
				CompletedRequest->FinishRequest();
				// Can't move BroadcastHttpRequestCompleted to CompletedRequest->FinishRequest because 
				// FinishRequest can be called before adding into HttpManager when pre-check failed
				BroadcastHttpRequestCompleted(CompletedRequestRef);
				MarkCurrentThreadCompletingRequest(false);
			}
		}
	}
	
	// Report csv stats.
	int32 TotalDownloadedMB = int32(HttpStats.TotalDownloadedBytes.load() >> 20);
	CSV_CUSTOM_STAT_DEFINED(RequestsInQueue, HttpStats.RequestsInQueue.load(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(MaxRequestsInQueue, int32(HttpStats.MaxRequestsInQueue.load()), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(RequestsInFlight, HttpStats.RequestsInFlight.load(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(MaxRequestsInFlight, int32(HttpStats.MaxRequestsInFlight.load()), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(MaxTimeToWaitInQueue, int32(HttpStats.MaxTimeToWaitInQueue.load()), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(DownloadedMB, TotalDownloadedMB, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(BandwidthMbps, int32(HttpStats.BandwidthMbps.load()), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_DEFINED(DurationMsAvg, int32(HttpStats.HttpDurationMsAvg.load()), ECsvCustomStatOp::Set);

	// keep ticking
	return true;
}

void FHttpManager::FlushTick(float DeltaSeconds)
{
	Tick(DeltaSeconds);
}

void FHttpManager::AddRequest(const FHttpRequestRef& Request)
{
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpManager_AddRequest);
		FScopeLock ScopeLock(&RequestLock);
		UE_CLOGF(bFlushing, LogHttp, Warning, "Adding request %ls to http manager while flushing", *Request->GetURL());
		Requests.Add(Request);
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpManager_RequestAddedDelegate);
	RequestAddedDelegate.ExecuteIfBound(Request);
}

void FHttpManager::RemoveRequest(const FHttpRequestRef& Request)
{
	FScopeLock ScopeLock(&RequestLock);

	Requests.Remove(Request);
}

void FHttpManager::AddThreadedRequest(const TSharedRef<FHttpRequestCommon, ESPMode::ThreadSafe>& Request)
{
	check(Thread);
	{
		AddRequest(Request);
	}
	Thread->AddRequest(&Request.Get());
}

void FHttpManager::CancelThreadedRequest(const TSharedRef<FHttpRequestCommon, ESPMode::ThreadSafe>& Request)
{
	check(Thread);
	Thread->CancelRequest(&Request.Get());
}

bool FHttpManager::IsValidRequest(const IHttpRequest* RequestPtr) const
{
	FScopeLock ScopeLock(&RequestLock);

	bool bResult = false;
	for (const FHttpRequestRef& Request : Requests)
	{
		if (&Request.Get() == RequestPtr)
		{
			bResult = true;
			break;
		}
	}

	return bResult;
}

void FHttpManager::SetRequestAddedDelegate(const FHttpManagerRequestAddedDelegate& Delegate)
{
	RequestAddedDelegate = Delegate;
}

void FHttpManager::SetRequestCompletedDelegate(const FHttpManagerRequestCompletedDelegate& Delegate)
{
	RequestCompletedDelegate = Delegate;
}

void FHttpManager::DumpRequests(FOutputDevice& Ar) const
{
	FScopeLock ScopeLock(&RequestLock);

	Ar.Logf(TEXT("------- (%d) Http Requests"), Requests.Num());
	for (const FHttpRequestRef& Request : Requests)
	{
		Ar.Logf(TEXT("	verb=[%s] url=[%s] status=%s"),
			*Request->GetVerb(), *Request->GetURL(), EHttpRequestStatus::ToString(Request->GetStatus()));
	}
}

bool FHttpManager::SupportsDynamicProxy() const
{
	return false;
}

void FHttpManager::BroadcastHttpRequestCompleted(const FHttpRequestRef& Request)
{
	FHttpResponsePtr Response = Request->GetResponse();
	if (Response.IsValid())
	{
		const int64 OldDuration = HttpStatsHistory.DurationMs[HttpStatsHistory.HistoryIndex];
		const int64 NewDuration = FMath::RoundToInt64(Request->GetElapsedTime() * 1000.0f);

		HttpStatsHistory.TotalDuration -= OldDuration;
		HttpStatsHistory.TotalDuration += NewDuration;
		HttpStatsHistory.DurationMs[HttpStatsHistory.HistoryIndex] = NewDuration;

		const int64 SizeBytes = Response->GetContentLength();
		HttpStats.TotalDownloadedBytes += SizeBytes;

		HttpStatsHistory.TotalDownloadedBytes -= HttpStatsHistory.DownloadedBytes[HttpStatsHistory.HistoryIndex];
		HttpStatsHistory.TotalDownloadedBytes += SizeBytes;
		HttpStatsHistory.DownloadedBytes[HttpStatsHistory.HistoryIndex] = SizeBytes;

		const int64 TotalDuration = FMath::Max<int64>(0, HttpStatsHistory.TotalDuration);
		const int64 DurationDivisor = TotalDuration + 1;
		HttpStats.BandwidthMbps = (HttpStatsHistory.TotalDownloadedBytes * 8) / DurationDivisor / 1000;
		if (FHttpStatsHistory::HttpHistoryCount > 0)
		{
			HttpStats.HttpDurationMsAvg = TotalDuration / FHttpStatsHistory::HttpHistoryCount;
			// Increment index
			HttpStatsHistory.HistoryIndex = (HttpStatsHistory.HistoryIndex + 1) % FHttpStatsHistory::HttpHistoryCount;
		}
		else
		{
			HttpStats.HttpDurationMsAvg = 0;
		}
	}
		
	RequestCompletedDelegate.ExecuteIfBound(Request);
}

FHttpThreadBase* FHttpManager::GetThread()
{
	return Thread;
}

void FHttpManager::RecordStatTimeToConnect(float Duration)
{
	HttpStats.MaxTimeToConnect = FGenericPlatformMath::Max(Duration, HttpStats.MaxTimeToConnect.load());
}

void FHttpManager::RecordStatRequestsInFlight(uint32 RequestsInFlight)
{
	HttpStats.RequestsInFlight = RequestsInFlight;
	HttpStats.MaxRequestsInFlight = FGenericPlatformMath::Max(RequestsInFlight, HttpStats.MaxRequestsInFlight.load());
}

void FHttpManager::RecordStatRequestsInQueue(uint32 RequestsInQueue)
{
	HttpStats.RequestsInQueue = RequestsInQueue;
	HttpStats.MaxRequestsInQueue = FGenericPlatformMath::Max(RequestsInQueue, HttpStats.MaxRequestsInQueue.load());
}

void FHttpManager::RecordMaxTimeToWaitInQueue(float Duration)
{
	HttpStats.MaxTimeToWaitInQueue = FGenericPlatformMath::Max(Duration, HttpStats.MaxTimeToWaitInQueue.load());
}

void FHttpManager::RecordStatRequestSucceeded()
{
	++HttpStats.RequestResults[static_cast<uint8>(EHttpFailureReason::None)];
}

void FHttpManager::RecordStatRequestFailed(EHttpFailureReason Reason)
{
	const uint8 Index = static_cast<uint8>(Reason);
	if (Index < static_cast<uint8>(EHttpFailureReason::Count))
	{
		++HttpStats.RequestResults[Index];
	}
}

void FHttpManager::RecordPlatformStats(const FHttpStatsPlatform& PlatformStats)
{
	HttpStats.PlatformStats = PlatformStats;
}

void FHttpManager::UpdateUrlPatternsToInspect(IConsoleVariable* CVar)
{
	const FScopeLock CacheLock(&UrlPatternsToInspectCriticalSection);
	const FString UrlPatternsToInspectStr = CVar->AsVariable()->GetString();
	UrlPatternsToInspectStr.ParseIntoArray(UrlPatternsToInspect, TEXT(" "));
}

bool FHttpManager::ShouldInspect(FStringView Url) const
{
	const FScopeLock CacheLock(&UrlPatternsToInspectCriticalSection);
	for (const FString& UrlPatternToInspect : UrlPatternsToInspect)
	{
		if (Url.Contains(UrlPatternToInspect))
		{
			return true;
		}
	}

	return false;
}

void FHttpManager::UpdateUrlPatternsToDisableFailedLog(IConsoleVariable* CVar)
{
	const FScopeLock CacheLock(&UrlPatternsToDisableFailedLogCriticalSection);
	const FString UrlPatternsToDisableFailedLogStr = CVar->AsVariable()->GetString();
	UrlPatternsToDisableFailedLogStr.ParseIntoArray(UrlPatternsToDisableFailedLog, TEXT(" "));
}

bool FHttpManager::ShouldDisableFailedLog(FStringView Url) const
{
	const FScopeLock CacheLock(&UrlPatternsToDisableFailedLogCriticalSection);
	for (const FString& UrlPatternToDisableFailedLog : UrlPatternsToDisableFailedLog)
	{
		if (Url.Contains(UrlPatternToDisableFailedLog))
		{
			return true;
		}
	}

	return false;
}

void FHttpManager::UpdateUrlPatternsToMockResponse(IConsoleVariable* CVar)
{
	const FScopeLock CacheLock(&UrlPatternsToMockResponseCriticalSection);
	const FString UrlPatternsToMockResponseStr = CVar->AsVariable()->GetString();

	TArray<FString> UrlPatternsToMockResponseStrings;
	UrlPatternsToMockResponseStr.ParseIntoArray(UrlPatternsToMockResponseStrings, TEXT(" "));

	UrlPatternsToMockResponse.Empty();

	for (const FString& UrlPatternToMockResponseString : UrlPatternsToMockResponseStrings)
	{
		// Example runtime args(NOTE: It can't contain space in body of each url pattern): 
		//  -ini:Engine:[ConsoleVariables]:http.UrlPatternsToMockResponse="www.mydomain.com/mypath++POST->403++{\"friendly_error\":\"online.presence.restricted\",\"error_code\":3456}++Content-Type:application/json"
		TArray<FString> UrlPattern;
		UrlPatternToMockResponseString.ParseIntoArray(UrlPattern, TEXT("->"));
		const FStringView PayloadSeparator = TEXT("++");

		if (UrlPattern.Num() == 2)
		{
			FMockResponse MockResponse;

			FString MockResponseStr = UrlPattern[1];
			const int32 PayloadSeparatorIndex = UE::String::FindFirst(MockResponseStr, PayloadSeparator);
			if (PayloadSeparatorIndex != INDEX_NONE)
			{
				MockResponse.StatusCode = FCString::Atoi(*MockResponseStr.Left(PayloadSeparatorIndex));
				FString ResponsePayload = MockResponseStr.RightChop(PayloadSeparatorIndex + PayloadSeparator.Len());
				const int32 HeaderSeparatorIndex = UE::String::FindFirst(ResponsePayload, PayloadSeparator);
				if (HeaderSeparatorIndex != INDEX_NONE)
				{
					FString HeadersStr = ResponsePayload.RightChop(HeaderSeparatorIndex + PayloadSeparator.Len());
					ResponsePayload = ResponsePayload.Left(HeaderSeparatorIndex);

					TArray<FString> ResponseHeaderStrings;
					HeadersStr.ParseIntoArray(ResponseHeaderStrings, TEXT(","));
					for (const FString& ResponseHeaderString : ResponseHeaderStrings)
					{
						FString HeaderKey;
						FString HeaderValue;
						if (ResponseHeaderString.Split(TEXT(":"), &HeaderKey, &HeaderValue))
						{
							MockResponse.ResponseHeaders.Emplace(MoveTemp(HeaderKey), MoveTemp(HeaderValue));
						}
					}
				}

				MockResponse.ResponsePayload = StringCast<UTF8CHAR>(*ResponsePayload);
			}
			else
			{
				MockResponse.StatusCode = FCString::Atoi(*MockResponseStr);
			}

			TArray<FString> UrlInfo;
			UrlPattern[0].ParseIntoArray(UrlInfo, PayloadSeparator.GetData());

			FUrlToMatch UrlToMatch;
			UrlToMatch.Str = UrlInfo[0];
			if (UrlInfo.Num() > 1)
			{
				UrlToMatch.Verb = UrlInfo[1];
			}
			UrlPatternsToMockResponse.Emplace(UrlToMatch, MoveTemp(MockResponse));
		}
	}
}

TOptional<FHttpManager::FMockResponse> FHttpManager::GetMockResponse(FStringView Url, FStringView Verb) const
{
	TOptional<FMockResponse> Result;
	const FScopeLock CacheLock(&UrlPatternsToMockResponseCriticalSection);

	for (const TPair<FUrlToMatch, FMockResponse>& UrlPatternPair : UrlPatternsToMockResponse)
	{
		const FUrlToMatch& UrlToMatch = UrlPatternPair.Key;
		if (Url.Contains(UrlToMatch.Str) && (UrlToMatch.Verb.IsEmpty() || UrlToMatch.Verb.Compare(FString(Verb), ESearchCase::IgnoreCase) == 0))
		{
			Result = UrlPatternPair.Value;
			break;
		}
	}

	return Result;
}

void FHttpManager::MarkCurrentThreadCompletingRequest(bool bCompleting)
{
#if !UE_BUILD_SHIPPING
	const FScopeLock CacheLock(&ThreadsCompletingRequestCriticalSection);
	bool& ThreadMark = ThreadsCompletingRequest.FindOrAdd(FPlatformTLS::GetCurrentThreadId());
	ThreadMark = bCompleting;
#endif
}

bool FHttpManager::IsCurrentThreadCompletingRequest() const
{
#if !UE_BUILD_SHIPPING
	const FScopeLock CacheLock(&ThreadsCompletingRequestCriticalSection);
	const bool* ThreadMark = ThreadsCompletingRequest.Find(FPlatformTLS::GetCurrentThreadId());
	if (ThreadMark == nullptr)
	{
		return false;
	}

	return *ThreadMark;
#else
	return false;
#endif
}
