// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "HttpManager.h"
#include "Http.h"
#include "NullHttp.h"
#include "TransactionallySafeHttpRequest.h"
#include "HttpTests.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"

// TODO: Create a public common header, similar to BSDSocketTypesPrivate.h to support more platforms
#if UE_HTTP_SOCKET_TEST_COMMAND_ENABLED
#include "Logging/StructuredLog.h"

#include <unistd.h>
#include <sys/socket.h>
#include <netinet/in.h>
#include <dirent.h>
#include <arpa/inet.h>
#endif

#if !UE_BUILD_SHIPPING
#include "HAL/IConsoleManager.h"
#endif

#if !UE_BUILD_SHIPPING
TAutoConsoleVariable<bool> CVarHttpInsecureProtocolEnabled(
	TEXT("Http.InsecureProtocolEnabled"),
	false,
	TEXT("Enable insecure http protocol")
);
#endif

DEFINE_LOG_CATEGORY(LogHttp);

// FHttpModule

IMPLEMENT_MODULE(FHttpModule, HTTP);

FHttpModule* FHttpModule::Singleton = nullptr;
bool FHttpModule::bWasEverLoaded = false;

static bool ShouldLaunchUrl(const TCHAR* Url)
{
	FString SchemeName;
	if (FParse::SchemeNameFromURI(Url, SchemeName) && (SchemeName == TEXT("http") || SchemeName == TEXT("https")))
	{
		FHttpManager& HttpManager = FHttpModule::Get().GetHttpManager();
		return HttpManager.IsDomainAllowed(Url);
	}

	return true;
}

void FHttpModule::UpdateConfigs()
{
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpTimeout"), HttpActivityTimeout, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpTotalTimeout"), HttpTotalTimeout, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpConnectionTimeout"), HttpConnectionTimeout, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpActivityTimeout"), HttpActivityTimeout, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpReceiveTimeout"), HttpReceiveTimeout, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpSendTimeout"), HttpSendTimeout, GEngineIni);
	GConfig->GetInt(TEXT("HTTP"), TEXT("HttpMaxConnectionsPerServer"), HttpMaxConnectionsPerServer, GEngineIni);
	GConfig->GetBool(TEXT("HTTP"), TEXT("bEnableHttp"), bEnableHttp, GEngineIni);
	GConfig->GetBool(TEXT("HTTP"), TEXT("bUseNullHttp"), bUseNullHttp, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpDelayTime"), HttpDelayTime, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpThreadActiveFrameTimeInSeconds"), HttpThreadActiveFrameTimeInSeconds, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpThreadActiveMinimumSleepTimeInSeconds"), HttpThreadActiveMinimumSleepTimeInSeconds, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpThreadIdleFrameTimeInSeconds"), HttpThreadIdleFrameTimeInSeconds, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpThreadIdleMinimumSleepTimeInSeconds"), HttpThreadIdleMinimumSleepTimeInSeconds, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpEventLoopThreadTickIntervalInSeconds"), HttpEventLoopThreadTickIntervalInSeconds, GEngineIni);

	if (!FParse::Value(FCommandLine::Get(), TEXT("HttpNoProxy="), HttpNoProxy))
	{
		GConfig->GetString(TEXT("HTTP"), TEXT("HttpNoProxy"), HttpNoProxy, GEngineIni);
	}

	AllowedDomains.Empty();
	GConfig->GetArray(TEXT("HTTP"), TEXT("AllowedDomains"), AllowedDomains, GEngineIni);

	if (HttpManager != nullptr)
	{
		HttpManager->UpdateConfigs();
	}
}

void FHttpModule::StartupModule()
{
	Singleton = this;
	bWasEverLoaded = true;

	MaxReadBufferSize = 256 * 1024;
	HttpTotalTimeout = 0.0f;
	HttpConnectionTimeout = 30.0f;
	HttpActivityTimeout = 30.0f;
	HttpReceiveTimeout = HttpConnectionTimeout;
	HttpSendTimeout = HttpConnectionTimeout;
	HttpMaxConnectionsPerServer = 16;
	bEnableHttp = true;
	bUseNullHttp = false;
	HttpDelayTime = 0;
	HttpThreadActiveFrameTimeInSeconds = 1.0f / 200.0f; // 200Hz
	HttpThreadActiveMinimumSleepTimeInSeconds = 0.0f;
	HttpThreadIdleFrameTimeInSeconds = 1.0f / 30.0f; // 30Hz
	HttpThreadIdleMinimumSleepTimeInSeconds = 0.0f;
	HttpEventLoopThreadTickIntervalInSeconds = 1.f / 10.f; // 10Hz

	// override the above defaults from configs
	FCoreDelegates::TSOnConfigSectionsChanged().AddRaw(this, &FHttpModule::OnConfigSectionsChanged);
	UpdateConfigs();

	ProxyAddress = FPlatformHttp::GetConfiguredProxyAddress();

	// Load from a configurable array of modules at this point, so things that need to bind to the SDK Manager init hooks can do so.
	TArray<FString> ModulesToLoad;
	GConfig->GetArray(TEXT("HTTP"), TEXT("ModulesToLoad"), ModulesToLoad, GEngineIni);
	for (const FString& ModuleToLoad : ModulesToLoad)
	{
		if (FModuleManager::Get().ModuleExists(*ModuleToLoad))
		{
			FModuleManager::Get().LoadModule(*ModuleToLoad);
		}
	}

	// Initialize FPlatformHttp after we have read config values
	FPlatformHttp::Init();

	HttpManager = FPlatformHttp::CreatePlatformHttpManager();
	if (nullptr == HttpManager)
	{
		// platform does not provide specific HTTP manager, use generic one
		HttpManager = new FHttpManager();
	}
	HttpManager->Initialize();

	bSupportsDynamicProxy = HttpManager->SupportsDynamicProxy();

	FCoreDelegates::ShouldLaunchUrl.BindStatic(ShouldLaunchUrl);
}

void FHttpModule::PostLoadCallback()
{

}

void FHttpModule::PreUnloadCallback()
{
}

void FHttpModule::ShutdownModule()
{
	FCoreDelegates::ShouldLaunchUrl.Unbind();

	if (HttpManager != nullptr)
	{
		// block on any http requests that have already been queued up
		HttpManager->Shutdown();
	}

	// at least on Linux, the code in HTTP manager (e.g. request destructors) expects platform to be initialized yet
	delete HttpManager;	// can be passed NULLs

	FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);

	FPlatformHttp::Shutdown();

	HttpManager = nullptr;
	Singleton = nullptr;
}

void FHttpModule::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (IniFilename == GEngineIni)
	{
		for (const FString& SectionName : SectionNames)
		{
			if (SectionName.StartsWith(TEXT("HTTP")))
			{
				UpdateConfigs();
				break;
			}
		}
	}
}

bool FHttpModule::HandleHTTPCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("TEST")))
	{
		int32 Iterations = 1;
		FString IterationsStr;
		FParse::Token(Cmd, IterationsStr, true);
		if (!IterationsStr.IsEmpty())
		{
			Iterations = FCString::Atoi(*IterationsStr);
		}
		FString Url;
		FParse::Token(Cmd, Url, true);
		if (Url.IsEmpty())
		{
			Url = TEXT("http://www.google.com");
		}
		FHttpTest* HttpTest = new FHttpTest(TEXT("GET"), TEXT(""), Url, Iterations);
		HttpTest->Run();
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPREQ")))
	{
		GetHttpManager().DumpRequests(Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("FLUSH")))
	{
		GetHttpManager().Flush(EHttpFlushReason::Default);
	}
#if !UE_BUILD_SHIPPING
	else if (FParse::Command(&Cmd, TEXT("FILEUPLOAD")))
	{
		FString UploadUrl, UploadFilename;
		bool bIsCmdOk = FParse::Token(Cmd, UploadUrl, false);
		bIsCmdOk &= FParse::Token(Cmd, UploadFilename, false);
		if (bIsCmdOk)
		{
			FString HttpMethod;
			if (!FParse::Token(Cmd, HttpMethod, false))
			{
				HttpMethod = TEXT("PUT");
			}

			TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Request = CreateRequest();
			Request->SetURL(UploadUrl);
			Request->SetVerb(HttpMethod);
			Request->SetHeader(TEXT("Content-Type"), TEXT("application/x-uehttp-upload-test"));
			Request->SetContentAsStreamedFile(UploadFilename);
			Request->ProcessRequest();
		}
		else
		{
			UE_LOGF(LogHttp, Warning, "Command expects params <upload url> <upload filename> [http verb]")
		}
	}
#endif
	else if (FParse::Command(&Cmd, TEXT("LAUNCHREQUESTS")))
	{
		FString Verb = FParse::Token(Cmd, false);
		FString Url = FParse::Token(Cmd, false);
		int32 NumRequests = FCString::Atoi(*FParse::Token(Cmd, false));
		bool bCancelRequests = FCString::ToBool(*FParse::Token(Cmd, false));

		TArray<TSharedRef<IHttpRequest, ESPMode::ThreadSafe>> Requests;

		for (int32 i = 0; i < NumRequests; ++i)
		{
			TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
			HttpRequest->SetURL(*Url);
			HttpRequest->SetVerb(*Verb);
			HttpRequest->OnProcessRequestComplete().BindLambda([](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded) {});
			HttpRequest->ProcessRequest();

			Requests.Add(HttpRequest);
		}

		if (bCancelRequests)
		{
			for (auto Request : Requests)
			{
				Request->CancelRequest();
			}
		}
	}
#if UE_HTTP_SOCKET_TEST_COMMAND_ENABLED
	else if (FParse::Command(&Cmd, TEXT("DUMPFDS")))
	{
		DIR* Dir = opendir("/proc/self/fd");
		if (!Dir)
		{
			UE_LOGF(LogHttp, Warning, "Failed to open directory /proc/self/fd");
		}
		else
		{
			struct dirent* Entry = nullptr;
			while ((Entry = readdir(Dir)) != nullptr)
			{
				if (strcmp(Entry->d_name, ".") == 0 || strcmp(Entry->d_name, "..") == 0)
				{
					continue;
				}

				char FullPath[2048];
				snprintf(FullPath, sizeof(FullPath), "/proc/self/fd/%s", Entry->d_name);
				FUtf8StringView FullPathStringView(FullPath);

				char Dest[1024] = { 0 };

				int32 Len = readlink(FullPath, Dest, sizeof(Dest));

				if (Len == -1)
				{
					UE_LOGFMT(LogHttp, Warning, "{FullPathStringView}", FullPathStringView);
					continue;
				}

				FUtf8StringView DestStringView(Dest, Len);
				UE_LOGFMT(LogHttp, Warning, "{FullPathStringView} -> {DestStringView}", FullPathStringView, DestStringView);
			}

			closedir(Dir);
		}
	}
	else if (FParse::Command(&Cmd, TEXT("LEAKSOCKET")))
	{
		// Leak sockets in purpose to verify high File Descriptor number can be handled properly in different places
		int NumLeakedSockets = 0;
		while (true)
		{
			NumLeakedSockets += 1;
			int Socket = socket(AF_INET, SOCK_DGRAM, IPPROTO_UDP);
			UE_LOGF(LogHttp, Log, "Leaking socket, Socket=%i, NumLeakedSockets=%i", Socket, NumLeakedSockets);
			if (Socket > 1024)
			{
				break;
			}
		}
	}
	else if (FParse::Command(&Cmd, TEXT("SHOWLATESTFD")))
	{
		int Socket = socket(AF_INET, SOCK_DGRAM, IPPROTO_UDP);
		if (Socket == 0)
		{
			UE_LOGF(LogHttp, Warning, "Failed to open new socket!");
		}
		else
		{
			UE_LOGF(LogHttp, Warning, "Latest socket fd is %i", Socket);
			close(Socket);
		}
	}
#endif // UE_HTTP_SOCKET_TEST_COMMAND_ENABLED

	return true;
}

bool FHttpModule::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FCString::Strnicmp(Cmd, TEXT("HTTP."), 5) == 0)
	{
		// It's changing console variable
		return false;
	}

	if (!FParse::Command(&Cmd, TEXT("HTTP")))
	{
		// Ignore any execs that don't start with HTTP
		return false;
	}

	return HandleHTTPCommand(Cmd, Ar);
}

FHttpModule& FHttpModule::Get()
{
	if (Singleton == nullptr)
	{
		// Only auto-load on first use. If the module was previously loaded and shut down
		// (e.g. by a test fixture), do not recreate it — callers must explicitly load it again.
		checkf(!bWasEverLoaded, TEXT("FHttpModule::Get() called after the HTTP module was shut down. Recreating it via Get() is not allowed. Call FModuleManager::LoadModuleChecked<FHttpModule>(\"HTTP\") explicitly if recreation is intended."));
		check(IsInGameThread());
		FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	}
	check(Singleton != nullptr);
	return *Singleton;
}

TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FHttpModule::CreateRequest()
{
	if (bUseNullHttp)
	{
		return MakeShared<FNullHttpRequest, ESPMode::ThreadSafe>();
	}

	if (AutoRTFM::IsClosed())
	{
		return MakeShared<FTransactionallySafeHttpRequest, ESPMode::ThreadSafe>();
	}

	// Create the platform specific Http request instance
	return TSharedRef<IHttpRequest, ESPMode::ThreadSafe>(FPlatformHttp::ConstructRequest());
}
