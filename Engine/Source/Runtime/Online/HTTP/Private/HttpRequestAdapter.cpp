// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpRequestAdapter.h"

FHttpRequestAdapterBase::FHttpRequestAdapterBase(const TSharedRef<IHttpRequest>& InHttpRequest)
	: HttpRequest(InHttpRequest)
{
}

FHttpRequestAdapterBase::~FHttpRequestAdapterBase() = default;

const FString& FHttpRequestAdapterBase::GetURL() const 
{ 
	return HttpRequest->GetURL(); 
}

FString FHttpRequestAdapterBase::GetURLParameter(const FString& ParameterName) const 
{ 
	return HttpRequest->GetURLParameter(ParameterName); 
}

FString FHttpRequestAdapterBase::GetHeader(const FString& HeaderName) const 
{ 
	return HttpRequest->GetHeader(HeaderName); 
}

TArray<FString> FHttpRequestAdapterBase::GetAllHeaders() const 
{ 
	return HttpRequest->GetAllHeaders(); 
}

FString FHttpRequestAdapterBase::GetContentType() const 
{ 
	return HttpRequest->GetContentType(); 
}

uint64 FHttpRequestAdapterBase::GetContentLength() const 
{ 
	return HttpRequest->GetContentLength(); 
}

const TArray<uint8>& FHttpRequestAdapterBase::GetContent() const 
{ 
	return HttpRequest->GetContent(); 
}

FString FHttpRequestAdapterBase::GetVerb() const 
{ 
	return HttpRequest->GetVerb(); 
}

FString FHttpRequestAdapterBase::GetOption(const FName Option) const
{
	return HttpRequest->GetOption(Option);
}

void FHttpRequestAdapterBase::SetVerb(const FString& Verb) 
{ 
	HttpRequest->SetVerb(Verb); 
}

void FHttpRequestAdapterBase::SetURL(const FString& URL) 
{ 
	HttpRequest->SetURL(URL); 
}

void FHttpRequestAdapterBase::SetOption(const FName Option, const FString& OptionValue)
{
	HttpRequest->SetOption(Option, OptionValue);
}

void FHttpRequestAdapterBase::SetContent(const TArray<uint8>& ContentPayload) 
{ 
	HttpRequest->SetContent(ContentPayload); 
}

void FHttpRequestAdapterBase::SetContent(TArray<uint8>&& ContentPayload) 
{ 
	HttpRequest->SetContent(MoveTemp(ContentPayload)); 
}

void FHttpRequestAdapterBase::SetContentAsString(const FString& ContentString) 
{ 
	HttpRequest->SetContentAsString(ContentString); 
}

bool FHttpRequestAdapterBase::SetContentAsStreamedFile(const FString& Filename) 
{ 
	return HttpRequest->SetContentAsStreamedFile(Filename); 
}

bool FHttpRequestAdapterBase::SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) 
{ 
	return HttpRequest->SetContentFromStream(Stream); 
}

bool FHttpRequestAdapterBase::SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) 
{ 
	return HttpRequest->SetResponseBodyReceiveStream(Stream); 
}

void FHttpRequestAdapterBase::SetHeader(const FString& HeaderName, const FString& HeaderValue) 
{ 
	HttpRequest->SetHeader(HeaderName, HeaderValue); 
}

void FHttpRequestAdapterBase::AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) 
{ 
	HttpRequest->AppendToHeader(HeaderName, AdditionalHeaderValue); 
}

void FHttpRequestAdapterBase::SetTimeout(float InTimeoutSecs) 
{ 
	HttpRequest->SetTimeout(InTimeoutSecs); 
}

void FHttpRequestAdapterBase::ClearTimeout() 
{ 
	HttpRequest->ClearTimeout(); 
}

void FHttpRequestAdapterBase::ResetTimeoutStatus() 
{ 
	HttpRequest->ResetTimeoutStatus(); 
}

TOptional<float> FHttpRequestAdapterBase::GetTimeout() const 
{ 
	return HttpRequest->GetTimeout(); 
}

void FHttpRequestAdapterBase::SetActivityTimeout(float InTimeoutSecs)
{
	return HttpRequest->SetActivityTimeout(InTimeoutSecs);
}

void FHttpRequestAdapterBase::ProcessRequestUntilComplete() 
{ 
	return HttpRequest->ProcessRequestUntilComplete(); 
}

const FHttpResponsePtr FHttpRequestAdapterBase::GetResponse() const 
{ 
	return HttpRequest->GetResponse(); 
}

float FHttpRequestAdapterBase::GetElapsedTime() const 
{ 
	return HttpRequest->GetElapsedTime(); 
}

EHttpRequestStatus::Type FHttpRequestAdapterBase::GetStatus() const 
{ 
	return HttpRequest->GetStatus(); 
}

EHttpFailureReason FHttpRequestAdapterBase::GetFailureReason() const 
{ 
	return HttpRequest->GetFailureReason(); 
}

const FString& FHttpRequestAdapterBase::GetEffectiveURL() const 
{ 
	return HttpRequest->GetEffectiveURL(); 
}

void FHttpRequestAdapterBase::Tick(float DeltaSeconds) 
{ 
	HttpRequest->Tick(DeltaSeconds); 
}

void FHttpRequestAdapterBase::SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy) 
{ 
	HttpRequest->SetDelegateThreadPolicy(InThreadPolicy); 
}

EHttpRequestDelegateThreadPolicy FHttpRequestAdapterBase::GetDelegateThreadPolicy() const 
{ 
	return HttpRequest->GetDelegateThreadPolicy(); 
}

void FHttpRequestAdapterBase::SetPriority(EHttpRequestPriority InPriority) 
{ 
	HttpRequest->SetPriority(InPriority); 
}

EHttpRequestPriority FHttpRequestAdapterBase::GetPriority() const 
{ 
	return HttpRequest->GetPriority(); 
}
