// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpRetrySystem.h"
#include "AutoRTFM.h"
#include "GenericPlatform/HttpRequestCommon.h"
#include "HAL/ConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/LowLevelMemTracker.h"
#include "Math/RandomStream.h"
#include "HttpModule.h"
#include "Http.h"
#include "HttpManager.h"
#include "HttpThread.h"
#include "Stats/Stats.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

LLM_DEFINE_TAG(HTTP);

TAutoConsoleVariable<bool> CVarHttpCancelSupportDuringWillRetryDelegate(
	TEXT("http.CancelSupportDuringWillRetryDelegate"),
	true,
	TEXT("When enabled, http request can be cancelled during WillRetry delegate."),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<bool> CVarHttpCancelSupportDuringBackOffEnabled(
	TEXT("http.CancelSupportDuringBackOffEnabled"),
	false,
	TEXT("When enabled, http request can be cancelled during back off."),
	ECVF_SaveForNextBoot
);

namespace FHttpRetrySystem
{

TOptional<double> ReadThrottledTimeFromResponseInSeconds(FHttpResponsePtr Response)
{
	TOptional<double> LockoutPeriod;
	// Check if there was a Retry-After header
	if (Response.IsValid())
	{
		FString RetryAfter = Response->GetHeader(TEXT("Retry-After"));
		if (!RetryAfter.IsEmpty())
		{
			if (RetryAfter.IsNumeric())
			{
				// seconds
				LockoutPeriod.Emplace(FCString::Atof(*RetryAfter));
			}
			else
			{
				// http date
				FDateTime UTCServerTime;
				if (FDateTime::ParseHttpDate(RetryAfter, UTCServerTime))
				{
					const FDateTime UTCNow = FDateTime::UtcNow();
					LockoutPeriod.Emplace((UTCServerTime - UTCNow).GetTotalSeconds());
				}
			}
		}
		else
		{
			FString RateLimitReset = Response->GetHeader(TEXT("X-Rate-Limit-Reset"));
			if (!RateLimitReset.IsEmpty())
			{
				// UTC seconds
				const FDateTime UTCServerTime = FDateTime::FromUnixTimestamp(FCString::Atoi64(*RateLimitReset));
				const FDateTime UTCNow = FDateTime::UtcNow();
				LockoutPeriod.Emplace((UTCServerTime - UTCNow).GetTotalSeconds());
			}
		}
	}
	return LockoutPeriod;
}

bool FHttpRetrySystem::FExponentialBackoffCurve::IsValid() const
{
	return Base > 1.0f
		&& ExponentBias >= 0.0f
		&& MinCoefficient <= MaxCoefficient
		&& MaxCoefficient > 0.001f
		&& MinCoefficient >= 0.0f;
}

float FHttpRetrySystem::FExponentialBackoffCurve::Compute(uint32 RetryNumber) const
{
	float BackOff = FMath::Pow(Base, static_cast<float>(RetryNumber) + ExponentBias);
	const float Coefficient = IsValid() ? FMath::RandRange(MinCoefficient, MaxCoefficient) : 1.0f;
	return FMath::Min(BackOff * Coefficient, MaxBackoffSeconds);
}

}

FHttpRetrySystem::FRequest::FRequest(
	TSharedRef<FManager, ESPMode::ThreadSafe> InManager,
	const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& HttpRequest,
	const FHttpRetrySystem::FRetryLimitCountSetting& InRetryLimitCountOverride,
	const FHttpRetrySystem::FRetryTimeoutRelativeSecondsSetting& InRetryTimeoutRelativeSecondsOverride,
	const FHttpRetrySystem::FRetryResponseCodes& InRetryResponseCodes,
	const FHttpRetrySystem::FRetryVerbs& InRetryVerbs,
	const FHttpRetrySystem::FRetryDomainsPtr& InRetryDomains,
	const FRetryLimitCountSetting& InRetryLimitCountForConnectionErrorOverride,
	const FExponentialBackoffCurve& InExponentialBackoffCurve
)
	: FHttpRequestAdapterBase(HttpRequest)
	, RetryStatus(FHttpRetrySystem::FRequest::EStatus::NotStarted)
	, RetryLimitCountOverride(InRetryLimitCountOverride)
	, RetryLimitCountForConnectionErrorOverride(InRetryLimitCountForConnectionErrorOverride)
	, RetryTimeoutRelativeSecondsOverride(InRetryTimeoutRelativeSecondsOverride)
	, RetryResponseCodes(InRetryResponseCodes)
	, RetryVerbs(InRetryVerbs)
	, RetryDomains(InRetryDomains)
	, RetryManager(InManager)
	, RetryExponentialBackoffCurve(InExponentialBackoffCurve)
{
	// if the InRetryTimeoutRelativeSecondsOverride override is being used the value cannot be negative
	check(!(InRetryTimeoutRelativeSecondsOverride.IsSet()) || (InRetryTimeoutRelativeSecondsOverride.GetValue() >= 0.0));

	if (RetryDomains.IsValid())
	{
		if (RetryDomains->Domains.Num() == 0)
		{
			// If there are no domains to cycle through, go through the simpler path
			RetryDomains.Reset();
		}
		else
		{
			// Start with the active index
			RetryDomainsIndex = RetryDomains->ActiveIndex;
			check(RetryDomains->Domains.IsValidIndex(RetryDomainsIndex));
		}
	}
}

void FHttpRetrySystem::FRequest::BindAdaptorDelegates()
{
	if (!bBoundAdaptorDelegates)
	{
		bBoundAdaptorDelegates = true;

		// Can't BindRaw&Unbind from ctor&dtor because with thread-safe delegate, it can cause issue when delete this request during complete callback, then unbind the callback
		HttpRequest->OnProcessRequestComplete().BindThreadSafeSP(this, &FHttpRetrySystem::FRequest::HttpOnProcessRequestComplete);
		HttpRequest->OnRequestProgress64().BindThreadSafeSP(this, &FHttpRetrySystem::FRequest::HttpOnRequestProgress);
		HttpRequest->OnStatusCodeReceived().BindThreadSafeSP(this, &FHttpRetrySystem::FRequest::HttpOnStatusCodeReceived);
		HttpRequest->OnHeaderReceived().BindThreadSafeSP(this, &FHttpRetrySystem::FRequest::HttpOnHeaderReceived);
	}
}

bool FHttpRetrySystem::FRequest::ProcessRequest()
{
	TSharedRef<FRequest, ESPMode::ThreadSafe> RetryRequest = StaticCastSharedRef<FRequest>(AsShared());

	OriginalUrl = HttpRequest->GetURL();

	if (RetryDomains.IsValid() && !RetryDomains->Domains.IsEmpty())
	{
		FString OriginalUrlDomainAndPort = FPlatformHttp::GetUrlDomainAndPort(OriginalUrl);
		int32 Index = RetryDomains->Domains.Find(OriginalUrlDomainAndPort);
		if (Index == INDEX_NONE)
		{
			RetryDomains->Domains.Insert(MoveTemp(OriginalUrlDomainAndPort), 0);
		}
		else if (Index > 0)
		{
			RetryDomains->Domains.RemoveAt(Index);
			RetryDomains->Domains.Insert(MoveTemp(OriginalUrlDomainAndPort), 0);
		}
	}

	// The ActiveIndex inside FRetryDomains could have been increased before, so to skip the domains failed to connect
	if (RetryDomains.IsValid())
	{
		SetUrlFromRetryDomains();
	}

	BindAdaptorDelegates();

	TSharedPtr<FManager, ESPMode::ThreadSafe> RetryManagerPtr = RetryManager.Pin();

	if (ensure(RetryManagerPtr))
	{
		return RetryManagerPtr->ProcessRequest(RetryRequest);
	}
	else
	{
		return false;
	}
}

void FHttpRetrySystem::FRequest::SetUrlFromRetryDomains()
{
	check(RetryDomains.IsValid());
	FString OriginalUrlDomainAndPort = FPlatformHttp::GetUrlDomainAndPort(OriginalUrl);
	if (!OriginalUrlDomainAndPort.IsEmpty())
	{
		const FString Url(OriginalUrl.Replace(*OriginalUrlDomainAndPort, *RetryDomains->Domains[RetryDomainsIndex]));
		HttpRequest->SetURL(Url);
	}
}

void FHttpRetrySystem::FRequest::MoveToNextRetryDomain()
{
	check(RetryDomains.IsValid());
	const int32 NextDomainIndex = (RetryDomainsIndex + 1) % RetryDomains->Domains.Num();
	if (RetryDomains->ActiveIndex.CompareExchange(RetryDomainsIndex, NextDomainIndex))
	{
		RetryDomainsIndex = NextDomainIndex;

		// We should updade config files only if two fields were explicitly set
		if (!RetryDomains->ConfigSection.IsEmpty() && !RetryDomains->ConfigKeyName.IsEmpty())
		{
			GConfig->RemoveKey(*RetryDomains->ConfigSection, *RetryDomains->ConfigKeyName, GGameUserSettingsIni);

			// If we come in a circle to the first index then no need to save it
			if (RetryDomains->ActiveIndex > 0)
			{
				// Save the configuration to reuse latest available endpoint during the next Launcher session
				GConfig->SetString(*RetryDomains->ConfigSection, *RetryDomains->ConfigKeyName, *RetryDomains->Domains[RetryDomains->ActiveIndex], GGameUserSettingsIni);
			}
		}
	}
	SetUrlFromRetryDomains();
}

void FHttpRetrySystem::FRequest::CancelRequest() 
{ 
	TSharedRef<FRequest, ESPMode::ThreadSafe> RetryRequest = StaticCastSharedRef<FRequest>(AsShared());

	BindAdaptorDelegates();

	if (TSharedPtr<FManager, ESPMode::ThreadSafe> RetryManagerPtr = RetryManager.Pin())
	{
		RetryManagerPtr->CancelRequest(RetryRequest);
	}
	else
	{
		HttpRequest->CancelRequest();
	}
}

void FHttpRetrySystem::FRequest::HttpOnRequestProgress(FHttpRequestPtr InHttpRequest, uint64 BytesSent, uint64 BytesRcv)
{
	OnRequestProgress64().ExecuteIfBound(AsShared(), BytesSent, BytesRcv);
}

void FHttpRetrySystem::FRequest::HttpOnProcessRequestComplete(FHttpRequestPtr InHttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	TSharedPtr<FManager, ESPMode::ThreadSafe> RetryManagerPtr = RetryManager.Pin();
	if (!RetryManagerPtr)
	{
		return;
	}

	TSharedRef<FRequest, ESPMode::ThreadSafe> SelfPtr = StaticCastSharedRef<FRequest>(AsShared()); // In case no ref after removing from RetryManager


	{
		FScopeLock ScopeLock(&RetryManagerPtr->RequestListLock);

		uint32 EntryIndex = RetryManagerPtr->RequestList.IndexOfByPredicate([this](const FManager::FHttpRetryRequestEntry& Entry) { return Entry.Request == AsShared(); });
		if (ensure(EntryIndex != INDEX_NONE))
		{
			FManager::FHttpRetryRequestEntry& HttpRetryRequestEntry = RetryManagerPtr->RequestList[EntryIndex];

			bool bCancelled = (RetryStatus == FHttpRetrySystem::FRequest::EStatus::Cancelled) || (GetFailureReason() == EHttpFailureReason::Cancelled);
			if (bCancelled)
			{
				// Do nothing here
			}
			else if (GetStatus() == EHttpRequestStatus::Failed)
			{
				if (GetFailureReason() == EHttpFailureReason::ConnectionError && RetryDomains.IsValid())
				{
					MoveToNextRetryDomain();
				}

				if (GetFailureReason() == EHttpFailureReason::TimedOut)
				{
					RetryStatus = FHttpRetrySystem::FRequest::EStatus::FailedTimeout;
				}
				else
				{
					RetryStatus = FHttpRetrySystem::FRequest::EStatus::FailedRetry;
				}
			}
			else
			{
				RetryStatus = FHttpRetrySystem::FRequest::EStatus::Succeeded;
			}

			if (!bCancelled && 
				RetryStatus != FHttpRetrySystem::FRequest::EStatus::FailedTimeout && 
				RetryManagerPtr->ShouldRetry(HttpRetryRequestEntry) && 
				RetryManagerPtr->CanRetry(HttpRetryRequestEntry))
			{
				const double NowAbsoluteSeconds = FPlatformTime::Seconds();
				float LockoutPeriod = RetryManagerPtr->GetLockoutPeriodSeconds(HttpRetryRequestEntry);

				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpRetrySystem_FManager_Update_OnRequestWillRetry);
					OnRequestWillRetry().ExecuteIfBound(HttpRetryRequestEntry.Request, GetResponse(), LockoutPeriod);
				}

				if (CVarHttpCancelSupportDuringWillRetryDelegate.GetValueOnAnyThread() &&
					RetryStatus == FHttpRetrySystem::FRequest::EStatus::Cancelled) // In case CancelRequest get called during OnRequestWillRetry callback
				{
					TSharedRef<FHttpRequestCommon, ESPMode::ThreadSafe> RequestCommon = StaticCastSharedRef<FHttpRequestCommon>(HttpRequest);
					RequestCommon->SetFailureReason(EHttpFailureReason::Cancelled);
					RequestCommon->SetStatus(EHttpRequestStatus::Failed);
					HttpResponse.Reset();
					bSucceeded = false;
				}
				else
				{
					RetryStatus = FHttpRetrySystem::FRequest::EStatus::ProcessingLockout;
					RetryManagerPtr->RetryHttpRequestWithDelay(HttpRetryRequestEntry, LockoutPeriod, bSucceeded);
					return;
				}
			}

			if (HttpRetryRequestEntry.CurrentRetryCount > 0)
			{
				FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::Get().DecrementRetriedRequests();
			}

			RetryManagerPtr->RequestList.RemoveAtSwap(EntryIndex);
		}
	}

	FHttpResponsePtr ResultResponse = HttpResponse;
	bool bResultSucceeded = bSucceeded;
	if (RetryStatus == FHttpRetrySystem::FRequest::EStatus::FailedTimeout && LastResponse != nullptr)
	{
		// Last response is better than nothing when it's timeout
		ResultResponse = LastResponse;
		LastResponse.Reset();

		bResultSucceeded = bLastSucceeded;
	}

	LLM_SCOPE_BYTAG(HTTP);
	OnProcessRequestComplete().ExecuteIfBound(SelfPtr, ResultResponse, bResultSucceeded);

	ClearTimeout();
}

void FHttpRetrySystem::FRequest::HttpOnStatusCodeReceived(FHttpRequestPtr Request, int32 StatusCode)
{
	TSharedRef<FRequest, ESPMode::ThreadSafe> SelfPtr = StaticCastSharedRef<FRequest>(AsShared());
	OnStatusCodeReceived().ExecuteIfBound(SelfPtr, StatusCode);
}

void FHttpRetrySystem::FRequest::HttpOnHeaderReceived(FHttpRequestPtr Request, const FString& HeaderName, const FString& NewHeaderValue)
{
	TSharedRef<FRequest, ESPMode::ThreadSafe> SelfPtr = StaticCastSharedRef<FRequest>(AsShared());
	OnHeaderReceived().ExecuteIfBound(SelfPtr, HeaderName, NewHeaderValue);
}

FHttpRetrySystem::FManager::FManager(
	const FRetryLimitCountSetting& InRetryLimitCountDefault,
	const FRetryTimeoutRelativeSecondsSetting& InRetryTimeoutRelativeSecondsDefault,
	const FRetryLimitCountSetting& InRetryLimitCountForConnectionErrorDefault
)
    : RandomFailureRate(FRandomFailureRateSetting())
    , RetryLimitCountDefault(InRetryLimitCountDefault)
    , RetryLimitCountForConnectionErrorDefault(InRetryLimitCountForConnectionErrorDefault)
	, RetryTimeoutRelativeSecondsDefault(InRetryTimeoutRelativeSecondsDefault)
{
	check(FHttpModule::Get().GetHttpManager().GetThread());
}

FHttpRetrySystem::FManager::~FManager()
{
	FScopeLock ScopeLock(&RequestListLock);

	// Decrement retried request for log verbosity tracker
	for (const FHttpRetryRequestEntry& Request : RequestList)
	{
		if (Request.CurrentRetryCount > 0)
		{
			FHttpLogVerbosityTracker::Get().DecrementRetriedRequests();
		}
	}
}

TSharedRef<FHttpRetrySystem::FRequest, ESPMode::ThreadSafe> FHttpRetrySystem::FManager::CreateRequest(
	const FRetryLimitCountSetting& InRetryLimitCountOverride,
	const FRetryTimeoutRelativeSecondsSetting& InRetryTimeoutRelativeSecondsOverride,
	const FRetryResponseCodes& InRetryResponseCodes,
	const FRetryVerbs& InRetryVerbs,
	const FRetryDomainsPtr& InRetryDomains,
	const FRetryLimitCountSetting& InRetryLimitCountForConnectionErrorOverride,
	const FExponentialBackoffCurve& InExponentialBackoffCurve
)
{
	return MakeShareable(new FRequest(
		AsShared(),
		FHttpModule::Get().CreateRequest(),
		InRetryLimitCountOverride,
		InRetryTimeoutRelativeSecondsOverride,
		InRetryResponseCodes,
		InRetryVerbs,
		InRetryDomains,
		InRetryLimitCountForConnectionErrorOverride,
		InExponentialBackoffCurve
		));
}

bool FHttpRetrySystem::FManager::ShouldRetry(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const
{
	FHttpResponsePtr Response = HttpRetryRequestEntry.Request->GetResponse();
	if (Response)
	{
		return HttpRetryRequestEntry.Request->RetryResponseCodes.Contains(Response->GetResponseCode());
	}

	// ONLY continue to check retry if no response. If there is any response, it means at least the http 
	// connection was established, we shouldn't attempt to retry. Otherwise request may be sent (and 
	// processed) twice

	// Safe check
	if (HttpRetryRequestEntry.Request->GetStatus() != EHttpRequestStatus::Failed)
	{
		// This shouldn't happen when response is null, but just in case
		return false;
	}

	// Should retry if couldn't connect at all
	if (HttpRetryRequestEntry.Request->GetFailureReason() == EHttpFailureReason::ConnectionError)
	{
		return true;
	}

	// Should retry for idempotent verbs if there is network error
	const FName Verb = FName(*HttpRetryRequestEntry.Request->GetVerb());

	if (!HttpRetryRequestEntry.Request->RetryVerbs.IsEmpty())
	{
		return HttpRetryRequestEntry.Request->RetryVerbs.Contains(Verb);
	}

	// Be default, we will also allow retry for GET and HEAD requests even if they may duplicate on the server
	static const TSet<FName> DefaultRetryVerbs(TArray<FName>({ FName("GET"), FName("HEAD") }));
	return DefaultRetryVerbs.Contains(Verb);
}

bool FHttpRetrySystem::FManager::RetryLimitForConnectionErrorIsSet(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const
{
	return HttpRetryRequestEntry.Request->RetryLimitCountForConnectionErrorOverride.IsSet() || RetryLimitCountForConnectionErrorDefault.IsSet();
}

bool FHttpRetrySystem::FManager::CanRetryForConnectionError(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const
{
	uint32 RetryLimitForConnectionError = HttpRetryRequestEntry.Request->RetryLimitCountForConnectionErrorOverride.Get(RetryLimitCountForConnectionErrorDefault.Get(0));
	return HttpRetryRequestEntry.CurrentRetryCountForConnectionError < RetryLimitForConnectionError;
}

bool FHttpRetrySystem::FManager::CanRetryInGeneral(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const
{
	uint32 RetryLimit = HttpRetryRequestEntry.Request->RetryLimitCountOverride.Get(RetryLimitCountDefault.Get(0));
	return HttpRetryRequestEntry.CurrentRetryCount < RetryLimit;
}

bool FHttpRetrySystem::FManager::CanRetry(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const
{
	if (HttpRetryRequestEntry.Request->GetFailureReason() == EHttpFailureReason::ConnectionError && RetryLimitForConnectionErrorIsSet(HttpRetryRequestEntry))
	{
		return CanRetryForConnectionError(HttpRetryRequestEntry);
	}

	return CanRetryInGeneral(HttpRetryRequestEntry);
}

bool FHttpRetrySystem::FManager::HasTimedOut(const FHttpRetryRequestEntry& HttpRetryRequestEntry, const double NowAbsoluteSeconds)
{
    bool bResult = false;

    bool bShouldTestRetryTimeout = false;
    double RetryTimeoutAbsoluteSeconds = HttpRetryRequestEntry.RequestStartTimeAbsoluteSeconds;
    if (HttpRetryRequestEntry.Request->RetryTimeoutRelativeSecondsOverride.IsSet())
    {
        bShouldTestRetryTimeout = true;
        RetryTimeoutAbsoluteSeconds += HttpRetryRequestEntry.Request->RetryTimeoutRelativeSecondsOverride.GetValue();
    }
    else if (RetryTimeoutRelativeSecondsDefault.IsSet())
    {
        bShouldTestRetryTimeout = true;
        RetryTimeoutAbsoluteSeconds += RetryTimeoutRelativeSecondsDefault.GetValue();
    }

    if (bShouldTestRetryTimeout)
    {
        if (NowAbsoluteSeconds >= RetryTimeoutAbsoluteSeconds)
        {
            bResult = true;
        }
    }

    return bResult;
}

void FHttpRetrySystem::FManager::RetryHttpRequest(FHttpRetryRequestEntry& RequestEntry)
{
	// if this fails the HttpRequest's state will be failed which will cause the retry logic to kick(as expected)
	if (RequestEntry.CurrentRetryCount == 0)
	{
		FHttpLogVerbosityTracker::Get().IncrementRetriedRequests();
	}
	++RequestEntry.CurrentRetryCount;
	if (RequestEntry.Request->LastFailureReason == EHttpFailureReason::ConnectionError)
	{
		++RequestEntry.CurrentRetryCountForConnectionError;
	}
	RequestEntry.Request->RetryStatus = FRequest::EStatus::Processing;

	if (const FHttpResponsePtr Response = RequestEntry.Request->LastResponse)
	{
		if (const int32 ResponseCode = Response->GetResponseCode(); ResponseCode < 400)
		{
			// 1XX, 2XX, and 3XX are non error responses, regular log level
			UE_LOGF(LogHttp, Log, "Retry %d on %ls with response %d",
				RequestEntry.CurrentRetryCount,
				*(RequestEntry.Request->GetURL()),
				ResponseCode);
		}
		else
		{
			// 4XX, 5XX are error responses, warning log level
			UE_LOGF(LogHttp, Warning, "Retry %d on %ls with response %d",
				RequestEntry.CurrentRetryCount,
				*(RequestEntry.Request->GetURL()),
				ResponseCode);	
		}
	}
	else
	{
		// We don't know the response code, default to warning log level
		UE_LOGF(LogHttp, Warning, "Retry %d on %ls", RequestEntry.CurrentRetryCount, *(RequestEntry.Request->GetURL()));
	}
	
	RequestEntry.Request->HttpRequest->ProcessRequest();
}

void FHttpRetrySystem::FManager::RetryHttpRequestWithDelay(FManager::FHttpRetryRequestEntry& RequestEntry, float InDelay, bool bWasSucceeded)
{
	RequestEntry.Request->LastResponse = RequestEntry.Request->GetResponse();
	RequestEntry.Request->LastFailureReason = RequestEntry.Request->GetFailureReason();
	RequestEntry.Request->bLastSucceeded = bWasSucceeded;

	// Timeout during lock out to keep existing behavior
	float TimeoutOrDefault = RequestEntry.Request->GetTimeout().Get(FHttpModule::Get().GetHttpTotalTimeout());
	if (TimeoutOrDefault != 0)
	{
		float TimeElapsedForTheRequest = FPlatformTime::Seconds() - RequestEntry.RequestStartTimeAbsoluteSeconds;
		float WillTimeoutInDelay = TimeoutOrDefault - TimeElapsedForTheRequest;
		if (WillTimeoutInDelay < InDelay)
		{
			HttpRequestTimeoutAfterDelay(RequestEntry, bWasSucceeded, WillTimeoutInDelay);
			return;
		}
	}

	if (CVarHttpCancelSupportDuringBackOffEnabled.GetValueOnAnyThread())
	{
		// We're sure it will be retried at this point without further access of response, clear old state, so CancelRequest can work well during this back off delay
		TSharedRef<FHttpRequestCommon, ESPMode::ThreadSafe> RequestCommon = StaticCastSharedRef<FHttpRequestCommon>(RequestEntry.Request->HttpRequest);
		RequestCommon->ClearInCaseOfRetry();
	}

	// Delay and start
	TWeakPtr<FRequest> RequestWeakPtr(RequestEntry.Request);
	FHttpModule::Get().GetHttpManager().AddHttpThreadTask([RequestWeakPtr]() {
		if (TSharedPtr<FRequest, ESPMode::ThreadSafe> RequestPtr = RequestWeakPtr.Pin())
		{
			if (TSharedPtr<FManager, ESPMode::ThreadSafe> RetryManagerPtr = RequestPtr->RetryManager.Pin())
			{
				FScopeLock ScopeLock(&RetryManagerPtr->RequestListLock);
				// Check if it's still there in case it has been cancelled during the delay period
				uint32 EntryIndex = RetryManagerPtr->RequestList.IndexOfByPredicate([RequestPtr](const FManager::FHttpRetryRequestEntry& Entry) { return Entry.Request == RequestPtr; });
				if (EntryIndex != INDEX_NONE)
				{
					FManager::FHttpRetryRequestEntry* HttpRetryRequestEntry = &RetryManagerPtr->RequestList[EntryIndex];
					// Check entry state as well, in case CancelRequest->AbortRequest is still queueing in http thread tasks
					if (!CVarHttpCancelSupportDuringBackOffEnabled.GetValueOnAnyThread() ||
						HttpRetryRequestEntry->Request->RetryStatus != FHttpRetrySystem::FRequest::EStatus::Cancelled)
					{
						RetryManagerPtr->RetryHttpRequest(*HttpRetryRequestEntry);
					}
				}
			}
		}
	}, InDelay);
}

void FHttpRetrySystem::FManager::HttpRequestTimeoutAfterDelay(FManager::FHttpRetryRequestEntry& RequestEntry, bool bWasSucceeded, float Delay)
{
	TWeakPtr<FRequest> RequestWeakPtr(RequestEntry.Request);
	TFunction<void()> Callback([RequestWeakPtr, bWasSucceeded]() {
		if (TSharedPtr<FRequest, ESPMode::ThreadSafe> RequestPtr = RequestWeakPtr.Pin())
		{
			if (TSharedPtr<FManager, ESPMode::ThreadSafe> RetryManagerPtr = RequestPtr->RetryManager.Pin())
			{
				FScopeLock ScopeLock(&RetryManagerPtr->RequestListLock);
				uint32 EntryIndex = RetryManagerPtr->RequestList.IndexOfByPredicate([RequestPtr](const FManager::FHttpRetryRequestEntry& Entry) { return Entry.Request == RequestPtr; });
				if (EntryIndex != INDEX_NONE)
				{
					FManager::FHttpRetryRequestEntry* HttpRetryRequestEntry = &RetryManagerPtr->RequestList[EntryIndex];
					if (HttpRetryRequestEntry->CurrentRetryCount > 0)
					{
						FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::Get().DecrementRetriedRequests();
					}
					RetryManagerPtr->RequestList.RemoveAtSwap(EntryIndex);
				}
			}

			// Same as existing behavior, when timeout during lock out period, it fails with result of last request before lockout
			RequestPtr->OnProcessRequestComplete().ExecuteIfBound(RequestPtr, RequestPtr->GetResponse(), bWasSucceeded);
		}
	});

	if (RequestEntry.Request->GetDelegateThreadPolicy() == EHttpRequestDelegateThreadPolicy::CompleteOnGameThread)
	{
		FHttpModule::Get().GetHttpManager().AddGameThreadTask(MoveTemp(Callback), Delay);
	}
	else
	{
		FHttpModule::Get().GetHttpManager().AddHttpThreadTask(MoveTemp(Callback), Delay);
	}
}

float FHttpRetrySystem::FManager::GetLockoutPeriodSeconds(const FHttpRetryRequestEntry& HttpRetryRequestEntry)
{
	float LockoutPeriod = 0.0f;
	TOptional<double> ResponseLockoutPeriod = FHttpRetrySystem::ReadThrottledTimeFromResponseInSeconds(HttpRetryRequestEntry.Request->GetResponse());
	if (ResponseLockoutPeriod.IsSet())
	{
		LockoutPeriod = static_cast<float>(ResponseLockoutPeriod.GetValue());
	}

	if (LockoutPeriod <= 0.0f)
	{
		const bool bFailedToConnect = (HttpRetryRequestEntry.Request->GetStatus() == EHttpRequestStatus::Failed && HttpRetryRequestEntry.Request->GetFailureReason() == EHttpFailureReason::ConnectionError);
		const bool bHasRetryDomains = HttpRetryRequestEntry.Request->RetryDomains.IsValid();
		// Skip the lockout period if we failed to connect to a domain and we have other domains to try
		if (bFailedToConnect && bHasRetryDomains)
		{
			return 0.0f;
		}
		// The first time through this function, the CurrentRetryCount is 0, the second time it's 1, etc. We automatically add 1 to make the input into the backoff function line up with expectations.
		LockoutPeriod = HttpRetryRequestEntry.Request->RetryExponentialBackoffCurve.Compute(HttpRetryRequestEntry.CurrentRetryCount+1);
	}

	return LockoutPeriod;
}

static FRandomStream TempRandomStream(4435261);

FHttpRetrySystem::FManager::FHttpRetryRequestEntry::FHttpRetryRequestEntry(TSharedRef<FHttpRetrySystem::FRequest, ESPMode::ThreadSafe>& InRequest)
	: CurrentRetryCount(0)
	, CurrentRetryCountForConnectionError(0)
	, RequestStartTimeAbsoluteSeconds(FPlatformTime::Seconds())
	, Request(InRequest)
{}

bool FHttpRetrySystem::FManager::ProcessRequest(TSharedRef<FHttpRetrySystem::FRequest, ESPMode::ThreadSafe>& HttpRetryRequest)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpRetrySystem_FManager_ProcessRequest);

	// Let the request trigger timeout by itself, instead of ticking it in retry system
	if (HttpRetryRequest->RetryTimeoutRelativeSecondsOverride.IsSet())
	{
		HttpRetryRequest->SetTimeout(HttpRetryRequest->RetryTimeoutRelativeSecondsOverride.GetValue());
	}
	else if (RetryTimeoutRelativeSecondsDefault.IsSet())
	{
		HttpRetryRequest->SetTimeout(RetryTimeoutRelativeSecondsDefault.GetValue());
	}

	FScopeLock ScopeLock(&RequestListLock);
	RequestList.Add(FHttpRetryRequestEntry(HttpRetryRequest));
	HttpRetryRequest->RetryStatus = FHttpRetrySystem::FRequest::EStatus::Processing;
	HttpRetryRequest->HttpRequest->ProcessRequest();

	return true;
}

void FHttpRetrySystem::FManager::CancelRequest(TSharedRef<FHttpRetrySystem::FRequest, ESPMode::ThreadSafe>& HttpRetryRequest)
{
	UE_AUTORTFM_ONCOMMIT(this, HttpRetryRequest)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpRetrySystem_FManager_CancelRequest);

		FScopeLock ScopeLock(&RequestListLock);

		// Find the existing request entry if it was previously processed.
		bool bFound = false;
		for (int32 i = 0; i < RequestList.Num(); ++i)
		{
			FHttpRetryRequestEntry& EntryRef = RequestList[i];

			if (EntryRef.Request == HttpRetryRequest)
			{
				bFound = true;
			}
		}
		// If we did not find the entry, likely ProcessRequest hasn't been called. Add it to RequestList 
		// anyway to go through the same flow of cancellation
		if (!bFound)
		{
			FHttpRetryRequestEntry RetryRequestEntry(HttpRetryRequest);
			RequestList.Add(RetryRequestEntry);
		}

		HttpRetryRequest->HttpRequest->CancelRequest();
		HttpRetryRequest->RetryStatus = FHttpRetrySystem::FRequest::EStatus::Cancelled;
	};
}

/* This should only be used when shutting down or suspending, to make sure 
	all pending HTTP requests are flushed to the network */
void FHttpRetrySystem::FManager::BlockUntilFlushed(float InTimeoutSec)
{
	const float SleepInterval = 0.016f;
	float TimeElapsed = 0.0f;

	while (TimeElapsed < InTimeoutSec)
	{
		{
			FScopeLock ScopeLock(&RequestListLock);
			if (RequestList.IsEmpty())
			{
				break;
			}
		}

		FHttpModule::Get().GetHttpManager().Tick(SleepInterval);

		FPlatformProcess::Sleep(SleepInterval);
		TimeElapsed += SleepInterval;
	}
}

FHttpRetrySystem::FManager::FHttpLogVerbosityTracker& FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::Get()
{
	static FHttpLogVerbosityTracker Tracker;
	return Tracker;
}

FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::FHttpLogVerbosityTracker()
{
	UpdateSettingsFromConfig();
	FCoreDelegates::TSOnConfigSectionsChanged().AddRaw(this, &FHttpLogVerbosityTracker::OnConfigSectionsChanged);
}

FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::~FHttpLogVerbosityTracker()
{
	FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);
}

void FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::IncrementRetriedRequests()
{
	FScopeLock ScopeLock(&NumRetriedRequestsLock);

	++NumRetriedRequests;
	if (NumRetriedRequests == 1)
	{
		OriginalVerbosity = UE_GET_LOG_VERBOSITY(LogHttp);
		if (TargetVerbosity != ELogVerbosity::NoLogging)
		{
			UE_LOGF(LogHttp, Warning, "HttpRetry: Increasing log verbosity from %ls to %ls due to requests being retried", ToString(OriginalVerbosity), ToString(TargetVerbosity));
			//UE_SET_LOG_VERBOSITY(LogHttp, TargetVerbosity); // Macro requires the value to be a ELogVerbosity constant
#if !NO_LOGGING
			LogHttp.SetVerbosity(TargetVerbosity);
#endif
		}
	}
}

void FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::DecrementRetriedRequests()
{
	FScopeLock ScopeLock(&NumRetriedRequestsLock);

	--NumRetriedRequests;
	check(NumRetriedRequests >= 0);
	if (NumRetriedRequests == 0)
	{
		UE_LOGF(LogHttp, Warning, "HttpRetry: Resetting log verbosity to %ls due to requests being retried", ToString(OriginalVerbosity));
		//UE_SET_LOG_VERBOSITY(LogHttp, OriginalVerbosity); // Macro requires the value to be a ELogVerbosity constant
#if !NO_LOGGING
		LogHttp.SetVerbosity(OriginalVerbosity);
#endif
	}
}

void FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::UpdateSettingsFromConfig()
{
	FString TargetVerbosityAsString;
	if (GConfig->GetString(TEXT("HTTP.Retry"), TEXT("RetryManagerVerbosityLevel"), TargetVerbosityAsString, GEngineIni))
	{
		TargetVerbosity = ParseLogVerbosityFromString(TargetVerbosityAsString);
	}
	else
	{
		TargetVerbosity = ELogVerbosity::NoLogging;
	}
}

void FHttpRetrySystem::FManager::FHttpLogVerbosityTracker::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionName)
{
	if (IniFilename == GEngineIni && SectionName.Contains(TEXT("HTTP.Retry")))
	{
		UpdateSettingsFromConfig();
	}
}
