// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpTests.h"
#include "HttpModule.h"
#include "Http.h"

// FHttpTest

FHttpTest::FHttpTest(const FString& InVerb, const FString& InPayload, const FString& InUrl, int32 InIterations)
	: Verb(InVerb)
	, Payload(InPayload)
	, Url(InUrl)
	, TestsToRun(InIterations)
{
	
}

void FHttpTest::Run(void)
{
	UE_LOGF(LogHttp, Log, "Starting test [%ls] Url=[%ls]", 
		*Verb, *Url);

	for (int Idx=0; Idx < TestsToRun; Idx++)
	{
		TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> Request = FHttpModule::Get().CreateRequest();
		Request->OnProcessRequestComplete().BindRaw(this, &FHttpTest::RequestComplete);
		Request->SetURL(Url);
		if (Payload.Len() > 0)
		{
			Request->SetContentAsString(Payload);
		}
		Request->SetVerb(Verb);
		Request->ProcessRequest();
	}
}

void FHttpTest::RequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	if (!HttpResponse.IsValid())
	{
		UE_LOGF(LogHttp, Log, "Test failed. NULL response");
	}
	else
	{
		UE_LOGF(LogHttp, Log, "Completed test [%ls] Url=[%ls] Response=[%d] [%ls]", 
			*HttpRequest->GetVerb(), 
			*HttpRequest->GetURL(), 
			HttpResponse->GetResponseCode(), 
			*HttpResponse->GetContentAsString());
	}
	
	if ((--TestsToRun) <= 0)
	{
		HttpRequest->OnProcessRequestComplete().Unbind();
		// Done with the test. Delegate should always gets called
		delete this;
	}
}

