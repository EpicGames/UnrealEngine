// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IHttpRequest.h"

/**
 * Test an Http request to a specified endpoint Url
 */
class FHttpTest
{
public:

	/**
	 * Constructor
	 *
	 * @param Verb - verb to use for request (GET,POST,DELETE,etc)
	 * @param Payload - optional payload string
	 * @param Url - url address to connect to
	 * @param InIterations - total test iterations to run
	 */
	FHttpTest(const FString& InVerb, const FString& InPayload, const FString& InUrl, int32 InIterations);

	/**
	 * Kick off the Http request for the test and wait for delegate to be called
	 */
	void Run(void);

	/**
	 * Delegate called when the request completes
	 *
	 * @param HttpRequest - object that started/processed the request
	 * @param HttpResponse - optional response object if request completed
	 * @param bSucceeded - true if Url connection was made and response was received
	 */
	void RequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);

private:
	FString Verb;
	FString Payload;
	FString Url;
	int32 TestsToRun;
};

