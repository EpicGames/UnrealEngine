// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpThread.h"
#include "GenericPlatform/HttpRequestCommon.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "HAL/IConsoleManager.h"
#include "HttpManager.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Misc/Fork.h"
#include "Misc/Parse.h"
#include "HttpModule.h"
#include "Http.h"
#include "PlatformHttp.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"
#include "Templates/UnrealTemplate.h"

CSV_DEFINE_CATEGORY(HttpRequest, true);

TAutoConsoleVariable<int32> CVarHttpMaxConcurrentRequests(
	TEXT("http.MaxConcurrentRequests"),
	UE_HTTP_DEFAULT_MAX_CONCURRENT_REQUESTS,
	TEXT("The max number of http requests to run in parallel"),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<int32> CVarHttpDurationInQueueToWarnInSeconds(
	TEXT("http.DurationInQueueToWarnInSeconds"),
	10,
	TEXT("If http request waited more than this duration in the queue, output a warning before starting")
);

TAutoConsoleVariable<bool> CVarHttpRequestPriorityEnabled(
	TEXT("http.RequestPriorityEnabled"),
	true,
	TEXT("Whether to enable http request priority in the queue when max requests limit reached."),
	ECVF_SaveForNextBoot
);

TAutoConsoleVariable<bool> CVarHttpCsvRegions(
	TEXT("http.csvRegions"),
	false,
	TEXT("Add CSV regions for requests"),
	ECVF_Default
);

static void CsvRegion(const FHttpRequestCommon* Request, bool bBegin)
{
#if CSV_PROFILER_STATS && !UE_BUILD_SHIPPING
	if (CVarHttpCsvRegions.GetValueOnAnyThread(true) && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(HttpRequest)))
	{
		// Trim the https:// and anything passed via URL params
		FString Name = Request->GetURL();
		if (Name.StartsWith(TEXT("https://")))
		{
			Name.RightChopInline(8);
		}
		int32 Index;
		if (Name.FindChar(TCHAR('?'), Index))
		{
			Name.MidInline(0, Index);
		}
		Name = FString::Printf(TEXT("%s (%s)"), *Name, *Request->GetVerb());
		if (bBegin)
		{
			CSV_REGION_BEGIN(HttpRequest, *Name);
		}
		else
		{
			CSV_REGION_END(HttpRequest, *Name);
		}
	}
#endif
}

extern TAutoConsoleVariable<bool> CVarHttpRemoveRequestUsingHttpThreadPolicyOnHttpThread;

// Thread priority cvar (settable at runtime)
// We declare these explicitly rather than just casting the cvar in case the enum changes in future
const int32 GHttpThreadPriorities[] =
{
	EThreadPriority::TPri_Lowest,
	EThreadPriority::TPri_BelowNormal,
	EThreadPriority::TPri_SlightlyBelowNormal,
	EThreadPriority::TPri_Normal,
	EThreadPriority::TPri_AboveNormal
};

const TCHAR* GHttpThreadPriortyNames[] =
{
	TEXT("TPri_Lowest"),
	TEXT("TPri_BelowNormal"),
	TEXT("TPri_SlightlyBelowNormal"),
	TEXT("TPri_Normal"),
	TEXT("TPri_AboveNormal")
};

static int32 GHttpThreadPriorityIndex = 3; // EThreadPriority::TPri_Normal
FAutoConsoleVariableRef CVarHttpThreadPriority(
	TEXT("http.ThreadPriority"), 
	GHttpThreadPriorityIndex, 
	TEXT("Thread priority of the Http Manager thread: 0=Lowest, 1=BelowNormal, 2=SlightlyBelowNormal, 3=Normal, 4=AboveNormal\n")
	TEXT("Note that this is switchable at runtime"),
	ECVF_Default);

DECLARE_STATS_GROUP(TEXT("HTTP Thread"), STATGROUP_HTTPThread, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("Process"), STAT_HTTPThread_Process, STATGROUP_HTTPThread);
DECLARE_CYCLE_STAT(TEXT("TickThreadedRequest"), STAT_HTTPThread_TickThreadedRequest, STATGROUP_HTTPThread);
DECLARE_CYCLE_STAT(TEXT("StartThreadedRequest"), STAT_HTTPThread_StartThreadedRequest, STATGROUP_HTTPThread);
DECLARE_CYCLE_STAT(TEXT("HttpThreadTick"), STAT_HTTPThread_HttpThreadTick, STATGROUP_HTTPThread);
DECLARE_CYCLE_STAT(TEXT("IsThreadedRequestComplete"), STAT_HTTPThread_IsThreadedRequestComplete, STATGROUP_HTTPThread);
DECLARE_CYCLE_STAT(TEXT("CompleteThreadedRequest"), STAT_HTTPThread_CompleteThreadedRequest, STATGROUP_HTTPThread);
DECLARE_CYCLE_STAT(TEXT("ActiveSleep"), STAT_HTTPThread_ActiveSleep, STATGROUP_HTTPThread);
DECLARE_CYCLE_STAT(TEXT("IdleSleep"), STAT_HTTPThread_IdleSleep, STATGROUP_HTTPThread);

class FHttpTaskTimerHandleFTSTicker : public IHttpTaskTimerHandle
{
public:
	FHttpTaskTimerHandleFTSTicker(FTSTicker::FDelegateHandle InHandle)
		: Handle(InHandle)
	{
	}

	virtual void RemoveTaskFrom(FHttpThreadBase* HttpThreadBase) 
	{ 
		HttpThreadBase->RemoveTimerHandle(Handle);
	}

private:
	FTSTicker::FDelegateHandle Handle;
};

// FHttpThread

FHttpThreadBase::FHttpThreadBase()
	: Thread(nullptr)
	, bIsSingleThread(false)
	, bIsStopped(true)
	, CurrentThreadPriority(EThreadPriority::TPri_Num)
	, MaxConcurrentRequests(CVarHttpMaxConcurrentRequests.GetValueOnAnyThread())
{
	UE_LOGF(LogInit, Log, "Creating http thread with maximum %d concurrent requests", MaxConcurrentRequests);
}

FHttpThreadBase::~FHttpThreadBase()
{
	StopThread();
}

void FHttpThreadBase::StartThread()
{
	bIsSingleThread = false;

	const bool bDisableForkedHTTPThread = FParse::Param(FCommandLine::Get(), TEXT("DisableForkedHTTPThread"));

	// Get the requested thread priority from the cvar
	CurrentThreadPriority = (EThreadPriority)GHttpThreadPriorities[FMath::Clamp(GHttpThreadPriorityIndex, 0, UE_ARRAY_COUNT(GHttpThreadPriorities)-1)];

	if (FForkProcessHelper::IsForkedMultithreadInstance() && bDisableForkedHTTPThread == false)
	{
		// We only create forkable threads on the forked instance since the HTTPManager cannot safely transition from fake to real seamlessly
		Thread = FForkProcessHelper::CreateForkableThread(this, TEXT("HttpManagerThread"), 128 * 1024, CurrentThreadPriority);
	}
	else
	{
		// If the runnable thread is fake.
		if (FGenericPlatformProcess::SupportsMultithreading() == false)
		{
			bIsSingleThread = true;
		}

		Thread = FRunnableThread::Create(this, TEXT("HttpManagerThread"), 128 * 1024, CurrentThreadPriority);
	}

	bIsStopped = false;
}

void FHttpThreadBase::UpdateThreadPriorityIfNeeded()
{
	if ( !bIsSingleThread && ensure(!IsInGameThread()))
	{
		int32 ThreadPriorityIndex = FMath::Clamp(GHttpThreadPriorityIndex, 0, UE_ARRAY_COUNT(GHttpThreadPriorities) - 1);
		EThreadPriority DesiredThreadPriority = (EThreadPriority)GHttpThreadPriorities[ThreadPriorityIndex];
		if (DesiredThreadPriority != CurrentThreadPriority)
		{
			UE_LOGF(LogHttp, Display, "Updating HTTP thread priority to %ls", GHttpThreadPriortyNames[ThreadPriorityIndex]);
			FPlatformProcess::SetThreadPriority(DesiredThreadPriority);
			CurrentThreadPriority = DesiredThreadPriority;
		}
	}
}

void FHttpThreadBase::StopThread()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}

	bIsStopped = true;
	bIsSingleThread = true;
}

void FHttpThreadBase::AddRequest(FHttpRequestCommon* Request)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpThreadBase_AddRequest);
	NewThreadedRequests.Enqueue(Request);
}

void FHttpThreadBase::CancelRequest(FHttpRequestCommon* Request)
{
	CancelledThreadedRequests.Enqueue(Request);
}

void FHttpThreadBase::GetCompletedRequests(TArray<FHttpRequestCommon*>& OutCompletedRequests)
{
	FHttpRequestCommon* Request = nullptr;
	while (CompletedThreadedRequests.Dequeue(Request))
	{
		OutCompletedRequests.Add(Request);
	}
}

bool FHttpThreadBase::Init()
{
	LastTime = FPlatformTime::Seconds();
	UpdateConfigs();
	return true;
}

uint32 FHttpThreadBase::Run()
{
	return 0;
}

void FHttpThreadBase::Tick()
{
	const double AppTime = FPlatformTime::Seconds();
	const double ElapsedTime = AppTime - LastTime;
	LastTime = AppTime;
	HttpThreadTick(ElapsedTime);
}

bool FHttpThreadBase::NeedsSingleThreadTick() const
{
	return bIsSingleThread;
}

void FHttpThreadBase::UpdateConfigs()
{
}

void FHttpThreadBase::HttpThreadTick(float DeltaSeconds)
{
}

bool FHttpThreadBase::StartThreadedRequest(FHttpRequestCommon* Request)
{
	return Request->StartThreadedRequest();
}

void FHttpThreadBase::CompleteThreadedRequest(FHttpRequestCommon* Request)
{
	// empty
}

void FHttpThreadBase::Stop()
{
	// empty
}

void FHttpThreadBase::Exit()
{
	// empty
}

void FHttpThreadBase::ConsumeCanceledRequestsAndNewRequests(TArray<FHttpRequestCommon*>& RequestsToCancel, TArray<FHttpRequestCommon*>& RequestsToComplete)
{
	// cache all cancelled and new requests
	{
		FHttpRequestCommon* Request = nullptr;

		RequestsToCancel.Reset();
		while (CancelledThreadedRequests.Dequeue(Request))
		{
			RequestsToCancel.Add(Request);
		}

		while (NewThreadedRequests.Dequeue(Request))
		{
			Request->StartWaitingInQueue();

			InsertToRateLimitedRequestsAccordingToPriority(Request);
		}
	}

	// Cancel any pending cancel requests
	for (FHttpRequestCommon* Request : RequestsToCancel)
	{
		if (RunningThreadedRequests.Remove(Request) > 0)
		{
			RequestsToComplete.AddUnique(Request);
		}
		else if (RateLimitedThreadedRequests.Remove(Request) > 0)
		{
			RequestsToComplete.AddUnique(Request);
		}
		else
		{
			// Don't make this a warning as these events can happen frequently when HTTP request timeouts are expected to happen
			UE_LOGF(LogHttp, Log, "Unable to find request (%p) in HttpThread", Request);
		}
	}
}

void FHttpThreadBase::InsertToRateLimitedRequestsAccordingToPriority(FHttpRequestCommon* Request)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpThreadBase_InsertToRateLimitedRequestsAccordingToPriority);

	if (CVarHttpRequestPriorityEnabled.GetValueOnAnyThread())
	{
		for (int32 i = 0; i < RateLimitedThreadedRequests.Num(); ++i)
		{
			FHttpRequestCommon* ExistingRequest = RateLimitedThreadedRequests[i];
			if (ExistingRequest->GetPriority() < Request->GetPriority())
			{
				RateLimitedThreadedRequests.Insert(Request, i);
				return;
			}
		}
	}

	RateLimitedThreadedRequests.Add(Request);
}

void FHttpThreadBase::StartRequestsWaitingInQueue(TArray<FHttpRequestCommon*>& RequestsToComplete)
{
	FHttpManager& HttpManager = FHttpModule::Get().GetHttpManager();

	// We'll start rate limited requests until we hit the limit
	// Tick new requests separately from existing RunningThreadedRequests so they get a chance 
	// to send unaffected by possibly large ElapsedTime above
	int32 RunningThreadedRequestsCounter = RunningThreadedRequests.Num();

#if !UE_HTTP_SUPPORT_TO_INCREASE_MAX_REQUESTS_AT_RUNTIME
	// This will enable shrinking but not growing the max concurrent requests at runtime, on platform where http memory pool was pre-allocated when boot
	if (CVarHttpMaxConcurrentRequests.GetValueOnAnyThread() < MaxConcurrentRequests)
#endif
	{
		MaxConcurrentRequests = CVarHttpMaxConcurrentRequests.GetValueOnAnyThread();
	}

	if (RunningThreadedRequestsCounter < MaxConcurrentRequests)
	{
		while(RunningThreadedRequestsCounter < MaxConcurrentRequests && !RateLimitedThreadedRequests.IsEmpty())
		{
			SCOPE_CYCLE_COUNTER(STAT_HTTPThread_StartThreadedRequest);

			FHttpRequestCommon* ReadyThreadedRequest = RateLimitedThreadedRequests[0];
			RateLimitedThreadedRequests.RemoveAt(0);

			const double DurationInQueue = FPlatformTime::Seconds() - ReadyThreadedRequest->GetTimeStartedWaitingInQueue();
			UE_CLOGF(!FApp::IsUnattended() && DurationInQueue > CVarHttpDurationInQueueToWarnInSeconds.GetValueOnAnyThread(), LogHttp, Warning, "Request (%p) %ls %ls waited in queue for %.2fs before starting", ReadyThreadedRequest, *ReadyThreadedRequest->GetVerb(), *ReadyThreadedRequest->GetURL(), DurationInQueue);
			constexpr float StartImmediately = 0.01f;
			if (DurationInQueue > StartImmediately)
			{
				HttpManager.RecordMaxTimeToWaitInQueue(DurationInQueue);
			}

			CsvRegion(ReadyThreadedRequest, true);
			if (StartThreadedRequest(ReadyThreadedRequest))
			{
				RunningThreadedRequestsCounter++;
				RunningThreadedRequests.Add(ReadyThreadedRequest);
				ReadyThreadedRequest->TickThreadedRequest(0.0f);
				UE_LOGF(LogHttp, Verbose, "Started http request in thread (%p). Waited in queue for (%.2fs) Running threaded requests (%d) Rate limited threaded requests (%d)", 
					ReadyThreadedRequest, DurationInQueue, RunningThreadedRequests.Num(), RateLimitedThreadedRequests.Num());
			}
			else
			{
				RequestsToComplete.AddUnique(ReadyThreadedRequest);
			}
		}
	}

	HttpManager.RecordStatRequestsInFlight(RunningThreadedRequestsCounter);
	if (!RateLimitedThreadedRequests.IsEmpty())
	{
		HttpManager.RecordStatRequestsInQueue(RateLimitedThreadedRequests.Num());
	}
}

void FHttpThreadBase::MoveCompletingRequestsToCompletedRequests(TArray<FHttpRequestCommon*>& RequestsToComplete)
{
	const double AppTime = FPlatformTime::Seconds();
	const double ElapsedTime = AppTime - LastTime;
	LastTime = AppTime;

	// Tick any running requests
	// as long as they properly finish in HttpThreadTick below they are unaffected by a possibly large ElapsedTime above
	for (FHttpRequestCommon* Request : RunningThreadedRequests)
	{
		SCOPE_CYCLE_COUNTER(STAT_HTTPThread_TickThreadedRequest);

		Request->TickThreadedRequest(ElapsedTime);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_HTTPThread_HttpThreadTick);

		// Every valid request in RunningThreadedRequests gets at least two calls to HttpThreadTick
		// Blocking loads still can affect things if the network stack can't keep its connections alive
		HttpThreadTick(ElapsedTime);
	}

	// Move any completed requests
	for (int32 Index = 0; Index < RunningThreadedRequests.Num(); ++Index)
	{
		SCOPE_CYCLE_COUNTER(STAT_HTTPThread_IsThreadedRequestComplete);

		FHttpRequestCommon* Request = RunningThreadedRequests[Index];

		if (Request->IsThreadedRequestComplete())
		{
			RequestsToComplete.AddUnique(Request);
			RunningThreadedRequests.RemoveAtSwap(Index);
			--Index;
			UE_LOGF(LogHttp, Verbose, "Threaded request (%p) completed. Running threaded requests (%d)", Request, RunningThreadedRequests.Num());
		}
	}
}

void FHttpThreadBase::FinishRequestsFromHttpThreadWithCallbacks(TArray<FHttpRequestCommon*>& RequestsToComplete)
{
	if (RequestsToComplete.Num() > 0)
	{
		for (FHttpRequestCommon* Request : RequestsToComplete)
		{
			SCOPE_CYCLE_COUNTER(STAT_HTTPThread_CompleteThreadedRequest);

			CsvRegion(Request, false);
			CompleteThreadedRequest(Request);

			FHttpRequestRef RequestRef = Request->AsShared();

			if (Request->GetDelegateThreadPolicy() == EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread)
			{
				FHttpManager& HttpManager = FHttpModule::Get().GetHttpManager();
				if (CVarHttpRemoveRequestUsingHttpThreadPolicyOnHttpThread.GetValueOnAnyThread())
				{
					HttpManager.RemoveRequest(RequestRef);
				}
				HttpManager.MarkCurrentThreadCompletingRequest(true);
				Request->FinishRequest();
				HttpManager.BroadcastHttpRequestCompleted(RequestRef);
				HttpManager.MarkCurrentThreadCompletingRequest(false);
			}

			if (!CVarHttpRemoveRequestUsingHttpThreadPolicyOnHttpThread.GetValueOnAnyThread() || RequestRef->GetDelegateThreadPolicy() == EHttpRequestDelegateThreadPolicy::CompleteOnGameThread)
			{
				CompletedThreadedRequests.Enqueue(Request);
			}
		}
		RequestsToComplete.Reset();
	}
}

void FHttpThreadBase::Process(TArray<FHttpRequestCommon*>& RequestsToCancel, TArray<FHttpRequestCommon*>& RequestsToComplete)
{
	SCOPE_CYCLE_COUNTER(STAT_HTTPThread_Process);

	UpdateThreadPriorityIfNeeded();

	ConsumeCanceledRequestsAndNewRequests(RequestsToCancel, RequestsToComplete);

	MoveCompletingRequestsToCompletedRequests(RequestsToComplete);

	StartRequestsWaitingInQueue(RequestsToComplete);

	FinishRequestsFromHttpThreadWithCallbacks(RequestsToComplete);
}

FLegacyHttpThread::FLegacyHttpThread()
{
	FPlatformHttp::AddDefaultUserAgentProjectComment(TEXT("http-legacy"));

	HttpThreadActiveFrameTimeInSeconds = FHttpModule::Get().GetHttpThreadActiveFrameTimeInSeconds();
	HttpThreadActiveMinimumSleepTimeInSeconds = FHttpModule::Get().GetHttpThreadActiveMinimumSleepTimeInSeconds();
	HttpThreadIdleFrameTimeInSeconds = FHttpModule::Get().GetHttpThreadIdleFrameTimeInSeconds();
	HttpThreadIdleMinimumSleepTimeInSeconds = FHttpModule::Get().GetHttpThreadIdleMinimumSleepTimeInSeconds();

	UE_LOGF(LogHttp, Log, "HTTP thread active frame time %.1f ms. Minimum active sleep time is %.1f ms. HTTP thread idle frame time %.1f ms. Minimum idle sleep time is %.1f ms.", HttpThreadActiveFrameTimeInSeconds * 1000.0, HttpThreadActiveMinimumSleepTimeInSeconds * 1000.0, HttpThreadIdleFrameTimeInSeconds * 1000.0, HttpThreadIdleMinimumSleepTimeInSeconds * 1000.0);
}

FLegacyHttpThread::~FLegacyHttpThread()
{
}

void FLegacyHttpThread::StartThread()
{
	FHttpThreadBase::StartThread();
}

void FLegacyHttpThread::StopThread()
{
	FHttpThreadBase::StopThread();
}

void FLegacyHttpThread::AddRequest(FHttpRequestCommon* Request)
{
	FHttpThreadBase::AddRequest(Request);
}

void FLegacyHttpThread::CancelRequest(FHttpRequestCommon* Request)
{
	FHttpThreadBase::CancelRequest(Request);
}

void FLegacyHttpThread::GetCompletedRequests(TArray<FHttpRequestCommon*>& OutCompletedRequests)
{
	FHttpThreadBase::GetCompletedRequests(OutCompletedRequests);
}

void FLegacyHttpThread::Tick()
{
	FHttpThreadBase::Tick();

	if (ensure(NeedsSingleThreadTick()))
	{
		TArray<FHttpRequestCommon*> RequestsToCancel;
		TArray<FHttpRequestCommon*> RequestsToComplete;
		Process(RequestsToCancel, RequestsToComplete);
	}
}

TSharedPtr<IHttpTaskTimerHandle> FLegacyHttpThread::AddHttpThreadTask(TFunction<void()>&& Task, float InDelay)
{
	return MakeShared<FHttpTaskTimerHandleFTSTicker>(Ticker.AddTicker(FTickerDelegate::CreateLambda([this, Task=MoveTemp(Task)](float) {
		Task();
		return false;
	}), InDelay));
}

void FLegacyHttpThread::RemoveTimerHandle(FTSTicker::FDelegateHandle DelegateHandle)
{
	Ticker.RemoveTicker(DelegateHandle);
}

void FLegacyHttpThread::RemoveTimerHandle(UE::EventLoop::FTimerHandle EventLoopTimerHandle)
{
	checkNoEntry();
}

void FLegacyHttpThread::HttpThreadTick(float DeltaSeconds)
{
	FHttpThreadBase::HttpThreadTick(DeltaSeconds);

	Ticker.Tick(DeltaSeconds);
}

bool FLegacyHttpThread::Init()
{
	ExitRequest.Set(false);
	return FHttpThreadBase::Init();
}

UE_DISABLE_OPTIMIZATION_SHIP
uint32 FLegacyHttpThread::Run()
{
	// Arrays declared outside of loop to re-use memory
	TArray<FHttpRequestCommon*> RequestsToCancel;
	TArray<FHttpRequestCommon*> RequestsToComplete;
	while (!ExitRequest.GetValue())
	{
		if (ensureMsgf(!NeedsSingleThreadTick(), TEXT("HTTP Thread was set to singlethread mode while it was running autonomously!")))
		{
			const double OuterLoopBegin = FPlatformTime::Seconds();
			double OuterLoopEnd = 0.0;
			bool bKeepProcessing = true;
			while (bKeepProcessing)
			{
				const double InnerLoopBegin = FPlatformTime::Seconds();
			
				Process(RequestsToCancel, RequestsToComplete);
			
				if (RunningThreadedRequests.Num() == 0)
				{
					bKeepProcessing = false;
				}

				const double InnerLoopEnd = FPlatformTime::Seconds();
				if (bKeepProcessing)
				{
					SCOPE_CYCLE_COUNTER(STAT_HTTPThread_ActiveSleep);
					double InnerLoopTime = InnerLoopEnd - InnerLoopBegin;

					// On Windows when optimization enabled, seems InnerLoopEnd can get a value without adding the 
					// const value 16777216.0 from FWindowsPlatformTime::Seoncds(), it could be caused by https://github.com/openssl/openssl/issues/21522
					// Until we upgrade to new openssl to confirm the fix, keep this along with PRAGMA_DISABLE_OPTIMIZATION
					// as an additional step to be safe
					if (InnerLoopTime < 0.0)
					{
						InnerLoopTime = 0.0;
					}

					double InnerSleep = FMath::Max(HttpThreadActiveFrameTimeInSeconds - InnerLoopTime, HttpThreadActiveMinimumSleepTimeInSeconds);
					FPlatformProcess::SleepNoStats(InnerSleep);
				}
				else
				{
					OuterLoopEnd = InnerLoopEnd;
				}
			}
			SCOPE_CYCLE_COUNTER(STAT_HTTPThread_IdleSleep)
			double OuterLoopTime = OuterLoopEnd - OuterLoopBegin;
			double OuterSleep = FMath::Max(HttpThreadIdleFrameTimeInSeconds - OuterLoopTime, HttpThreadIdleMinimumSleepTimeInSeconds);
			FPlatformProcess::SleepNoStats(OuterSleep);
		}
		else
		{
			break;
		}
	}
	return 0;
}
UE_ENABLE_OPTIMIZATION_SHIP

void FLegacyHttpThread::Stop()
{
	FHttpThreadBase::Stop();
	ExitRequest.Set(true);
}
