// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EventLoop/EventLoopTimer.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/Runnable.h"
#include "HttpPackage.h"
#include "Misc/SingleThreadRunnable.h"
#include "Misc/Timespan.h"
#include "Containers/Queue.h"
#include "Containers/SpscQueue.h"
#include "Containers/Ticker.h"

#include <atomic>

class FHttpRequestCommon;
class FHttpThreadBase;

class IHttpTaskTimerHandle
{
public:
	virtual ~IHttpTaskTimerHandle() = default;

	virtual void RemoveTaskFrom(FHttpThreadBase* HttpThreadBase) = 0;
};

/**
 * Manages Http thread
 * Assumes any requests entering the system will remain valid (not deleted) until they exit the system
 */
class FHttpThreadBase
	: FRunnable, FSingleThreadRunnable
{
public:

	FHttpThreadBase();
	virtual ~FHttpThreadBase();

	/**
	 * Start the HTTP thread.
	 */
	virtual void StartThread();

	/**
	 * Stop the HTTP thread.  Blocks until thread has stopped.
	 */
	virtual void StopThread();

	/**
	 * Is the HTTP thread started or stopped.
	 */
	bool IsStopped() const { return bIsStopped; }

	/**
	 * Add a request to begin processing on HTTP thread.
	 *
	 * @param Request the request to be processed on the HTTP thread
	 */
	virtual void AddRequest(FHttpRequestCommon* Request);

	/**
	 * Mark a request as cancelled.    Called on non-HTTP thread.
	 *
	 * @param Request the request to be processed on the HTTP thread
	 */
	virtual void CancelRequest(FHttpRequestCommon* Request);

	/**
	 * Get completed requests.  Clears internal arrays.  Called on non-HTTP thread.
	 *
	 * @param OutCompletedRequests array of requests that have been completed
	 */
	virtual void GetCompletedRequests(TArray<FHttpRequestCommon*>& OutCompletedRequests);

	//~ Begin FSingleThreadRunnable Interface
	virtual void Tick() override;
	//~ End FSingleThreadRunnable Interface

	/**
	 * When true the owner of the HTTPThread needs to manually call Tick() since no automomous threads are
	 * executing the runnable object
	 */
	bool NeedsSingleThreadTick() const;

	/**
	 * Update configuration. Called when config has been updated and we need to apply any changes.
	 */
	virtual void UpdateConfigs();

	/**
	 * Add task to be ran on the http thread next tick
	 *
	 * @param Task The task to be ran
	 * @param InDelay The delay to wait before running the task
	 * @return The handle of the timer, which could be used to remove the task before it get triggered
	 */
	virtual TSharedPtr<IHttpTaskTimerHandle> AddHttpThreadTask(TFunction<void()>&& Task, float InDelay) = 0;

	virtual void RemoveTimerHandle(FTSTicker::FDelegateHandle DelegateHandle) = 0;

	virtual void RemoveTimerHandle(UE::EventLoop::FTimerHandle EventLoopTimerHandle) = 0;

protected:

	/**
	 * Tick on http thread
	 */
	virtual void HttpThreadTick(float DeltaSeconds);
	
	/** 
	 * Start processing a request on the http thread
	 */
	virtual bool StartThreadedRequest(FHttpRequestCommon* Request);

	/** 
	 * Complete a request on the http thread
	 */
	virtual void CompleteThreadedRequest(FHttpRequestCommon* Request);

protected:

	// Threading functions

	//~ Begin FRunnable Interface
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;
	//~ End FRunnable Interface

	void Process(TArray<FHttpRequestCommon*>& RequestsToCancel, TArray<FHttpRequestCommon*>& RequestsToComplete);

	/**
	*  FSingleThreadRunnable accessor for ticking this FRunnable when multi-threading is disabled.
	*  @return FSingleThreadRunnable Interface for this FRunnable object.
	*/
	virtual class FSingleThreadRunnable* GetSingleThreadInterface() override { return this; }

private:
	void ConsumeCanceledRequestsAndNewRequests(TArray<FHttpRequestCommon*>& RequestsToCancel, TArray<FHttpRequestCommon*>& RequestsToComplete);
	void InsertToRateLimitedRequestsAccordingToPriority(FHttpRequestCommon* Request);
	void MoveCompletingRequestsToCompletedRequests(TArray<FHttpRequestCommon*>& RequestsToComplete);
	void StartRequestsWaitingInQueue(TArray<FHttpRequestCommon*>& RequestsToComplete);
	void FinishRequestsFromHttpThreadWithCallbacks(TArray<FHttpRequestCommon*>& RequestsToComplete);
	void UpdateThreadPriorityIfNeeded();

protected:
	/** Pointer to Runnable Thread */
	FRunnableThread* Thread;

private:
	/** Are we holding a fake thread and we need to be ticked manually when Flushing */
	bool bIsSingleThread;

	/** Tells if the runnable thread is running or stopped */
	bool bIsStopped;

	/** Last time the thread has been processed. Used in the non-game thread. */
	double LastTime;

	/** Current thread priority of the thread. Used to detect when a priority change is requested */
	EThreadPriority CurrentThreadPriority;

	/** Max concurrent requests the thread can run, the rest of the requests will wait in the queue */
	int32 MaxConcurrentRequests;

protected:
	/** 
	 * Threaded requests that are waiting to be processed on the http thread.
	 * Added to on (any) non-HTTP thread, processed then cleared on HTTP thread.
	 */
	TMpscQueue<FHttpRequestCommon*> NewThreadedRequests;

	/**
	 * Threaded requests that are waiting to be cancelled on the http thread.
	 * Added to on (any) non-HTTP thread, processed then cleared on HTTP thread.
	 */
	TMpscQueue<FHttpRequestCommon*> CancelledThreadedRequests;

	/**
	 * Threaded requests that are ready to run, but waiting due to the running request limit (not in any of the other lists, except potentially CancelledThreadedRequests).
	 * Only accessed on the HTTP thread.
	 */
	TArray<FHttpRequestCommon*> RateLimitedThreadedRequests;

	/**
	 * Currently running threaded requests (not in any of the other lists, except potentially CancelledThreadedRequests).
	 * Only accessed on the HTTP thread.
	 */
	TArray<FHttpRequestCommon*> RunningThreadedRequests;

	/**
	 * Threaded requests that have completed and are waiting for the game thread to process.
	 * Added to on HTTP thread, processed then cleared on game thread (Single producer, single consumer)
	 */
	TSpscQueue<FHttpRequestCommon*> CompletedThreadedRequests;
};

class FLegacyHttpThread	: public FHttpThreadBase
{
public:

	FLegacyHttpThread();
	virtual ~FLegacyHttpThread();

	virtual void StartThread() override final;
	virtual void StopThread() override final;
	virtual void AddRequest(FHttpRequestCommon* Request) override final;
	virtual void CancelRequest(FHttpRequestCommon* Request) override final;
	virtual void GetCompletedRequests(TArray<FHttpRequestCommon*>& OutCompletedRequests) override final;

	//~ Begin FSingleThreadRunnable Interface
	// Cannot be overridden to ensure identical behavior with the threaded tick
	virtual void Tick() override final;
	//~ End FSingleThreadRunnable Interface

protected:
	//~ Begin FRunnable Interface
	virtual bool Init() override;
	// Cannot be overridden to ensure identical behavior with the single threaded tick
	virtual uint32 Run() override final;
	virtual void Stop() override;
	//~ End FRunnable Interface

	virtual TSharedPtr<IHttpTaskTimerHandle> AddHttpThreadTask(TFunction<void()>&& Task, float InDelay) override;
	virtual void HttpThreadTick(float DeltaSeconds) override;

	virtual void RemoveTimerHandle(FTSTicker::FDelegateHandle DelegateHandle) override;
	virtual void RemoveTimerHandle(UE::EventLoop::FTimerHandle EventLoopTimerHandle) override;

	/** signal request to stop and exit thread */
	FThreadSafeCounter ExitRequest;

	/** Time in seconds to use as frame time when actively processing requests. 0 means no frame time. */
	double HttpThreadActiveFrameTimeInSeconds;
	/** Time in seconds to sleep minimally when actively processing requests. */
	double HttpThreadActiveMinimumSleepTimeInSeconds;
	/** Time in seconds to use as frame time when idle, waiting for requests. 0 means no frame time. */
	double HttpThreadIdleFrameTimeInSeconds;
	/** Time in seconds to sleep minimally when idle, waiting for requests. */
	double HttpThreadIdleMinimumSleepTimeInSeconds;

	/* Ticker for functions to run in HTTP thread */
	FTSTicker Ticker;
};
