// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IHttpRequest.h"

namespace HttpRequestOptions
{
	const FName HttpVersion("HttpVersion");
	const FName RequestMode("RequestMode");
#if UE_HTTP_SUPPORT_UNIX_SOCKET
	const FName UnixSocketPath("UnixSocketPath");
#endif //UE_HTTP_SUPPORT_UNIX_SOCKET
	const FName EnableCookies("EnableCookies");
}

const TCHAR* LexToString(EHttpRequestMode Mode)
{
	switch (Mode)
	{
		case EHttpRequestMode::ThreadedRequest: return TEXT("ThreadedRequest");
		case EHttpRequestMode::ImmediateRequest: return TEXT("ImmediateRequest");

		default:
		checkf(false, TEXT("Please, implement new modes here"));
		return TEXT("Unknown");
	}
}

class FArchiveWithDelegateV2 final : public FArchive
{
public:
	FArchiveWithDelegateV2(FHttpRequestStreamDelegateV2 InStreamDelegateV2)
		: StreamDelegateV2(InStreamDelegateV2)
	{
	}

	virtual void Serialize(void* V, int64 Length) override
	{
		int64 LengthProcessed = Length;
		StreamDelegateV2.ExecuteIfBound(V, LengthProcessed);
		if (LengthProcessed != Length)
		{
			SetError();
		}
	}

private:
	FHttpRequestStreamDelegateV2 StreamDelegateV2;
};

bool IHttpRequest::SetResponseBodyReceiveStreamDelegateV2(FHttpRequestStreamDelegateV2 StreamDelegate) 
{ 
	return SetResponseBodyReceiveStream(MakeShared<FArchiveWithDelegateV2>(StreamDelegate)); 
}
