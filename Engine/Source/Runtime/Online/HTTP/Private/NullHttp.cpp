// Copyright Epic Games, Inc. All Rights Reserved.

#include "NullHttp.h"
#include "HttpManager.h"
#include "HttpModule.h"
#include "Http.h"

// FNullHttpRequest

const FString& FNullHttpRequest::GetURL() const
{
	return Url;
}

FString FNullHttpRequest::GetURLParameter(const FString& ParameterName) const
{
	return FString();
}

FString FNullHttpRequest::GetHeader(const FString& HeaderName) const
{
	const FString* Header = Headers.Find(HeaderName);
	if (Header != NULL)
	{
		return *Header;
	}
	return FString();
}

TArray<FString> FNullHttpRequest::GetAllHeaders() const
{
	TArray<FString> Result;
	for (TMap<FString, FString>::TConstIterator It(Headers); It; ++It)
	{
		Result.Add(It.Key() + TEXT(": ") + It.Value());
	}
	return Result;
}

FString FNullHttpRequest::GetContentType() const
{
	return GetHeader(TEXT("Content-Type"));
}

uint64 FNullHttpRequest::GetContentLength() const
{
	return Payload.Num();
}

const TArray<uint8>& FNullHttpRequest::GetContent() const
{
	return Payload;
}

FString FNullHttpRequest::GetVerb() const
{
	return Verb;
}

void FNullHttpRequest::SetVerb(const FString& InVerb)
{
	Verb = InVerb;
}

void FNullHttpRequest::SetURL(const FString& InURL)
{
	Url = InURL;
}

FString FNullHttpRequest::GetOption(const FName Option) const
{
	const FString* OptionValue = Options.Find(Option);
	if (OptionValue)
	{
		return *OptionValue;
	}
	return TEXT("");
}

void FNullHttpRequest::SetOption(const FName Option, const FString& OptionValue)
{
	Options.Add(Option, OptionValue);
}

void FNullHttpRequest::SetContent(const TArray<uint8>& ContentPayload)
{
	Payload = ContentPayload;
}

void FNullHttpRequest::SetContent(TArray<uint8>&& ContentPayload)
{
	Payload = MoveTemp(ContentPayload);
}

void FNullHttpRequest::SetContentAsString(const FString& ContentString)
{
	int32 Utf8Length = FPlatformString::ConvertedLength<UTF8CHAR>(*ContentString, ContentString.Len());
	Payload.SetNumUninitialized(Utf8Length);
	FPlatformString::Convert((UTF8CHAR*)Payload.GetData(), Payload.Num(), *ContentString, ContentString.Len());
}

bool FNullHttpRequest::SetContentAsStreamedFile(const FString& Filename)
{
	UE_LOGF(LogHttp, Warning, "FNullHttpRequest::SetContentAsStreamedFile is not implemented");
	return false;
}

bool FNullHttpRequest::SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream)
{
	// TODO: Not implemented.
	UE_LOGF(LogHttp, Warning, "FNullHttpRequest::SetContentFromStream is not implemented");
	return false;
}

bool FNullHttpRequest::SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream)
{
	UE_LOGF(LogHttp, Warning, "FNullHttpRequest::SetResponseBodyReceiveStream is not implemented");
	return false;
}

void FNullHttpRequest::SetHeader(const FString& HeaderName, const FString& HeaderValue)
{
	Headers.Add(HeaderName, HeaderValue);
}

void FNullHttpRequest::AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue)
{
	if (HeaderName.Len() > 0 && AdditionalHeaderValue.Len() > 0)
	{
		FString* PreviousValue = Headers.Find(HeaderName);
		FString NewValue;
		if (PreviousValue != NULL && PreviousValue->Len() > 0)
		{
			NewValue = (*PreviousValue) + TEXT(", ");
		}
		NewValue += AdditionalHeaderValue;

		SetHeader(HeaderName, NewValue);
	}
}

bool FNullHttpRequest::ProcessRequest()
{
	ElapsedTime = 0;
	CompletionStatus = EHttpRequestStatus::Processing;

	UE_LOGF(LogHttp, Log, "Start request. %p %ls url=%ls", this, *GetVerb(), *GetURL());
	return true;
}

void FNullHttpRequest::CancelRequest()
{
	if (!IsInGameThread())
	{
		FHttpModule::Get().GetHttpManager().AddGameThreadTask([StrongThis = StaticCastSharedRef<FNullHttpRequest>(AsShared())]()
		{
			StrongThis->FinishedRequest();
		});
	}
	else
	{
		FinishedRequest();
	}
}

EHttpRequestStatus::Type FNullHttpRequest::GetStatus() const
{
	return CompletionStatus;
}

EHttpFailureReason FNullHttpRequest::GetFailureReason() const
{
	return FailureReason;
}

const FString& FNullHttpRequest::GetEffectiveURL() const
{
	return EffectiveUrl;
}

const FHttpResponsePtr FNullHttpRequest::GetResponse() const
{
	return FHttpResponsePtr(nullptr);
}

void FNullHttpRequest::Tick(float DeltaSeconds)
{
	if (CompletionStatus == EHttpRequestStatus::Processing)
	{
		ElapsedTime += DeltaSeconds;
		const float HttpTimeout = GetTimeout().Get(FHttpModule::Get().GetHttpTotalTimeout());
		if (HttpTimeout > 0 && ElapsedTime >= HttpTimeout)
		{
			UE_LOGF(LogHttp, Warning, "Timeout processing Http request. %p",
				this);

			FinishedRequest();
		}
	}
}

float FNullHttpRequest::GetElapsedTime() const
{
	return ElapsedTime;
}

void FNullHttpRequest::FinishedRequest()
{
	CompletionStatus = EHttpRequestStatus::Failed;
	FailureReason = EHttpFailureReason::Other;

	UE_LOGF(LogHttp, Log, "Finished request %p. no response %ls url=%ls elapsed=%.3f",
		this, *GetVerb(), *GetURL(), ElapsedTime);

	OnProcessRequestComplete().ExecuteIfBound(SharedThis(this), NULL, false);
}

void FNullHttpRequest::SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy)
{
}

EHttpRequestDelegateThreadPolicy FNullHttpRequest::GetDelegateThreadPolicy() const
{
	return EHttpRequestDelegateThreadPolicy::CompleteOnGameThread;
}

void FNullHttpRequest::SetPriority(EHttpRequestPriority InPriority)
{
}

EHttpRequestPriority FNullHttpRequest::GetPriority() const
{
	return EHttpRequestPriority::Normal;
}

void FNullHttpRequest::SetTimeout(float InTimeoutSecs) 
{
	TimeoutSecs = InTimeoutSecs;
}

void FNullHttpRequest::ClearTimeout() 
{
	TimeoutSecs.Reset();
}

void FNullHttpRequest::ResetTimeoutStatus() 
{
}

TOptional<float> FNullHttpRequest::GetTimeout() const 
{ 
	return TimeoutSecs; 
}

void FNullHttpRequest::SetActivityTimeout(float InTimeoutSecs)
{
}

void FNullHttpRequest::ProcessRequestUntilComplete()
{
}

// FNullHttpResponse

const FString& FNullHttpResponse::GetURL() const
{
	return URL;
}

FString FNullHttpResponse::GetURLParameter(const FString& ParameterName) const
{
	return FString();
}

FString FNullHttpResponse::GetHeader(const FString& HeaderName) const
{
	return FString();
}

TArray<FString> FNullHttpResponse::GetAllHeaders() const
{
	return TArray<FString>();
}

FString FNullHttpResponse::GetContentType() const
{
	return FString();
}

uint64 FNullHttpResponse::GetContentLength() const
{
	return 0;
}

const TArray<uint8>& FNullHttpResponse::GetContent() const
{
	return Payload;
}

int32 FNullHttpResponse::GetResponseCode() const
{
	return 0;
}

FString FNullHttpResponse::GetContentAsString() const
{
	return FString();
}

FUtf8StringView FNullHttpResponse::GetContentAsUtf8StringView() const
{
	return FUtf8StringView();
}
