// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/HttpRequestImpl.h"
#include "Interfaces/IHttpResponse.h"

/**
 * Null (mock) implementation of an HTTP request
 */
class FNullHttpRequest : public FHttpRequestImpl
{
public:

	// IHttpBase
	virtual const FString& GetURL() const override;
	virtual FString GetURLParameter(const FString& ParameterName) const override;
	virtual FString GetHeader(const FString& HeaderName) const override;
	virtual TArray<FString> GetAllHeaders() const override;	
	virtual FString GetContentType() const override;
	virtual uint64 GetContentLength() const override;
	virtual const TArray<uint8>& GetContent() const override;
	// IHttpRequest 
	virtual FString GetVerb() const override;
	virtual void SetVerb(const FString& InVerb) override;
	virtual void SetURL(const FString& InURL) override;
	virtual FString GetOption(const FName Option) const override;
	virtual void SetOption(const FName Option, const FString& OptionValue) override;
	virtual void SetContent(const TArray<uint8>& ContentPayload) override;
	virtual void SetContent(TArray<uint8>&& ContentPayload) override;
	virtual void SetContentAsString(const FString& ContentString) override;
    virtual bool SetContentAsStreamedFile(const FString& Filename) override;
	virtual bool SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) override;
	virtual bool SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) override;
	virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) override;
	virtual void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) override;
	virtual bool ProcessRequest() override;
	virtual void CancelRequest() override;
	virtual EHttpRequestStatus::Type GetStatus() const override;
	virtual EHttpFailureReason GetFailureReason() const override;
	virtual const FString& GetEffectiveURL() const override;
	virtual const FHttpResponsePtr GetResponse() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual float GetElapsedTime() const override;
	virtual void SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy) override;
	virtual EHttpRequestDelegateThreadPolicy GetDelegateThreadPolicy() const override;
	virtual void SetPriority(EHttpRequestPriority InPriority) override;
	virtual EHttpRequestPriority GetPriority() const override;
	virtual void SetTimeout(float InTimeoutSecs) override;
	virtual void ClearTimeout() override;
	virtual void ResetTimeoutStatus() override;
	virtual TOptional<float> GetTimeout() const override;
	virtual void SetActivityTimeout(float InTimeoutSecs) override;
	virtual void ProcessRequestUntilComplete() override;

	FNullHttpRequest()
		: CompletionStatus(EHttpRequestStatus::NotStarted)
		, FailureReason(EHttpFailureReason::None)
		, ElapsedTime(0)
	{}
	virtual ~FNullHttpRequest() = default;

private:
	void FinishedRequest();

	FString Url;
	FString EffectiveUrl;
	FString Verb;
	TArray<uint8> Payload;
	EHttpRequestStatus::Type CompletionStatus;
	EHttpFailureReason FailureReason;
	TMap<FString, FString> Headers;
	TMap<const FName, FString> Options;
	float ElapsedTime;
	TOptional<float> TimeoutSecs;
};

/**
 * Null (mock) implementation of an HTTP request
 */
class FNullHttpResponse : public IHttpResponse
{
	// IHttpBase 
	virtual const FString& GetURL() const override;
	virtual FString GetURLParameter(const FString& ParameterName) const override;
	virtual FString GetHeader(const FString& HeaderName) const override;
	virtual TArray<FString> GetAllHeaders() const override;	
	virtual FString GetContentType() const override;
	virtual uint64 GetContentLength() const override;
	virtual const TArray<uint8>& GetContent() const override;
	//~ Begin IHttpResponse Interface
	virtual int32 GetResponseCode() const override;
	virtual FString GetContentAsString() const override;
	virtual FUtf8StringView GetContentAsUtf8StringView() const override;

	FNullHttpResponse() {}
	virtual ~FNullHttpResponse() = default;

private:
	TArray<uint8> Payload;
	FString URL;
};
