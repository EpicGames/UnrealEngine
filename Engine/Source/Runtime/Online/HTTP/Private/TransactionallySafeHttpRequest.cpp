// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransactionallySafeHttpRequest.h"
#include "Http.h"
#include "PlatformHttp.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "HAL/Platform.h"
#include "Logging/StructuredLog.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"

class FTransactionallySafeHttpRequest::FClosedHttpRequest : public IHttpRequest
{
public:
	FClosedHttpRequest(FTransactionallySafeHttpRequest* Owner)
	{
		AutoRTFM::PushOnCommitHandler(this, [this, Owner]
		{
			TSharedPtr<IHttpRequest> Req{FPlatformHttp::ConstructRequest()};
			
			Req->SetDelegateThreadPolicy(ThreadPolicy);
			Req->SetPriority(Priority);
			Req->SetVerb(MoveTemp(Verb));

			if (!Url.IsEmpty())
			{
				Req->SetURL(MoveTemp(Url));
			}
			for (TPair<const FName, FString>& Option : Options)
			{
				Req->SetOption(Option.Key, MoveTemp(Option.Value));
			}
			for (TPair<FString, FString>& Header : Headers)
			{
				Req->SetHeader(MoveTemp(Header.Key), MoveTemp(Header.Value));
			}
			if (ResponseBodyReceiveStream)
			{
				Req->SetResponseBodyReceiveStream(ResponseBodyReceiveStream.ToSharedRef());
			}
			if (TimeoutSecs.IsSet())
			{
				Req->SetTimeout(TimeoutSecs.GetValue());
			}
			if (ActivityTimeoutSecs.IsSet())
			{
				Req->SetActivityTimeout(ActivityTimeoutSecs.GetValue());
			}
			if (CompleteDelegate.IsSet())
			{
				Req->OnProcessRequestComplete() = MoveTemp(CompleteDelegate.GetValue());
			}
			if (ProgressDelegate.IsSet())
			{
				Req->OnRequestProgress64() = MoveTemp(ProgressDelegate.GetValue());
			}
			if (WillRetryDelegate.IsSet())
			{
				Req->OnRequestWillRetry() = MoveTemp(WillRetryDelegate.GetValue());
			}
			if (HeaderReceivedDelegate.IsSet())
			{
				Req->OnHeaderReceived() = MoveTemp(HeaderReceivedDelegate.GetValue());
			}
			if (StatusCodeReceivedDelegate.IsSet())
			{
				Req->OnStatusCodeReceived() = MoveTemp(StatusCodeReceivedDelegate.GetValue());
			}
			if (Payload.IsType<RawPayload>())
			{
				Req->SetContent(MoveTemp(Payload.Get<RawPayload>().Content));
			}
			else if (Payload.IsType<FilePayload>())
			{
				Req->SetContentAsStreamedFile(Payload.Get<FilePayload>().Filename);
			}
			else if (Payload.IsType<StreamPayload>())
			{
				Req->SetContentFromStream(MoveTemp(Payload.Get<StreamPayload>().Stream));
			}
			if (bProcessRequest)
			{
				Req->ProcessRequest();
			}

			Owner->InnerRequest = MoveTemp(Req);
		});
	}

	~FClosedHttpRequest()
	{
		// If the request is destroyed before the transaction is committed, 
		// there's nothing more to do.
		AutoRTFM::PopOnCommitHandler(this);
	}

	const FString& GetURL() const override
	{
		return Url;
	}

	void SetURL(const FString& InURL) override
	{
		Url = InURL;
	}

	void SetHeader(const FString& HeaderName, const FString& HeaderValue) override
	{
		Headers.Add(HeaderName, HeaderValue);
	}

	FString GetHeader(const FString& HeaderName) const override
	{
		const FString* Header = Headers.Find(HeaderName);
		return Header ? *Header : FString();
	}

	void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) override
	{
		ensure(!"Implement AddToHeader() if it becomes necessary.");
	}

	TArray<FString> GetAllHeaders() const override
	{
		ensure(!"Implement GetAllHeaders() if it becomes necessary.");
		return TArray<FString>{};
	}

	FString GetVerb() const override
	{
		return Verb;
	}

	void SetVerb(const FString& InVerb) override
	{
		Verb = InVerb;
	}

	FString GetOption(const FName Option) const override
	{
		const FString* OptionValue = Options.Find(Option);
		return OptionValue ? *OptionValue : FString();
	}

	void SetOption(const FName Option, const FString& OptionValue) override
	{
		Options.Add(Option, OptionValue);
	}

	void SetContent(const TArray<uint8>& InPayload) override
	{
		Payload.Set<RawPayload>({ InPayload });
	}

	void SetContent(TArray<uint8>&& InPayload) override
	{
		Payload.Set<RawPayload>({ MoveTemp(InPayload) });
	}

	const TArray<uint8>& GetContent() const override
	{
		return Payload.IsType<RawPayload>() ? Payload.Get<RawPayload>().Content
			                                : EmptyContent;
	}

	void SetContentAsString(const FString& ContentString) override
	{
		FTCHARToUTF8 Converter(*ContentString);
		TArray<uint8> ContentStringAsUTF8;
		ContentStringAsUTF8.SetNum(Converter.Length());
		FMemory::Memcpy(ContentStringAsUTF8.GetData(), (const uint8*)Converter.Get(), ContentStringAsUTF8.Num());
		Payload.Set<RawPayload>({ MoveTemp(ContentStringAsUTF8) });
	}

	bool SetContentAsStreamedFile(const FString& Filename) override
	{
		Payload.Set<FilePayload>({Filename});
		return true;
	}

	bool SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) override
	{
		Payload.Set<StreamPayload>({MoveTemp(Stream)});
		return true;
	}

	EHttpRequestStatus::Type GetStatus() const override
	{
		return EHttpRequestStatus::NotStarted;
	}

	const FString& GetEffectiveURL() const override
	{
		// The effective URL will always be an empty string at this point in the request lifecycle.
		return EmptyString;
	}

	FString GetURLParameter(const FString& ParameterName) const override
	{
		TOptional<FString> Param = FGenericPlatformHttp::GetUrlParameter(Url, ParameterName);
		return Param.Get(FString());
	}

	uint64 GetContentLength() const override
	{
		if (Payload.IsType<RawPayload>())
		{
			return Payload.Get<RawPayload>().Content.Num();
		}
		if (Payload.IsType<StreamPayload>())
		{
			return Payload.Get<StreamPayload>().Stream->TotalSize();
		}
		return 0;
	}

	FString GetContentType() const override
	{
		return GetHeader(TEXT("Content-Type"));
	}

	bool SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) override
	{
		ResponseBodyReceiveStream = MoveTemp(Stream);
		return true;
	}

	EHttpFailureReason GetFailureReason() const override
	{
		return EHttpFailureReason::None;
	}

	const FHttpResponsePtr GetResponse() const override
	{
		return nullptr;
	}

	void Tick(float) override
	{
		ensure(!"Tick() shouldn't be called on a FClosedHttpRequest.");
	}

	float GetElapsedTime() const override
	{
		return 0.0f;
	}

	void SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy) override
	{
		ThreadPolicy = InThreadPolicy;
	}

	EHttpRequestDelegateThreadPolicy GetDelegateThreadPolicy() const override
	{
		return ThreadPolicy;
	}

	void SetPriority(EHttpRequestPriority InPriority) override
	{
		Priority = InPriority;
	}

	EHttpRequestPriority GetPriority() const override
	{
		return Priority;
	}

	bool ProcessRequest() override
	{
		bProcessRequest = true;
		return true;
	}

	void CancelRequest() override
	{
		check(!"CancelRequest() shouldn't be called on a FClosedHttpRequest.");
	}

	void ProcessRequestUntilComplete() override
	{
		// We can't do a blocking HTTP load inside of a transaction. We don't know if the transaction
		// will succeed or not at this point, so we can't issue the HTTP request.
		// If we reach this point, the code needs to be restructured to use a non-blocking load.
		check(!"ProcessRequestUntilComplete shouldn't be called on a FClosedHttpRequest.");
		abort();
	}

	void SetTimeout(float InTimeoutSecs) override
	{
		TimeoutSecs = InTimeoutSecs;
	}

	void SetActivityTimeout(float InTimeoutSecs) override
	{
		ActivityTimeoutSecs = InTimeoutSecs;
	}

	void ClearTimeout() override
	{
		TimeoutSecs.Reset();
	}

	void ResetTimeoutStatus() override
	{
	}

	TOptional<float> GetTimeout() const override
	{
		return TimeoutSecs;
	}

	FHttpRequestCompleteDelegate& OnProcessRequestComplete() override
	{
		return CompleteDelegate.IsSet() ? CompleteDelegate.GetValue()
			                            : CompleteDelegate.Emplace();
	}

	FHttpRequestProgressDelegate64& OnRequestProgress64() override
	{
		return ProgressDelegate.IsSet() ? ProgressDelegate.GetValue()
			                            : ProgressDelegate.Emplace();
	}

	FHttpRequestWillRetryDelegate& OnRequestWillRetry() override
	{
		return WillRetryDelegate.IsSet() ? WillRetryDelegate.GetValue()
			                             : WillRetryDelegate.Emplace();
	}

	FHttpRequestHeaderReceivedDelegate& OnHeaderReceived() override
	{
		return HeaderReceivedDelegate.IsSet() ? HeaderReceivedDelegate.GetValue()
			                                  : HeaderReceivedDelegate.Emplace();
	}

	FHttpRequestStatusCodeReceivedDelegate& OnStatusCodeReceived() override
	{
		return StatusCodeReceivedDelegate.IsSet() ? StatusCodeReceivedDelegate.GetValue()
			                                      : StatusCodeReceivedDelegate.Emplace();
	}

private:
	FString Url;
	FString Verb = "GET";
	TMap<const FName, FString> Options;
	TMap<FString, FString> Headers;
	TSharedPtr<FArchive> ResponseBodyReceiveStream;
	EHttpRequestDelegateThreadPolicy ThreadPolicy = EHttpRequestDelegateThreadPolicy::CompleteOnGameThread;
	EHttpRequestPriority Priority = EHttpRequestPriority::Normal;
	TOptional<float> TimeoutSecs;
	TOptional<float> ActivityTimeoutSecs;
	TOptional<FHttpRequestCompleteDelegate> CompleteDelegate;
	TOptional<FHttpRequestProgressDelegate64> ProgressDelegate;
	TOptional<FHttpRequestWillRetryDelegate> WillRetryDelegate;
	TOptional<FHttpRequestHeaderReceivedDelegate> HeaderReceivedDelegate;
	TOptional<FHttpRequestStatusCodeReceivedDelegate> StatusCodeReceivedDelegate;
	bool bProcessRequest = false;

	const TArray<uint8> EmptyContent;
	const FString EmptyString;

	// The caller can specify a number of different payload types.
	struct NoPayload {};
	struct RawPayload { TArray<uint8> Content; };
	struct FilePayload { FString Filename; };
	struct StreamPayload { TSharedRef<FArchive, ESPMode::ThreadSafe> Stream; };

	TVariant<NoPayload, RawPayload, FilePayload, StreamPayload> Payload;
};

FTransactionallySafeHttpRequest::FTransactionallySafeHttpRequest()
{
	InnerRequest = AutoRTFM::IsClosed() ? MakeShared<FTransactionallySafeHttpRequest::FClosedHttpRequest>(this)
                                        : TSharedPtr<IHttpRequest>(FPlatformHttp::ConstructRequest());
}

const FString& FTransactionallySafeHttpRequest::GetURL() const 
{
	return InnerRequest->GetURL();
}

void FTransactionallySafeHttpRequest::SetURL(const FString& InURL) 
{
	InnerRequest->SetURL(InURL);
}

void FTransactionallySafeHttpRequest::SetHeader(const FString& HeaderName, const FString& HeaderValue) 
{
	InnerRequest->SetHeader(HeaderName, HeaderValue);
}

FString FTransactionallySafeHttpRequest::GetHeader(const FString& HeaderName) const 
{
	return InnerRequest->GetHeader(HeaderName);
}

TArray<FString> FTransactionallySafeHttpRequest::GetAllHeaders() const 
{
	return InnerRequest->GetAllHeaders();
}

FString FTransactionallySafeHttpRequest::GetVerb() const
{
	return InnerRequest->GetVerb();
}

void FTransactionallySafeHttpRequest::SetVerb(const FString& InVerb) 
{
	InnerRequest->SetVerb(InVerb);
}

FString FTransactionallySafeHttpRequest::GetOption(const FName Option) const 
{
	return InnerRequest->GetOption(Option);
}

void FTransactionallySafeHttpRequest::SetOption(const FName Option, const FString& OptionValue)
{
	InnerRequest->SetOption(Option, OptionValue);
}

void FTransactionallySafeHttpRequest::SetContent(const TArray<uint8>& InPayload)
{
	InnerRequest->SetContent(InPayload);
}

void FTransactionallySafeHttpRequest::SetContent(TArray<uint8>&& InPayload)
{
	InnerRequest->SetContent(MoveTemp(InPayload));
}

const TArray<uint8>& FTransactionallySafeHttpRequest::GetContent() const 
{
	return InnerRequest->GetContent();
}

void FTransactionallySafeHttpRequest::SetContentAsString(const FString& ContentString) 
{
	InnerRequest->SetContentAsString(ContentString);
}

bool FTransactionallySafeHttpRequest::SetContentAsStreamedFile(const FString& Filename) 
{
	return InnerRequest->SetContentAsStreamedFile(Filename);
}

bool FTransactionallySafeHttpRequest::SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream)
{
	return InnerRequest->SetContentFromStream(Stream);
}

EHttpRequestStatus::Type FTransactionallySafeHttpRequest::GetStatus() const
{
	return InnerRequest->GetStatus();
}

const FString& FTransactionallySafeHttpRequest::GetEffectiveURL() const
{
	return InnerRequest->GetEffectiveURL();
}

FString FTransactionallySafeHttpRequest::GetURLParameter(const FString& ParameterName) const
{
	return InnerRequest->GetURLParameter(ParameterName);
}

uint64 FTransactionallySafeHttpRequest::GetContentLength() const 
{
	return InnerRequest->GetContentLength();
}

FString FTransactionallySafeHttpRequest::GetContentType() const 
{
	return InnerRequest->GetContentType();
}

bool FTransactionallySafeHttpRequest::SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream)
{
	return InnerRequest->SetResponseBodyReceiveStream(MoveTemp(Stream));
}

void FTransactionallySafeHttpRequest::AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) 
{
	InnerRequest->AppendToHeader(HeaderName, AdditionalHeaderValue);
}

bool FTransactionallySafeHttpRequest::ProcessRequest() 
{
	return InnerRequest->ProcessRequest();
}

void FTransactionallySafeHttpRequest::CancelRequest() 
{
	InnerRequest->CancelRequest();
}

EHttpFailureReason FTransactionallySafeHttpRequest::GetFailureReason() const
{
	return InnerRequest->GetFailureReason();
}

const FHttpResponsePtr FTransactionallySafeHttpRequest::GetResponse() const
{
	return InnerRequest->GetResponse();
}

void FTransactionallySafeHttpRequest::Tick(float DeltaSeconds)
{
	InnerRequest->Tick(DeltaSeconds);
}

float FTransactionallySafeHttpRequest::GetElapsedTime() const
{
	return InnerRequest->GetElapsedTime();
}

void FTransactionallySafeHttpRequest::SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy)
{
	InnerRequest->SetDelegateThreadPolicy(InThreadPolicy);
}

EHttpRequestDelegateThreadPolicy FTransactionallySafeHttpRequest::GetDelegateThreadPolicy() const
{
	return InnerRequest->GetDelegateThreadPolicy();
}

void FTransactionallySafeHttpRequest::SetPriority(EHttpRequestPriority InPriority)
{
	InnerRequest->SetPriority(InPriority);
}

EHttpRequestPriority FTransactionallySafeHttpRequest::GetPriority() const
{
	return InnerRequest->GetPriority();
}

void FTransactionallySafeHttpRequest::SetTimeout(float InTimeoutSecs)
{
	InnerRequest->SetTimeout(InTimeoutSecs);
}

void FTransactionallySafeHttpRequest::ClearTimeout() 
{
	InnerRequest->ClearTimeout();
}

void FTransactionallySafeHttpRequest::ResetTimeoutStatus() 
{
	InnerRequest->ResetTimeoutStatus();
}

TOptional<float> FTransactionallySafeHttpRequest::GetTimeout() const 
{
	return InnerRequest->GetTimeout();
}

void FTransactionallySafeHttpRequest::SetActivityTimeout(float InTimeoutSecs) 
{
	InnerRequest->SetActivityTimeout(InTimeoutSecs);
}

void FTransactionallySafeHttpRequest::ProcessRequestUntilComplete()
{
	InnerRequest->ProcessRequestUntilComplete();
}

FHttpRequestCompleteDelegate& FTransactionallySafeHttpRequest::OnProcessRequestComplete() 
{
	return InnerRequest->OnProcessRequestComplete();
}

FHttpRequestProgressDelegate64& FTransactionallySafeHttpRequest::OnRequestProgress64()
{
	return InnerRequest->OnRequestProgress64();
}

FHttpRequestWillRetryDelegate& FTransactionallySafeHttpRequest::OnRequestWillRetry()
{
	return InnerRequest->OnRequestWillRetry();
}

FHttpRequestHeaderReceivedDelegate& FTransactionallySafeHttpRequest::OnHeaderReceived()
{
	return InnerRequest->OnHeaderReceived();
}

FHttpRequestStatusCodeReceivedDelegate& FTransactionallySafeHttpRequest::OnStatusCodeReceived()
{
	return InnerRequest->OnStatusCodeReceived();
}
