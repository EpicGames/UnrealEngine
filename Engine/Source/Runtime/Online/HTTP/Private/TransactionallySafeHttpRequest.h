// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/HttpRequestImpl.h"
#include "Interfaces/IHttpResponse.h"
#include "Templates/UniquePtr.h"

/**
 * Wraps a FPlatformHttp request when one is created inside an AutoRTFM transaction. 
 * Basic getters and setters are cached and played back when the transaction succeeds.
 * Once the transaction is committed, a real HttpRequest is instantiated and all calls
 * are passed through as-is.
 */
class FTransactionallySafeHttpRequest : public IHttpRequest
{
public:
	FTransactionallySafeHttpRequest();
	virtual ~FTransactionallySafeHttpRequest() = default;

	// IHttpBase
	const FString& GetURL() const override;
	FString GetURLParameter(const FString& ParameterName) const override;
	FString GetHeader(const FString& HeaderName) const override;
	TArray<FString> GetAllHeaders() const override;	
	FString GetContentType() const override;
	uint64 GetContentLength() const override;
	const TArray<uint8>& GetContent() const override;

	// IHttpRequest 
	FString GetVerb() const override;
	void SetVerb(const FString& InVerb) override;
	void SetURL(const FString& InURL) override;
	FString GetOption(const FName Option) const override;
	void SetOption(const FName Option, const FString& OptionValue) override;
	void SetContent(const TArray<uint8>& ContentPayload) override;
	void SetContent(TArray<uint8>&& ContentPayload) override;
	void SetContentAsString(const FString& ContentString) override;
    bool SetContentAsStreamedFile(const FString& Filename) override;
	bool SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) override;
	bool SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) override;
	void SetHeader(const FString& HeaderName, const FString& HeaderValue) override;
	void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) override;
	bool ProcessRequest() override;
	void CancelRequest() override;
	EHttpRequestStatus::Type GetStatus() const override;
	EHttpFailureReason GetFailureReason() const override;
	const FString& GetEffectiveURL() const override;
	const FHttpResponsePtr GetResponse() const override;
	void Tick(float DeltaSeconds) override;
	float GetElapsedTime() const override;
	void SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy) override;
	EHttpRequestDelegateThreadPolicy GetDelegateThreadPolicy() const override;
	virtual void SetPriority(EHttpRequestPriority InPriority) override;
	virtual EHttpRequestPriority GetPriority() const override;
	void SetTimeout(float InTimeoutSecs) override;
	void ClearTimeout() override;
	void ResetTimeoutStatus() override;
	TOptional<float> GetTimeout() const override;
	void SetActivityTimeout(float InTimeoutSecs) override;
	void ProcessRequestUntilComplete() override;

	FHttpRequestCompleteDelegate& OnProcessRequestComplete() override;
	FHttpRequestProgressDelegate64& OnRequestProgress64() override;
	FHttpRequestWillRetryDelegate& OnRequestWillRetry() override;
	FHttpRequestHeaderReceivedDelegate& OnHeaderReceived() override;
	FHttpRequestStatusCodeReceivedDelegate& OnStatusCodeReceived() override;

private:
	// The InnerRequest can be one of two things:
	// 
	// * An FClosedHttpRequest, which is created when a transactionally-safe HTTP request is created from inside of a transaction.
	//   A closed request will queue up work and then play it back into a PlatformRequest when the transaction commits.
	// * An real request created from a call to FPlatformHttp::ConstructRequest. 
	//   This replaces the original InnerRequest once we reach the open.
	TSharedPtr<IHttpRequest> InnerRequest;

	class FClosedHttpRequest;
	friend class FClosedHttpRequest;
};
