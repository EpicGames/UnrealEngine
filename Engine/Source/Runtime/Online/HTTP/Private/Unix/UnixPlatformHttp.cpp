// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformHttp.h"
#include "Curl/CurlHttp.h"
#include "Curl/CurlHttpManager.h"

void FUnixPlatformHttp::Init()
{
	FCurlHttpManager::InitCurl();
}

class FHttpManager * FUnixPlatformHttp::CreatePlatformHttpManager()
{
	return new FCurlHttpManager();
}

void FUnixPlatformHttp::Shutdown()
{
	FCurlHttpManager::ShutdownCurl();
}

IHttpRequest* FUnixPlatformHttp::ConstructRequest()
{
	return new FCurlHttpRequest();
}

