// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WINHTTP

#include "WinHttp/Support/WinHttpConnectionHttp.h"
#include "WinHttp/Support/WinHttpSession.h"
#include "WinHttp/Support/WinHttpErrorHelper.h"
#include "WinHttp/Support/WinHttpTypes.h"
#include "WinHttp/WinHttpHttpManager.h"

#include "Http.h"
#include "GenericPlatform/HttpRequestPayload.h"
#include "Misc/ScopeLock.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <errhandlingapi.h>

#define UE_WINHTTP_WRITE_BUFFER_BYTES (8*1024)
#define UE_WINHTTP_READ_BUFFER_BYTES (8*1024)

void CALLBACK UE_WinHttpStatusHttpCallback(HINTERNET hInternet, DWORD_PTR dwContext, DWORD dwInternetStatus, LPVOID lpvStatusInformation, DWORD dwStatusInformationLength)
{
	const EWinHttpCallbackStatus Status = static_cast<EWinHttpCallbackStatus>(dwInternetStatus);
	if (!IsValidStatus(Status))
	{
		UE_LOGF(LogWinHttp, Warning, "Received unknown WinHttp Status %lu", dwInternetStatus);
		return;
	}

	// If there is a dwContext, this is a normal HTTP request
	if (dwContext)
	{
		FWinHttpConnectionHttp* const RequestContext = reinterpret_cast<FWinHttpConnectionHttp*>(dwContext);
		RequestContext->HandleHttpStatusCallback(hInternet, Status, lpvStatusInformation, dwStatusInformationLength);
	}
}

TSharedPtr<FWinHttpConnectionHttp, ESPMode::ThreadSafe> FWinHttpConnectionHttp::CreateHttpConnection(
	FWinHttpSession& InSession,
	const FString& InVerb,
	const FString& InUrl,
	const TMap<FString, FString>& InHeaders,
	const TSharedPtr<FRequestPayload, ESPMode::ThreadSafe>& InPayload)
{
	if (!InSession.IsValid())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp Request with no active session");
		return nullptr;
	}

	if (InVerb.IsEmpty())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp Request with an empty verb");
		return nullptr;
	}
	
	if (InUrl.IsEmpty())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp Request with an empty url");
		return nullptr;
	}

	const bool bIsSecure = FGenericPlatformHttp::IsSecureProtocol(InUrl).Get(false);
	if (!bIsSecure && InSession.AreOnlySecureConnectionsAllowed())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create an insecure WinHttp Request which is disabled on this platform");
		return nullptr;
	}

	const FString Domain = FGenericPlatformHttp::GetUrlDomain(InUrl);
	if (Domain.IsEmpty())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp Request with an unset domain");
		return nullptr;
	}

	const FString PathAndQuery = FGenericPlatformHttp::GetUrlPath(InUrl, true, false);
	if (PathAndQuery.IsEmpty())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp Request with an unset path");
		return nullptr;
	}
	
	TOptional<uint16> Port = FGenericPlatformHttp::GetUrlPort(InUrl);
	TSharedRef<FWinHttpConnectionHttp, ESPMode::ThreadSafe> Result = MakeShareable(new FWinHttpConnectionHttp(InSession, InVerb, InUrl, bIsSecure, Domain, Port, PathAndQuery, InHeaders, InPayload));
	if (!Result->IsValid())
	{
		return nullptr;
	}

	return Result;
}

FWinHttpConnectionHttp::~FWinHttpConnectionHttp()
{
	UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Destructing WinHttp request", this);

	RequestHandle.Reset();
	ConnectionHandle.Reset();

	if (FWinHttpHttpManager* const HttpManager = FWinHttpHttpManager::GetManager())
	{
		HttpManager->ReleaseRequestResources(*this);
	}
}

bool FWinHttpConnectionHttp::IsValid() const
{
	FScopeLock ScopeLock(&SyncObject);
	return ConnectionHandle.IsValid() && RequestHandle.IsValid();
}

const FString& FWinHttpConnectionHttp::GetRequestUrl() const
{
	// No lock because this data is constant for the entire request
	return RequestUrl;
}

void* FWinHttpConnectionHttp::GetHandle()
{
	// No lock, as this is to be called while inside a callback (which gives us a lock)
	return RequestHandle.Get();
}

bool FWinHttpConnectionHttp::StartRequest()
{
	FScopeLock ScopeLock(&SyncObject);
	if (CurrentAction != EState::WaitToStart)
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Attempted to start a request that has already started", this);
		return false;
	}

	KeepAlive = StaticCastSharedRef<FWinHttpConnectionHttp>(AsShared());

	if (WinHttpSetStatusCallback(RequestHandle.Get(), UE_WinHttpStatusHttpCallback, WINHTTP_CALLBACK_FLAG_ALL_NOTIFICATIONS, 0) == WINHTTP_INVALID_STATUS_CALLBACK)
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpSetStatusCallbackFailure(ErrorCode);
		KeepAlive.Reset();
		return false;
	}

	// Setup our buffer
	if (Payload.IsValid())
	{
		const uint64 NumBytesToWriteNow = FMath::Min((uint64)(UE_WINHTTP_WRITE_BUFFER_BYTES), Payload->GetContentLength());
		PayloadBuffer.SetNumUninitialized(NumBytesToWriteNow, EAllowShrinking::No);

		const uint64 BufferSize = Payload->FillOutputBuffer(MakeArrayView(PayloadBuffer), 0);
		PayloadBuffer.SetNumUninitialized(BufferSize, EAllowShrinking::No);
	}

	CurrentAction = EState::SendRequest;
	if (!SendRequest())
	{
		CurrentAction = EState::WaitToStart;

		if (!WinHttpSetStatusCallback(RequestHandle.Get(), nullptr, WINHTTP_CALLBACK_FLAG_ALL_NOTIFICATIONS, 0))
		{
			const DWORD ErrorCode = GetLastError();
			FWinHttpErrorHelper::LogWinHttpSetStatusCallbackFailure(ErrorCode);
		}

		FinalState.Reset();
		KeepAlive.Reset();
		return false;
	}

	return true;
}

bool FWinHttpConnectionHttp::CancelRequest()
{
	FScopeLock ScopeLock(&SyncObject);

	if (bRequestCancelled)
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Attempting to cancel an already cancelling WinHttp Request", this);
		return true;
	}
	
	if (FinalState.IsSet())
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Attempted to cancel finished WinHttp Request", this);
		return false;
	}

	UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Cancelling WinHttp Request", this);
	bRequestCancelled = true;
	
	FinishRequest(EHttpRequestStatus::Failed);
	return true;
}

bool FWinHttpConnectionHttp::IsComplete() const
{
	FScopeLock ScopeLock(&SyncObject);
	return FinalState.IsSet();
}

void FWinHttpConnectionHttp::PumpMessages()
{
	// Don't lock our object if we don't have any events ready (this is an optimization to skip locking on game thread every tick if there's no events waiting)
	if (!bHasPendingDelegate)
	{
		return;
	}

	// Reset our pending delegate flag now that we're entering the real loop
	bHasPendingDelegate = false;

	// A keep-alive object that will keep us alive in case we're killed in one of the callbacks
	TSharedRef<IWinHttpConnection, ESPMode::ThreadSafe> LocalKeepAlive(AsShared());

	FScopeLock ScopeLock(&SyncObject);

	if (bRequestCancelled)
	{
		return;
	}

	if (CurrentChunk.Num() > 0)
	{
		// This is safe because currently CurrentChunk is only accessed within 
		// a SyncObject critical section.
		BytesWrittenToGameThreadChunk += CurrentChunk.Num();
		if (GameThreadChunk.Num() == 0)
		{
			GameThreadChunk = MoveTemp(CurrentChunk);
		}
		else
		{
			GameThreadChunk.Append(CurrentChunk);
		}
		const uint64 ReserveChunkSize = ResponseContentLength >= BytesWrittenToGameThreadChunk 
			? (ResponseContentLength - BytesWrittenToGameThreadChunk) 
			: UE_WINHTTP_READ_BUFFER_BYTES;
		CurrentChunk.Reset(ReserveChunkSize);
	}

	// Process Data Transfer callbacks
	if (BytesToReportSent.IsSet() || BytesToReportReceived.IsSet())
	{
		const uint64 BytesSent = BytesToReportSent.Get(0);
		BytesToReportSent.Reset();

		const uint64 BytesReceived = BytesToReportReceived.Get(0);
		BytesToReportReceived.Reset();

		OnDataTransferredHandler.ExecuteIfBound(BytesSent, BytesReceived);
		if (bRequestCancelled)
		{
			// If we get canceled in our callback, don't send any more messages
			return;
		}
	}

	// Process New Header callbacks
	if (HeaderKeysToReportReceived.Num())
	{
		TArray<FString> LocalHeadersToReport = MoveTemp(HeaderKeysToReportReceived);
		HeaderKeysToReportReceived.Reset();

		for (const FString& HeaderKey : LocalHeadersToReport)
		{
			OnHeaderReceivedHandler.ExecuteIfBound(HeaderKey, HeadersReceived.FindChecked(HeaderKey));
			if (bRequestCancelled)
			{
				// If we get canceled in our callback, don't send any more messages
				return;
			}
		}
	}

	// Send our completion callback if the request is now finished
	if (OnRequestCompleteHandler.IsBound() && FinalState.IsSet())
	{
		FWinHttpConnectionHttpOnRequestComplete LocalCompleteHandler = MoveTemp(OnRequestCompleteHandler);
		OnRequestCompleteHandler.Unbind();

		LocalCompleteHandler.ExecuteIfBound(FinalState.GetValue());
		if (bRequestCancelled)
		{
			// If we get canceled in our callback, don't send any more messages
			return;
		}
	}
}

void FWinHttpConnectionHttp::PumpStates()
{
	FScopeLock ScopeLock(&SyncObject);

	const int32 MaxStateChanges = 20;
	for (int32 NumStateChanges = 0; NumStateChanges < MaxStateChanges; ++NumStateChanges)
	{
		switch (CurrentAction)
		{
			case EState::WaitToStart:
			case EState::WaitForSendComplete:
			case EState::WaitForResponseHeaders:
			case EState::WaitForNextResponseBodyChunkSize:
			case EState::WaitForNextResponseBodyChunkData:
			case EState::Finished:
			{
				// We don't have anything to do but wait
				return;
			}

			case EState::SendRequest:
			{
				if (!SendRequest())
				{
					FinishRequest(EHttpRequestStatus::Failed);
					return;
				}
				continue;
			}

			case EState::SendAdditionalRequestBody:
			{
				check(HasRequestBodyToSend());
				if (!SendAdditionalRequestBody())
				{
					FinishRequest(EHttpRequestStatus::Failed);
					return;
				}
				continue;
			}

			case EState::RequestResponse:
			{
				if (!RequestResponse())
				{
					FinishRequest(EHttpRequestStatus::Failed);
					return;
				}
				continue;
			}

			case EState::ProcessResponseHeaders:
			{
				if (!ProcessResponseHeaders())
				{
					FinishRequest(EHttpRequestStatus::Failed);
					return;
				}

				continue;
			}

			case EState::RequestNextResponseBodyChunkSize:
			{
				if (!RequestNextResponseBodyChunkSize())
				{
					FinishRequest(EHttpRequestStatus::Failed);
					return;
				}
				continue;
			}

			case EState::RequestNextResponseBodyChunkData:
			{
				if (!RequestNextResponseBodyChunkData())
				{
					FinishRequest(EHttpRequestStatus::Failed);
					return;
				}
				continue;
			}
		}

		checkNoEntry();
	}
}

void FWinHttpConnectionHttp::SetDataTransferredHandler(FWinHttpConnectionHttpOnDataTransferred&& Handler)
{
	FScopeLock ScopeLock(&SyncObject);
	if (CurrentAction != EState::WaitToStart)
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Attempting to set the Data Transfer handler on a request that has already started", this);
		return;
	}

	OnDataTransferredHandler = MoveTemp(Handler);
}

void FWinHttpConnectionHttp::SetHeaderReceivedHandler(FWinHttpConnectionHttpOnHeaderReceived&& Handler)
{
	FScopeLock ScopeLock(&SyncObject);
	if (CurrentAction != EState::WaitToStart)
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Attempting to set the Header Received handler on a request that has already started", this);
		return;
	}

	OnHeaderReceivedHandler = MoveTemp(Handler);
}

void FWinHttpConnectionHttp::SetRequestCompletedHandler(FWinHttpConnectionHttpOnRequestComplete&& Handler)
{
	FScopeLock ScopeLock(&SyncObject);
	if (CurrentAction != EState::WaitToStart)
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Attempting to set the Request Complete handler on a request that has already started", this);
		return;
	}

	OnRequestCompleteHandler = MoveTemp(Handler);
}

FWinHttpConnectionHttp::FWinHttpConnectionHttp(
	FWinHttpSession& InSession,
	const FString& InVerb,
	const FString& InUrl,
	const bool bInIsSecure,
	const FString& InDomain,
	const TOptional<uint16> InPort,
	const FString& InPathAndQuery,
	const TMap<FString, FString>& InHeaders,
	const TSharedPtr<FRequestPayload, ESPMode::ThreadSafe>& InPayload)
	: RequestUrl(InUrl)
{
	const uint32 LogPort = InPort.Get(bInIsSecure ? 443 : 80);
	const uint64 LogPayloadSize = InPayload.IsValid() ? InPayload->GetContentLength() : 0;

	UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Creating request. InVerb=[%ls] bIsSecure=[%d] Domain=[%ls] Port=[%u] Path=[%ls] PaylodSize=[%llu]", this, *InVerb, bInIsSecure, *InDomain, LogPort, *InPathAndQuery, LogPayloadSize);

	// Note: Microsoft say not to reuse Connection Handles for multiple requests, despite what the API would suggest!  Microsoft say the Session handle
	// is to be reused amongst requests with the same Security Protocol. If the same Session is used, backing sockets for connections will be reused
	// whenever possible.  td;dr: try to reuse session handles, never re-use connections/request handles!

	INTERNET_PORT InternetPort = InPort.Get(INTERNET_DEFAULT_PORT);
	ConnectionHandle = WinHttpConnect(InSession.Get(), TCHAR_TO_WCHAR(*InDomain), InternetPort, 0);
	if (!ConnectionHandle.IsValid())
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpConnectFailure(ErrorCode);
		return;
	}

	const LPWSTR HttpVersion = NULL; // Setting to null defaults
	const LPWSTR RequestReferrer = WINHTTP_NO_REFERER;
	LPCWSTR* AcceptedMediaTypes = WINHTTP_DEFAULT_ACCEPT_TYPES;
	const DWORD Flags = bInIsSecure ? WINHTTP_FLAG_SECURE : 0;

	RequestHandle = WinHttpOpenRequest(ConnectionHandle.Get(), TCHAR_TO_WCHAR(*InVerb), TCHAR_TO_WCHAR(*InPathAndQuery), HttpVersion, RequestReferrer, AcceptedMediaTypes, Flags);
	if (!RequestHandle.IsValid())
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpOpenRequestFailure(ErrorCode);
		ConnectionHandle.Reset();
		return;
	}

	if (InHeaders.Num() > 0)
	{
		if (!SetHeaders(InHeaders))
		{
			RequestHandle.Reset();
			ConnectionHandle.Reset();
			return;
		}
	}
	if (InPayload.IsValid())
	{
		if (!SetPayload(InPayload.ToSharedRef()))
		{
			RequestHandle.Reset();
			ConnectionHandle.Reset();
			return;
		}
	}
}

bool FWinHttpConnectionHttp::SetHeaders(const TMap<FString, FString>& Headers)
{
	if (Headers.Num() == 0)
	{
		return true;
	}

	FScopeLock ScopeLock(&SyncObject);
	if (CurrentAction != EState::WaitToStart)
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Attempted to set headers on request that has already started", this);
		return false;
	}

	// Validate headers
	for (const TPair<FString, FString>& HeaderPair : Headers)
	{
		if (HeaderPair.Key.IsEmpty())
		{
			UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Attempted to set empty header key", this);
			return false;
		}
	}

	FString HeaderBuffer;

	const int32 NumCharsColonSpace = 2;
	for (const TPair<FString, FString>& HeaderPair : Headers)
	{
		if (HeaderPair.Key.IsEmpty())
		{
			continue;
		}

		const int32 StringSizeNeeded = HeaderPair.Key.Len() + NumCharsColonSpace + HeaderPair.Value.Len();
		HeaderBuffer.Reset(StringSizeNeeded);

		HeaderBuffer.Append(HeaderPair.Key);
		HeaderBuffer.AppendChar(TEXT(':'));
		HeaderBuffer.AppendChar(TEXT(' '));
		HeaderBuffer.Append(HeaderPair.Value);
		check(HeaderBuffer.Len() == StringSizeNeeded);
	
		const DWORD HeadersLength = HeaderBuffer.Len();
		const DWORD Flags = WINHTTP_ADDREQ_FLAG_ADD | WINHTTP_ADDREQ_FLAG_REPLACE;

		if (!WinHttpAddRequestHeaders(RequestHandle.Get(), TCHAR_TO_WCHAR(*HeaderBuffer), HeadersLength, Flags))
		{
			const DWORD ErrorCode = GetLastError();
			// ERROR_WINHTTP_HEADER_NOT_FOUND can be returned if `Value` is empty and the header does not exist.
			// The latest WinHttp implementation sees this as a request to delete the header. This is done because
			// according to the latest HTTP1.1 RFC (7230) a blank header is not allowed.
			// Therefore, acknowleding the fact that a blank header was not added is valid.
			if (HeaderPair.Value.IsEmpty() && ErrorCode == ERROR_WINHTTP_HEADER_NOT_FOUND)
			{
				UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Ignoring request to clear header as it was not set. HeaderKey=[%ls]", this, *HeaderPair.Key);
				continue;
			}

			FWinHttpErrorHelper::LogWinHttpAddRequestHeadersFailure(ErrorCode);

			UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Failed to add header to request. Header=[%ls]", this, *HeaderBuffer);
			return false;
		}
	}

	return true;
}

bool FWinHttpConnectionHttp::SetHeader(const FString& Key, const FString& Value)
{
	if (Key.IsEmpty())
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Attempted to set empty header key", this);
		return false;
	}

	FScopeLock ScopeLock(&SyncObject);
	if (CurrentAction != EState::WaitToStart)
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Attempted to set headers on request that has already started", this);
		return false;
	}

	FString HeaderBuffer;

	const int32 NumCharsColonSpace = 2;

	const int32 StringSizeNeeded = Key.Len() + NumCharsColonSpace + Value.Len();
	HeaderBuffer.Reset(StringSizeNeeded);

	HeaderBuffer.Append(Key);
	HeaderBuffer.AppendChar(TEXT(':'));
	HeaderBuffer.AppendChar(TEXT(' '));
	HeaderBuffer.Append(Value);
	check(HeaderBuffer.Len() == StringSizeNeeded);

	const DWORD HeaderLength = HeaderBuffer.Len();
	const DWORD Flags = WINHTTP_ADDREQ_FLAG_ADD | WINHTTP_ADDREQ_FLAG_REPLACE;

	if (!WinHttpAddRequestHeaders(RequestHandle.Get(), TCHAR_TO_WCHAR(*HeaderBuffer), HeaderLength, Flags))
	{
		const DWORD ErrorCode = GetLastError();
		// ERROR_WINHTTP_HEADER_NOT_FOUND can be returned if `Value` is empty and the header does not exist.
		// The latest WinHttp implementation sees this as a request to delete the header. This is done because
		// according to the latest HTTP1.1 RFC (7230) a blank header is not allowed.
		// Therefore, acknowleding the fact that a blank header was not added is valid.
		if (Value.IsEmpty() && ErrorCode == ERROR_WINHTTP_HEADER_NOT_FOUND)
		{
			UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Ignoring request to clear header as it was not set. HeaderKey=[%ls]", this, *Key);
			return true;
		}

		FWinHttpErrorHelper::LogWinHttpAddRequestHeadersFailure(ErrorCode);

		UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Failed to add header to request. Header=[%ls]", this, *HeaderBuffer);
		return false;
	}

	return true;
}

bool FWinHttpConnectionHttp::SetPayload(const TSharedRef<FRequestPayload, ESPMode::ThreadSafe>& InPayload)
{
	FScopeLock ScopeLock(&SyncObject);
	if (CurrentAction != EState::WaitToStart)
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Attempted to set playload for request that has already started", this);
		return false;
	}

	Payload = InPayload;
	return true;
}

bool FWinHttpConnectionHttp::SendRequest()
{
	check(CurrentAction == EState::WaitToStart || CurrentAction == EState::SendRequest);
	const EState StateAtStart = CurrentAction;

	// Reset our sent bytes to 0
	NumBytesSuccessfullySent = 0;

	const LPCWSTR AdditionalHeaders = WINHTTP_NO_ADDITIONAL_HEADERS;
	const DWORD AdditionalHeaderCount = 0;
	const LPVOID DataToSendImmediately = (PayloadBuffer.Num() > 0) ? PayloadBuffer.GetData() : WINHTTP_NO_REQUEST_DATA;
	const DWORD DataToSendImmediatelyLength = PayloadBuffer.Num();
	const DWORD TotalDataLength = Payload.IsValid() ? Payload->GetContentLength() : 0;
	DWORD_PTR Context = reinterpret_cast<DWORD_PTR>(this);
	
	CurrentAction = EState::WaitForSendComplete;
	while (!WinHttpSendRequest(RequestHandle.Get(), AdditionalHeaders, AdditionalHeaderCount, DataToSendImmediately, DataToSendImmediatelyLength, TotalDataLength, Context))
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpSendRequestFailure(ErrorCode);

		if (ErrorCode == ERROR_WINHTTP_RESEND_REQUEST && RequestHandle.IsValid())
		{
			// Resend the request if we didn't cancel it in a callback invoked by SendRequest.
			// If a request needs to be redirected, we need to call SendRequest again until it is not redirected
			continue;
		}

		CurrentAction = StateAtStart;
		return false;
	}

	// We are successful
	return true;
}

void FWinHttpConnectionHttp::IncrementSentByteCounts(const uint64 AmountSent)
{
	check(CurrentAction == EState::WaitForSendComplete);

	if (!BytesToReportSent.IsSet())
	{
		BytesToReportSent = AmountSent;
	}
	else
	{
		BytesToReportSent.GetValue() += AmountSent;
	}

	bHasPendingDelegate = true;

	NumBytesSuccessfullySent += AmountSent;
}

void FWinHttpConnectionHttp::IncrementReceivedByteCounts(const uint64 AmountReceived)
{
	check(CurrentAction == EState::WaitForNextResponseBodyChunkData);

	if (!BytesToReportReceived.IsSet())
	{
		BytesToReportReceived = AmountReceived;
	}
	else
	{
		BytesToReportReceived.GetValue() += AmountReceived;
	}

	bHasPendingDelegate = true;
}

bool FWinHttpConnectionHttp::HasRequestBodyToSend() const
{
	if (!Payload.IsValid())
	{
		return false;
	}

	return Payload->GetContentLength() > NumBytesSuccessfullySent;
}

bool FWinHttpConnectionHttp::SendAdditionalRequestBody()
{
	check(CurrentAction == EState::SendAdditionalRequestBody);
	check(HasRequestBodyToSend());
	check(Payload.IsValid());

	const int64 TotalBytesLeftToWrite = Payload->GetContentLength() - NumBytesSuccessfullySent;
	check(TotalBytesLeftToWrite > 0);
	
	// Resize buffer to max amount of data we can write
	const int64 OptimalAmountToWrite = FMath::Min(TotalBytesLeftToWrite, UE_WINHTTP_WRITE_BUFFER_BYTES);
	PayloadBuffer.SetNumUninitialized(OptimalAmountToWrite, EAllowShrinking::No);

	// Read data into our buffer if possible
	const int64 ActualDataSize = Payload->FillOutputBuffer(MakeArrayView(PayloadBuffer), NumBytesSuccessfullySent);
	if (ActualDataSize < 1)
	{
		// Set our buffer to be empty since we didn't write anything into it
		PayloadBuffer.Reset(0);
		return true;
	}

	// Resize our buffer based on how much was actually written to it
	PayloadBuffer.SetNumUninitialized(ActualDataSize, EAllowShrinking::No);

	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Writing Data. NumBytes=[%u] TotalBytesWritten=[%llu] TotalBytes=[%llu]", this, PayloadBuffer.Num(), NumBytesSuccessfullySent, Payload->GetContentLength())

	CurrentAction = EState::WaitForSendComplete;
	if (!WinHttpWriteData(RequestHandle.Get(), PayloadBuffer.GetData(), PayloadBuffer.Num(), NULL))
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpWriteDataFailure(ErrorCode);

		CurrentAction = EState::SendAdditionalRequestBody;
		return false;
	}

	return true;
}

bool FWinHttpConnectionHttp::RequestResponse()
{
	check(CurrentAction == EState::RequestResponse);
	check(!Payload.IsValid() || Payload->GetContentLength() == NumBytesSuccessfullySent);

	// Now that we have all of our content sent, we can receive our response
	CurrentAction = EState::WaitForResponseHeaders;
	if (!WinHttpReceiveResponse(RequestHandle.Get(), NULL))
	{
		// Log error
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpReceiveResponseFailure(ErrorCode);

		// Check for the error that asks us to resend the request
		if (ErrorCode == ERROR_WINHTTP_RESEND_REQUEST)
		{
			CurrentAction = EState::SendRequest;
			return SendRequest();
		}

		CurrentAction = EState::RequestResponse;
		return false;
	}

	return true;
}

bool FWinHttpConnectionHttp::ProcessResponseHeaders()
{
	check(CurrentAction == EState::ProcessResponseHeaders);

	// Read the size of buffer we'll need to store our headers
	DWORD InfoLevel = WINHTTP_QUERY_RAW_HEADERS;
	LPCWSTR HeaderName = WINHTTP_HEADER_NAME_BY_INDEX;
	LPVOID BufferDestination = WINHTTP_NO_OUTPUT_BUFFER;
	DWORD OutHeaderByteSize = 0u;
	LPDWORD OutHeaderIndex = WINHTTP_NO_HEADER_INDEX;

	// Call WinHttpQueryHeaders with a NO_OUTPUT_BUFFER to set OutHeaderByteSize to the correct value to store this response (we expect this call to return false)
	if (WinHttpQueryHeaders(RequestHandle.Get(), InfoLevel, HeaderName, BufferDestination, &OutHeaderByteSize, OutHeaderIndex))
	{
		// This should not return true (this code shouldn't happen), as passing WINHTTP_NO_OUTPUT_BUFFER is supposed to make the request fail
		return false;
	}

	// Make sure this error was the one we were expecting (insufficient space in our empty and null buffer)
	{
		const DWORD ErrorCode = GetLastError();
		if (ErrorCode != ERROR_INSUFFICIENT_BUFFER)
		{
			// We're expecting an insufficent buffer error, but this is a real error, so fail the request
			FWinHttpErrorHelper::LogWinHttpQueryHeadersFailure(ErrorCode);
			return false;
		}
	}

	// Read our HTTP headers now that we have the size of them as a string
	if (LIKELY(OutHeaderByteSize > 0u))
	{
		// Setup buffer to write headers into
		TArray<wchar_t> AllHeadersBuffer;
		AllHeadersBuffer.SetNumUninitialized(OutHeaderByteSize / sizeof(wchar_t), EAllowShrinking::No);
		BufferDestination = AllHeadersBuffer.GetData();

		// Read headers into our buffer
		if (!WinHttpQueryHeaders(RequestHandle.Get(), InfoLevel, HeaderName, BufferDestination, &OutHeaderByteSize, OutHeaderIndex))
		{
			// Unexpected failure
			const DWORD ErrorCode = GetLastError();
			FWinHttpErrorHelper::LogWinHttpQueryHeadersFailure(ErrorCode);
			return false;
		}

		// Parse the headers into Key and Value and save them in into our containers until we read an empty header (end of list)
		int32 ReadIndex = 0;
		while (const int32 StringLength = FCStringWide::Strlen(AllHeadersBuffer.GetData() + ReadIndex))
		{
			FWideStringView CompleteHeader(AllHeadersBuffer.GetData() + ReadIndex, StringLength + 1);
			ReadIndex += StringLength + 1;

			// This log is only safe because these headers are actually null-terminated!
			UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Received Header Header=[%ls]", this, CompleteHeader.GetData());

			int32 OutIndex = INDEX_NONE;
			if (CompleteHeader.FindChar(L':', OutIndex))
			{
				FWideStringView HeaderKey(CompleteHeader.Left(OutIndex));
				FWideStringView HeaderValue(CompleteHeader.RightChop(OutIndex + 1));

				// Remove trailing NULL terminator from the view. Views should not contain the terminator or
				// the resulting string will end up double terminated.
				HeaderValue.RemoveSuffix(1);

				HeaderValue.TrimStartAndEndInline();

				HeadersReceived.Emplace(HeaderKey, HeaderValue);
				HeaderKeysToReportReceived.Emplace(HeaderKey);

				bHasPendingDelegate = true;
			}
		}
	}

	// Read our HTTP status code now that we're done reading string headers
	{
		DWORD StatusCode = 0;
		InfoLevel = WINHTTP_QUERY_STATUS_CODE | WINHTTP_QUERY_FLAG_NUMBER;
		BufferDestination = &StatusCode;
		OutHeaderByteSize = sizeof(StatusCode);
		if (!WinHttpQueryHeaders(RequestHandle.Get(), InfoLevel, HeaderName, BufferDestination, &OutHeaderByteSize, OutHeaderIndex))
		{
			// Unexpected failure
			const DWORD ErrorCode = GetLastError();
			FWinHttpErrorHelper::LogWinHttpQueryHeadersFailure(ErrorCode);
			return false;
		}
		ResponseCode = static_cast<EHttpResponseCodes::Type>(StatusCode);
	}

	// Read our HTTP content length now that we have our status code (if there is one)
	{
		DWORD ContentLength = 0;
		InfoLevel = WINHTTP_QUERY_CONTENT_LENGTH | WINHTTP_QUERY_FLAG_NUMBER;
		BufferDestination = &ContentLength;
		OutHeaderByteSize = sizeof(ContentLength);
		if (WinHttpQueryHeaders(RequestHandle.Get(), InfoLevel, HeaderName, BufferDestination, &OutHeaderByteSize, OutHeaderIndex))
		{
			// It's ok if this fails, it just means we don't have a content-length header (such as when the response is chunk-encoded, for example)
			if (ContentLength > 0)
			{
				ResponseContentLength = ContentLength;
			}
		}
		CurrentChunk.Reserve(ResponseContentLength > 0 ? ResponseContentLength : UE_WINHTTP_READ_BUFFER_BYTES);
	}

	// Nothing above was async, so we can move to the next action state now
	CurrentAction = EState::RequestNextResponseBodyChunkSize;
	return true;
}

bool FWinHttpConnectionHttp::RequestNextResponseBodyChunkSize()
{
	check(CurrentAction == EState::RequestNextResponseBodyChunkSize);

	CurrentAction = EState::WaitForNextResponseBodyChunkSize;
	if (!WinHttpQueryDataAvailable(RequestHandle.Get(), NULL))
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpQueryDataAvailableFailure(ErrorCode);
		
		CurrentAction = EState::RequestNextResponseBodyChunkSize;
		return false;
	}

	return true;
}

bool FWinHttpConnectionHttp::RequestNextResponseBodyChunkData()
{
	check(CurrentAction == EState::RequestNextResponseBodyChunkData);
	check(ResponseBytesAvailable.IsSet());

	const uint64 NumBytesAvailable = ResponseBytesAvailable.GetValue();
	ResponseBytesAvailable.Reset();

	if (NumBytesAvailable == 0)
	{
		// We're finished reading data
		UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Request Completed Successfully", this);
		FinishRequest(EHttpRequestStatus::Succeeded);
		return true;
	}

	uint64 ResponseBytesWritten = CurrentChunk.Num();
	CurrentChunk.AddUninitialized(NumBytesAvailable);

	CurrentAction = EState::WaitForNextResponseBodyChunkData;
	if (!WinHttpReadData(RequestHandle.Get(), CurrentChunk.GetData() + ResponseBytesWritten, NumBytesAvailable, NULL))
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpReadDataFailure(ErrorCode);

		CurrentAction = EState::RequestNextResponseBodyChunkData;
		return false;
	}

	return true;

}

bool FWinHttpConnectionHttp::FinishRequest(const EHttpRequestStatus::Type NewFinalState)
{
	// Make sure this state is a "final" state
	if (!EHttpRequestStatus::IsFinished(NewFinalState))
	{
		return false;
	}
	// Make sure we didn't already finish this request
	if (FinalState.IsSet())
	{
		return false;
	}

	// Set our finished state
	FinalState = NewFinalState;

	// Set our state machine to being finished now that we're not processing
	CurrentAction = EState::Finished;

	// Log-level Log if successful, Warning if failure
	if (FinalState == EHttpRequestStatus::Succeeded)
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Request Complete. State=[%ls]", this, EHttpRequestStatus::ToString(FinalState.GetValue()));
	}
	else
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Request Complete. State=[%ls]", this, EHttpRequestStatus::ToString(FinalState.GetValue()));
	}

	// We have a pending action now that our final state is set
	bHasPendingDelegate = true;

	// Reset our handles (begins shutdown)
	RequestHandle.Reset();
	ConnectionHandle.Reset();
	return true;
}

void FWinHttpConnectionHttp::HandleConnectedToServer(const wchar_t* ServerIP)
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[CONNECTED_TO_SERVER] ServerIP=[%ls]", this, WCHAR_TO_TCHAR(ServerIP));
}

void FWinHttpConnectionHttp::HandleSendingRequest()
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[SENDING_REQUEST]", this);
	
	FScopeLock ScopeLock(&SyncObject);

	if (FWinHttpHttpManager* ActiveManager = FWinHttpHttpManager::GetManager())
	{
		if (!ActiveManager->ValidateRequestCertificates(*this))
		{
			FinishRequest(EHttpRequestStatus::Failed);
			return;
		}
	}
}

void FWinHttpConnectionHttp::HandleWriteComplete(const uint64 NumBytesSent)
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[WRITE_COMPLETE] NumBytesSent=[%llu]", this, NumBytesSent);

	FScopeLock ScopeLock(&SyncObject);

	// If we're in a different state, an error probably happened or we were cancelled
	if (CurrentAction != EState::WaitForSendComplete)
	{
		return;
	}

	// Update data written stats
	IncrementSentByteCounts(NumBytesSent);

	// Update our state machine
	if (HasRequestBodyToSend())
	{
		// Tell the main thread to ask for more data
		CurrentAction = EState::SendAdditionalRequestBody;
	}
	else
	{
		// Tell the main thread to request the response
		CurrentAction = EState::RequestResponse;
	}
}

void FWinHttpConnectionHttp::HandleSendRequestComplete()
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[SEND_REQUEST_COMPLETE]", this);

	FScopeLock ScopeLock(&SyncObject);
	
	// If we're in a different state, an error probably happened or we were cancelled
	if (CurrentAction != EState::WaitForSendComplete)
	{
		return;
	}

	// If we finished sending our request, we didn't have a connection error
	bConnectedToServer = true;

	// Update data written stats
	IncrementSentByteCounts(PayloadBuffer.Num());

	// Update our state machine
	if (HasRequestBodyToSend())
	{
		// Tell the main thread to ask for more data
		CurrentAction = EState::SendAdditionalRequestBody;
	}
	else
	{
		// Tell the main thread to request the response
		CurrentAction = EState::RequestResponse;
	}
}

void FWinHttpConnectionHttp::HandleHeadersAvailable()
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[HEADERS_AVAILABLE]", this);

	FScopeLock ScopeLock(&SyncObject);

	if (!RequestHandle.IsValid())
	{
		// Request was probably cancelled, just stop here
		return;
	}

	if (ensure(CurrentAction == EState::WaitForResponseHeaders))
	{
		CurrentAction = EState::ProcessResponseHeaders;
	}

	// The fact we received a headers available message indicates request success, so we don't need the payload data anymore
	ReleasePayloadData();
}

void FWinHttpConnectionHttp::HandleDataAvailable(const uint64 NumBytesAvailable)
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[DATA_AVAILABLE] NumBytesAvailable=[%llu]", this, NumBytesAvailable);

	FScopeLock ScopeLock(&SyncObject);

	if (CurrentAction != EState::WaitForNextResponseBodyChunkSize)
	{
		return;
	}

	ResponseBytesAvailable = NumBytesAvailable;
	CurrentAction = EState::RequestNextResponseBodyChunkData;
}

void FWinHttpConnectionHttp::HandleReadComplete(const uint64 NumBytesRead)
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[READ_COMPLETE] NumBytesRead=[%llu]", this, NumBytesRead);

	FScopeLock ScopeLock(&SyncObject);
	
	if (CurrentAction != EState::WaitForNextResponseBodyChunkData)
	{
		return;
	}

	IncrementReceivedByteCounts(NumBytesRead);

	CurrentAction = EState::RequestNextResponseBodyChunkSize;
}

namespace
{
	static bool IsErrorCodeAConnectionError(const uint32 ErrorCode)
	{
		switch (ErrorCode)
		{
		case ERROR_WINHTTP_NAME_NOT_RESOLVED:
		case ERROR_WINHTTP_CANNOT_CONNECT:
		case ERROR_WINHTTP_TIMEOUT:
			return true;
		}

		return false;
	}
}

void FWinHttpConnectionHttp::HandleRequestError(const uint32 ErrorApiId, const uint32 ErrorCode)
{
	UE_LOGF(LogWinHttp, Warning, "WinHttp Http[%p]: Callback Status=[REQUEST_ERROR] ErrorCode=[0x%0.8X]", this, ErrorCode);

	FScopeLock ScopeLock(&SyncObject);

	if (!RequestHandle.IsValid())
	{
		// Request was probably cancelled, just stop here
		return;
	}

	switch (ErrorApiId)
	{
	case API_RECEIVE_RESPONSE:
		FWinHttpErrorHelper::LogWinHttpReceiveResponseFailure(ErrorCode);
		break;
	case API_QUERY_DATA_AVAILABLE:
		FWinHttpErrorHelper::LogWinHttpQueryDataAvailableFailure(ErrorCode);
		break;
	case API_READ_DATA:
		FWinHttpErrorHelper::LogWinHttpReadDataFailure(ErrorCode);
		break;
	case API_WRITE_DATA:
		FWinHttpErrorHelper::LogWinHttpWriteDataFailure(ErrorCode);
		break;
	case API_SEND_REQUEST:
		FWinHttpErrorHelper::LogWinHttpSendRequestFailure(ErrorCode);
		break;
	case API_GET_PROXY_FOR_URL:
		// We don't call this function so this shouldn't happen, but here for completions sake
		UE_LOGF(LogWinHttp, Error, "GetProxyForUrl failed with error code 0x%0.8X", ErrorCode);
		break;
	default:
		UE_LOGF(LogWinHttp, Error, "Unknown API (%u) failed with error code 0x%0.8X", ErrorApiId, ErrorCode);
		break;
	}

	if (!FinalState.IsSet())
	{
		FinishRequest(EHttpRequestStatus::Failed);
	}

	// If the request was cancelled, we can release the payload memory
	if (ErrorCode == ERROR_WINHTTP_OPERATION_CANCELLED)
	{
		ReleasePayloadData();
	}
}

void FWinHttpConnectionHttp::HandleHandleClosing()
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[HANDLE_CLOSING]", this);

	// If we are closing the request handle, we can release the payload memory
	ReleasePayloadData();

	KeepAlive.Reset();
}

void FWinHttpConnectionHttp::HandleHttpStatusCallback(HINTERNET ResourceHandle, EWinHttpCallbackStatus Status, void* StatusInformation, uint32 StatusInformationLength)
{
	// Prevent the request from dying while we're any callback
	TSharedPtr<FWinHttpConnectionHttp, ESPMode::ThreadSafe> LocalKeepAlive = KeepAlive;

	switch (Status)
	{
		case EWinHttpCallbackStatus::ConnectedToServer:
		{
			check(StatusInformationLength > 0);
			check(StatusInformation != nullptr);

			const wchar_t* const ServerIPString = static_cast<const wchar_t*>(StatusInformation);
			HandleConnectedToServer(ServerIPString);
			return;
		}
		case EWinHttpCallbackStatus::SendingRequest:
		{
			HandleSendingRequest();
			return;
		}
		case EWinHttpCallbackStatus::SendRequestComplete:
		{
			HandleSendRequestComplete();
			return;
		}
		case EWinHttpCallbackStatus::WriteComplete:
		{
			check(StatusInformationLength == sizeof(DWORD));
			check(StatusInformation != nullptr);
			const DWORD NumBytesSent = *static_cast<DWORD*>(StatusInformation);
			HandleWriteComplete(NumBytesSent);
			return;
		}
		case EWinHttpCallbackStatus::HeadersAvailable:
		{
			HandleHeadersAvailable();
			return;
		}
		case EWinHttpCallbackStatus::DataAvailable:
		{
			check(StatusInformationLength == sizeof(DWORD));
			check(StatusInformation != nullptr);
			const DWORD NumBytesAvailable = *static_cast<DWORD*>(StatusInformation);
			HandleDataAvailable(NumBytesAvailable);
			return;
		}
		case EWinHttpCallbackStatus::ReadComplete:
		{
			const DWORD NumBytesRead = StatusInformationLength;
			HandleReadComplete(NumBytesRead);
			return;
		}
		case EWinHttpCallbackStatus::RequestError:
		{
			check(StatusInformation != nullptr);
			const WINHTTP_ASYNC_RESULT* const AsyncResult = static_cast<WINHTTP_ASYNC_RESULT*>(StatusInformation);

			const DWORD ErrorApiId = AsyncResult->dwResult;
			const DWORD ErrorCode = AsyncResult->dwError;

			HandleRequestError(ErrorApiId, ErrorCode);
			return;
		}
		case EWinHttpCallbackStatus::HandleClosing:
		{
			HandleHandleClosing();
			// We don't exist anymore now, don't access anything here
			return;
		}

		/// Below are for logging only and aren't specifically handled by us
		
		case EWinHttpCallbackStatus::ResolvingName:
		{
			check(StatusInformationLength > 0);
			check(StatusInformation != nullptr);

			const wchar_t* const ServerName = static_cast<const wchar_t*>(StatusInformation);
			UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[%ls] ServerName=[%ls]", this, LexToString(Status), WCHAR_TO_TCHAR(ServerName));
			return;
		}
		case EWinHttpCallbackStatus::NameResolved:
		{
			check(StatusInformationLength > 0);
			check(StatusInformation != nullptr);

			const wchar_t* const ServerIPString = static_cast<const wchar_t*>(StatusInformation);
			UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[%ls] ServerIP=[%ls]", this, LexToString(Status), WCHAR_TO_TCHAR(ServerIPString));
			return;
		}
		case EWinHttpCallbackStatus::RequestSent:
		{
			check(StatusInformationLength == sizeof(DWORD));
			check(StatusInformation != nullptr);
			const DWORD NumBytesSent = *static_cast<DWORD*>(StatusInformation);
			UE_LOGF(LogWinHttp, Verbose, "WinHttp Http[%p]: Callback Status=[REQUEST_SENT] NumBytesSent=[%u]", this, NumBytesSent);
			return;
		}
		case EWinHttpCallbackStatus::ConnectingToServer:
		case EWinHttpCallbackStatus::ReceivingResponse:
		case EWinHttpCallbackStatus::ResponseReceived:
		case EWinHttpCallbackStatus::ClosingConnection:
		case EWinHttpCallbackStatus::ConnectionClosed:
		case EWinHttpCallbackStatus::HandleCreated:
		case EWinHttpCallbackStatus::DetectingProxy:
		case EWinHttpCallbackStatus::Redirect:
		case EWinHttpCallbackStatus::IntermediateResponse:
		case EWinHttpCallbackStatus::SecureFailure:
		case EWinHttpCallbackStatus::GetProxyForUrlComplete:
		case EWinHttpCallbackStatus::CloseComplete:
		case EWinHttpCallbackStatus::ShutdownComplete:
		case EWinHttpCallbackStatus::SettingsWriteComplete:
		case EWinHttpCallbackStatus::SettingsReadComplete:
			UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp Http[%p]: Callback Status=[%ls]", this, LexToString(Status));
			return;
	}

	checkNoEntry();
}

void FWinHttpConnectionHttp::ReleasePayloadData()
{
	// We don't need our payload data anymore, release the memory
	PayloadBuffer.Empty();
	Payload.Reset();
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // WITH_WINHTTP
