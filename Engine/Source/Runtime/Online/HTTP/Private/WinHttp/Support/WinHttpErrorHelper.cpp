// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WINHTTP

#include "WinHttp/Support/WinHttpErrorHelper.h"
#include "WinHttp/Support/WinHttpTypes.h"
#include "Http.h"

#include "Windows/AllowWindowsPlatformTypes.h"

void FWinHttpErrorHelper::LogWinHttpOpenFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpen failed to open session due to an internal error");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpen failed to open session due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpen failed to open session due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpCloseHandleFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_SHUTDOWN:
			UE_LOGF(LogWinHttp, Error, "WinHttpCloseHandle failed due to function support having already been shutdown");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpCloseHandle failed due to an internal error");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpCloseHandle failed due an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpCloseHandle failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpSetTimeoutsFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetTimeouts failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetTimeouts failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetTimeouts failed due to an internal error");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetTimeouts failed due an insufficient memory error");
			break;
		case ERROR_INVALID_PARAMETER:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetTimeouts failed due an invalid parameter");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetTimeouts failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpConnectFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpConnect failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpConnect failed due to an internal error");
			break;
		case ERROR_WINHTTP_INVALID_URL:
			UE_LOGF(LogWinHttp, Error, "WinHttpConnect failed due to an invalid URL");
			break;
		case ERROR_WINHTTP_OPERATION_CANCELLED:
			UE_LOGF(LogWinHttp, Error, "WinHttpConnect failed due to the operation being cancelled");
			break;
		case ERROR_WINHTTP_UNRECOGNIZED_SCHEME:
			UE_LOGF(LogWinHttp, Error, "WinHttpConnect failed due to an unrecognized uri scheme");
			break;
		case ERROR_WINHTTP_SHUTDOWN:
			UE_LOGF(LogWinHttp, Error, "WinHttpConnect failed due to the required functions having been unloaded");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpConnect failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpConnect failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpOpenRequestFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpenRequest failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpenRequest failed due to an internal error");
			break;
		case ERROR_WINHTTP_INVALID_URL:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpenRequest failed due to an invalid URL");
			break;
		case ERROR_WINHTTP_OPERATION_CANCELLED:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpenRequest failed due to the operation being cancelled");
			break;
		case ERROR_WINHTTP_UNRECOGNIZED_SCHEME:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpenRequest failed due to an unrecognized uri scheme");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpenRequest failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpOpenRequest failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpSetOptionFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetOption failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetOption failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetOption failed due to an internal error");
			break;
		case ERROR_WINHTTP_INVALID_OPTION:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetOption failed due to an invalid option");
			break;
		case ERROR_INVALID_PARAMETER:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetOption failed due to an invalid parameter");
			break;
		case ERROR_WINHTTP_OPTION_NOT_SETTABLE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetOption failed due to being called on an unsettable option");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetOption failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetOption failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpAddRequestHeadersFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpAddRequestHeaders failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpAddRequestHeaders failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpAddRequestHeaders failed due to an internal error");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpAddRequestHeaders failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpAddRequestHeaders failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpSetStatusCallbackFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetStatusCallback failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetStatusCallback failed due to an internal error");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetStatusCallback failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpSetStatusCallback failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpSendRequestFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_CANNOT_CONNECT:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to the connection to the server failing");
			break;
		case ERROR_WINHTTP_CLIENT_AUTH_CERT_NEEDED:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to a client auth certificate being needed");
			break;
		case ERROR_WINHTTP_CONNECTION_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an error establishing a secure connection to the server");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an internal error");
			break;
		case ERROR_WINHTTP_INVALID_URL:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an invalid Url");
			break;
		case ERROR_WINHTTP_LOGIN_FAILURE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to a login failure");
			break;
		case ERROR_WINHTTP_NAME_NOT_RESOLVED:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to a DNS name resolution failure");
			break;
		case ERROR_WINHTTP_OPERATION_CANCELLED:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to the operation being cancelled");
			break;
		case ERROR_WINHTTP_RESPONSE_DRAIN_OVERFLOW:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to a response overflowing an internal buffer");
			break;
		case ERROR_WINHTTP_SECURE_FAILURE:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to a security failure");
			break;
		case ERROR_WINHTTP_SHUTDOWN:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to WinHttp being shutdown");
			break;
		case ERROR_WINHTTP_TIMEOUT:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to a timeout");
			break;
		case ERROR_WINHTTP_UNRECOGNIZED_SCHEME:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an unrecognized scheme");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an insufficient memory error");
			break;
		case ERROR_INVALID_PARAMETER:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an invalid parameter");
			break;
		case ERROR_WINHTTP_RESEND_REQUEST:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed but the request should be sent again");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpSendRequest failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpReceiveResponseFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_CANNOT_CONNECT:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to the connection to the server failing");
			break;
		case ERROR_WINHTTP_CHUNKED_ENCODING_HEADER_SIZE_OVERFLOW:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an overflow parsing chunked encoding");
			break;
		case ERROR_WINHTTP_CLIENT_AUTH_CERT_NEEDED:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to client auth certificate being needed");
			break;
		case ERROR_WINHTTP_CONNECTION_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an error establishing a secure connection to the server");
			break;
		case ERROR_WINHTTP_HEADER_COUNT_EXCEEDED:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to a very large amount of headers in the response");
			break;
		case ERROR_WINHTTP_HEADER_SIZE_OVERFLOW:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to the response headers being too large");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an internal error");
			break;
		case ERROR_WINHTTP_INVALID_SERVER_RESPONSE:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an invalid server response");
			break;
		case ERROR_WINHTTP_INVALID_URL:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an invalid URL");
			break;
		case ERROR_WINHTTP_LOGIN_FAILURE:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to a login failure");
			break;
		case ERROR_WINHTTP_NAME_NOT_RESOLVED:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to a DNS name resolution failure");
			break;
		case ERROR_WINHTTP_OPERATION_CANCELLED:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to the operation being cancelled");
			break;
		case ERROR_WINHTTP_REDIRECT_FAILED:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to a redirect failure");
			break;
		case ERROR_WINHTTP_RESEND_REQUEST:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed but the request should be sent again");
			break;
		case ERROR_WINHTTP_RESPONSE_DRAIN_OVERFLOW:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to a response overflowing an internal buffer");
			break;
		case ERROR_WINHTTP_SECURE_FAILURE:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to a security failure");
			break;
		case ERROR_WINHTTP_TIMEOUT:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to a timeout");
			break;
		case ERROR_WINHTTP_UNRECOGNIZED_SCHEME:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an unrecognized scheme");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpReceiveResponse failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpQueryHeadersFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_HEADER_NOT_FOUND:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryHeaders failed due to the requested header not being found");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryHeaders failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryHeaders failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryHeaders failed due to an internal error");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryHeaders failed due to an insufficient memory error");
			break;
		case ERROR_INSUFFICIENT_BUFFER:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryHeaders failed due to an insufficently sized buffer");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryHeaders failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpQueryDataAvailableFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_CONNECTION_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryDataAvailable failed due to a connection error");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryDataAvailable failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryDataAvailable failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryDataAvailable failed due to an internal error");
			break;
		case ERROR_WINHTTP_OPERATION_CANCELLED:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryDataAvailable failed due to an operation being cancelled");
			break;
		case ERROR_WINHTTP_TIMEOUT:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryDataAvailable failed due to a timeout ");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryDataAvailable failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpQueryDataAvailable failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpReadDataFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_CONNECTION_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to a connection error");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to an internal error");
			break;
		case ERROR_WINHTTP_OPERATION_CANCELLED:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to an operation being cancelled");
			break;
		case ERROR_WINHTTP_RESPONSE_DRAIN_OVERFLOW:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to a response overflowing an internal buffer");
			break;
		case ERROR_WINHTTP_TIMEOUT:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to a timeout");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpReadData failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

void FWinHttpErrorHelper::LogWinHttpWriteDataFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_WINHTTP_CONNECTION_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpWriteData failed due to a connection error");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_STATE:
			UE_LOGF(LogWinHttp, Error, "WinHttpWriteData failed due to an incorrect handle state");
			break;
		case ERROR_WINHTTP_INCORRECT_HANDLE_TYPE:
			UE_LOGF(LogWinHttp, Error, "WinHttpWriteData failed due to an incorrect handle type");
			break;
		case ERROR_WINHTTP_INTERNAL_ERROR:
			UE_LOGF(LogWinHttp, Error, "WinHttpWriteData failed due to an internal error");
			break;
		case ERROR_WINHTTP_OPERATION_CANCELLED:
			UE_LOGF(LogWinHttp, Error, "WinHttpWriteData failed due to an operation being cancelled");
			break;
		case ERROR_WINHTTP_TIMEOUT:
			UE_LOGF(LogWinHttp, Error, "WinHttpWriteData failed due to a timeout");
			break;
		case ERROR_NOT_ENOUGH_MEMORY:
			UE_LOGF(LogWinHttp, Error, "WinHttpWriteData failed due to an insufficient memory error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpWriteData failed due to an unknown error (%0.8X)", ErrorCode);
			break;
	}
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // WITH_WINHTTP
