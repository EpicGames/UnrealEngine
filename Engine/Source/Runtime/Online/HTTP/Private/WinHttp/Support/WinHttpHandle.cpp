// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WINHTTP

#include "WinHttp/Support/WinHttpHandle.h"
#include "WinHttp/Support/WinHttpErrorHelper.h"
#include "WinHttp/Support/WinHttpTypes.h"
#include "Http.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <errhandlingapi.h>

FWinHttpHandle::FWinHttpHandle(HINTERNET NewHandle)
	: Handle(NewHandle)
{
}

FWinHttpHandle::~FWinHttpHandle()
{
	if (Handle != nullptr)
	{
		if (!WinHttpCloseHandle(Handle))
		{
			const DWORD ErrorCode = GetLastError();
			FWinHttpErrorHelper::LogWinHttpCloseHandleFailure(ErrorCode);
		}
		Handle = nullptr;
	}
}

FWinHttpHandle::FWinHttpHandle(FWinHttpHandle&& Other)
	: Handle(Other.Handle)
{
	Other.Handle = nullptr;
}

FWinHttpHandle& FWinHttpHandle::operator=(FWinHttpHandle&& Other)
{
	if (this != &Other)
	{
		HINTERNET TempHandle = Other.Handle;
		Other.Handle = Handle;
		Handle = TempHandle;
	}

	return *this;
}

FWinHttpHandle& FWinHttpHandle::operator=(HINTERNET NewHandle)
{
	if (Handle != NewHandle)
	{
		*this = FWinHttpHandle(NewHandle);
	}

	return *this;
}

void FWinHttpHandle::Reset()
{
	*this = FWinHttpHandle();
}

FWinHttpHandle::operator bool() const
{
	return IsValid();
}

bool FWinHttpHandle::IsValid() const
{
	return Handle != nullptr;
}

HINTERNET FWinHttpHandle::Get() const
{
	return Handle;
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // WITH_WINHTTP
