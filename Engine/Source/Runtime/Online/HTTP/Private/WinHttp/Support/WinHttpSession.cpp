// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WINHTTP

#include "WinHttp/Support/WinHttpSession.h"
#include "WinHttp/Support/WinHttpErrorHelper.h"
#include "WinHttp/Support/WinHttpConnection.h"
#include "WinHttp/Support/WinHttpTypes.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "Misc/ConfigCacheIni.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <errhandlingapi.h>

// Check if we support HTTP2
#if defined(WINHTTP_PROTOCOL_FLAG_HTTP2) && defined(WINHTTP_OPTION_ENABLE_HTTP_PROTOCOL)
#define UE_HAS_HTTP2_SUPPORT 1
#else // ^^^ defined(WINHTTP_PROTOCOL_FLAG_HTTP2) && defined(WINHTTP_OPTION_ENABLE_HTTP_PROTOCOL) ^^^ /// vvv !defined(WINHTTP_PROTOCOL_FLAG_HTTP2) || !defined(WINHTTP_OPTION_ENABLE_HTTP_PROTOCOL) vvv
#define UE_HAS_HTTP2_SUPPORT 0
#endif

FWinHttpSession::FWinHttpSession(uint32 SecurityProtocolFlags, const bool bInForceSecureConnections)
	: bForceSecureConnections(bInForceSecureConnections)
{
	const FString UserAgent = FGenericPlatformHttp::GetDefaultUserAgent();
	const DWORD AccessType = WINHTTP_ACCESS_TYPE_AUTOMATIC_PROXY;
	LPCWSTR ProxyAddress = WINHTTP_NO_PROXY_NAME;
	LPCWSTR ProxyBypass = WINHTTP_NO_PROXY_BYPASS;
	DWORD Flags = WINHTTP_FLAG_ASYNC;

	// Disable this on Windows now until we can do a runtime check for a future version of Windows10 that supports this flag
#if defined(WINHTTP_FLAG_SECURE_DEFAULTS) && !PLATFORM_WINDOWS
	if (bForceSecureConnections)
	{
		Flags |= WINHTTP_FLAG_SECURE_DEFAULTS;
	}
#endif

	SessionHandle = WinHttpOpen(TCHAR_TO_WCHAR(*UserAgent), AccessType, ProxyAddress, ProxyBypass, Flags);
	if (!SessionHandle.IsValid())
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpOpenFailure(ErrorCode);
		return;
	}

	DWORD dwSecurityProtocolFlags = SecurityProtocolFlags;
	if (!WinHttpSetOption(SessionHandle.Get(), WINHTTP_OPTION_SECURE_PROTOCOLS, &dwSecurityProtocolFlags, sizeof(dwSecurityProtocolFlags)))
	{
		// Get last error
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpSetOptionFailure(ErrorCode);

		// Reset handle to signify we failed
		SessionHandle.Reset();
		return;
	}

	// Opportunistically Enable HTTP2 if we can
#if UE_HAS_HTTP2_SUPPORT
	DWORD FlagEnableHTTP2 = WINHTTP_PROTOCOL_FLAG_HTTP2;
	if (WinHttpSetOption(SessionHandle.Get(), WINHTTP_OPTION_ENABLE_HTTP_PROTOCOL, &FlagEnableHTTP2, sizeof(FlagEnableHTTP2)))
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp local machine has support for HTTP/2");
	}
	else
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp local machine does not support HTTP/2, HTTP/1.1 will be used");
	}
#else // ^^^ UE_HAS_HTTP2_SUPPORT ^^^ // vvv !UE_HAS_HTTP2_SUPPORT vvv
	UE_LOGF(LogWinHttp, Verbose, "UE WinHttp compiled without HTTP/2 support");
#endif // !UE_HAS_HTTP2_SUPPORT

	// Opportunistically enable request compression if we can
	DWORD FlagEnableCompression = WINHTTP_DECOMPRESSION_FLAG_ALL;
	if (WinHttpSetOption(SessionHandle.Get(), WINHTTP_OPTION_DECOMPRESSION, &FlagEnableCompression, sizeof(FlagEnableCompression)))
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp local machine has support for compression");
	}
	else
	{
		UE_LOGF(LogWinHttp, Verbose, "WinHttp local machine does not support for compression");
	}

	float HttpConnectionTimeout = 30.0f;
	float HttpActivityTimeout = 30.0f;
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpConnectionTimeout"), HttpConnectionTimeout, GEngineIni);
	GConfig->GetFloat(TEXT("HTTP"), TEXT("HttpActivityTimeout"), HttpActivityTimeout, GEngineIni);
	const FTimespan ConnectionTimeout = FTimespan::FromSeconds(HttpConnectionTimeout > 0 ? HttpConnectionTimeout : 0);
	const FTimespan ResolveTimeout = ConnectionTimeout;
	const FTimespan ReceiveTimeout = FTimespan::FromSeconds(HttpActivityTimeout > 0 ? HttpActivityTimeout : 0);
	const FTimespan SendTimeout = ReceiveTimeout;

	if (!WinHttpSetTimeouts(SessionHandle.Get(), ResolveTimeout.GetTotalMilliseconds(), ConnectionTimeout.GetTotalMilliseconds(), SendTimeout.GetTotalMilliseconds(), ReceiveTimeout.GetTotalMilliseconds()))
	{
		// Get last error
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpSetTimeoutsFailure(ErrorCode);
	}

	// Success
}

bool FWinHttpSession::IsValid() const
{
	return SessionHandle.IsValid();
}

HINTERNET FWinHttpSession::Get() const
{
	return SessionHandle.Get();
}

bool FWinHttpSession::AreOnlySecureConnectionsAllowed() const
{
	return bForceSecureConnections;
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // WITH_WINHTTP
