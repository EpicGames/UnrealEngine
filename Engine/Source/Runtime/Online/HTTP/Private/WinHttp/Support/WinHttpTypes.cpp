// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WINHTTP

#include "WinHttp/Support/WinHttpTypes.h"

const TCHAR* LexToString(const EWinHttpCallbackStatus Status)
{
	switch (Status)
	{
		case EWinHttpCallbackStatus::ResolvingName: return TEXT("RESOLVING_NAME");
		case EWinHttpCallbackStatus::NameResolved: return TEXT("NAME_RESOLVED");
		case EWinHttpCallbackStatus::ConnectingToServer: return TEXT("CONNECTING_TO_SERVER");
		case EWinHttpCallbackStatus::ConnectedToServer: return TEXT("CONNECTED_TO_SERVER");
		case EWinHttpCallbackStatus::SendingRequest: return TEXT("SENDING_REQUEST");
		case EWinHttpCallbackStatus::RequestSent: return TEXT("REQUEST_SENT");
		case EWinHttpCallbackStatus::ReceivingResponse: return TEXT("RECEIVING_RESPONSE");
		case EWinHttpCallbackStatus::ResponseReceived: return TEXT("RESPONSE_RECEIVED");
		case EWinHttpCallbackStatus::ClosingConnection: return TEXT("CLOSING_CONNECTION");
		case EWinHttpCallbackStatus::ConnectionClosed: return TEXT("CONNECTION_CLOSED");
		case EWinHttpCallbackStatus::HandleCreated: return TEXT("HANDLE_CREATED");
		case EWinHttpCallbackStatus::HandleClosing: return TEXT("HANDLE_CLOSING");
		case EWinHttpCallbackStatus::DetectingProxy: return TEXT("DETECTING_PROXY");
		case EWinHttpCallbackStatus::Redirect: return TEXT("REDIRECT");
		case EWinHttpCallbackStatus::IntermediateResponse: return TEXT("INTERMEDIATE_RESPONSE");
		case EWinHttpCallbackStatus::SecureFailure: return TEXT("SECURE_FAILURE");
		case EWinHttpCallbackStatus::HeadersAvailable: return TEXT("HEADERS_AVAILABLE");
		case EWinHttpCallbackStatus::DataAvailable: return TEXT("DATA_AVAILABLE");
		case EWinHttpCallbackStatus::ReadComplete: return TEXT("READ_COMPLETE");
		case EWinHttpCallbackStatus::WriteComplete: return TEXT("WRITE_COMPLETE");
		case EWinHttpCallbackStatus::RequestError: return TEXT("REQUEST_ERROR");
		case EWinHttpCallbackStatus::SendRequestComplete: return TEXT("SENDREQUEST_COMPLETE");
		case EWinHttpCallbackStatus::GetProxyForUrlComplete: return TEXT("GETPROXYFORURL_COMPLETE");
		case EWinHttpCallbackStatus::CloseComplete: return TEXT("CLOSE_COMPLETE");
		case EWinHttpCallbackStatus::ShutdownComplete: return TEXT("SHUTDOWN_COMPLETE");
		case EWinHttpCallbackStatus::SettingsWriteComplete: return TEXT("SETTINGS_WRITE_COMPLETE");
		case EWinHttpCallbackStatus::SettingsReadComplete: return TEXT("SETTINGS_READ_COMPLETE");
	}

	checkNoEntry();
	return TEXT("");
}

bool IsValidStatus(const EWinHttpCallbackStatus Status)
{
	switch (Status)
	{
		case EWinHttpCallbackStatus::ResolvingName:
		case EWinHttpCallbackStatus::NameResolved:
		case EWinHttpCallbackStatus::ConnectingToServer:
		case EWinHttpCallbackStatus::ConnectedToServer:
		case EWinHttpCallbackStatus::SendingRequest:
		case EWinHttpCallbackStatus::RequestSent:
		case EWinHttpCallbackStatus::ReceivingResponse:
		case EWinHttpCallbackStatus::ResponseReceived:
		case EWinHttpCallbackStatus::ClosingConnection:
		case EWinHttpCallbackStatus::ConnectionClosed:
		case EWinHttpCallbackStatus::HandleCreated:
		case EWinHttpCallbackStatus::HandleClosing:
		case EWinHttpCallbackStatus::DetectingProxy:
		case EWinHttpCallbackStatus::Redirect:
		case EWinHttpCallbackStatus::IntermediateResponse:
		case EWinHttpCallbackStatus::SecureFailure:
		case EWinHttpCallbackStatus::HeadersAvailable:
		case EWinHttpCallbackStatus::DataAvailable:
		case EWinHttpCallbackStatus::ReadComplete:
		case EWinHttpCallbackStatus::WriteComplete:
		case EWinHttpCallbackStatus::RequestError:
		case EWinHttpCallbackStatus::SendRequestComplete:
		case EWinHttpCallbackStatus::GetProxyForUrlComplete:
		case EWinHttpCallbackStatus::CloseComplete:
		case EWinHttpCallbackStatus::ShutdownComplete:
		case EWinHttpCallbackStatus::SettingsWriteComplete:
		case EWinHttpCallbackStatus::SettingsReadComplete:
			return true;
	}

	return false;
}

DEFINE_LOG_CATEGORY(LogWinHttp);

#endif // WITH_WINHTTP