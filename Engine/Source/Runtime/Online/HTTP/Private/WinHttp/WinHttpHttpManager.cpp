// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WINHTTP

#include "WinHttp/WinHttpHttpManager.h"
#include "Containers/BackgroundableTicker.h"
#include "HttpModule.h"
#include "WinHttp/Support/WinHttpSession.h"
#include "WinHttp/Support/WinHttpTypes.h"
#include "Http.h"
#include "Stats/Stats.h"

#if !UE_BUILD_SHIPPING
#include "HAL/IConsoleManager.h"
#endif

#if !UE_BUILD_SHIPPING
extern TAutoConsoleVariable<bool> CVarHttpInsecureProtocolEnabled;
#endif

namespace
{
	FWinHttpHttpManager* GWinHttpManager = nullptr;
}

FWinHttpHttpManager* FWinHttpHttpManager::GetManager()
{
	return GWinHttpManager;
}

FWinHttpHttpManager::FWinHttpHttpManager()
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
{
	if (GWinHttpManager == nullptr)
	{
		GWinHttpManager = this;
	}
}

FWinHttpHttpManager::~FWinHttpHttpManager()
{
	if (GWinHttpManager == this)
	{
		GWinHttpManager = nullptr;
	}
}

#endif //WITH_WINHTTP
