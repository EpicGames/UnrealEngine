// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformHttp.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/ConfigCacheIni.h"
#include "Curl/CurlHttp.h"
#include "Curl/CurlHttpManager.h"
#include "Http.h"
#include "WinHttp/Support/WinHttpTypes.h" // Always include for OS proxy settings

static bool IsUnsignedInteger(const FString& InString)
{
	bool bResult = true;
	for (auto CharacterIter : InString)
	{
		if (!FChar::IsDigit(CharacterIter))
		{
			bResult = false;
			break;
		}
	}
	return bResult;
}

static bool IsValidIPv4Address(const FString& InString)
{
	bool bResult = false;

	FString Temp;
	FString AStr, BStr, CStr, DStr, PortStr;

	bool bWasPatternMatched = false;
	if (InString.Split(TEXT("."), &AStr, &Temp))
	{
		if (Temp.Split(TEXT("."), &BStr, &Temp))
		{
			if (Temp.Split(TEXT("."), &CStr, &Temp))
			{
				if (Temp.Split(TEXT(":"), &DStr, &PortStr))
				{
					bWasPatternMatched = true;
				}
			}
		}
	}

	if (bWasPatternMatched)
	{
		if (IsUnsignedInteger(AStr) && IsUnsignedInteger(BStr) && IsUnsignedInteger(CStr) && IsUnsignedInteger(DStr) && IsUnsignedInteger(PortStr))
		{
			uint32 A, B, C, D, Port;
			LexFromString(A, *AStr);
			LexFromString(B, *BStr);
			LexFromString(C, *CStr);
			LexFromString(D, *DStr);
			LexFromString(Port, *PortStr);

			if (A < 256 && B < 256 && C < 256 && D < 256 && Port < 65536)
			{
				bResult = true;
			}
		}
	}

	return bResult;
}

void FWindowsPlatformHttp::Init()
{
	FString HttpMode;
	if (FParse::Value(FCommandLine::Get(), TEXT("HTTP="), HttpMode) &&
		(HttpMode.Equals(TEXT("WinInet"), ESearchCase::IgnoreCase)))
	{
		UE_LOGF(LogHttp, Warning, "-HTTP=WinInet is no longer valid");
	}

	FGenericPlatformHttp::Init();

	FCurlHttpManager::InitCurl();
}

void FWindowsPlatformHttp::Shutdown()
{
	FCurlHttpManager::ShutdownCurl();

	FGenericPlatformHttp::Shutdown();
}

FHttpManager * FWindowsPlatformHttp::CreatePlatformHttpManager()
{
	return new FCurlHttpManager();
}

IHttpRequest* FWindowsPlatformHttp::ConstructRequest()
{
	return new FCurlHttpRequest();
}

FString FWindowsPlatformHttp::GetMimeType(const FString& FilePath)
{
	FString MimeType = "application/unknown";
	const FString FileExtension = FPaths::GetExtension(FilePath, true);

	HKEY hKey;
	if ( ::RegOpenKeyEx(HKEY_CLASSES_ROOT, *FileExtension, 0, KEY_READ, &hKey) == ERROR_SUCCESS )
	{
		TCHAR MimeTypeBuffer[128];
		DWORD MimeTypeBufferSize = sizeof(MimeTypeBuffer);
		DWORD KeyType = 0;

		if ( ::RegQueryValueEx(hKey, TEXT("Content Type"), NULL, &KeyType, (BYTE*)MimeTypeBuffer, &MimeTypeBufferSize) == ERROR_SUCCESS && KeyType == REG_SZ )
		{
			MimeType = MimeTypeBuffer;
		}

		::RegCloseKey(hKey);
	}

	return MimeType;
}

TOptional<FString> FWindowsPlatformHttp::GetOperatingSystemProxyAddress()
{
	FString ProxyAddress;

	// Retrieve the default proxy configuration.
	WINHTTP_PROXY_INFO DefaultProxyInfo;
	memset(&DefaultProxyInfo, 0, sizeof(DefaultProxyInfo));
	WinHttpGetDefaultProxyConfiguration(&DefaultProxyInfo);

	if (DefaultProxyInfo.lpszProxy != nullptr)
	{
		FString TempProxy(DefaultProxyInfo.lpszProxy);
		if (IsValidIPv4Address(TempProxy))
		{
			ProxyAddress = MoveTemp(TempProxy);
		}
		else
		{
			if (TempProxy.Split(TEXT("https="), nullptr, &TempProxy))
			{
				TempProxy.Split(TEXT(";"), &TempProxy, nullptr);
				if (IsValidIPv4Address(TempProxy))
				{
					ProxyAddress = MoveTemp(TempProxy);
				}
			}
		}
	}

	// Look for the proxy setting for the current user. Charles proxies count in here.
	if (ProxyAddress.IsEmpty())
	{
		WINHTTP_CURRENT_USER_IE_PROXY_CONFIG IeProxyInfo;
		memset(&IeProxyInfo, 0, sizeof(IeProxyInfo));
		WinHttpGetIEProxyConfigForCurrentUser(&IeProxyInfo);

		if (IeProxyInfo.lpszProxy != nullptr)
		{
			FString TempProxy(IeProxyInfo.lpszProxy);
			if (IsValidIPv4Address(TempProxy))
			{
				ProxyAddress = MoveTemp(TempProxy);
			}
			else
			{
				if (TempProxy.Split(TEXT("https="), nullptr, &TempProxy))
				{
					TempProxy.Split(TEXT(";"), &TempProxy, nullptr);
					if (IsValidIPv4Address(TempProxy))
					{
						ProxyAddress = MoveTemp(TempProxy);
					}
				}
			}
		}
	}
	return ProxyAddress;
}

bool FWindowsPlatformHttp::IsOperatingSystemProxyInformationSupported()
{
	return true;
}

FString FWindowsPlatformHttp::GetConfiguredProxyAddress()
{
	if (TOptional<FString> ConfigProxy = GetProxyAddressFromConfig())
	{
		return ConfigProxy.GetValue();
	}
	if (TOptional<FString> OsProxy = FWindowsPlatformHttp::GetOperatingSystemProxyAddress())
	{
		return OsProxy.GetValue();
	}
	return FString();
}

bool FWindowsPlatformHttp::VerifyPeerSslCertificate(bool verify)
{
	bool bPreviousValue = FCurlHttpManager::CurlRequestOptions.bVerifyPeer;
	FCurlHttpManager::CurlRequestOptions.bVerifyPeer = verify;
	return bPreviousValue;
}
