// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformHttp.h"

@class NSURLSessionTask;
@class NSURLSession;

/**
 * Platform specific Http implementations
 */
class HTTP_API FApplePlatformHttp : public FGenericPlatformHttp
{
public:
	/**
	 * Platform initialization step
	 */
	static void Init();

	/**
	 * Creates a platform-specific HTTP manager.
	 *
	 * @return NULL if default implementation is to be used
	 */
	static FHttpManager* CreatePlatformHttpManager();

	/**
	 * Platform shutdown step
	 */
	static void Shutdown();

	/**
	 * Creates a new Http request instance for the current platform
	 *
	 * @return request object
	 */
	static IHttpRequest* ConstructRequest();

	static bool IsBackgroundRequestFeatureEnabled();

	static TSharedPtr<class FAppleHttpRequest>& Add(NSURLSessionTask* Task, const TSharedPtr<class FAppleHttpRequest>& Request);
	static bool Find(NSURLSessionTask* Task, OUT TSharedPtr<class FAppleHttpRequest>& Request);
	static bool RemoveAndCopyValue(NSURLSessionTask* Task, OUT TSharedPtr<class FAppleHttpRequest>& Request);

private:
    /** Session used to create Apple based requests */
    static inline NSURLSession* Session = nullptr;
	static inline NSURLSession* BackgroundSession = nullptr;

	static FRWLock TaskRequestMapLock;
	static TMap<NSURLSessionTask*, TSharedPtr<class FAppleHttpRequest>> TaskRequestMap;

	static void InitWithNSUrlSession();
	static void ShutdownWithNSUrlSession();
};


typedef FApplePlatformHttp FPlatformHttp;
