// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/Optional.h"

class FHttpManager;
class IHttpRequest;

/**
 * Utility structure to help with building the HTTP user agent string.
 */
class FDefaultUserAgentBuilder
{
public:
	HTTP_API FDefaultUserAgentBuilder();

	// Build the user agent string. Passing a filter will limit the comments to only those allowed.
	// Passing nullptr for a filter implies that all comments are allowed.
	HTTP_API FString BuildUserAgentString(const TSet<FString>* AllowedProjectCommentsFilter = nullptr, const TSet<FString>* AllowedPlatformCommentsFilter = nullptr) const;

	HTTP_API void SetProjectName(const FString& InProjectName);
	HTTP_API void SetProjectVersion(const FString& InProjectVersion);
	HTTP_API void AddProjectComment(const FString& InComment);
	HTTP_API void SetPlatformName(const FString& InPlatformName);
	HTTP_API void SetPlatformVersion(const FString& InPlatformVersion);
	HTTP_API void AddPlatformComment(const FString& InComment);

	// Get the builder version. The version is used for tracking app changes to the default user
	// agent and is not included in the user agent string.
	HTTP_API uint32 GetAgentVersion() const;

private:
	static FString BuildCommentString(const TArray<FString>& Comments, const TSet<FString>* AllowedCommentsFilter);

	FString ProjectName;
	FString ProjectVersion;
	TArray<FString> ProjectComments;
	FString PlatformName;
	FString PlatformVersion;
	TArray<FString> PlatformComments;
	uint32 AgentVersion;
};

/**
 * Platform specific Http implementations
 * Intended usage is to use FPlatformHttp instead of FGenericPlatformHttp
 */
class FGenericPlatformHttp
{
public:

	/**
	 * Platform initialization step
	 */
	static HTTP_API void Init();

	/**
	 * Creates a platform-specific HTTP manager.
	 *
	 * @return nullptr if default implementation is to be used
	 */
	static FHttpManager* CreatePlatformHttpManager()
	{
		return nullptr;
	}

	/**
	 * Platform shutdown step
	 */
	static HTTP_API void Shutdown();

	/**
	 * Platform shutdown custom ssl if supported
	 */
	static void ShutdownPlatformCustomSsl() {}

	/**
	 * Creates a new Http request instance for the current platform
	 *
	 * @return request object
	 */
	static HTTP_API IHttpRequest* ConstructRequest();

	/**
	 * Returns a percent-encoded version of the passed in string
	 *
	 * @param UnencodedString The unencoded string to convert to percent-encoding
	 * @return The percent-encoded string
	 */
	static HTTP_API FString UrlEncode(const FStringView UnencodedString);

	/**
	 * Returns a decoded version of the percent-encoded passed in string
	 *
	 * @param EncodedString The percent encoded string to convert to string
	 * @return The decoded string
	 */
	static HTTP_API FString UrlDecode(const FStringView EncodedString);

	/**
	 * Returns the &lt; &gt...etc encoding for strings between HTML elements.
	 *
	 * @param UnencodedString The unencoded string to convert to html encoding.
	 * @return The html encoded string
	 */
	static HTTP_API FString HtmlEncode(const FStringView UnencodedString);

	/** 
	* Returns the domain and port portion of the URL, e.g., "a.b.c:d" of "http://a.b.c:d/e"
	* @param Url the URL to return the domain and port of
	* @return the domain and port of the specified URL
	*/
	static HTTP_API FString GetUrlDomainAndPort(const FStringView Url);

	/** 
	 * Returns the domain portion of the URL, e.g., "a.b.c" of "http://a.b.c:d/e"
	 * @param Url the URL to return the domain of
	 * @return the domain of the specified URL
	 */
	static HTTP_API FString GetUrlDomain(const FStringView Url);

	/** 
	* Returns the base of the URL, e.g., "http://a.b.c:d" of "http://a.b.c:d/e"
	* @param Url the URL to return the base of
	* @return the base of the specified URL
	*/
	static HTTP_API FString GetUrlBase(const FStringView Url);

	/**
	 * Get the mime type for the file
	 * @return the mime type for the file.
	 */
	static HTTP_API FString GetMimeType(const FString& FilePath);

	/**
	 * Returns the default User-Agent string to use in HTTP requests.
	 * Requests that explicitly set the User-Agent header will not use this value.
	 * 
	 * The default User-Agent is built in the format "project/version (comments) platform/OS version (comments)"
	 *
	 * @return the default User-Agent string that requests should use.
	 */
	static HTTP_API FString GetDefaultUserAgent();
	static HTTP_API FString EscapeUserAgentString(const FString& UnescapedString);

	/**
	 * Add a comment to be included in the project section of the default User-Agent string.
	 */
	static HTTP_API void AddDefaultUserAgentProjectComment(const FString& Comment);

	/**
	 * Add a comment to be included in the platform section of the default User-Agent string.
	 */
	static HTTP_API void AddDefaultUserAgentPlatformComment(const FString& Comment);

	/**
	 * Get the version of the default user agent. The version is incremented any time the default
	 * user agent is changed. Used to invalidate any cached user agent value.
	 * 
	 * @return the current version of the default user agent.
	 */
	static HTTP_API uint32 GetDefaultUserAgentVersion();

	/**
	 * Gets a copy of the values used to build the default user agent string. Used for modifying
	 * default user agent values before sending HTTP requests.
	 */
	static HTTP_API FDefaultUserAgentBuilder GetDefaultUserAgentBuilder();

	/**
	 * Utility for escaping a string used in the HTTP user agent. Removes disallowed characters.
	 * 
	 * @return the escaped string.
	 */

	/**
	 * Get the configured proxy address, using the same precedence as the HTTP module:
	 * 1. Command-line: -httpproxy=
	 * 2. INI: [HTTP] HttpProxyAddress
	 * 3. Operating system proxy
	 *
	 * @return the proxy address, or an empty string if none is configured
	 */
	static HTTP_API FString GetConfiguredProxyAddress();

	/**
	 * Get the proxy address specified by the operating system
	 *
	 * @return optional FString: If unset: we are unable to get information from the operating system. If set: the proxy address set by the operating system (may be blank)
	 */
	static HTTP_API TOptional<FString> GetOperatingSystemProxyAddress();

	/**
	 * Check if getting proxy information from the current operating system is supported
	 * Useful for "Network Settings" type pages.  GetProxyAddress may return an empty or populated string but that does not imply
	 * the operating system does or does not support proxies (or that it has been implemented here)
	 *
	 * @return true if we are able to get proxy information from the current operating system, false if not
	 */
	static HTTP_API bool IsOperatingSystemProxyInformationSupported();

	/**
	 * Internal helper that checks command-line and INI config for a proxy address.
	 * Returns the proxy if found (set), or unset if neither cmdline nor INI has a proxy.
	 * Callers use this to determine whether to fallback to OS-specific proxy queries.
	 *
	 * @return Set with proxy address if found in cmdline/INI, unset otherwise
	 */
	static HTTP_API TOptional<FString> GetProxyAddressFromConfig();

	/**
	 * Helper function for checking if a byte array is in URL encoded format.
	 */
	static HTTP_API bool IsURLEncoded(const TArray<uint8>& Payload);

	/**
	 * Extract the URL-Decoded value of the specified ParameterName from Url. An unset return means the parameter was not present in Url, while an empty value means it was present, but had no value.
	 * 
	 * @param Url The URL to parse for ParameterName
	 * @param ParameterName The parameter name to look for
	 * @return If ParameterName was found, the string value of its value (in URL Decoded format), otherwise the return value is unset
	 */
	static HTTP_API TOptional<FString> GetUrlParameter(const FStringView Url, const FStringView ParameterName);

	/**
	 * Extract the Port part of a URL, or an unset object if there was non specified. Example: "http://example.org:23/" would return 23, while "https://example.org/" would return an unset object
	 * 
	 * @param Url The URL to parse for a Port
	 */
	static HTTP_API TOptional<uint16> GetUrlPort(const FStringView Url);

	/**
	 * Extract the Path portion of a URL, optionally including the Query String, and optionally including the Fragment (when also including the query string.)
	 * The return value will always contain a leading forward slash, even if no path is found.
	 * If bIncludeFragment is set true, bIncludeQueryString must also be true
	 *
	 * @param Url The URL to parse for a Path
	 * @param bIncludeQueryString include the URL's query string in the return value (if one is found)
	 * @param bIncludeFragment include the URL's fragement in the return value (if one is found)
	 */
	static HTTP_API FString GetUrlPath(const FStringView Url, const bool bIncludeQueryString = false, const bool bIncludeFragment = false);

	/**
	 * Check the protocol of the provided URL to determine if this is for a secure connection (HTTPS, WSS, etc)
	 *
	 * @param The URL to check for a scheme
	 * @return True if secure, false if insecure, unset if unknown
	 */
	static HTTP_API TOptional<bool> IsSecureProtocol(const FStringView Url);
};
