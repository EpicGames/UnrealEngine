// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IHttpRequest.h"
#include "Misc/Optional.h"

/**
 * Contains implementation of some common functions that don't vary between implementation
 */
class FHttpRequestImpl : public IHttpRequest
{
public:
	// IHttpRequest
	HTTP_API virtual FHttpRequestCompleteDelegate& OnProcessRequestComplete() override;
	HTTP_API virtual FHttpRequestProgressDelegate64& OnRequestProgress64() override;
	HTTP_API virtual FHttpRequestStatusCodeReceivedDelegate& OnStatusCodeReceived() override;
	HTTP_API virtual FHttpRequestHeaderReceivedDelegate& OnHeaderReceived() override;
	HTTP_API virtual FHttpRequestWillRetryDelegate& OnRequestWillRetry() override;

	/**
	 * Clear callbacks before http module get deleted
	 */
	HTTP_API virtual void Shutdown();

protected:
	/** 
	 * Broadcast all of our response's headers as having been received
	 * Used when we don't know when we receive headers in our HTTP implementation
	 */
	HTTP_API void BroadcastResponseHeadersReceived();

protected:
	/** Delegate that will get called once request completes or on any error */
	FHttpRequestCompleteDelegate RequestCompleteDelegate;

	/** Delegate that will get called once per tick with bytes downloaded so far */
	FHttpRequestProgressDelegate64 RequestProgressDelegate64;

	/** Delegate that will get called when status code received */
	FHttpRequestStatusCodeReceivedDelegate StatusCodeReceivedDelegate;

	/** Delegate that will get called for each new header received */
	FHttpRequestHeaderReceivedDelegate HeaderReceivedDelegate;
	
	/** Delegate that will get called when request will be retried */
	FHttpRequestWillRetryDelegate OnRequestWillRetryDelegate;
};
