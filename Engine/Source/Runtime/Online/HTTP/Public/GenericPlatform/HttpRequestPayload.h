// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "HAL/PlatformMath.h"
#include "Templates/SharedPointer.h"

class FArchive;

/**
* Abstraction that encapsulates the location of a request payload
*/
class FRequestPayload
{
public:
	virtual ~FRequestPayload() = default;
	/** Get the total content length of the request payload in bytes */
	virtual uint64 GetContentLength() const = 0;
	/** Return a reference to the underlying memory buffer. Only valid for in-memory request payloads */
	virtual const TArray<uint8>& GetContent() const = 0;
	/** Check if the request payload is URL encoded. This check is only performed for in-memory request payloads */
	virtual bool IsURLEncoded() const = 0;
	/**
	 * Read part of the underlying request payload into an output buffer.
	 * @param OutputBuffer - the destination memory address where the payload should be copied
	 * @param MaxOutputBufferSize - capacity of OutputBuffer in bytes
	 * @param SizeAlreadySent - how much of payload has previously been sent.
	 * @return Returns the number of bytes copied into OutputBuffer
	 */
	virtual size_t FillOutputBuffer(void* OutputBuffer, size_t MaxOutputBufferSize, size_t SizeAlreadySent) = 0;

	/**
	 * Read part of the underlying request payload into an output buffer.
	 * @param OutputBuffer - the destination memory address where the payload should be copied
	 * @param MaxOutputBufferSize - capacity of OutputBuffer in bytes
	 * @param SizeAlreadySent - how much of payload has previously been sent.
	 * @return Returns the number of bytes copied into OutputBuffer
	 */
	virtual size_t FillOutputBuffer(TArrayView<uint8> OutputBuffer, size_t SizeAlreadySent) = 0;

	/** Open the upload payload before start */
	virtual bool Open() = 0;

	/** Close the upload payload when the http request don't need to use it anymore */
	virtual void Close() = 0;
};

class FRequestPayloadInFileStream : public FRequestPayload
{
public:
	FRequestPayloadInFileStream(const FString& InFilename);
	FRequestPayloadInFileStream(TSharedRef<FArchive> InFile, bool bInCloseWhenComplete = false);
	virtual ~FRequestPayloadInFileStream();
	virtual uint64 GetContentLength() const override;
	virtual const TArray<uint8>& GetContent() const override;
	virtual bool IsURLEncoded() const override;
	virtual size_t FillOutputBuffer(void* OutputBuffer, size_t MaxOutputBufferSize, size_t SizeAlreadySent) override;
	virtual size_t FillOutputBuffer(TArrayView<uint8> OutputBuffer, size_t SizeAlreadySent) override;
	virtual bool Open() override;
	virtual void Close() override;

private:
	FString Filename;
	TSharedPtr<FArchive> File;
	bool bCloseWhenComplete = false;
};

class FRequestPayloadInMemory : public FRequestPayload
{
public:
	FRequestPayloadInMemory(const TArray<uint8>& Array);
	FRequestPayloadInMemory(TArray<uint8>&& Array);
	virtual ~FRequestPayloadInMemory();
	virtual uint64 GetContentLength() const override;
	virtual const TArray<uint8>& GetContent() const override;
	virtual bool IsURLEncoded() const override;
	virtual size_t FillOutputBuffer(void* OutputBuffer, size_t MaxOutputBufferSize, size_t SizeAlreadySent) override;
	virtual size_t FillOutputBuffer(TArrayView<uint8> OutputBuffer, size_t SizeAlreadySent) override;
	virtual bool Open() override;
	virtual void Close() override;
private:
	TArray<uint8> Buffer;
};
