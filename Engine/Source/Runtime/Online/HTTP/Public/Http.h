// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Module dependencies

#include "CoreMinimal.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"

// Module includes
#include "PlatformHttp.h"

HTTP_API DECLARE_LOG_CATEGORY_EXTERN(LogHttp, Display, All);
