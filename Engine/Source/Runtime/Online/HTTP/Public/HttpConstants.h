// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API HTTP_API

/**
 * Universal HTTP Constants
 */
struct FHttpConstants
{
	/** Basic auth */
	static UE_API const TCHAR* const AUTH_BASIC;
	/** Bearer auth */
	static UE_API const TCHAR* const AUTH_BEARER;

	/** DELETE */
	static UE_API const TCHAR* const VERB_DELETE;
	/** GET */
	static UE_API const TCHAR* const VERB_GET;
	/** HEAD */
	static UE_API const TCHAR* const VERB_HEAD;
	/** PATCH */
	static UE_API const TCHAR* const VERB_PATCH;
	/** POST */
	static UE_API const TCHAR* const VERB_POST;
	/** PUT */
	static UE_API const TCHAR* const VERB_PUT;

	/** Accept */
	static UE_API const TCHAR* const HEADER_ACCEPT;
	/** Accept-Encoding */
	static UE_API const TCHAR* const HEADER_ACCEPT_ENCODING;
	/** Authorization */
	static UE_API const TCHAR* const HEADER_AUTHORIZATION;
	/** Content-Length */
	static UE_API const TCHAR* const HEADER_CONTENT_LENGTH;
	/** Content-Type */
	static UE_API const TCHAR* const HEADER_CONTENT_TYPE;
	/** Server Date */
	static UE_API const TCHAR* const HEADER_DATE;
	/** User Agent */
	static UE_API const TCHAR* const HEADER_USER_AGENT;

	/** application/json */
	static UE_API const TCHAR* const MEDIATYPE_JSON;
	/** application/x-www-form-urlencoded */
	static UE_API const TCHAR* const MEDIATYPE_FORM_URLENCODED;
	/** application/octet-stream */
	static UE_API const TCHAR* const MEDIATYPE_OCTET_STREAM;

	/** gzip */
	static UE_API const TCHAR* const MEDIAENCODING_GZIP;

	/** version */
	static UE_API const TCHAR* const VERSION_2TLS;
	static UE_API const TCHAR* const VERSION_1_1;
};

#undef UE_API
