// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"
#include "HttpConstants.h"

class IHttpRequest;
class IHttpResponse;

typedef TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> FHttpRequestPtr;
typedef TSharedPtr<IHttpResponse, ESPMode::ThreadSafe> FHttpResponsePtr;

typedef TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FHttpRequestRef;
typedef TSharedRef<IHttpResponse, ESPMode::ThreadSafe> FHttpResponseRef;

namespace FHttpRetrySystem
{
	using RetryLimitCountType = uint32;
	using RetryTimeoutRelativeSecondsType = double;

	using FRandomFailureRateSetting = TOptional<float>;
	using FRetryLimitCountSetting = TOptional<RetryLimitCountType>;
	using FRetryTimeoutRelativeSecondsSetting = TOptional<RetryTimeoutRelativeSecondsType>;
	using FRetryResponseCodes = TSet<int32>;
	using FRetryVerbs = TSet<FName>;

	struct FRetryDomains;
	using FRetryDomainsPtr = TSharedPtr<FRetryDomains, ESPMode::ThreadSafe>;
};
