// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HttpPackage.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/EnumRange.h"
#include "Misc/URLRequestFilter.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

class FHttpThreadBase;
class FHttpRequestCommon;
class FOutputDevice;
class IHttpTaskTimerHandle;

enum class EHttpFlushReason : uint8
{
	/** Reasonable, typically higher time limits */
	Default,
	/** Shorter time limits depending on platform requirements */
	Shutdown,
	/** Infinite wait, should only be used in non-game scenarios where longer waits are acceptable */
	FullFlush,
	Count
};
ENUM_RANGE_BY_COUNT(EHttpFlushReason, EHttpFlushReason::Count)
const TCHAR* LexToString(const EHttpFlushReason& FlushReason);

/**
 * Delegate called when an Http request added. Intended to be used for analytics. Called from the thread that adds the HTTP request.
 * @param Request Http request that was added
 */
DECLARE_DELEGATE_OneParam(FHttpManagerRequestAddedDelegate, const FHttpRequestRef& /*Request*/);

/**
 * Delegate called when an Http request completes. Intended to be used for analytics. Called from the game thread.
 * @param Request Http request that completed
 */
DECLARE_DELEGATE_OneParam(FHttpManagerRequestCompletedDelegate, const FHttpRequestRef& /*Request*/);

struct FHttpStatsPlatformMemoryPool
{
	uint64 PoolSize = 0;
	uint64 MaxInUseSize = 0;
	uint64 CurrentInUseSize = 0;
};

struct FHttpStatsPlatform
{
	FHttpStatsPlatformMemoryPool MemoryPoolConnection;
	FHttpStatsPlatformMemoryPool MemoryPoolSsl;
	FHttpStatsPlatformMemoryPool MemoryPoolNet;
};

struct FHttpStats
{
	// Use atomic for the following fields because csv profiler reads them from game thread while 
	// http thread record them from http thread

	/** The number of requests waiting in queue in http manager */
	std::atomic<int32> RequestsInQueue = 0;
	/** The number of requests in flight in http manager */
	std::atomic<int32> RequestsInFlight = 0;
	/** The max time to successfully connect the backend */
	std::atomic<float> MaxTimeToConnect = -1.0f;
	/** The max waiting queue in http manager */
	std::atomic<uint32> MaxRequestsInQueue = 0;
	/** The max number of requests in flight in http manager */
	std::atomic<uint32> MaxRequestsInFlight = 0;
	/** The max waiting time in queue of http manager */
	std::atomic<float> MaxTimeToWaitInQueue = 0.0f;
	/** The total bytes downloaded so far */
	std::atomic<int64> TotalDownloadedBytes = 0;
	/** Approximate download bandwidth used */
	std::atomic<int64> BandwidthMbps = 0;
	/** Avg duration (in milliseconds) from request to response */
	std::atomic<int64> HttpDurationMsAvg = 0;
	/** The optional http stats on specific platform */
	TOptional<FHttpStatsPlatform> PlatformStats;

	/**
	 * Request completion counters indexed by EHttpFailureReason.
	 * Index 0 (None) counts succeeded requests; indices 1+ count each failure reason.
	 */
	std::atomic<int64> RequestResults[static_cast<uint8>(EHttpFailureReason::Count)] = {};

	FHttpStats()
	{
	}

	FHttpStats(const FHttpStats& Other)
		: RequestsInQueue(Other.RequestsInQueue.load())
		, RequestsInFlight(Other.RequestsInFlight.load())
		, MaxTimeToConnect(Other.MaxTimeToConnect.load())
		, MaxRequestsInQueue(Other.MaxRequestsInQueue.load())
		, MaxRequestsInFlight(Other.MaxRequestsInFlight.load())
		, MaxTimeToWaitInQueue(Other.MaxTimeToWaitInQueue.load())
		, TotalDownloadedBytes(Other.TotalDownloadedBytes.load())
		, BandwidthMbps(Other.BandwidthMbps.load())
		, HttpDurationMsAvg(Other.HttpDurationMsAvg.load())
		, PlatformStats(Other.PlatformStats)
	{
		for (uint8 i = 0; i < static_cast<uint8>(EHttpFailureReason::Count); ++i)
		{
			RequestResults[i] = Other.RequestResults[i].load();
		}
	}
};

/**
 * Manages Http request that are currently being processed
 */
class FHttpManager
	: public FTSTickerObjectBase
{
public:

	// FHttpManager

	/**
	 * Constructor
	 */
	HTTP_API FHttpManager();

	/**
	 * Destructor
	 */
	HTTP_API virtual ~FHttpManager();

	/**
	 * Initialize
	 */
	HTTP_API void Initialize();

	/**
	 * Shutdown logic should be called before quiting
	 */
	HTTP_API void Shutdown();

	/**
	 * Set a delegate to be triggered when an http request added to http manager. 
	 * NOTE: The delegate can be triggered from different threads, depends on which
	 * thread the request created. So make sure the delegate set here is thread-safe.
	 */
	HTTP_API void SetRequestAddedDelegate(const FHttpManagerRequestAddedDelegate& Delegate);

	/**
	 * Set a delegate to be triggered when an http request completed. 
	 * NOTE: The delegate can be triggered from different threads, it depends on the delegate policy set 
	 * to each http request. So make sure the delegate set here is thread-safe.
	 */
	HTTP_API void SetRequestCompletedDelegate(const FHttpManagerRequestCompletedDelegate& Delegate);

	/**
	 * Removes an Http request instance from the manager
	 * Presumably it is done being processed
	 *
	 * @param Request - the request object to remove
	 */
	HTTP_API void RemoveRequest(const FHttpRequestRef& Request);

	/**
	* Find an Http request in the lists of current valid requests
	*
	* @param RequestPtr - ptr to the http request object to find
	*
	* @return true if the request is being tracked, false if not
	*/
	HTTP_API bool IsValidRequest(const IHttpRequest* RequestPtr) const;

	/**
	 * Block until all pending requests are finished processing
	 *
	 * @param FlushReason the flush reason will influence times waited to cancel ongoing http requests
	 */
	HTTP_API void Flush(EHttpFlushReason FlushReason);

	/**
	 * FTSTicker callback
	 *
	 * @param DeltaSeconds - time in seconds since the last tick
	 *
	 * @return false if no longer needs ticking
	 */
	HTTP_API bool Tick(float DeltaSeconds) override;

	/**
	 * Tick called during Flush
	 *
	 * @param DeltaSeconds - time in seconds since the last tick
	 */
	HTTP_API virtual void FlushTick(float DeltaSeconds);

	/** 
	 * Add a http request to be executed on the http thread
	 *
	 * @param Request - the request object to add
	 */
	HTTP_API void AddThreadedRequest(const TSharedRef<FHttpRequestCommon, ESPMode::ThreadSafe>& Request);

	/**
	 * Mark a threaded http request as cancelled to be removed from the http thread
	 *
	 * @param Request - the request object to cancel
	 */
	HTTP_API void CancelThreadedRequest(const TSharedRef<FHttpRequestCommon, ESPMode::ThreadSafe>& Request);

	/**
	 * List all of the Http requests currently being processed
	 *
	 * @param Ar - output device to log with
	 */
	HTTP_API void DumpRequests(FOutputDevice& Ar) const;

	/**
	 * Method to check dynamic proxy setting support.
	 *
	 * @returns Whether this http implementation supports dynamic proxy setting.
	 */
	HTTP_API virtual bool SupportsDynamicProxy() const;

	/**
	 * Set the method used to set a Correlation id on each request, if one is not already specified.
	 *
	 * This method allows you to override the Engine default method.
	 *
	 * @param InCorrelationIdMethod The method to use when sending a request, if no Correlation id is already set
	 */
	HTTP_API void SetCorrelationIdMethod(TFunction<FString()> InCorrelationIdMethod);

	/**
	 * Create a new correlation id for a request
	 *
	 * @return The new correlationid string
	 */
	HTTP_API FString CreateCorrelationId() const;

	/**
	 * Determine if the domain is allowed to be accessed
	 *
	 * @param Url the path to check domain on
	 *
	 * @return true if domain is allowed
	 */
	HTTP_API bool IsDomainAllowed(const FString& Url) const;

	/**
	 * Get the default method for creating new correlation ids for a request
	 *
	 * @return The default correlationid creation method
	 */
	static HTTP_API TFunction<FString()> GetDefaultCorrelationIdMethod();

	/**
	 * Inform that HTTP Manager that we are about to fork(). Will block to flush all outstanding http requests
	 */
	HTTP_API virtual void OnBeforeFork();

	/**
	 * Inform that HTTP Manager that we have completed a fork(). Must be called in both the client and parent process
	 */
	HTTP_API virtual void OnAfterFork();

	/**
	 * Inform the HTTP Manager that we finished ticking right after forking. Only called on the forked process
	 */
	HTTP_API virtual void OnEndFramePostFork();


	/**
	 * Update configuration. Called when config has been updated and we need to apply any changes.
	 */
	HTTP_API virtual void UpdateConfigs();

	/**
	 * Add task to be ran on the game thread next tick
	 *
	 * @param Task The task to be ran next tick
	 */
	HTTP_API void AddGameThreadTask(TFunction<void()>&& Task, float Delay = 0.0f);

	/**
	 * Add task to be ran on the http thread
	 *
	 * @param Task The task to be ran
	 * @param InDelay The delay to wait before running the task
	 * @return The handle of the timer, which could be used to remove the task before it's triggered
	 */
	HTTP_API TSharedPtr<IHttpTaskTimerHandle> AddHttpThreadTask(TFunction<void()>&& Task, float InDelay = 0.0f);

	/**
	 * Remove the task from the http thread before it's triggered
	 *
	 * @param HttpTaskTimerHandle The handle of the timer
	 */
	HTTP_API void RemoveHttpThreadTask(TSharedPtr<IHttpTaskTimerHandle> HttpTaskTimerHandle);

	/**
	 * Set url request filter through code, instead of setting it through config.
	 *
	 * @param InURLRequestFilter The request filter to set
	 */
	void SetURLRequestFilter(const UE::Core::FURLRequestFilter& InURLRequestFilter) { URLRequestFilter = InURLRequestFilter; }

	FHttpStats GetHttpStats() const { return HttpStats; }

protected:
	/** 
	 * Create HTTP thread object
	 *
	 * @return the HTTP thread object
	 */
	HTTP_API virtual FHttpThreadBase* CreateHttpThread();

	HTTP_API void ReloadFlushTimeLimits();

	HTTP_API bool HasAnyBoundDelegate() const;

	HTTP_API void UpdateUrlPatternsToInspect(IConsoleVariable* CVar);
	HTTP_API void UpdateUrlPatternsToDisableFailedLog(IConsoleVariable* CVar);
	HTTP_API void UpdateUrlPatternsToMockResponse(IConsoleVariable* CVar);

protected:
	/** List of Http requests that are actively being processed */
	TArray<FHttpRequestRef> Requests;

	FHttpThreadBase* Thread;

	/** This method will be called to generate a CorrelationId on all requests being sent if one is not already set */
	TFunction<FString()> CorrelationIdMethod;

	/** Ticker to run game thread tasks*/
	FTSTicker GameThreadTicker;
	FCriticalSection GameThreadTickerLock;

	// This variable is set to true in Flush(EHttpFlushReason), and prevents new Http requests from being launched
	bool bFlushing;

	/** Delegate that will get called once request added */
	FHttpManagerRequestAddedDelegate RequestAddedDelegate;

	/** Delegate that will get called when a request completes */
	FHttpManagerRequestCompletedDelegate RequestCompletedDelegate;

	/** Url request filter, if specified in the config, it will launch http request only when the url is in the allowlist */
	UE::Core::FURLRequestFilter URLRequestFilter;

	struct FHttpFlushTimeLimit
	{
		/**
		 * Designates the amount of time we will wait during a flush before we try to cancel the request.
		 * This MUST be strictly < FlushTimeHardLimitSeconds for the logic to work and actually cancel the request, since we must Tick at least one time for the cancel to work.
		 * Setting this to 0 will immediately cancel all ongoing requests. A hard limit is still required for this to work.
		 * Setting this to < 0 will disable the cancel, but FlushTimeHardLimitSeconds can still be used to stop waiting on requests.
		 */		
		double SoftLimitSeconds;

		/**
		 * After we hit the soft time limit and cancel the requests, we wait some additional time for the canceled requests to go away.
		 * If they don't go away in time, we will hit this "hard" time limit that will just stop waiting.
		 * If we are shutting down, this is probably fine. If we are flushing for other reasons,
		 * This could indicate things lying around, and we'll put out some warning log messages to indicate this.
		 * Setting this to < 0 will disable all time limits and the code will wait infinitely for all requests to complete.
		 */
		double HardLimitSeconds;

		FHttpFlushTimeLimit(double InSoftLimitSeconds, double InHardLimitSeconds)
			: SoftLimitSeconds(InSoftLimitSeconds), HardLimitSeconds(InHardLimitSeconds)
		{}
	};

	TMap<EHttpFlushReason, FHttpFlushTimeLimit> FlushTimeLimitsMap;

	FHttpStats HttpStats;
	struct FHttpStatsHistory
	{
		static constexpr int32 HttpHistoryCount = 16;
		int32 HistoryIndex = 0;

		int64 DownloadedBytes[HttpHistoryCount] = { 0 };
		int64 DurationMs[HttpHistoryCount] = { 0 };

		int64 TotalDownloadedBytes = 0;
		int64 TotalUploadedBytes = 0;
		int64 TotalDuration = 0;
	};
	FHttpStatsHistory HttpStatsHistory;

	bool bUseEventLoop = true;

	TArray<FString> UrlPatternsToInspect;
	mutable FCriticalSection UrlPatternsToInspectCriticalSection;

	TArray<FString> UrlPatternsToDisableFailedLog;
	mutable FCriticalSection UrlPatternsToDisableFailedLogCriticalSection;

#if !UE_BUILD_SHIPPING
	mutable FCriticalSection ThreadsCompletingRequestCriticalSection;
	TMap<uint32, bool> ThreadsCompletingRequest;
#endif

PACKAGE_SCOPE:
	struct FMockResponse
	{
		int32 StatusCode = 0;
		FUtf8String ResponsePayload;
		TMap<FString, FString> ResponseHeaders;
	};

	struct FUrlToMatch
	{
		FString Str;
		FString Verb;
	};

	/** Used to lock access to add/remove/find requests */
	mutable FCriticalSection RequestLock;

	/** Used to lock access to get completed requests */
	FCriticalSection CompletedRequestLock;

	/**
	 * Broadcast that a non-threaded HTTP request is complete.
	 * Called automatically internally for threaded requests.
	 * Called explicitly by non-threaded requests
	 */
	HTTP_API void BroadcastHttpRequestCompleted(const FHttpRequestRef& Request);

	/**
	 * Access http thread of http manager for internal usage
	 */
	HTTP_API FHttpThreadBase* GetThread();

	/** Record the time to connect, to have a general idea how long the client usually take to connect for success requests, to adjust the connection timeout */
	HTTP_API void RecordStatTimeToConnect(float Duration);

	/** Record the requests waiting in flight, to have an idea if there are too many concurrent requests */
	HTTP_API void RecordStatRequestsInFlight(uint32 RequestsInFlight);

	/** Record the requests waiting in queue, to have an idea if there are too many requests or if request number limit is too small */
	HTTP_API void RecordStatRequestsInQueue(uint32 RequestsInQueue);

	/** Record the time to wait in queue, to have a general idea how long the client usually wait before actually starting, to adjust the requests */
	HTTP_API void RecordMaxTimeToWaitInQueue(float Duration);

	/** Record a succeeded request */
	HTTP_API void RecordStatRequestSucceeded();

	/** Record a failed request by its failure reason */
	HTTP_API void RecordStatRequestFailed(EHttpFailureReason Reason);

	/** Record platform specific stats */
	HTTP_API void RecordPlatformStats(const FHttpStatsPlatform& PlatformStats);

	HTTP_API bool ShouldInspect(FStringView Url) const;
	HTTP_API bool ShouldDisableFailedLog(FStringView Url) const;

	HTTP_API TOptional<FMockResponse> GetMockResponse(FStringView Url, FStringView Verb) const;

	HTTP_API void MarkCurrentThreadCompletingRequest(bool bCompleting);
	HTTP_API bool IsCurrentThreadCompletingRequest() const;

	/**
	 * Add a http request to the manager without enqueueing the threaded execution
	 * 
	 * @param Request - the request object to add
	 */
	HTTP_API void AddRequest(const FHttpRequestRef& Request);

protected:
	TArray<TPair<FUrlToMatch, FMockResponse>> UrlPatternsToMockResponse;
	mutable FCriticalSection UrlPatternsToMockResponseCriticalSection;
};
