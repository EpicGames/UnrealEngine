// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformMath.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FHttpManager;
class FOutputDevice;
class IHttpRequest;
class UWorld;

/**
 * Module for Http request implementations
 * Use FHttpFactory to create a new Http request
 */
class FHttpModule : 
	public IModuleInterface, public FSelfRegisteringExec
{

protected:

	// FSelfRegisteringExec

	/**
	 * Handle exec commands starting with "HTTP"
	 *
	 * @param InWorld	the world context
	 * @param Cmd		the exec command being executed
	 * @param Ar		the archive to log results to
	 *
	 * @return true if the handler consumed the input, false to continue searching handlers
	 */
	HTTP_API virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

public:

	/** 
	 * Exec command handlers
	 */
	bool HandleHTTPCommand( const TCHAR* Cmd, FOutputDevice& Ar );

	// FHttpModule

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	HTTP_API static FHttpModule& Get();

	/**
	 * Update all config-based values
	 */
	HTTP_API void UpdateConfigs();

	/**
	 * Instantiates a new Http request for the current platform
	 *
	 * @return new Http request instance
	 */
	HTTP_API virtual TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateRequest();

	/**
	 * Only meant to be used by Http request/response implementations
	 *
	 * @return Http request manager used by the module
	 */
	inline FHttpManager& GetHttpManager()
	{
		check(HttpManager != NULL);
		return *HttpManager;
	}

	/**
	 * @return total timeout in seconds for the entire http request to complete
	 */
	inline float GetHttpTotalTimeout() const
	{
		return HttpTotalTimeout;
	}

	/**
	 * @return timeout in seconds to establish the connection
	 */
	inline float GetHttpConnectionTimeout() const
	{
		return HttpConnectionTimeout;
	}

	/**
	 * @return timeout in seconds to check there is any ongoing activity on the established connection
	 */
	inline float GetHttpActivityTimeout() const
	{
		return HttpActivityTimeout;
	}

	/**
	 * @return max number of simultaneous connections to a specific server
	 */
	inline int32 GetHttpMaxConnectionsPerServer() const
	{
		return HttpMaxConnectionsPerServer;
	}

	/**
	 * @return max read buffer size for http requests
	 */
	inline int32 GetMaxReadBufferSize() const
	{
		return MaxReadBufferSize;
	}

	/**
	 * Sets the maximum size for the read buffer
	 * @param SizeInBytes	The maximum number of bytes to use for the read buffer
	 */
	inline void SetMaxReadBufferSize(int32 SizeInBytes)
	{
		MaxReadBufferSize = SizeInBytes;
	}

	/**
	 * @return true if http requests are enabled
	 */
	inline bool IsHttpEnabled() const
	{
		return bEnableHttp;
	}

	/**
	 * toggle null http implementation
	 */
	inline void ToggleNullHttp(bool bEnabled)
	{
		bUseNullHttp = bEnabled;
	}

	/**
	 * @return true if null http is being used
	 */
	inline bool IsNullHttpEnabled() const
	{
		return bUseNullHttp;
	}

	/**
	 * @return min delay time for each http request
	 */
	inline float GetHttpDelayTime() const
	{
		return HttpDelayTime;
	}

	/**
	 * Set the min delay time for each http request
	 */
	inline void SetHttpDelayTime(float InHttpDelayTime)
	{
		HttpDelayTime = InHttpDelayTime;
	}

	/**
	 * @return Target tick rate of an active http thread
	 */
	inline float GetHttpThreadActiveFrameTimeInSeconds() const
	{
		return HttpThreadActiveFrameTimeInSeconds;
	}

	/**
	 * Set the target tick rate of an active http thread
	 */
	inline void SetHttpThreadActiveFrameTimeInSeconds(float InHttpThreadActiveFrameTimeInSeconds)
	{
		HttpThreadActiveFrameTimeInSeconds = InHttpThreadActiveFrameTimeInSeconds;
	}

	/**
	 * @return Minimum sleep time of an active http thread
	 */
	inline float GetHttpThreadActiveMinimumSleepTimeInSeconds() const
	{
		return HttpThreadActiveMinimumSleepTimeInSeconds;
	}

	/**
	 * Set the minimum sleep time of an active http thread
	 */
	inline void SetHttpThreadActiveMinimumSleepTimeInSeconds(float InHttpThreadActiveMinimumSleepTimeInSeconds)
	{
		HttpThreadActiveMinimumSleepTimeInSeconds = InHttpThreadActiveMinimumSleepTimeInSeconds;
	}

	/**
	 * @return Target tick rate of an idle http thread
	 */
	inline float GetHttpThreadIdleFrameTimeInSeconds() const
	{
		return HttpThreadIdleFrameTimeInSeconds;
	}

	/**
	 * Set the target tick rate of an idle http thread
	 */
	inline void SetHttpThreadIdleFrameTimeInSeconds(float InHttpThreadIdleFrameTimeInSeconds)
	{
		HttpThreadIdleFrameTimeInSeconds = InHttpThreadIdleFrameTimeInSeconds;
	}

	/**
	 * @return Minimum sleep time when idle, waiting for requests
	 */
	inline float GetHttpThreadIdleMinimumSleepTimeInSeconds() const
	{
		return HttpThreadIdleMinimumSleepTimeInSeconds;
	}

	/**
	 * Set the minimum sleep time when idle, waiting for requests
	 */
	inline void SetHttpThreadIdleMinimumSleepTimeInSeconds(float InHttpThreadIdleMinimumSleepTimeInSeconds)
	{
		HttpThreadIdleMinimumSleepTimeInSeconds = InHttpThreadIdleMinimumSleepTimeInSeconds;
	}

	/**
	 * @return Duration between explicit tick calls when running an event loop http thread.
	 */
	inline float GetHttpEventLoopThreadTickIntervalInSeconds() const
	{
		return HttpEventLoopThreadTickIntervalInSeconds;
	}

	/**
	 * Get the default headers that are appended to every request
	 * @return the default headers
	 */
	const TMap<FString, FString>& GetDefaultHeaders() const { return DefaultHeaders; }

	/**
	 * Add a default header to be appended to future requests
	 * If a request already specifies this header, then the defaulted version will not be used
	 * @param HeaderName - Name of the header (e.g., "Content-Type")
	 * @param HeaderValue - Value of the header
	 */
	void AddDefaultHeader(const FString& HeaderName, const FString& HeaderValue) { DefaultHeaders.Emplace(HeaderName, HeaderValue); }

	/**
	 * @returns The current proxy address.
	 */
	inline const FString& GetProxyAddress() const
	{
		return ProxyAddress;
	}

	/**
	 * Setter for the proxy address.
	 * @param InProxyAddress - New proxy address to use.
	 */
	inline void SetProxyAddress(const FString& InProxyAddress)
	{
		ProxyAddress = InProxyAddress;
	}

	/**
	 * @returns the domains which won't use proxy even if ProxyAddress is set
	 */
	inline const FString& GetHttpNoProxy() const
	{
		return HttpNoProxy;
	}

	/**
	 * Method to check dynamic proxy setting support.
	 * @returns Whether this http implementation supports dynamic proxy setting.
	 */
	inline bool SupportsDynamicProxy() const
	{
		return bSupportsDynamicProxy;
	}

	// IModuleInterface

	/**
	 * Called when Http module is loaded
	 * load dependant modules
	 */
	HTTP_API virtual void StartupModule() override;

	/**
	 * Called after Http module is loaded
	 * Initialize platform specific parts of Http handling
	 */
	HTTP_API virtual void PostLoadCallback() override;
	
	/**
	 * Called before Http module is unloaded
	 * Shutdown platform specific parts of Http handling
	 */
	HTTP_API virtual void PreUnloadCallback() override;

	/**
	 * Called when Http module is unloaded
	 */
	HTTP_API virtual void ShutdownModule() override;

protected:
	/** timeout in seconds to establish the connection */
	float HttpConnectionTimeout;
	/** timeout in seconds for the entire http request to complete. 0 is no timeout */
	float HttpTotalTimeout;
	/**  timeout in seconds to check there is any ongoing activity on the established connection */
	float HttpActivityTimeout;

private:
	/**
	 * Delegate for config file changes.
	 */
	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);

	/** Keeps track of Http requests while they are being processed */
	FHttpManager* HttpManager = nullptr;
	/** timeout in seconds to receive a response on the connection */
	float HttpReceiveTimeout;
	/** timeout in seconds to send a request on the connection */
	float HttpSendTimeout;
	/** total time to delay the request */
	float HttpDelayTime;
	/** Time in seconds to use as frame time when actively processing requests. 0 means no frame time. */
	float HttpThreadActiveFrameTimeInSeconds;
	/** Time in seconds to sleep minimally when actively processing requests. */
	float HttpThreadActiveMinimumSleepTimeInSeconds;
	/** Time in seconds to use as frame time when idle, waiting for requests. 0 means no frame time. */
	float HttpThreadIdleFrameTimeInSeconds;
	/** Time in seconds to sleep minimally when idle, waiting for requests. */
	float HttpThreadIdleMinimumSleepTimeInSeconds;
	/** Time in seconds between explicit calls to tick requests when using an event loop to run http requests. */
	float HttpEventLoopThreadTickIntervalInSeconds;
	/** Max number of simultaneous connections to a specific server */
	int32 HttpMaxConnectionsPerServer;
	/** Max buffer size for individual http reads */
	int32 MaxReadBufferSize;
	/** toggles http requests */
	bool bEnableHttp;
	/** toggles null (mock) http requests */
	bool bUseNullHttp;
	/** Default headers - each request will include these headers, using the default value if not overridden */
	TMap<FString, FString> DefaultHeaders;
	/** singleton for the module while loaded and available */
	static FHttpModule* Singleton;
	/** true once StartupModule has been called at least once; prevents Get() from auto-loading after shutdown */
	static bool bWasEverLoaded;
	/** The address to use for proxy, in format IPADDRESS:PORT */
	FString ProxyAddress;
	/** The domains which won't use proxy even if the ProxyAddress is set, in format "127.0.0.1,localhost,example.com" */
	FString HttpNoProxy;
	/** Whether or not the http implementation we are using supports dynamic proxy setting. */
	bool bSupportsDynamicProxy;
	/** List of domains that can be accessed. If Empty then no filtering is applied */
	TArray<FString> AllowedDomains;
};
