// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/HttpRequestImpl.h"

/** 
  * Adapter class for IHttpRequest abstract interface
  * does not fully expose the wrapped interface in the base. This allows client defined marshalling of the requests when end point permissions are at issue.
  */

class FHttpRequestAdapterBase : public FHttpRequestImpl
{
public:
	HTTP_API FHttpRequestAdapterBase(const TSharedRef<IHttpRequest>& InHttpRequest);
	HTTP_API ~FHttpRequestAdapterBase();

	// IHttpRequest interface
	HTTP_API virtual const FString& GetURL() const override;
	HTTP_API virtual FString GetURLParameter(const FString& ParameterName) const override;
	HTTP_API virtual FString GetHeader(const FString& HeaderName) const override;
	HTTP_API virtual TArray<FString> GetAllHeaders() const override;
	HTTP_API virtual FString GetContentType() const override;
	HTTP_API virtual uint64 GetContentLength() const override;
	HTTP_API virtual const TArray<uint8>& GetContent() const override;
	HTTP_API virtual FString GetVerb() const override;
	HTTP_API virtual FString GetOption(const FName Option) const override;
	HTTP_API virtual void SetVerb(const FString& Verb) override;
	HTTP_API virtual void SetURL(const FString& URL) override;
	HTTP_API virtual void SetOption(const FName Option, const FString& OptionValue) override;
	HTTP_API virtual void SetContent(const TArray<uint8>& ContentPayload) override;
	HTTP_API virtual void SetContent(TArray<uint8>&& ContentPayload) override;
	HTTP_API virtual void SetContentAsString(const FString& ContentString) override;
	HTTP_API virtual bool SetContentAsStreamedFile(const FString& Filename) override;
	HTTP_API virtual bool SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) override;
	HTTP_API virtual bool SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) override;
	HTTP_API virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) override;
	HTTP_API virtual void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) override;
	HTTP_API virtual void SetTimeout(float InTimeoutSecs) override;
	HTTP_API virtual void ClearTimeout() override;
	HTTP_API virtual void ResetTimeoutStatus() override;
	HTTP_API virtual TOptional<float> GetTimeout() const override;
	HTTP_API virtual void SetActivityTimeout(float InTimeoutSecs) override;
	HTTP_API virtual void ProcessRequestUntilComplete() override;
	HTTP_API virtual const FHttpResponsePtr GetResponse() const override;
	HTTP_API virtual float GetElapsedTime() const override;
	HTTP_API virtual EHttpRequestStatus::Type GetStatus() const override;
	HTTP_API virtual EHttpFailureReason GetFailureReason() const override;
	HTTP_API virtual const FString& GetEffectiveURL() const override;
	HTTP_API virtual void Tick(float DeltaSeconds) override;
	HTTP_API virtual void SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy) override;
	HTTP_API virtual EHttpRequestDelegateThreadPolicy GetDelegateThreadPolicy() const override;
	HTTP_API virtual void SetPriority(EHttpRequestPriority InPriority) override;
	HTTP_API virtual EHttpRequestPriority GetPriority() const override;

protected:
	TSharedRef<IHttpRequest> HttpRequest;
};

