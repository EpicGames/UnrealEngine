// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "HttpRequestAdapter.h"
#include "Interfaces/IHttpRequest.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Optional.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

/**
 * Helpers of various types for the retry system
 */
namespace FHttpRetrySystem
{
	class FManager;

	typedef uint32 RetryLimitCountType;
	typedef double RetryTimeoutRelativeSecondsType;

	inline RetryLimitCountType             RetryLimitCount(uint32 Value) { return Value; }
	inline RetryTimeoutRelativeSecondsType RetryTimeoutRelativeSeconds(double Value) { return Value; }

	template <typename  IntrinsicType>
	IntrinsicType TZero();

	template <> inline float                           TZero<float>() { return 0.0f; }
	template <> inline RetryLimitCountType             TZero<RetryLimitCountType>() { return RetryLimitCount(0); }
	template <> inline RetryTimeoutRelativeSecondsType TZero<RetryTimeoutRelativeSecondsType>() { return RetryTimeoutRelativeSeconds(0.0); }

	typedef TOptional<float>                           FRandomFailureRateSetting;
	typedef TOptional<RetryLimitCountType>             FRetryLimitCountSetting;
	typedef TOptional<RetryTimeoutRelativeSecondsType> FRetryTimeoutRelativeSecondsSetting;
	typedef TSet<int32> FRetryResponseCodes;
	typedef TSet<FName> FRetryVerbs;

	struct FRetryDomains
	{
		FRetryDomains(TArray<FString>&& InDomains)
			: Domains(MoveTemp(InDomains))
			, ActiveIndex(0)
		{}

		FRetryDomains(TArray<FString>&& InDomains, const int32 InActiveIndex, const FString& InConfigSection, const FString& InConfigKeyName)
			: Domains(MoveTemp(InDomains))
			, ActiveIndex(InActiveIndex)
			, ConfigSection(InConfigSection)
			, ConfigKeyName(InConfigKeyName)
		{}

		/** The domains to use */
		TArray<FString> Domains;
		/**
		 * Index into Domains to attempt
		 * Domains are cycled through on some errors, and when we succeed on one domain, we remain on that domain until that domain results in an error
		 */
		TAtomic<int32> ActiveIndex;

		/** Config section name that can be used for storing active domain for reusing during next sessions */
		const FString ConfigSection;

		/** Config key name that can be used for storing active domain for reusing during next sessions */
		const FString ConfigKeyName;
	};
	typedef TSharedPtr<FRetryDomains, ESPMode::ThreadSafe> FRetryDomainsPtr;

	/** 
	 * Model for computing exponential backoff using the formula: Base**(CurrentRetryAttempt + Bias) 
	 * Then applying jitter to the backoff. Jitter application is performed by selecting a random value in the [Min, Max] range and multiplying it against the computed backoff.
	 * Half jitter can be implemented using { 0.5, 1.0 }, which becomes Backoff' = Backoff * Rand(0.5, 1.0) = Backoff/2 + Rand(0, Backoff/2)
	 * Full jitter can be implemented using { 0.0, 1.0 }, which becomes Backoff' = Backoff * Rand(0.0, 1.0) = Rand(0.0, Backoff)
	 * No jitter can be implemented using { 0.0, 0.0 } or any pair that where Min > Max. Backoff' = Backoff
	*/
	struct FExponentialBackoffCurve
	{
		/** Exponential backoff base */
		float Base = 2.0f;
		/** Exponential backoff bias added to the current retry number */
		float ExponentBias = 1.0f;
		/** Exponential backoff jitter coefficient minimum value. Defaults to half jitter */
		float MinCoefficient = 0.5f;
		/** Exponential backoff jitter coefficient maximum value. Defaults to half jitter */
		float MaxCoefficient = 1.0f;
		/** Max back off seconds */
		float MaxBackoffSeconds = 60.0f;

		HTTP_API bool IsValid() const;
		HTTP_API float Compute(uint32 RetryNumber) const;
	};
	using FRetryExponentialBackoffCurvePtr = TSharedPtr<FExponentialBackoffCurve, ESPMode::ThreadSafe>;

	/**
	 * Read the number of seconds a HTTP request is throttled for from the response
	 * @param Response the HTTP response to read the value from
	 * @return If found, the number of seconds the request is rate limited for.  If not found, an unset TOptional
	 */
	TOptional<double> HTTP_API ReadThrottledTimeFromResponseInSeconds(FHttpResponsePtr Response);
};


namespace FHttpRetrySystem
{
	/**
	 * class FRequest is what the retry system accepts as inputs
	 */
	class FRequest
		: public FHttpRequestAdapterBase
	{
	public:
		struct EStatus
		{
			enum Type
			{
				NotStarted = 0,
				Processing,
				ProcessingLockout,
				Cancelled,
				FailedRetry,
				FailedTimeout,
				Succeeded
			};
		};

	public:
		// IHttpRequest interface
		HTTP_API virtual bool ProcessRequest() override;
		HTTP_API virtual void CancelRequest() override;

		// FRequest
		EStatus::Type GetRetryStatus() const { return RetryStatus; }

	protected:
		friend class FManager;

		HTTP_API FRequest(
			TSharedRef<FManager, ESPMode::ThreadSafe> InManager,
			const TSharedRef<IHttpRequest, ESPMode::ThreadSafe>& HttpRequest,
			const FRetryLimitCountSetting& InRetryLimitCountOverride = FRetryLimitCountSetting(),
			const FRetryTimeoutRelativeSecondsSetting& InRetryTimeoutRelativeSecondsOverride = FRetryTimeoutRelativeSecondsSetting(),
			const FRetryResponseCodes& InRetryResponseCodes = FRetryResponseCodes(),
			const FRetryVerbs& InRetryVerbs = FRetryVerbs(),
			const FRetryDomainsPtr& InRetryDomains = FRetryDomainsPtr(),
			const FRetryLimitCountSetting& InRetryLimitCountForConnectionErrorOverride = FRetryLimitCountSetting(),
			const FExponentialBackoffCurve& InExponentialBackoffCurve = FExponentialBackoffCurve()
		);

		void HttpOnRequestProgress(FHttpRequestPtr InHttpRequest, uint64 BytesSent, uint64 BytesRcv);
		void HttpOnProcessRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
		void HttpOnStatusCodeReceived(FHttpRequestPtr Request, int32 StatusCode);
		void HttpOnHeaderReceived(FHttpRequestPtr Request, const FString& HeaderName, const FString& NewHeaderValue);

		/** Update our HTTP request's URL's domain from our RetryDomains */
		void SetUrlFromRetryDomains();
		/** Move to the next retry domain from our RetryDomains */
		void MoveToNextRetryDomain();

		void BindAdaptorDelegates();

		EStatus::Type RetryStatus;

		FRetryLimitCountSetting RetryLimitCountOverride;
		FRetryLimitCountSetting RetryLimitCountForConnectionErrorOverride;
		FRetryTimeoutRelativeSecondsSetting  RetryTimeoutRelativeSecondsOverride;
		FRetryResponseCodes RetryResponseCodes;
		FRetryVerbs RetryVerbs;
		FRetryDomainsPtr RetryDomains;
		/** The current index in RetryDomains we are attempting */
		int32 RetryDomainsIndex = 0;
		/** The original URL before replacing anything from RetryDomains */
		FString OriginalUrl;

		TWeakPtr<FManager> RetryManager;
		/** Save the last response before the retry */
		FHttpResponsePtr LastResponse;
		EHttpFailureReason LastFailureReason = EHttpFailureReason::None;
		bool bLastSucceeded = false;

		/** Exponential backoff curve */
		FExponentialBackoffCurve RetryExponentialBackoffCurve;

		bool bBoundAdaptorDelegates = false;
	};
}

namespace FHttpRetrySystem
{

class FManager : public TSharedFromThis<FManager>
{
public:
	// FManager
	HTTP_API FManager(
		const FRetryLimitCountSetting& InRetryLimitCountDefault, 
		const FRetryTimeoutRelativeSecondsSetting& InRetryTimeoutRelativeSecondsDefault,
		const FRetryLimitCountSetting& InRetryLimitCountForConnectionErrorDefault = FRetryLimitCountSetting()
	);

	/**
	 * Create a new http request with retries
	 */
	HTTP_API TSharedRef<class FHttpRetrySystem::FRequest, ESPMode::ThreadSafe> CreateRequest(
		const FRetryLimitCountSetting& InRetryLimitCountOverride = FRetryLimitCountSetting(),
		const FRetryTimeoutRelativeSecondsSetting& InRetryTimeoutRelativeSecondsOverride = FRetryTimeoutRelativeSecondsSetting(),
		const FRetryResponseCodes& InRetryResponseCodes = FRetryResponseCodes(),
		const FRetryVerbs& InRetryVerbs = FRetryVerbs(),
		const FRetryDomainsPtr& InRetryDomains = FRetryDomainsPtr(),
		const FRetryLimitCountSetting& InRetryLimitCountForConnectionErrorOverride = FRetryLimitCountSetting(),
		const FExponentialBackoffCurve & InExponentialBackoffCurve = FExponentialBackoffCurve()
	);

	HTTP_API virtual ~FManager();

	void SetRandomFailureRate(float Value) { RandomFailureRate = FRandomFailureRateSetting(Value); }
	void SetDefaultRetryLimit(uint32 Value) { RetryLimitCountDefault = FRetryLimitCountSetting(Value); }

	// @return Block the current process until all requests are flushed, or timeout has elapsed
	HTTP_API void BlockUntilFlushed(float TimeoutSec);

protected:
	friend class FRequest;

	struct FHttpRetryRequestEntry
	{
		FHttpRetryRequestEntry(TSharedRef<FRequest, ESPMode::ThreadSafe>& InRequest);

		uint32 CurrentRetryCount;
		uint32 CurrentRetryCountForConnectionError;
		double RequestStartTimeAbsoluteSeconds;
		double LockoutEndTimeAbsoluteSeconds;

		TSharedRef<FRequest, ESPMode::ThreadSafe>	Request;
	};

	class FHttpLogVerbosityTracker
	{
	public:
		FHttpLogVerbosityTracker();
		~FHttpLogVerbosityTracker();

		/** Mark that a request is being retried */
		void IncrementRetriedRequests();
		/** Mark that a retried request is no longer being retried */
		void DecrementRetriedRequests();
		static FHttpLogVerbosityTracker& Get();
	protected:
		/** Update settings from config */
		void UpdateSettingsFromConfig();
		void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionName);

		/** Number of requests that are in a retried state.  When this is non-zero, verbosity will be adjusted. */
		int32 NumRetriedRequests = 0;
		/** DecrementRetriedRequests can be called from game thread or http thread depends on the http request thread policy, make sure it's thread-safe */
		FCriticalSection NumRetriedRequestsLock;
		/** Verbosity to restore to when there are no requests being retried */
		ELogVerbosity::Type OriginalVerbosity = ELogVerbosity::Error;
		/** Config driven target verbosity to set to when requests are being retried.  NoLogging means the verbosity will not be modified. */
		ELogVerbosity::Type TargetVerbosity = ELogVerbosity::NoLogging;
	};

	bool ProcessRequest(TSharedRef<FRequest, ESPMode::ThreadSafe>& HttpRequest);
	void CancelRequest(TSharedRef<FRequest, ESPMode::ThreadSafe>& HttpRequest);

	// @return true if there is a no formal response to the request
	// @TODO return true if a variety of 5xx errors are the result of a formal response
	bool ShouldRetry(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const;

	// @return true if retry chances have not been exhausted
	bool CanRetry(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const;

	// @return true if the retry request has timed out
	bool HasTimedOut(const FHttpRetryRequestEntry& HttpRetryRequestEntry, const double NowAbsoluteSeconds);

	/**
	 * Retry an HTTP request
	 * @param RequestEntry request to retry
	 */
	void RetryHttpRequest(FHttpRetryRequestEntry& RequestEntry);

	/**
	 * Retry an HTTP request with delay
	 * @param RequestEntry request retry
	 * @param InDelay the delay to wait before retrying
	 * @param bWasSucceeded was the request succeeded before retry
	 */
	void RetryHttpRequestWithDelay(FManager::FHttpRetryRequestEntry& RequestEntry, float InDelay, bool bWasSucceeded);

	void HttpRequestTimeoutAfterDelay(FManager::FHttpRetryRequestEntry& RequestEntry, bool bWasSucceeded, float Delay);

	// @return number of seconds to lockout for
	float GetLockoutPeriodSeconds(const FHttpRetryRequestEntry& HttpRetryRequestEntry);

	bool RetryLimitForConnectionErrorIsSet(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const;
	bool CanRetryForConnectionError(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const;
	bool CanRetryInGeneral(const FHttpRetryRequestEntry& HttpRetryRequestEntry) const;

	// Default configuration for the retry system
	FRandomFailureRateSetting RandomFailureRate;
	FRetryLimitCountSetting RetryLimitCountDefault;
	FRetryLimitCountSetting RetryLimitCountForConnectionErrorDefault;
	FRetryTimeoutRelativeSecondsSetting RetryTimeoutRelativeSecondsDefault;

	TArray<FHttpRetryRequestEntry> RequestList;
	FCriticalSection RequestListLock;
};

}
