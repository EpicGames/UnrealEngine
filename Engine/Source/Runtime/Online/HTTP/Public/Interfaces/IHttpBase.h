// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace EHttpRequestStatus
{
	/**
	 * Enumerates the current state of an Http request
	 */
	enum Type
	{
		/** Has not been started via ProcessRequest() */
		NotStarted,
		/** Currently being ticked and processed */
		Processing,
		/** Finished but failed */
		Failed,
		/** Finished and was successful */
		Succeeded
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(EHttpRequestStatus::Type EnumVal)
	{
		switch (EnumVal)
		{
			case NotStarted:
			{
				return TEXT("NotStarted");
			}
			case Processing:
			{
				return TEXT("Processing");
			}
			case Failed:
			{
				return TEXT("Failed");
			}
			case Succeeded:
			{
				return TEXT("Succeeded");
			}
		}
		return TEXT("");
	}

	inline bool IsFinished(const EHttpRequestStatus::Type Value)
	{
		return Value != NotStarted && Value != Processing;
	}
}

/**
 * The reason of the failure when HTTP request failed
 */
enum class EHttpFailureReason : uint8
{
	None = 0,
	ConnectionError,
	Cancelled,
	// This acts differently than the platform timeout. It's the time out of the entire request including 
	// retries, which configured by the user. The platform timeout has different meanings/APIs on different 
	// platforms, such as resolve host timeout, connect timeout, send timeout, receive timeout, entire 
	// request timeout etc. It's not practical to unify to same behavior between them through their APIs. 
	// When user's configured time is up, it's TimedOut no matter which step this Http request is in.
	TimedOut,
	Other,
	ResponseTooLarge,
	Count
};

/** @return the stringified version of the enum passed in */
inline const TCHAR* LexToString(EHttpFailureReason HttpFailureReason)
{
	switch (HttpFailureReason)
	{
	case EHttpFailureReason::None: return TEXT("None");
	case EHttpFailureReason::ConnectionError: return TEXT("ConnectionError");
	case EHttpFailureReason::Cancelled: return TEXT("Cancelled");
	case EHttpFailureReason::TimedOut: return TEXT("TimedOut");
	case EHttpFailureReason::Other: return TEXT("Other");
	case EHttpFailureReason::ResponseTooLarge: return TEXT("ResponseTooLarge");
	default: checkNoEntry(); return TEXT("Invalid");
	}
}

/**
 * Base interface for Http Requests and Responses.
 */
class IHttpBase
{
public:
	/**
	 * Get the URL used to send the request.
	 *
	 * @return the URL string.
	 */
	virtual const FString& GetURL() const = 0;

	/**
	 * Get the effective URL in case of redirected. If not redirected, it's the same as GetURL
	 *
	 * @return the effective URL string.
	 */
	virtual const FString& GetEffectiveURL() const = 0;

	/**
	 * Get the current status of the request being processed
	 *
	 * @return the current status
	 */
	virtual EHttpRequestStatus::Type GetStatus() const = 0;

	/**
	 * Get the reason of th failure if GetStatus returns Failed
	 *
	 * @return the reason of the failure
	 */
	virtual EHttpFailureReason GetFailureReason() const = 0;

	/** 
	 * Gets an URL parameter.
	 * expected format is ?Key=Value&Key=Value...
	 * If that format is not used, this function will not work.
	 * 
	 * @param ParameterName - the parameter to request.
	 * @return the parameter value string.
	 */
	virtual FString GetURLParameter(const FString& ParameterName) const = 0;

	/** 
	 * Gets the value of a header, or empty string if not found. 
	 * 
	 * @param HeaderName - name of the header to set.
	 */
	virtual FString GetHeader(const FString& HeaderName) const = 0;

	/**
	 * Return all headers in an array in "Name: Value" format.
	 *
	 * @return the header array of strings
	 */
	virtual TArray<FString> GetAllHeaders() const = 0;

	/**
	 * Shortcut to get the Content-Type header value (if available)
	 *
	 * @return the content type.
	 */
	virtual FString GetContentType() const = 0;

	/**
	 * Shortcut to get the Content-Length header value. Will not always return non-zero.
	 * If you want the real length of the payload, get the payload and check it's length.
	 *
	 * @return the content length (if available)
	 */
	virtual uint64 GetContentLength() const = 0;

	/**
	 * Get the content payload of the request or response.
	 *
	 * @param Content - array that will be filled with the content.
	 */
	virtual const TArray<uint8>& GetContent() const = 0;

	/** 
	 * Destructor for overrides 
	 */
	virtual ~IHttpBase() = default;
};

#if !defined(UE_HTTP_CONNECTION_TIMEOUT_SUPPORT_RETRY)
#define UE_HTTP_CONNECTION_TIMEOUT_SUPPORT_RETRY UE_DEPRECATED_MACRO(5.6, "UE_HTTP_CONNECTION_TIMEOUT_SUPPORT_RETRY has been deprecated because it is no longer necessary.") 1
#endif 
