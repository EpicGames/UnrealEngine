// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IHttpBase.h"

class IHttpRequest;
class IHttpResponse;

/**
 * Enumerates thread policy about which thread to complete the http request
 */
enum class EHttpRequestDelegateThreadPolicy : uint8
{
	CompleteOnGameThread = 0,
	CompleteOnHttpThread,
};

/**
 * Enumerates priority to decide how the request will be queued when max concurrent limit reached
 * The priority will be compared through value, so it's important to keep the order from low to high
 */
enum class EHttpRequestPriority : uint8
{
	Lowest = 0,
	Low,
	Normal,
	High,
	Highest
};

class IHttpRequest;
class IHttpResponse;

typedef TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> FHttpRequestPtr;
typedef TSharedPtr<IHttpResponse, ESPMode::ThreadSafe> FHttpResponsePtr;

typedef TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FHttpRequestRef;
typedef TSharedRef<IHttpResponse, ESPMode::ThreadSafe> FHttpResponseRef;

/**
 * Delegate called when an Http request completes
 *
 * @param Request original Http request that started things
 * @param Response response received from the server if a successful connection was established. Note that
 *        even if response is not null, it doesn't mean the request was processed successfully. Use
 *        Response->GetStatus(), Response->GetFailureReason() and Response->GetResponseCode() to decide
 *        how to handle it
 * @param bProcessedSuccessfully this flag indicates whether or not the request was able to be processed successfully.
 *        It could be connect error, timeout error or platform http error. Note that when this flag is true
 *        doesn't mean the http request get 2xx success status code, it just means this request got response
 *        from http server and completed the processing
 */
using FHttpRequestCompleteDelegate = TTSDelegate<void(FHttpRequestPtr /*Request*/, FHttpResponsePtr /*Response*/, bool /*bProcessedSuccessfully*/)>;

/**
 * Delegate called when an Http request receives status code
 *
 * @param Request original Http request that started things
 * @param status code
 */
using FHttpRequestStatusCodeReceivedDelegate = TTSDelegate<void(FHttpRequestPtr /*Request*/, int32 /*StatusCode*/)>;

/**
 * Delegate called when an Http request receives a header
 *
 * @param Request original Http request that started things
 * @param HeaderName the name of the header
 * @param NewHeaderValue the value of the header
 */
using FHttpRequestHeaderReceivedDelegate = TTSDelegate<void(FHttpRequestPtr /*Request*/, const FString& /*HeaderName*/, const FString& /*NewHeaderValue*/)>;

/**
 * Delegate called per tick to update an Http request upload or download size progress
 *
 * @param Request original Http request that started things
 * @param BytesSent the number of bytes sent / uploaded in the request so far.
 * @param BytesReceived the number of bytes received / downloaded in the response so far.
 */
using FHttpRequestProgressDelegate = TTSDelegate<void(FHttpRequestPtr /*Request*/, int32 /*BytesSent*/, int32 /*BytesReceived*/)>;

/**
 * Delegate called per tick to update an Http request upload or download size progress
 *
 * @param Request original Http request that started things
 * @param BytesSent the number of bytes sent / uploaded in the request so far.
 * @param BytesReceived the number of bytes received / downloaded in the response so far.
 */
using FHttpRequestProgressDelegate64 = TTSDelegate<void(FHttpRequestPtr /*Request*/, uint64 /*BytesSent*/, uint64 /*BytesReceived*/)>;

/**
 * Delegate called when an Http request will be retried in the future
 *
 * @param Request - original Http request that started things
 * @param Response - response received from the server if a successful connection was established
 * @param SecondsToRetry - seconds in the future when the response will be retried
 */
using FHttpRequestWillRetryDelegate = TTSDelegate<void(FHttpRequestPtr /*Request*/, FHttpResponsePtr /*Response*/, float /*SecondsToRetry*/)>;

/**
 * Delegate called when an Http request will send/recv data through stream
 *
 * @param Ptr - The buffer ptr to read/write
 * @param Length - The length of buffer to read/write
 * @return true if succeed, false if failed to read/write data
 */
using FHttpRequestStreamDelegate = TTSDelegate<bool(void*/*Ptr*/, int64/*Length*/)>;

/**
 * Delegate called when an Http request will send/recv data through stream
 *
 * @param Ptr - The buffer ptr to read/write
 * @param InOutLength - The int64 reference length of buffer to read/write, if there is any error when serialize set it to 0
 */
using FHttpRequestStreamDelegateV2 = TTSDelegate<void(void*/*Ptr*/, int64&/*InOutLength*/)>;

/**
 * Delegate version of FArchive, for streaming interface
 */
class UE_DEPRECATED(5.5, "FArchiveWithDelegate is deprecated and will be moved to internal") FArchiveWithDelegate final : public FArchive
{
public:
	FArchiveWithDelegate(FHttpRequestStreamDelegate InStreamDelegate)
		: StreamDelegate(InStreamDelegate)
	{
	}

	virtual void Serialize(void* V, int64 Length) override
	{
		if (!StreamDelegate.IsBound() || !StreamDelegate.Execute(V, Length))
		{
			SetError();
		}
	}

private:
	FHttpRequestStreamDelegate StreamDelegate;
};

/**
 * Options that can be specified on a Http Request
 */
namespace HttpRequestOptions
{
	extern HTTP_API const FName HttpVersion;
	extern HTTP_API const FName RequestMode;
#if UE_HTTP_SUPPORT_UNIX_SOCKET
	extern HTTP_API const FName UnixSocketPath;
#endif //UE_HTTP_SUPPORT_UNIX_SOCKET
	extern HTTP_API const FName EnableCookies;
}

/**
 * Mode that can be specified through IHttpRequest::SetOption(HttpRequestOptions::RequestMode, LexToString(EHttpRequestMode::Value));
 */
enum class EHttpRequestMode : uint8
{
	/** The request will be enqueued to the http manager thread system */
	ThreadedRequest,
	/** The request will be immediately sent when the platform supports it */
	ImmediateRequest
};

HTTP_API const TCHAR* LexToString(EHttpRequestMode Mode);

/**
 * Interface for Http requests (created using FHttpFactory)
 */
class IHttpRequest :
	public IHttpBase, public TSharedFromThis<IHttpRequest, ESPMode::ThreadSafe>
{
public:

	/**
	 * Gets the verb (GET, PUT, POST) used by the request.
	 *
	 * @return the verb string
	 */
	virtual FString GetVerb() const = 0;

	/**
	 * Sets the verb used by the request.
	 * Eg. (GET, PUT, POST)
	 * Should be set before calling ProcessRequest.
	 * If not specified then a GET is assumed.
	 *
	 * @param Verb - verb to use.
	 */
	virtual void SetVerb(const FString& Verb) = 0;

	/**
	 * Sets the URL for the request
	 * Eg. (http://my.domain.com/something.ext?key=value&key2=value).
	 * Must be set before calling ProcessRequest.
	 *
	 * @param URL - URL to use.
	 */
	virtual void SetURL(const FString& URL) = 0;

	/**
	 * Get the current value for the given option
	 *
	 * @return the current value set for this option or an empty string if no value has been specified
	 */
	virtual FString GetOption(const FName Option) const = 0;

	/**
	 * Sets the given option for this Request
	 * Must be set before calling ProcessRequest.
	 *
	 * @param Option - The option to set, see 'HttpRequestOptions' for supported options
	 * @param OptionValue - The value of the option to set
	 */
	virtual void SetOption(const FName Option, const FString& OptionValue) = 0;

	/**
	 * Sets the content of the request (optional data).
	 * Usually only set for POST requests.
	 *
	 * @param ContentPayload - payload to set.
	 */
	virtual void SetContent(const TArray<uint8>& ContentPayload) = 0;

	/**
	 * Sets the content of the request (optional data).
	 * Usually only set for POST requests.
	 *
	 * This version lets the API take ownership of the payload directly, helpful for larger payloads.
	 *
	 * @param ContentPayload - payload to set.
	 */
	virtual void SetContent(TArray<uint8>&& ContentPayload) = 0;

	/**
	 * Sets the content of the request as a string encoded as UTF8.
	 *
	 * @param ContentString - payload to set.
	 */
	virtual void SetContentAsString(const FString& ContentString) = 0;

	/**
	 * Sets the content of the request to stream from a file.
	 *
	 * @param FileName - filename from which to stream the body.
	 * @return True if the file is valid and will be used to stream the request. False otherwise.
	 */
	virtual bool SetContentAsStreamedFile(const FString& Filename) = 0;

	/**
	 * Sets the content of the request to stream directly from an archive.
	 * NOTE: The Stream->Serialize will be called from another thread other than the game thread
	 *
	 * @param Stream - archive from which the payload should be streamed.
	 * @return True if the archive can be used to stream the request. False otherwise.
	 */
	virtual bool SetContentFromStream(TSharedRef<FArchive, ESPMode::ThreadSafe> Stream) = 0;

	/**
	 * Sets the stream to receive the response body. Make sure to handle the cleanup of stream when
	 * Serialize generated error(Stream->GetError returns true after Stream->Serialize call), this
	 * http request will fail and quit.
	 *
	 * NOTE: Once set, the data will no longer be cached in response, IHttpResponse::GetContent() and
	 * IHttpResponse::GetContentAsString() will return empty result. The Stream->Serialize will be called
	 * from another thread other than the game thread
	 *
	 * @param Stream - will be used to receive the response body
	 * @return True if the stream can be used. False otherwise.
	 */
	virtual bool SetResponseBodyReceiveStream(TSharedRef<FArchive> Stream) = 0;

	/**
	 * Sets the delegate to receive the response body. Make sure to handle the cleanup of received data when
	 * failed to process the data(StreamDelegate return false), this http request will fail and quit.
	 *
	 * NOTE: Once set, the data will no longer be cached in response, IHttpResponse::GetContent() and
	 * IHttpResponse::GetContentAsString() will return empty result. The delegate will be called from
	 * another thread other than the game thread
	 *
	 * @param StreamDelegate - will be used to receive the response body
	 * @return True if the delegate can be used. False otherwise.
	 */
	HTTP_API bool SetResponseBodyReceiveStreamDelegateV2(FHttpRequestStreamDelegateV2 StreamDelegate);

	/**
	 * Sets optional header info.
	 * SetHeader for a given HeaderName will overwrite any previous values
	 * Use AppendToHeader to append more values for the same header
	 * Content-Length is the only header set for you.
	 * Required headers depends on the request itself.
	 * Eg. "multipart/form-data" needed for a form post
	 *
	 * @param HeaderName - Name of the header (ie, Content-Type)
	 * @param HeaderValue - Value of the header
	 */
	virtual void SetHeader(const FString& HeaderName, const FString& HeaderValue) = 0;

	/**
	* Appends to the value already set in the header.
	* If there is already content in that header, a comma delimiter is used.
	* If the header is as of yet unset, the result is the same as calling SetHeader
	* Content-Length is the only header set for you.
	* Also see: SetHeader()
	*
	* @param HeaderName - Name of the header (ie, Content-Type)
	* @param AdditionalHeaderValue - Value to add to the existing contents of the specified header.
	*	comma is inserted between old value and new value, per HTTP specifications
	*/
	virtual void AppendToHeader(const FString& HeaderName, const FString& AdditionalHeaderValue) = 0;

	/**
	 * Sets an optional timeout in seconds for this entire HTTP request to complete.
	 * If set, this value overrides the default HTTP timeout set via FHttpModule::SetTimeout().
	 *
	 * @param InTimeoutSecs - Timeout for this HTTP request instance, in seconds
	 */
	virtual void SetTimeout(float InTimeoutSecs) = 0;

	/**
	 * Sets an optional activity timeout in seconds for this HTTP request. After connecting to 
	 * web server, if there is no activity(send or receive) happen for this time period, it will
	 * trigger activity timeout
	 * If set, this value overrides the default HTTP activity timeout
	 *
	 * @param InTimeoutSecs - Timeout for this HTTP request instance, in seconds
	 */
	virtual void SetActivityTimeout(float InTimeoutSecs) = 0;

	/**
	 * Clears the optional timeout in seconds for this HTTP request, causing the default value
	 * from FHttpModule::GetTimeout() to be used.
	 */
	virtual void ClearTimeout() = 0;

	/**
	 * Reset the elapsed timeout duration and flag, after the request completed and need to be reused
	 */
	virtual void ResetTimeoutStatus() = 0;

	/**
	 * Gets the optional timeout in seconds for this entire HTTP request to complete.
	 * If valid, this value overrides the default HTTP timeout set via FHttpModule::SetTimeout().
	 *
	 * @return the timeout for this HTTP request instance, in seconds
	 */
	virtual TOptional<float> GetTimeout() const = 0;

	/**
	 * Called to begin processing the request.
	 * OnProcessRequestComplete delegate is always called when the request completes or on error if it is bound.
	 * A request can be re-used but not while still being processed.
	 *
	 * @return if the request was successfully started.
	 */
	virtual bool ProcessRequest() = 0;

	/**
	 * Delegate called when the request is complete. See FHttpRequestCompleteDelegate
	 */
	virtual FHttpRequestCompleteDelegate& OnProcessRequestComplete() = 0;

	/**
	 * Delegate called to update the request/response progress. See FHttpRequestProgressDelegate64
	 */
	virtual FHttpRequestProgressDelegate64& OnRequestProgress64() = 0;

	/**
	* Delegate called when the request will be retried
	*/
	virtual FHttpRequestWillRetryDelegate& OnRequestWillRetry() = 0;

	/**
	 * Delegate called to signal the receipt of a header.  See FHttpRequestHeaderReceivedDelegate
	 */
	virtual FHttpRequestHeaderReceivedDelegate& OnHeaderReceived() = 0;

	/**
	 * Delegate called to signal the receipt of a header.  See FHttpRequestStatusCodeReceivedDelegate
	 */
	virtual FHttpRequestStatusCodeReceivedDelegate& OnStatusCodeReceived() = 0;

	/**
	 * Called to cancel a request that is still being processed
	 */
	virtual void CancelRequest() = 0;

	/**
	 * Get the associated Response
	 *
	 * @return the response
	 */
	virtual const FHttpResponsePtr GetResponse() const = 0;

	/**
	 * Used to tick the request
	 *
	 * @param DeltaSeconds - seconds since last ticked
	 */
	virtual void Tick(float DeltaSeconds) = 0;

	/**
	 * Gets the time that it took for the server to fully respond to the request.
	 *
	 * @return elapsed time in seconds.
	 */
	virtual float GetElapsedTime() const = 0;

	/**
	 * Set thread policy about which thread to trigger the delegates, set by FHttpManager::SetRequestCompletedDelegate,
	 * IHttpRequest::OnStatusCodeReceived, IHttpRequest::OnHeaderReceived, IHttpRequest::OnRequestProgress64 and IHttpRequest::OnProcessRequestComplete.
	 *
	 * Note that when set it as CompleteOnHttpThread, the thread to trigger delegates could be any thread
	 * depends on the implementation. User code should make the delegate thread-safe and shouldn't assume
	 * it's triggered by the thread where this request get created.
	 *
	 * @param InThreadPolicy - The thread policy to indicate which thread to trigger the delegates
	 */
	virtual void SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy InThreadPolicy) = 0;

	/**
	 * Get thread policy about which thread to complete this request
	 *
	 * @return The thread policy
	 */
	virtual EHttpRequestDelegateThreadPolicy GetDelegateThreadPolicy() const = 0;

	/**
	 * Blocking call to wait the request until it's completed
	 *
	 * WARNINGS:
	 *
	 * - This is a blocking call, DON'T use this in a time-sensitive context
	 * - Complete delegate will be used in this function so customized complete delegate is not supported
	 * - This will force the usage of EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread to make sure the
	 *   request can complete, when this function get called from main thread. So if any other delegate is
	 *   bound, make sure the bound delegate can handle the custom logic in a thread-safe way
	 */
	virtual void ProcessRequestUntilComplete() = 0;

	/**
	 * Set priority of the request, which will be used to decide the position in the queue when max concurrent limit reached
	 *
	 * @param InHttpRequestPriority - The priority to set
	 */
	virtual void SetPriority(EHttpRequestPriority InPriority) = 0;

	/**
	 * Get request priority, which is used to decide the position in the queue when max concurrent limit reached
	 *
	 * @return The request priority
	 */
	virtual EHttpRequestPriority GetPriority() const = 0;

	/**
	 * Destructor for overrides
	 */
	virtual ~IHttpRequest() = default;
};

