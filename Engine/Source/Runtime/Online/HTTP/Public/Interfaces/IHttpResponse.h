// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IHttpBase.h"

#define LOCTEXT_NAMESPACE "IHttpResponse"

namespace EHttpResponseCodes
{
	static constexpr TCHAR ErrorNamespace[] = TEXT("errors.com.epicgames.httpresponse");
	/**
	 * Response codes that can come back from an Http request
	 */
	 #define HTTP_RESPONSE_CODE(Name, Value) Name = Value,
	enum Type
	{
		#include "IHttpResponseCodes.inl"
	};
	#undef HTTP_RESPONSE_CODE


	#define HTTP_RESPONSE_CODE(Name,Value) case Name: return TEXT(#Value":"#Name);
	inline FString ResponseCodeToString(Type ResponseCode)
	{
		switch (ResponseCode)
		{
		#include "IHttpResponseCodes.inl"
		}
		return FString::Printf(TEXT("%d"), (int32)(ResponseCode));
	}
	#undef HTTP_RESPONSE_CODE
	/**
	 * @param StatusCode http response code to check
	 * @return true if the status code is an Ok response
	 */
	inline bool IsOk(int32 StatusCode)
	{
		return StatusCode >= Ok && StatusCode <= PartialContent;
	}

	inline FString GetResponseCodeAsErrorCode(int32 StatusCode)
	{
		return FString::Printf(TEXT("%s.%s"), ErrorNamespace, *ResponseCodeToString((Type)StatusCode));
	}

	/**
	 * @param StatusCode http response code to retrieve
	 * @return The status code description
	 */
	inline FText GetDescription(EHttpResponseCodes::Type StatusCode)
	{
		switch (StatusCode)
		{
		case EHttpResponseCodes::Continue: return LOCTEXT("HttpResponseCode_100", "Continue");
		case EHttpResponseCodes::SwitchProtocol: return LOCTEXT("HttpResponseCode_101", "Switching Protocols");
		case EHttpResponseCodes::Ok: return LOCTEXT("HttpResponseCode_200", "OK");
		case EHttpResponseCodes::Created: return LOCTEXT("HttpResponseCode_201", "Created");
		case EHttpResponseCodes::Accepted: return LOCTEXT("HttpResponseCode_202", "Accepted");
		case EHttpResponseCodes::Partial: return LOCTEXT("HttpResponseCode_203", "Non-Authoritative Information");
		case EHttpResponseCodes::NoContent: return LOCTEXT("HttpResponseCode_204", "No Content");
		case EHttpResponseCodes::ResetContent: return LOCTEXT("HttpResponseCode_205", "Reset Content");
		case EHttpResponseCodes::PartialContent: return LOCTEXT("HttpResponseCode_206", "Partial Content");

		case EHttpResponseCodes::Ambiguous: return LOCTEXT("HttpResponseCode_300", "Multiple Choices");
		case EHttpResponseCodes::Moved: return LOCTEXT("HttpResponseCode_301", "Moved Permanently");
		case EHttpResponseCodes::Redirect: return LOCTEXT("HttpResponseCode_302", "Found/Moved temporarily");
		case EHttpResponseCodes::RedirectMethod: return LOCTEXT("HttpResponseCode_303", "See Other");
		case EHttpResponseCodes::NotModified: return LOCTEXT("HttpResponseCode_304", "Not Modified");
		case EHttpResponseCodes::UseProxy: return LOCTEXT("HttpResponseCode_305", "Use Proxy");
		case EHttpResponseCodes::RedirectKeepVerb: return LOCTEXT("HttpResponseCode_307", "Temporary Redirect");

		case EHttpResponseCodes::BadRequest: return LOCTEXT("HttpResponseCode_400", "Bad Request");
		case EHttpResponseCodes::Denied: return LOCTEXT("HttpResponseCode_401", "Unauthorized");
		case EHttpResponseCodes::PaymentReq: return LOCTEXT("HttpResponseCode_402", "Payment Required");
		case EHttpResponseCodes::Forbidden: return LOCTEXT("HttpResponseCode_403", "Forbidden");
		case EHttpResponseCodes::NotFound: return LOCTEXT("HttpResponseCode_404", "Not Found");
		case EHttpResponseCodes::BadMethod: return LOCTEXT("HttpResponseCode_405", "Method Not Allowed");
		case EHttpResponseCodes::NoneAcceptable: return LOCTEXT("HttpResponseCode_406", "Not Acceptable");
		case EHttpResponseCodes::ProxyAuthReq: return LOCTEXT("HttpResponseCode_407", "Proxy Authentication Required");
		case EHttpResponseCodes::RequestTimeout: return LOCTEXT("HttpResponseCode_408", "Request Timeout");
		case EHttpResponseCodes::Conflict: return LOCTEXT("HttpResponseCode_409", "Conflict");
		case EHttpResponseCodes::Gone: return LOCTEXT("HttpResponseCode_410", "Gone");
		case EHttpResponseCodes::LengthRequired: return LOCTEXT("HttpResponseCode_411", "Length Required");
		case EHttpResponseCodes::PrecondFailed: return LOCTEXT("HttpResponseCode_412", "Precondition Failed");
		case EHttpResponseCodes::RequestTooLarge: return LOCTEXT("HttpResponseCode_413", "Payload Too Large");
		case EHttpResponseCodes::UriTooLong: return LOCTEXT("HttpResponseCode_414", "URI Too Long");
		case EHttpResponseCodes::UnsupportedMedia: return LOCTEXT("HttpResponseCode_415", "Unsupported Media Type");
		case EHttpResponseCodes::TooManyRequests: return LOCTEXT("HttpResponseCode_429", "Too Many Requests");
		case EHttpResponseCodes::RetryWith: return LOCTEXT("HttpResponseCode_449", "Retry With");

		case EHttpResponseCodes::ServerError: return LOCTEXT("HttpResponseCode_500", "Internal Server Error");
		case EHttpResponseCodes::NotSupported: return LOCTEXT("HttpResponseCode_501", "Not Implemented");
		case EHttpResponseCodes::BadGateway: return LOCTEXT("HttpResponseCode_502", "Bad Gateway");
		case EHttpResponseCodes::ServiceUnavail: return LOCTEXT("HttpResponseCode_503", "Service Unavailable");
		case EHttpResponseCodes::GatewayTimeout: return LOCTEXT("HttpResponseCode_504", "Gateway Timeout");
		case EHttpResponseCodes::VersionNotSup: return LOCTEXT("HttpResponseCode_505", "HTTP Version Not Supported");

		case EHttpResponseCodes::Unknown:
		default:
			return LOCTEXT("HttpResponseCode_0", "Unknown");
		}
	}
}

/**
 * Interface for Http responses that come back after starting an Http request
 */
class IHttpResponse : public IHttpBase
{
public:
	/**
	* Return the content with a MoveTemp if possible.
	*
	* @return - The content array
	*/
	virtual TArray<uint8> TakeContent() = 0;
	
	/**
	 * Gets the response code returned by the requested server.
	 * See EHttpResponseCodes for known response codes
	 *
	 * @return the response code.
	 */
	virtual int32 GetResponseCode() const = 0;

	/**
	 * Returns the payload as a string, assuming the payload is UTF8.
	 *
	 * @return the payload as a string.
	 */
	virtual FString GetContentAsString() const = 0;

	/**
	 * Returns the payload as a utf8 string view. This does not validate that the response is valid 
	 * utf8. It is the caller's responsibility, for example, by checking the Content-Type header
	 *
	 * @return the payload as a utf8 string view.
	 */
	virtual FUtf8StringView GetContentAsUtf8StringView() const = 0;

	/** 
	 * Destructor for overrides 
	 */
	virtual ~IHttpResponse() = default;
};

#undef LOCTEXT_NAMESPACE
