// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformHttp.h"

class FHttpManager;
class IHttpRequest;

/**
 * Platform specific Http implementations
 */
class FUnixPlatformHttp : public FGenericPlatformHttp
{
public:

	/**
	 * Platform initialization step
	 */
	static HTTP_API void Init();

	/**
	 * Creates a platform-specific HTTP manager.
	 *
	 * @return NULL if default implementation is to be used
	 */
	static HTTP_API FHttpManager* CreatePlatformHttpManager();

	/**
	 * Platform shutdown step
	 */
	static HTTP_API void Shutdown();

	/**
	 * Creates a new Http request instance for the current platform
	 *
	 * @return request object
	 */
	static HTTP_API IHttpRequest* ConstructRequest();
};


typedef FUnixPlatformHttp FPlatformHttp;
