// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformHttp.h"


/**
 * Platform specific HTTP implementations.
 */
class FWindowsPlatformHttp : public FGenericPlatformHttp
{
public:

	/** Platform initialization step. */
	static HTTP_API void Init();

	/**
	 * Creates a platform-specific HTTP manager.
	 *
	 * @return nullptr if default implementation is to be used.
	 */
	static HTTP_API FHttpManager* CreatePlatformHttpManager();

	/** Platform shutdown step. */
	static HTTP_API void Shutdown();

	/**
	 * Creates a new HTTP request instance for the current platform.
	 *
	 * @return The request object.
	 */
	static HTTP_API IHttpRequest* ConstructRequest();

	/**
	 * @return the mime type for the file.
	 */
	static HTTP_API FString GetMimeType(const FString& FilePath);

	/**
	 * Get the proxy address from the highest-priority available source:
	 * command-line (-httpproxy=), INI ([HTTP] HttpProxyAddress), or OS proxy settings.
	 * It's to ensure FWindowsPlatformHttp::GetOperatingSystemProxyAddress() is called.
	 */
	static HTTP_API FString GetConfiguredProxyAddress();

	/**
	 * Get the proxy address specified by the operating system
	 *
	 * @return optional FString: If unset: we are unable to get information from the operating system. If set: the proxy address set by the operating system (may be blank)
	 */
	static HTTP_API TOptional<FString> GetOperatingSystemProxyAddress();

	/**
	 * Check if getting proxy information from the current operating system is supported
	 * Useful for "Network Settings" type pages.  GetProxyAddress may return an empty or populated string but that does not imply
	 * the operating system does or does not support proxies (or that it has been implemented here)
	 *
	 * @return true if we are able to get proxy information from the current operating system, false if not
	 */
	static HTTP_API bool IsOperatingSystemProxyInformationSupported();
	
	/**
	 * Verify Peer Ssl Certificate
	 *
	 * @return optional bool: the previous value
	 */
	static HTTP_API bool VerifyPeerSslCertificate(bool verify);
};

#if WINDOWS_USE_FEATURE_PLATFORMHTTP_CLASS
typedef FWindowsPlatformHttp FPlatformHttp;
#endif // WINDOWS_USE_FEATURE_PLATFORMHTTP_CLASS
