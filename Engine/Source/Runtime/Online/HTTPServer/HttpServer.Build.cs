// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HTTPServer : ModuleRules
{
    public HTTPServer(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
				"HTTP",
				"Sockets",
            }
        );
    }
}
