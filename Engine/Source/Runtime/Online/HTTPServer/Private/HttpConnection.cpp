// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpConnection.h"
#include "HttpConnectionContext.h"
#include "HttpConnectionRequestReadContext.h"
#include "HttpConnectionResponseWriteContext.h"
#include "HttpRouter.h"
#include "HttpServerRequest.h"
#include "HttpServerResponse.h"
#include "HttpRequestHandlerIterator.h"
#include "HttpServerConstants.h"
#include "HttpServerConstantsPrivate.h"

#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Stats/Stats.h"
#include "Containers/Array.h"

DEFINE_LOG_CATEGORY(LogHttpConnection)

FHttpConnection::FHttpConnection(FSocket* InSocket, TSharedPtr<FHttpRouter> InRouter, uint32 InOriginPort, uint32 InConnectionId)
	: Socket(InSocket)
	,Router(InRouter)
	,OriginPort(InOriginPort)
	,ConnectionId(InConnectionId)
	,ReadContext(InSocket)
	,WriteContext(InSocket)
{
	check(nullptr != Socket);

	Config = FHttpServerConfig::GetConnectionConfig();
}

FHttpConnection::~FHttpConnection()
{
	check(nullptr == Socket);
}

void FHttpConnection::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpConnection_Tick);
	const float AwaitReadTimeout = bKeepAlive ?
		ConnectionKeepAliveTimeout : ConnectionTimeout;

	switch (State)
	{
	case EHttpConnectionState::AwaitingRead:
		if (ReadContext.GetElapsedIdleTime() > AwaitReadTimeout || ReadContext.GetSecondsWaitingForReadableSocket() > AwaitReadTimeout)
		{
			Destroy(EConnectionDestroyReason::AwaitReadTimeout);
			return;
		}
		BeginRead(DeltaTime);
		break;

	case EHttpConnectionState::Reading:
		if (ReadContext.GetElapsedIdleTime() > ConnectionTimeout)
		{
			Destroy(EConnectionDestroyReason::ReadTimeout);
			return;
		}
		ContinueRead(DeltaTime);
		break;

	case EHttpConnectionState::AwaitingProcessing:
		break;

	case EHttpConnectionState::Writing:
		if (WriteContext.GetElapsedIdleTime() > ConnectionTimeout)
		{
			Destroy(EConnectionDestroyReason::WriteTimeout);
			return;
		}
		ContinueWrite(DeltaTime);
		break;

	case EHttpConnectionState::Destroyed:
		ensure(false);
		break;
	}
}

void FHttpConnection::ChangeState(EHttpConnectionState NewState)
{
	check(EHttpConnectionState::Destroyed != State);
	check(NewState != State);

	UE_LOGF(LogHttpConnection, Verbose,
				 "ChangingState: %d => %d", int(State), int(NewState));
	State = NewState;
}

void FHttpConnection::TransferState(EHttpConnectionState CurrentState, EHttpConnectionState NextState)
{
	check(CurrentState == State);
	check(NextState != CurrentState);
	ChangeState(NextState);
}

void FHttpConnection::BeginRead(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpConnection_BeginRead);
	const FTimespan WaitTime = FTimespan::FromMilliseconds(Config.BeginReadWaitTimeMS);
	if (!Socket->Wait(ESocketWaitConditions::WaitForRead, WaitTime))
	{
		const float SecondsPollingSocketThisFrame = WaitTime.GetTotalSeconds();
		// We don't add DeltaTime on the first attempt as only 1ms has passed since we started waiting for the socket to be reachable.
		const float SecondsPollingSocketSinceLastFrame = ReadContext.GetSecondsWaitingForReadableSocket() == 0.f ? SecondsPollingSocketThisFrame : DeltaTime + SecondsPollingSocketThisFrame;
		ReadContext.AddSecondsWaitingForReadableSocket(SecondsPollingSocketSinceLastFrame);
	}
	else
	{
		UE_LOGF(LogHttpConnection, Verbose,
			"SecondsWaitingForReadableSocket\t [%d][%u]-%u : %f",
				OriginPort, ConnectionId, LastRequestNumber, ReadContext.GetSecondsWaitingForReadableSocket());

		ReadContext.ResetSecondsWaitingForReadableSocket();

		// The socket is reachable, however there may not be data in the pipe
		uint32 PendingDataSize = 0;
		if (Socket->HasPendingData(PendingDataSize))
		{
			TransferState(EHttpConnectionState::AwaitingRead, EHttpConnectionState::Reading);
			LastRequestNumber++;
			ReadContext.ResetContext();
			ContinueRead(DeltaTime);
		}
		else
		{
			ReadContext.AddElapsedIdleTime(DeltaTime);
		}
	}
}

void FHttpConnection::ContinueRead(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpConnection_ContinueRead);
	check(State == EHttpConnectionState::Reading);

	auto ReaderState = ReadContext.ReadStream(DeltaTime);

	switch (ReaderState)
	{
	case EHttpConnectionContextState::Continue:
		break;

	case EHttpConnectionContextState::Done:
		CompleteRead(ReadContext.GetRequest());
		break;

	case EHttpConnectionContextState::Error:
		HandleReadError(ReadContext.GetErrorCode(), *ReadContext.GetErrorStr());
		break;
	}
}

void FHttpConnection::CompleteRead(const TSharedPtr<FHttpServerRequest>& Request)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpConnection_CompleteRead);
	TArray<FString>* ConnectionHeaders = Request->Headers.Find(UE_HTTP_SERVER_HEADER_KEYS_CONNECTION);
	if (ConnectionHeaders)
	{
		bKeepAlive = ResolveKeepAlive(Request->HttpVersion, *ConnectionHeaders);
	}

	TWeakPtr<FHttpConnection> WeakThisPtr(AsShared());
	FHttpResultCallback OnProcessingComplete =
		[WeakThisPtr,
		OriginPortCapture = OriginPort,
		ConnectionIdCapture = ConnectionId,
		LastRequestNumberCapture = LastRequestNumber,
		ResponseVersionCapture = Request->HttpVersion,
		bIsHeadRequest = (Request->Verb == EHttpServerRequestVerbs::VERB_HEAD)]
	(TUniquePtr<FHttpServerResponse>&& Response)
	{
		TSharedPtr<FHttpConnection> SharedThisPtr = WeakThisPtr.Pin();
		if (SharedThisPtr.IsValid())
		{
			UE_LOGF(LogHttpConnection, Verbose,
				"EndProcessRequest\t [%d][%u]-%u : %u",
						 OriginPortCapture, ConnectionIdCapture, LastRequestNumberCapture, int(Response->Code));

			// A streaming handler using `MultipleWriteStream | HasAdditionalWrites` may invoke this callback again after the connection has already been torn down by `HandleWriteError` mid-stream. The `WeakThisPtr` still pins during the window before `FHttpListener::RemoveDestroyedConnections` runs, so the only way to detect the disconnect here is the connection state. Drop the response silently.
			if (EHttpConnectionState::Destroyed == SharedThisPtr->GetState())
			{
				UE_LOGF(LogHttpConnection, Verbose,
					"EndProcessRequest dropped (connection destroyed)\t [%d][%u]-%u : %u",
							 OriginPortCapture, ConnectionIdCapture, LastRequestNumberCapture, int(Response->Code));
				return;
			}

			// Normalise headers up-front so the response is ready to write whether we dispatch now or queue.
			Response->HttpVersion = ResponseVersionCapture;
			// HEAD responses must have the same headers as GET but no body
			if (bIsHeadRequest)
			{
				Response->Body.Empty();
			}

			// On-stack reentry while the previous response is still draining (state == `Writing`):
			// capture into the single-slot pending response and let `CompleteWrite` drain it.
			if (EHttpConnectionState::Writing == SharedThisPtr->GetState())
			{
				checkf(!SharedThisPtr->PendingResponse,
					TEXT("FHttpConnection [%d][%u]-%u received a third synchronous OnComplete while a response was already queued."),
					OriginPortCapture, ConnectionIdCapture, LastRequestNumberCapture);
				SharedThisPtr->PendingResponse = MoveTemp(Response);
				UE_LOGF(LogHttpConnection, Verbose,
					"EndProcessRequest queued (mid-Writing)\t [%d][%u]-%u",
					OriginPortCapture, ConnectionIdCapture, LastRequestNumberCapture);
				return;
			}

			// Ensure this result callback was called at least once
			check(EHttpConnectionState::AwaitingProcessing == SharedThisPtr->GetState());

			SharedThisPtr->BeginWrite(MoveTemp(Response), LastRequestNumberCapture);
		}
	};

	ProcessRequest(Request, OnProcessingComplete);
}

void FHttpConnection::ProcessRequest(const TSharedPtr<FHttpServerRequest>& Request, const FHttpResultCallback& OnProcessingComplete)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpConnection_ProcessRequest);
	TransferState(EHttpConnectionState::Reading, EHttpConnectionState::AwaitingProcessing);

	UE_LOGF(LogHttpConnection, Verbose,
		"BeginProcessRequest\t [%d][%u]-%u : %ls", 
		OriginPort, ConnectionId, LastRequestNumber, *Request->RelativePath.GetPath());

	bool bRequestHandled = false;
	auto RequestHandlerIterator = Router->CreateRequestHandlerIterator(Request);
	while (const FHttpRequestHandler* RequestHandlerPtr = RequestHandlerIterator.Next())
	{
		if (!RequestHandlerPtr->IsBound())
		{
			UE_LOGF(LogHttpConnection, Verbose, "Skipping an unbound FHttpRequestHandler.");
			continue;
		}

		bRequestHandled = RequestHandlerPtr->Execute(*Request, OnProcessingComplete);
		if (bRequestHandled)
		{
			break;
		}
		// If this handler didn't accept, ensure they did not invoke the result callback
		check(State == EHttpConnectionState::AwaitingProcessing);
	}

	if (!bRequestHandled)
	{
		auto Response = FHttpServerResponse::Error(EHttpServerResponseCodes::NotFound, UE_HTTP_SERVER_ERROR_STR_ROUTE_HANDLER_NOT_FOUND);
		OnProcessingComplete(MoveTemp(Response));
	}
}

void FHttpConnection::BeginWrite(TUniquePtr<FHttpServerResponse>&& Response, uint32 RequestNumber)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpConnection_BeginWrite);
	// Ensure the passed-in request number is the one we expect
	check(RequestNumber == LastRequestNumber);

	ChangeState(EHttpConnectionState::Writing);

	if (bKeepAlive)
	{
		FString KeepAliveTimeoutStr = FString::Printf(TEXT("timeout=%f"), ConnectionKeepAliveTimeout);
		TArray<FString> KeepAliveTimeoutValue = { MoveTemp(KeepAliveTimeoutStr) };
		Response->Headers.Add(UE_HTTP_SERVER_HEADER_KEYS_KEEP_ALIVE, MoveTemp(KeepAliveTimeoutValue));
	}

	WriteContext.ResetContext(MoveTemp(Response));
	ContinueWrite(0.0f);
}

void FHttpConnection::ContinueWrite(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpConnection_ContinueWrite);
	check(State == EHttpConnectionState::Writing);

	auto WriterState = WriteContext.WriteStream(DeltaTime);
	switch (WriterState)
	{
	case EHttpConnectionContextState::Continue:
		break;

	case EHttpConnectionContextState::Done:
		CompleteWrite();
		break;

	case EHttpConnectionContextState::Error:
		HandleWriteError(*WriteContext.GetErrorStr());
		break;
	}
}

void FHttpConnection::CompleteWrite()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpConnection_CompleteWrite);
	check(EHttpConnectionState::Writing == State);

	// `CloseAfterWrite` overrides keep-alive: the just-written response explicitly requested
	// that the connection be torn down. Used by streaming responses (`MultipleWriteStream`)
	// without `Content-Length` or chunked framing - the socket close is the client's only
	// end-of-message signal. Any queued `PendingResponse` is dropped along with the
	// connection, matching the existing teardown drop semantics below.
	const bool bCloseAfterWrite = WriteContext.ShouldCloseAfterWrite();

	if (bKeepAlive && !bGracefulDestroyRequested && !bCloseAfterWrite)
	{
		// Drain a response queued by `OnProcessingComplete` while we were mid-`Writing`. Transition
		// through `AwaitingProcessing` so `BeginWrite` can re-enter `Writing` cleanly. `LastRequestNumber`
		// is reused as-is: the queued response was produced by the same handler invocation as the
		// in-flight one, and `LastRequestNumber` only advances on a new read - which cannot happen
		// while state is `Writing`.
		if (PendingResponse)
		{
			UE_LOGF(LogHttpConnection, Verbose,
				"EndProcessRequest drained (queued mid-Writing)\t [%d][%u]-%u",
				OriginPort, ConnectionId, LastRequestNumber);
			ChangeState(EHttpConnectionState::AwaitingProcessing);
			BeginWrite(MoveTemp(PendingResponse), LastRequestNumber);
			return;
		}

		if (WriteContext.HasAdditionalWrites())
		{
			ChangeState(EHttpConnectionState::AwaitingProcessing);
		}
		else
		{
			ChangeState(EHttpConnectionState::AwaitingRead);
		}
	}
	else
	{
		if (PendingResponse)
		{
			UE_LOGF(LogHttpConnection, Verbose,
				"EndProcessRequest dropped (connection closing)\t [%d][%u]-%u",
				OriginPort, ConnectionId, LastRequestNumber);
		}
		Destroy(EConnectionDestroyReason::WriteComplete);
	}
}

void FHttpConnection::RequestDestroy(bool bGraceful)
{
	if (EHttpConnectionState::Destroyed == State)
	{
		return;
	}

	bGracefulDestroyRequested = bGraceful;

	// If we aren't gracefully destroying, or we are otherwise already 
	// awaiting a read operation (not started yet), destroy() immediately.
	if (!bGracefulDestroyRequested || State == EHttpConnectionState::AwaitingRead)
	{
		Destroy(EConnectionDestroyReason::DestroyRequest);
	}
}

void FHttpConnection::Destroy(EConnectionDestroyReason Reason)
{
	check(State != EHttpConnectionState::Destroyed);
	ChangeState(EHttpConnectionState::Destroyed);

	if (Socket)
	{
		ISocketSubsystem *SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		if (SocketSubsystem)
		{
			SocketSubsystem->DestroySocket(Socket);
		}
		Socket = nullptr;

		UE_LOGF(LogHttpConnection, Verbose, "ConnectionDestroyed, Reason: %ls", LexToString(Reason));
	}
}

void FHttpConnection::HandleReadError(EHttpServerResponseCodes ErrorCode, const TCHAR* ErrorCodeStr)
{
	UE_LOGF(LogHttpConnection, Error, "%ls", ErrorCodeStr);

	// Forcibly Reply
	bKeepAlive = false;
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Error(ErrorCode, ErrorCodeStr);
	TSharedPtr<FHttpServerRequest> Request = ReadContext.GetRequest();
	if (Request.IsValid())
	{
		Response->HttpVersion = Request->HttpVersion;
	}
	else
	{
		UE_LOGF(LogHttpConnection, Verbose, "Http Request is null, unable to parse version.");
		Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_UNKNOWN;
	}

	BeginWrite(MoveTemp(Response), LastRequestNumber);
}

void FHttpConnection::HandleWriteError(const TCHAR* ErrorCodeStr)
{
	UE_LOGF(LogHttpConnection, Error, "%ls", ErrorCodeStr);

	if (PendingResponse)
	{
		UE_LOGF(LogHttpConnection, Verbose,
			"EndProcessRequest dropped (write error)\t [%d][%u]-%u",
			OriginPort, ConnectionId, LastRequestNumber);
	}

	// Forcibly Close
	bKeepAlive = false;
	Destroy(EConnectionDestroyReason::WriteError);
}

bool FHttpConnection::ResolveKeepAlive(HttpVersion::EHttpServerHttpVersion HttpVersion, const TArray<FString>& ConnectionHeaders)
{
	const bool bDefaultKeepAlive = HttpVersion == HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;

	if (bDefaultKeepAlive)
	{
		return !ConnectionHeaders.Contains(TEXT("close"));
	}
	else
	{
		return ConnectionHeaders.Contains(TEXT("Keep-Alive"));
	}
}
