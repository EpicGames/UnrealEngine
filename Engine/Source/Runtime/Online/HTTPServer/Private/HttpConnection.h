// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HttpConnectionTypes.h"
#include "HttpConnectionRequestReadContext.h"
#include "HttpConnectionResponseWriteContext.h"
#include "HttpResultCallback.h"
#include "HttpServerConfig.h"
#include "HttpServerConstants.h"
#include "HttpServerHttpVersion.h"
#include "Misc/Timespan.h"

class FSocket;
class ISocketSubsystem;
class FHttpRouter;
struct FHttpServerRequest;
struct FHttpServerResponse;

DECLARE_LOG_CATEGORY_EXTERN(LogHttpConnection, Log, All);

struct FHttpConnection final : public TSharedFromThis<FHttpConnection>
{
public:

	/**
	 * Constructor
	 * 
	 * @param InSocket The underlying file descriptor
	 */
	FHttpConnection(FSocket* InSocket, TSharedPtr<FHttpRouter> InRouter, uint32 InOriginPort, uint32 InConnectionId);
	/**
	 * Destructor
	 */
	~FHttpConnection();

	/** 
	* Ticks the connection to drive internal state
	*
	* @param DeltaTime The elapsed time since the last Tick()
	*/
	void Tick(float DeltaTime);

	/**
	 * Returns the current state of the connection
	 */
	FORCEINLINE EHttpConnectionState GetState() const
	{
		return State;
	}

	/**
	 * Signals the connection to destroy itself
	 *
	 * @param bGraceful Whether to gracefully destroy pending current operations
	 */
	void RequestDestroy(bool bGraceful);

	/**
	*  Determines whether this connection should remain open after writing
	*/
	FORCEINLINE bool IsHttpKeepAliveEnabled() const 
	{ 
		return bKeepAlive;
	}

	FORCEINLINE bool operator==(const FHttpConnection& Other) const
	{
		return this->Socket == Other.Socket;
	}

	friend uint32 GetTypeHash(const FHttpConnection& Conn)
	{
		return GetTypeHash(Conn.Socket);
	}

private:
	/**
	 * The reason why the connection was destroyed.
	 */
	enum class EConnectionDestroyReason : uint8
	{
		WriteError,
        AwaitReadTimeout,
        BeginReadTimeout,
        ReadTimeout,
        WriteTimeout,
        WriteComplete,
        DestroyRequest
    };

private:

	/**
     * Changes the internal state of the connection to NewState
	 * 
     * @param NewState The state to transition to
     */
	void ChangeState(EHttpConnectionState NewState);

	/**
	 * Changes the internal state from the caller-supplied current state to the next state
	 *
	 * @param CurrentState The expected current state
	 * @param NextState    The next state
	 */
	void TransferState(EHttpConnectionState CurrentState, EHttpConnectionState NextState);

	/**
     * Begins a read operation
	 *
	 * @param DeltaTime The elapsed time since the last invocation
     */
	void BeginRead(float DeltaTime);

	/**
	 * Continues an in-progress read operation
	 *
	 * @param DeltaTime The elapsed time since the last invocation
	 */
	void ContinueRead(float DeltaTime);

	/**
	 * Completes a previously begun read operation
	 *
	 * @param Request The instantiated request object 
	 */
	void CompleteRead(const TSharedPtr<FHttpServerRequest>& Request);

	/**
	 * Proxies the respective request to a bound handler
	 * @param Request              The request to process
	 * @param OnProcessingComplete The callback to be invoked upon completion
	 * @return                     true if the request was accepted by a handler, false otherwise
	 */
	void ProcessRequest(const TSharedPtr<FHttpServerRequest>& Request, const FHttpResultCallback& OnProcessingComplete);

	/**
     * Begins a write operation
	 * 
     * @param Response      The response to write
	 * @param RequestNumber The expected and unique request number
     */
	void BeginWrite(TUniquePtr<FHttpServerResponse>&& Response, uint32 RequestNumber);

	/**
	 * Continues an in-progress write operation
     *
	 * @param DeltaTime The elapsed time since the last invocation
	 */
	void ContinueWrite(float DeltaTime);

	/**
	 * Completes a previously begun write operation
	 */
	void CompleteWrite();

	/**
	 * Closes and nullifies the underlying connection
	 * @param DestroyReason The reason why the connection should be destroyed.
	 */
	void Destroy(EConnectionDestroyReason DestroyReason);

	/**
	 * Logs and responds with the caller-supplied error code
	 *
	 * @param ErrorCode The HTTP error code
	 * @param ErrorCodeStr The machine-readable error description
	 */
	void HandleReadError(EHttpServerResponseCodes ErrorCode, const TCHAR* ErrorCodeStr);

	/**
	 * Logs the caller-supplied error code and closes the connection
	 *
	 * @param ErrorCodeStr The machine-readable error description
	 */
	void HandleWriteError(const TCHAR* ErrorCodeStr);

	/**
	 * Determines whether KeepAlive is set based on the caller-supplied http version and connection headers
	 *
	 * @param HttpVersion         The http version of the request
	 * @param ConnectionHeaders   The request "Connection:" headers
	 * @return                    true if KeepAlive should be set, false otherwise
     */
	static bool ResolveKeepAlive(HttpVersion::EHttpServerHttpVersion HttpVersion, const TArray<FString>& ConnectionHeaders);

	/**
	 * Convert a connection destroy reason to a string.
	 * @param DestroyReason The reason why the connection was destroyed.
	 * @return				The output string.
	 */
	static const TCHAR* LexToString(EConnectionDestroyReason DestroyReason)
	{
		switch (DestroyReason)
		{
			case EConnectionDestroyReason::WriteError:
				return TEXT("Write error");
		    case EConnectionDestroyReason::AwaitReadTimeout:
		    	return TEXT("Await read timeout");
		    case EConnectionDestroyReason::BeginReadTimeout:
		    	return TEXT("Begin read timeout");
		    case EConnectionDestroyReason::ReadTimeout:
		    	return TEXT("Read timeout");
		    case EConnectionDestroyReason::WriteTimeout:
		    	return TEXT("Write timeout");
		    case EConnectionDestroyReason::WriteComplete:
		    	return TEXT("Write complete");
		    case EConnectionDestroyReason::DestroyRequest:
		    	return TEXT("Destroy connection request");
			default:
			{
				checkNoEntry();
				return TEXT("Unknown");
			}
		}
	}

private:

	/** Accepted external socket */
	FSocket* Socket;

	/** State of the connection  */
	EHttpConnectionState State = EHttpConnectionState::AwaitingRead;

	/** Routing mechanism  */
	const TSharedPtr<FHttpRouter> Router;

	/** The origin port on which this connection was accepted */
	uint32 OriginPort;

	/** The connection identifier (used for logging purposes) */
	uint32 ConnectionId;

	/** Helper reader context to track the state of streaming request reads */
	FHttpConnectionRequestReadContext ReadContext;

	/** Helper writer context to track the state of streaming response writes */
	FHttpConnectionResponseWriteContext WriteContext;

	/** Whether to keep this connection alive after writing */
	bool bKeepAlive = true;

	/** Whether to gracefully close pending current operations */
	bool bGracefulDestroyRequested = false;

	/** Internal state tracker (incremented per-request-read) used to validate request/response throughput */
	uint32 LastRequestNumber = 0;

	/**
	 * Single-slot pending response captured by `OnProcessingComplete` while the connection is still draining
	 * a previous write (state == `Writing`). `CompleteWrite` drains it once the in-flight response finishes;
	 * a `checkf(!PendingResponse)` at the queue site enforces the two-deep `FHttpResultCallback` contract.
	 *
	 * Silently dropped if the connection transitions to teardown before the drain runs: `bGracefulDestroyRequested`,
	 * `!bKeepAlive`, or `HandleWriteError`. A `Verbose` log is emitted on the drop path so the disappearance is
	 * traceable in production logs.
	 */
	TUniquePtr<FHttpServerResponse> PendingResponse;

	/** The duration (seconds) at which connections are forcefully timed out */
	static constexpr float ConnectionTimeout = 5.0f;

	/** The duration (seconds) at which idle keep-alive connections are forcefully timed out */
	static constexpr float ConnectionKeepAliveTimeout = 15.0f;

	/** Connection configuration data */
	FHttpServerConnectionConfig Config;
};

