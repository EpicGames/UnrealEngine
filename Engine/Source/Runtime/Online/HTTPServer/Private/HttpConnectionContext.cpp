// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpConnectionContext.h"
#include "CoreMinimal.h"

FHttpConnectionContext::FHttpConnectionContext()
{
	ErrorBuilder.SetAutoEmitLineTerminator(true);
}

FHttpConnectionContext::~FHttpConnectionContext()
{
}

void FHttpConnectionContext::AddElapsedIdleTime(float DeltaTime)
{
	ElapsedIdleTime += DeltaTime;
}

void FHttpConnectionContext::AddError(const FString& ErrorCodeStr, EHttpServerResponseCodes HttpErrorCode)
{
	if (!ErrorBuilder.IsEmpty())
	{
		ErrorBuilder.AppendChar(TCHAR(' '));
	}
	ErrorBuilder.Append(ErrorCodeStr);

	if (EHttpServerResponseCodes::Unknown != HttpErrorCode)
	{
		ErrorCode = HttpErrorCode;
	}
}
