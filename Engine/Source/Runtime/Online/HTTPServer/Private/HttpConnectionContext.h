// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "HttpServerConstants.h"
#include "Misc/StringOutputDevice.h"

enum class EHttpConnectionContextState
{
	Continue,
	Done,
	Error
};


struct FHttpConnectionContext
{

public:

	/**
	 * Adds to the context elapsed time
	 * 
	 * @param DeltaTime The elapsed time (seconds) to add
	 */
	void AddElapsedIdleTime(float DeltaTime);

	/**
	 * Gets the elapsed time (seconds) since last activity
	 */
	FORCEINLINE float GetElapsedIdleTime() const
	{
		return ElapsedIdleTime;
	}

	/**
    * Gets the respective error code
	*/
	FORCEINLINE EHttpServerResponseCodes GetErrorCode() const
	{
		return ErrorCode;
	}

	/**
	 * Gets the cumulative error string
	 */
	FORCEINLINE const FString& GetErrorStr() const
	{
		return ErrorBuilder;
	}

protected:

	/**
	 * Constructor
	 */
	FHttpConnectionContext();

	/**
	 * Destructor
	 */
	virtual ~FHttpConnectionContext();

	/**
	 * Adds the caller-supplied error to the context

	 * @param ErrorCode The machine-readable error code
	 */
	void AddError(const FString& ErrorCodeStr, EHttpServerResponseCodes ErrorCode = EHttpServerResponseCodes::Unknown);

	/** Tracks time since last read/write activity */
	float ElapsedIdleTime = 0.0f;

	/** Tracks the respective ErrorCode  */
	EHttpServerResponseCodes ErrorCode = EHttpServerResponseCodes::Unknown;

	/** Tracks cumulative context errors */
	FStringOutputDevice ErrorBuilder;
};