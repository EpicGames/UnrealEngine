// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpConnectionRequestReadContext.h"
#include "HttpServerConstants.h"
#include "HttpServerConstantsPrivate.h"
#include "HttpServerRequest.h"
#include "HttpConnectionContext.h"
#include "IPAddress.h"
#include "PlatformHttp.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

DEFINE_LOG_CATEGORY(LogHttpConnectionRequestReadContext)

FHttpConnectionRequestReadContext::FHttpConnectionRequestReadContext(FSocket* InSocket)
	: Socket(InSocket)
{
}

void FHttpConnectionRequestReadContext::ResetContext()
{
	Request = nullptr;
	ElapsedIdleTime = 0.0f;
	SecondsWaitingForReadableSocket = 0.f;
	ErrorBuilder.Empty();
	HeaderBytes.Empty();
	IncomingRequestBodyBytesToRead = 0;

	bParseHeaderComplete = false;
	bParseBodyComplete = false;
}

EHttpConnectionContextState FHttpConnectionRequestReadContext::ReadStream(float DeltaTime)
{
	ElapsedIdleTime += DeltaTime;

	const uint32 ByteBufferSize = 1024 * 64; // 64k - safe value to remain under SCA limit (/analyze:stacksize 81940)
	uint8 ByteBuffer[ByteBufferSize] = { 0 };
	int32 BytesRead = 0;
	if (!Socket->Recv(ByteBuffer, sizeof(ByteBuffer) - 1, BytesRead, ESocketReceiveFlags::None))
	{
		AddError(UE_HTTP_SERVER_ERROR_STR_SOCKET_RECV_FAILURE);
		return EHttpConnectionContextState::Error;
	}

	if (0 == BytesRead)
	{
		return EHttpConnectionContextState::Continue;
	}
	else
	{
		UE_LOGF(LogHttpConnectionRequestReadContext, Verbose,
			"ElapsedIdleTime\t %f", ElapsedIdleTime);
		ElapsedIdleTime = 0.0f;
	}

	if (!bParseHeaderComplete)
	{
		if (!ParseHeader(ByteBuffer, BytesRead))
		{
			return EHttpConnectionContextState::Error;
		}
	}
	else if (!bParseBodyComplete)
	{
		if (!ParseBody(ByteBuffer, BytesRead))
		{
			return EHttpConnectionContextState::Error;
		}
	}
		
	// If parsing is complete, we are done
	if (bParseHeaderComplete && bParseBodyComplete)
	{
		check(Request);
		return EHttpConnectionContextState::Done;
	}
	
	return EHttpConnectionContextState::Continue;
}

bool FHttpConnectionRequestReadContext::ParseHeader(uint8* ByteBuffer, int32 BufferLen)
{
	// Append this chunk to our header bytes
	int PreviousHeaderBytesLen = HeaderBytes.Num();
	HeaderBytes.Append(ByteBuffer, BufferLen);

	// We are hunting for the header terminator sequence '\r\n\r\n'
	// Start 3 bytes prior to this most recent appendage
	int32 SearchIndex = FMath::Max(0, PreviousHeaderBytesLen - 3);
	for ( ; SearchIndex < HeaderBytes.Num() - 3; ++SearchIndex)
	{
		// To find this sequence we will parse byte-by-byte in ANSICHAR
		if (ANSICHAR(HeaderBytes[SearchIndex])     == '\r' &&
			ANSICHAR(HeaderBytes[SearchIndex + 1]) == '\n' &&
			ANSICHAR(HeaderBytes[SearchIndex + 2]) == '\r' &&
			ANSICHAR(HeaderBytes[SearchIndex + 3]) == '\n')
		{
			// We found the terminator
			bParseHeaderComplete = true;

			// Remove any extra header bytes, but retain the first trailing \r\n 
			HeaderBytes.SetNum(SearchIndex + 2);
			HeaderBytes.Add('\0');

			// Build header string
			FUTF8ToTCHAR WByteBuffer(reinterpret_cast<const ANSICHAR*>(HeaderBytes.GetData()), HeaderBytes.Num());
			const FString IncomingRequestHeaderStr = FString::ConstructFromPtrSize(WByteBuffer.Get(), WByteBuffer.Length());
			Request = BuildRequest(IncomingRequestHeaderStr);
			if (!Request)
			{
				// Error if we cannot build a request object
				return false;
			}

			if (!IsRequestValid(*Request.Get()))
			{
				// Error if request object is invalid
				return false;
			}

			// Try to parse the rest of the request body here in the same frame
			ParseContentLength(*Request.Get(), IncomingRequestBodyBytesToRead);

			//                    base         relative offset    absolute offset
			auto BodyBytesPtr = ByteBuffer +  (SearchIndex + 4 - PreviousHeaderBytesLen);
			auto BufferBytesRemaining = BufferLen - (BodyBytesPtr - ByteBuffer);
			return ParseBody(BodyBytesPtr, BufferBytesRemaining);
		}
	}

	// Keep parsing.
	return true;
}

bool FHttpConnectionRequestReadContext::ParseBody(uint8* ByteBuffer, int32 BufferLen)
{
	if (BufferLen > IncomingRequestBodyBytesToRead)
	{
		// Some HTTP clients (e.g. xCurl with CURL_HTTP_VERSION_2TLS falling back to HTTP/1.1) send
		// "Transfer-Encoding: chunked" even for bodyless requests (GET, HEAD, etc.) and append the
		// chunked empty-body terminator "0\r\n\r\n" (5 bytes) after the request headers. Since the
		// server does not implement chunked request-body decoding, accept this specific case:
		// when no body was expected (IncomingRequestBodyBytesToRead == 0) and the extra bytes are
		// exactly the chunked terminator, treat the body as empty and complete.
		static const uint8 ChunkedTerminator[] = { '0', '\r', '\n', '\r', '\n' };
		static constexpr int32 ChunkedTerminatorLen = UE_ARRAY_COUNT(ChunkedTerminator);
		if (IncomingRequestBodyBytesToRead == 0 &&
			BufferLen >= ChunkedTerminatorLen &&
			FMemory::Memcmp(ByteBuffer, ChunkedTerminator, ChunkedTerminatorLen) == 0)
		{
			// Empty chunked body — consume the terminator and complete body parsing
			bParseBodyComplete = true;
			return true;
		}

		// Error - Sent data size exceeds expected
		AddError(UE_HTTP_SERVER_ERROR_STR_MISMATCHED_CONTENT_LENGTH_BODY_TOO_LARGE, EHttpServerResponseCodes::BadRequest);
		return false;
	}

	// Append bytes
	if (BufferLen > 0)
	{
		Request->Body.Append(ByteBuffer, BufferLen);
		IncomingRequestBodyBytesToRead -= BufferLen;
	}

	bParseBodyComplete = (0 == IncomingRequestBodyBytesToRead);
	return true;
}

bool FHttpConnectionRequestReadContext::IsRequestValid(const FHttpServerRequest& InRequest) 
{
	int32 RequestContentLength; 
	bool bContentLengthSpecified = ParseContentLength(InRequest, RequestContentLength);

	switch (InRequest.Verb)
	{
	case EHttpServerRequestVerbs::VERB_GET:
		// Enforce content length missing or 0
		if (bContentLengthSpecified && 0 != RequestContentLength)
		{
			AddError(UE_HTTP_SERVER_ERROR_STR_INVALID_CONTENT_LENGTH_HEADER, EHttpServerResponseCodes::BadRequest);
			return false;
		}
		break;

	case EHttpServerRequestVerbs::VERB_PUT:
	case EHttpServerRequestVerbs::VERB_POST:
		// Content length must be set
		if (!bContentLengthSpecified)
		{
			AddError(UE_HTTP_SERVER_ERROR_STR_MISSING_CONTENT_LENGTH_HEADER, EHttpServerResponseCodes::LengthRequired);
			return false;
		}
		// Content length must be valid
		if(RequestContentLength < 0)
		{
			AddError(UE_HTTP_SERVER_ERROR_STR_INVALID_CONTENT_LENGTH_HEADER, EHttpServerResponseCodes::LengthRequired);
			return false;
		}
		break;
	}

	return true;
}

bool FHttpConnectionRequestReadContext::ParseContentLength(const FHttpServerRequest& InRequest, int32& OutContentLength)
{
	const TArray<FString>* ContentLengthValues = InRequest.Headers.Find(UE_HTTP_SERVER_HEADER_KEYS_CONTENT_LENGTH);

	if (ContentLengthValues && ContentLengthValues->Num() > 0)
	{
		const FString& ContentLengthStr = (*ContentLengthValues)[0];
		OutContentLength = FMath::Max(0, FCString::Atoi(*ContentLengthStr));
		return true;
	}
	return false;
}

TSharedPtr<FHttpServerRequest> FHttpConnectionRequestReadContext::BuildRequest(const FString& RequestHeader)
{
	if (IsEngineExitRequested())
	{
		return nullptr;
	}
	
	TArray<FString> ParsedHeader;
	RequestHeader.ParseIntoArrayLines(ParsedHeader);

	if (0 == ParsedHeader.Num())
	{
		AddError(UE_HTTP_SERVER_ERROR_STR_MISSING_REQUEST_HEADERS, EHttpServerResponseCodes::BadRequest);
		return nullptr;
	}

	// Split HTTP Method Line (Path/Verb/Version)
	TArray<FString> HttpMethodTokens;
	const FString& HttpMethod = ParsedHeader[0];
	HttpMethod.ParseIntoArrayWS(HttpMethodTokens);
	if (HttpMethodTokens.Num() < 3)
	{
		AddError(UE_HTTP_SERVER_ERROR_STR_MALFORMED_REQUEST_HEADER, EHttpServerResponseCodes::BadRequest);
		return nullptr;
	}

	Request = MakeShared<FHttpServerRequest>();

	if (Socket)
	{
		if (ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
		{
			TSharedRef<FInternetAddr> RemoteAddress = SocketSubsystem->CreateInternetAddr();
			if (Socket->GetPeerAddress(*RemoteAddress))
			{
				Request->PeerAddress = MoveTemp(RemoteAddress);
			}
		}
	}

	auto RequestVerb = HttpMethodTokens[0];
	if (0 == RequestVerb.Compare(TEXT("GET"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_GET;
	}
	else if (0 == RequestVerb.Compare(TEXT("POST"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_POST;
	}
	else if (0 == RequestVerb.Compare(TEXT("PUT"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_PUT;
	}
	else if (0 == RequestVerb.Compare(TEXT("DELETE"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_DELETE;
	}
	else if (0 == RequestVerb.Compare(TEXT("PATCH"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_PATCH;
	}
	else if (0 == RequestVerb.Compare(TEXT("OPTIONS"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_OPTIONS;
	}
	else if (0 == RequestVerb.Compare(TEXT("HEAD"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_HEAD;
	}
	else if (0 == RequestVerb.Compare(TEXT("TRACE"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_TRACE;
	}
	else if (0 == RequestVerb.Compare(TEXT("CONNECT"),
		ESearchCase::IgnoreCase))
	{
		Request->Verb = EHttpServerRequestVerbs::VERB_CONNECT;
	}
	else
	{
		// Unknown Verb
		AddError(UE_HTTP_SERVER_ERROR_STR_UNKNOWN_REQUEST_VERB, EHttpServerResponseCodes::BadMethod);
		return nullptr;
	}

	// Parse/store path+query params
	auto RequestHttpPath = HttpMethodTokens[1].TrimStartAndEnd();
	int32 QueryParamsIndex = 0;
	if (RequestHttpPath.FindChar(TCHAR('?'), QueryParamsIndex))
	{
		FString QueryParamsStr = RequestHttpPath.Mid(QueryParamsIndex+1);
		RequestHttpPath.MidInline(0, QueryParamsIndex, EAllowShrinking::No);

		// Split query params
		TArray<FString> QueryParamPairs;
		QueryParamsStr.ParseIntoArray(QueryParamPairs, TEXT("&"), true);
		for (const FString& QueryParamPair : QueryParamPairs)
		{
			int32 Equalsindex = 0;
			if (QueryParamPair.FindChar(TCHAR('='), Equalsindex))
			{
				FString QueryParamKey = FGenericPlatformHttp::UrlDecode(QueryParamPair.Mid(0, Equalsindex));
				FString QueryParamValue = FGenericPlatformHttp::UrlDecode(QueryParamPair.Mid(Equalsindex + 1));
				Request->QueryParams.Emplace(MoveTemp(QueryParamKey), MoveTemp(QueryParamValue));
			}
		}
	}
	Request->RelativePath.SetPath(RequestHttpPath);
	Request->FullPath.SetPath(RequestHttpPath);

	// Parse/store http version
	const FString& RequestHttpVersion = HttpMethodTokens[2];
	if(!HttpVersion::FromString(RequestHttpVersion, Request->HttpVersion))
	{
		AddError(UE_HTTP_SERVER_ERROR_STR_UNSUPPORTED_HTTP_VERSION, EHttpServerResponseCodes::VersionNotSup);
		return nullptr;
	}

	// Parse/store headers
	for (int i = 1; i < ParsedHeader.Num(); ++i)
	{
		const FString& HeaderLine = ParsedHeader[i];
		int32 SplitIndex = 0;
		if (HeaderLine.FindChar(TCHAR(':'), SplitIndex))
		{
			const auto& HeaderKey = HeaderLine.Mid(0, SplitIndex).TrimStartAndEnd().ToLower();
			const auto& HeaderValuesStr = HeaderLine.Mid(SplitIndex + 1).TrimStartAndEnd();

			TArray<FString> HeaderValues;
			HeaderValuesStr.ParseIntoArray(HeaderValues, TEXT(","), true);

			if (HeaderValues.Num() > 0)
			{
				TArray<FString>* ExistingHeaderValues = Request->Headers.Find(HeaderKey);
				if (ExistingHeaderValues)
				{
					ExistingHeaderValues->Append(HeaderValues);
				}
				else
				{
					Request->Headers.Emplace(HeaderKey, MoveTemp(HeaderValues));
				}
			}
		}
	}

	return Request;
}