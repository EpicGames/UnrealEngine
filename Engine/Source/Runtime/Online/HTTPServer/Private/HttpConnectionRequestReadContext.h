// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "HttpConnectionContext.h"

struct FHttpServerRequest;
struct FHttpConnection;

class FSocket;

DECLARE_LOG_CATEGORY_EXTERN(LogHttpConnectionRequestReadContext, Log, All);

struct FHttpConnectionRequestReadContext final : public FHttpConnectionContext
{
public:

	/**
	 * Constructor
	 *
	 * @param Socket The underlying connection
	 */
	FHttpConnectionRequestReadContext(FSocket* InSocket);

	/**
	 * Reads a request from the connection

	 * @param DeltaTime The elapsed time since the last invocation
	 * @return The state of the read context
	 */
	EHttpConnectionContextState ReadStream(float DeltaTime);

	/**
	 * Gets the internally stored request object
	 */
	FORCEINLINE TSharedPtr<FHttpServerRequest> GetRequest() const
	{
		return Request;
	}

	/**
	 * Resets the internal state.
	 * Should be called for every read.
	 */
	void ResetContext();

	/**
	 * Get the time in seconds spent waiting for the socket to be readable.
	 */
	float GetSecondsWaitingForReadableSocket() const
	{
		return SecondsWaitingForReadableSocket;
	}

	/**
	 * Add time (in seconds) spent waiting for the socket to be readable.
	 */
	void AddSecondsWaitingForReadableSocket(float Seconds)
	{
		SecondsWaitingForReadableSocket += Seconds;
	}

	/**
	 * Reset the time spent waiting for the socket to be readable.
	 */
	void ResetSecondsWaitingForReadableSocket()
	{
		SecondsWaitingForReadableSocket = 0.f;
	}

private:

	/**
	 * Parses the request header
	 *
	 * @param  ByteBuffer The bytes to parse
	 * @param  BufferLen The length of the supplied byte buffer
	 * @return true if no error was encountered, false otherwise 
	 */
	bool ParseHeader(uint8* ByteBuffer, int32 BufferLen);

	/**
	 * Parses the request body
	 *
	 * @param  ByteBuffer The bytes to parse
	 * @param  BufferLen The length of the supplied byte buffer
	 * @return true if no error was encountered, false otherwise
	 */
	bool ParseBody(uint8* ByteBuffer, int32 BufferLen);

	/**
	 * Determines whether the member Request object is valid
	 * @param InRequest The request to validate
	 * eg: POST requests with ContentLength > 0 are invalid
	 * eg: Body-capable requests with mismatched content-length headers and bodies are invalid
	 *
	 * @return true if the current Request object is valid, false otherwise
	 */
	bool IsRequestValid(const FHttpServerRequest& InRequest);

	/**
	* Builds an FHttpServerRequest from the caller-supplied http header string
	*
	* @param RequestHeader The complete http request header string
	* @return An instantiated request object with valid input, nullptr otherwise
	*/
	TSharedPtr<FHttpServerRequest> BuildRequest(const FString& RequestHeader);

	/**
	 * Gets the expected Content-Length from the member Request object

	 * @param InRequest The request to validate
	 * @param OutContentLength Set to the respective content length when returns true
	 * @return true if content length is specified, false otherwise
	 */
	static bool ParseContentLength(const FHttpServerRequest& InRequest, int32& OutContentLength);

private:

	/** The underlying connection */
	FSocket* Socket;
	/** Internal request state. */
	TSharedPtr<FHttpServerRequest> Request;
	/** Whether the header has been fully parsed */
	bool bParseHeaderComplete = false;
	/** Whether the body has been fully parsed */
	bool bParseBodyComplete = false;
	/** State variable which stores incoming header bytes */
	TArray<uint8> HeaderBytes;
	/** State variable which tracks bytes read for an incoming request */
	int32 IncomingRequestBodyBytesToRead = 0;
	/** The time spent waiting for the socket to be readable. */
	float SecondsWaitingForReadableSocket = 0.f;
};