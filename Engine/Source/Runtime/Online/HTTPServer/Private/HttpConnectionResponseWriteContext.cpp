// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpConnectionResponseWriteContext.h"
#include "HttpServerResponse.h"
#include "HttpServerHttpVersion.h"
#include "HttpServerConstantsPrivate.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

DEFINE_LOG_CATEGORY(LogHttpConnectionResponseWriteContext);

FHttpConnectionResponseWriteContext::FHttpConnectionResponseWriteContext(FSocket* InSocket)
	: Socket(InSocket)
{
}

void FHttpConnectionResponseWriteContext::ResetContext(TUniquePtr<FHttpServerResponse>&& Resp)
{
	Response = MoveTemp(Resp);
	ElapsedIdleTime = 0.0f;
	ErrorBuilder.Empty();

	HeaderBytesWritten = 0;
	BodyBytesWritten = 0;
	HeaderBytes.Empty();
	bStreamingHeadersPending = false;

	if (Response)
	{
		const bool bSkipHeaders = EnumHasAnyFlags(Response->Flags, EHttpServerResponseFlags::SkipHeaderWrite);
		const bool bMultipleWriteStream = EnumHasAnyFlags(Response->Flags, EHttpServerResponseFlags::MultipleWriteStream);
		const bool bHasStreamingQueue = Response->StreamingBodyQueue.IsValid();

		if (bSkipHeaders)
		{
			// Mid-stream or continuation write: body only. HeaderBytes stays empty so
			// IsWriteHeaderComplete() returns true immediately and WriteStream proceeds to the body.
		}
		else if (bHasStreamingQueue)
		{
			// Queue-based streaming: defer header serialization until the first chunk arrives
			// (or the producer signals completion with an empty queue) to avoid premature close.
			bStreamingHeadersPending = true;
		}
		else if (bMultipleWriteStream)
		{
			// Callback re-invocation streaming: serialize headers now, omit Content-Length since
			// the total stream size is unknown. The connection stays open across writes via HAW.
			HeaderBytes.Append(SerializeHeadersUtf8(Response->HttpVersion, Response->Code, Response->Headers));
		}
		else
		{
			// Non-streaming response: include Content-Length and serialize headers now.
			TArray<FString> ContentLengthValue = { FString::FromInt(Response->Body.Num()) };
			Response->Headers.Add(UE_HTTP_SERVER_HEADER_KEYS_CONTENT_LENGTH, MoveTemp(ContentLengthValue));
			HeaderBytes.Append(SerializeHeadersUtf8(Response->HttpVersion, Response->Code, Response->Headers));
		}
	}
}

EHttpConnectionContextState FHttpConnectionResponseWriteContext::WriteStream(float DeltaTime)
{
	ElapsedIdleTime += DeltaTime;

	int32 BytesWritten = 0;

	// -----------------------------------------------------------------------
	// Streaming body: defer headers until first chunk, then send deferred headers
	// -----------------------------------------------------------------------
	if (bStreamingHeadersPending && Response->StreamingBodyQueue.IsValid())
	{
		TArray<uint8> Chunk;
		if (Response->StreamingBodyQueue->Dequeue(Chunk))
		{
			// First chunk arrived - serialize and send headers now
			HeaderBytes.Append(SerializeHeadersUtf8(Response->HttpVersion, Response->Code, Response->Headers));
			bStreamingHeadersPending = false;
			Response->Body.Append(Chunk);
		}
	}
	else if (IsWriteHeaderComplete() && Response->StreamingBodyQueue.IsValid())
	{
		TArray<uint8> Chunk;
		while (Response->StreamingBodyQueue->Dequeue(Chunk))
		{
			Response->Body.Append(Chunk);
		}
	}

	// -----------------------------------------------------------------------
	// Write headers
	// -----------------------------------------------------------------------
	if (!IsWriteHeaderComplete())
	{
		const uint8* DataOffset = HeaderBytes.GetData() + HeaderBytesWritten;
		int32 DataLen = HeaderBytes.Num() - HeaderBytesWritten;
		if (!WriteBytes(DataOffset, DataLen, BytesWritten))
		{
			AddError(UE_HTTP_SERVER_ERROR_STR_SOCKET_SEND_FAILURE);
			return EHttpConnectionContextState::Error;
		}
		HeaderBytesWritten += BytesWritten;
	}

	// -----------------------------------------------------------------------
	// Handle idle timeout for streaming responses
	// -----------------------------------------------------------------------
	if (!bStreamingHeadersPending && IsWriteHeaderComplete() && Response->StreamingBodyQueue.IsValid())
	{
		// While waiting for the next chunk the queue will be empty and WriteBytes won't fire,
		// causing ElapsedIdleTime to accumulate and prematurely trigger the write-idle timeout.
		// This is not truly idle — we are mid-stream waiting for the producer — so keep the
		// timer from expiring by resetting it each tick as long as streaming is still in progress.
		if (!Response->StreamingBodyComplete.IsValid() || !Response->StreamingBodyComplete->Load(EMemoryOrder::Relaxed))
		{
			ElapsedIdleTime = 0.0f;
		}
	}

	if (IsWriteHeaderComplete() && !IsWriteBodyComplete())
	{
		const uint8* DataOffset = Response->Body.GetData() + BodyBytesWritten;
		int32 DataLen = Response->Body.Num() - BodyBytesWritten;
		if (!WriteBytes(DataOffset, DataLen, BytesWritten))
		{
			AddError(UE_HTTP_SERVER_ERROR_STR_SOCKET_SEND_FAILURE);
			return EHttpConnectionContextState::Error;
		}
		BodyBytesWritten += BytesWritten;
	}

	if (IsWriteHeaderComplete() && IsWriteBodyComplete())
	{
		// If streaming is in progress, stay in Continue until all chunks are enqueued and written
		if (Response->StreamingBodyQueue.IsValid() &&
			(!Response->StreamingBodyComplete.IsValid() || !Response->StreamingBodyComplete->Load(EMemoryOrder::Relaxed)))
		{
			return EHttpConnectionContextState::Continue;
		}
		// Complete flag is set — do one final drain to catch any chunks enqueued just before the flag was stored
		if (Response->StreamingBodyQueue.IsValid())
		{
			TArray<uint8> Chunk;
			while (Response->StreamingBodyQueue->Dequeue(Chunk))
			{
				Response->Body.Append(Chunk);
			}
			if (!IsWriteBodyComplete())
			{
				return EHttpConnectionContextState::Continue;
			}
		}
		ensure(BodyBytesWritten+HeaderBytesWritten == HeaderBytes.Num()+Response->Body.Num());
		return EHttpConnectionContextState::Done;
	}

	return EHttpConnectionContextState::Continue;
}

bool FHttpConnectionResponseWriteContext::WriteBytes(const uint8* Bytes, int32 BytesLen, int32 &OutBytesWritten)
{
	OutBytesWritten = 0;
	bool bWriteSuccess = Socket->Send(Bytes, BytesLen, OutBytesWritten);
	if (!bWriteSuccess)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		const ESocketErrors Err = SocketSubsystem->GetLastErrorCode();
		if (Err == SE_EWOULDBLOCK || Err == SE_TRY_AGAIN)
		{
			bWriteSuccess = true;
			OutBytesWritten = 0;
		}
		else
		{
			UE_LOGF(LogHttpConnectionResponseWriteContext, Warning, "WriteBytes sent %d/%d bytes", OutBytesWritten, BytesLen);
		}
	}

	if (OutBytesWritten > 0)
	{
		UE_LOGF(LogHttpConnectionResponseWriteContext, Verbose,
			"ElapsedIdleTime\t %f", ElapsedIdleTime);
		ElapsedIdleTime = 0.0f;
	}

	return bWriteSuccess;
}

bool FHttpConnectionResponseWriteContext::IsWriteHeaderComplete() const
{
	if (bStreamingHeadersPending)
	{
		return false;
	}
	return HeaderBytesWritten >= HeaderBytes.Num();
}

bool FHttpConnectionResponseWriteContext::IsWriteBodyComplete() const
{
	check(Response);
	return BodyBytesWritten >= Response->Body.Num();
}

TArray<uint8> FHttpConnectionResponseWriteContext::SerializeHeadersUtf8(HttpVersion::EHttpServerHttpVersion HttpVersion, EHttpServerResponseCodes ResponseCode, const TMap<FString, TArray<FString>>& HeadersMap)
{
	FString ResponseHeaderStr = FString::Printf(TEXT("%s %d\r\n"),  *HttpVersion::ToString(HttpVersion), static_cast<int16>(ResponseCode));

	for (const auto& KeyValuePair : HeadersMap)
	{
		const FString& HeaderKey = KeyValuePair.Key;
		const TArray<FString>& HeaderValues = KeyValuePair.Value;
		for (const auto& HeaderValue : HeaderValues)
		{
			const FString HeaderItem = FString::Printf(TEXT("%s: %s\r\n"), *HeaderKey, *HeaderValue);
			ResponseHeaderStr.Append(HeaderItem);
		}
	}
	ResponseHeaderStr.Append(TEXT("\r\n"));

	TArray<uint8> HeaderRawBytes;
	FTCHARToUTF8 HeaderUtf8(*ResponseHeaderStr);
	const uint8* HeaderUtf8Bytes = reinterpret_cast<const uint8*>(HeaderUtf8.Get());
	HeaderRawBytes.Append(HeaderUtf8Bytes, HeaderUtf8.Length());
	return HeaderRawBytes;
}
