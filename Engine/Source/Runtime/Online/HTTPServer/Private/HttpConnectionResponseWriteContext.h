// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "HttpConnectionContext.h"
#include "HttpServerHttpVersion.h"
#include "HttpServerResponse.h"


class FSocket;

DECLARE_LOG_CATEGORY_EXTERN(LogHttpConnectionResponseWriteContext, Log, All);

struct FHttpConnectionResponseWriteContext final : public FHttpConnectionContext
{
public:

	/**
	 * Constructor
	 *
	 * @param InSocket The underlying connection
	 */
	FHttpConnectionResponseWriteContext(FSocket* InSocket);

	/**
     * Writes to the connection
	 *
	 * @param DeltaTime The elapsed time since the last invocation
	 * @return The state of the write context
     */
	EHttpConnectionContextState WriteStream(float DeltaTime);

	/**
	 * Resets the internal state.  
	 * Should be called before every write.
	 *
	 * @param Response The response to be written
	 */
	void ResetContext(TUniquePtr<FHttpServerResponse>&&  Response);

	/**
	 * Determine if the connection should expect multiple writes as part of this response before the next 
	 * read/close by re-using an existing FHttpResultCallback.  Streams should issue the first response callback 
	 * with an empty body to write headers.
	 *
	 * @return true if there are multiple writes expected
	 */
	bool IsMultipleWriteStream() const { return Response.IsValid() && EnumHasAnyFlags(Response->Flags, EHttpServerResponseFlags::MultipleWriteStream); }

	/**
	 * Determine if the context's current state is expecting additional body writes
	 *
	 * @return true if there are more writes
	 */
	bool HasAdditionalWrites() const { return Response.IsValid() && EnumHasAllFlags(Response->Flags, EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites); }

	/**
	 * Determine if the connection should be destroyed once this response finishes writing.
	 *
	 * @return true if the current response carries the `CloseAfterWrite` flag
	 */
	bool ShouldCloseAfterWrite() const { return Response.IsValid() && EnumHasAnyFlags(Response->Flags, EHttpServerResponseFlags::CloseAfterWrite); }

private:

	/**
	 * Writes the caller-supplied Bytes up to BytesLen as permitted by the socket
	 *
     * @param  Bytes The bytes to write
	 * @param  The desired number of bytes to write
	 * @param  OutBytesWritten The number of bytes actually written
	 * @return true if the write was successful, false otherwise
	 */
	bool WriteBytes(const uint8* Bytes, int32 BytesLen, int32 &OutBytesWritten);


	/**
	 * Determines whether the header has been completely written
	 * @return true if the header has been completely written, false otherwise
	 */
	bool IsWriteHeaderComplete() const;

	/**
	 * Determines whether the body has been completely written
	 * @return true if the body has been completely written, false otherwise
	 */
	bool IsWriteBodyComplete() const;

	/**
	 * Creates a UTF8-serialized HTTP header in byte representation
	 *
	 * @param  HttpVersion  The HTTP version
	 * @param  ResponseCode The HTTP response code
	 * @param  HeadersMap   The associative header key to values map
	 * @return              The UTF8-encoded byte representation
	 */
	static TArray<uint8> SerializeHeadersUtf8(HttpVersion::EHttpServerHttpVersion HttpVersion, EHttpServerResponseCodes ResponseCode, const TMap<FString, TArray<FString>>& HeadersMap);

private:

	/** The underlying connection */
	FSocket* Socket;
	/** The response to write */
	TUniquePtr<FHttpServerResponse> Response;
	/** State  */
	TArray<uint8> HeaderBytes;
	/** The total number of header bytes written */
	int32 HeaderBytesWritten = 0;
	/** The total number of body bytes written */
	int32 BodyBytesWritten = 0;
	/** For streaming responses, defer sending headers until first chunk arrives */
	bool bStreamingHeadersPending = false;
};