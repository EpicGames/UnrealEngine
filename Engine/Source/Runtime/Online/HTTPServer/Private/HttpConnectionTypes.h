// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Containers/Set.h"

struct FHttpConnection;

/**
 * Utility unique identifier for FHttpConnections
 */
typedef FGuid HttpConnectionIdType;

/**
 * FHttpConnectionPool
 */
typedef TSet<TSharedPtr<FHttpConnection>> FHttpConnectionPool;

/**
 * FHttpConnection state model
 */
enum class EHttpConnectionState
{
	// Awaiting available data
	AwaitingRead,
	// Reading available data
	Reading,
	// Awaiting request handler completion
	AwaitingProcessing,
	// Writing response to client
	Writing,
	// Awaiting destruction via listener
	Destroyed,
};

