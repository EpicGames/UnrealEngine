// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpListener.h"
#include "HttpConnection.h"
#include "HttpRequestHandler.h"
#include "HttpRouter.h"
#include "HttpServerConfig.h"

#include "Sockets.h"
#include "Stats/Stats.h"
#include "IPAddress.h"

DEFINE_LOG_CATEGORY(LogHttpListener)

FHttpListener::FHttpListener(uint32 InListenPort)
{ 
	check(InListenPort > 0);
	ListenPort = InListenPort;
	Router = MakeShared<FHttpRouter>();
}

FHttpListener::~FHttpListener() 
{ 
	check(!ListenSocket);
	check(!bIsListening);

	const bool bRequestGracefulExit = false;
	for (const auto& Connection : Connections)
	{
		Connection->RequestDestroy(bRequestGracefulExit);
	}
	Connections.Empty();
}

// --------------------------------------------------------------------------------------------
// Public Interface
// --------------------------------------------------------------------------------------------
bool FHttpListener::StartListening()
{
	check(!ListenSocket);
	check(!bIsListening);

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	if (nullptr == SocketSubsystem)
	{
		UE_LOGF(LogHttpListener, Error, 
			"HttpListener - SocketSubsystem Initialization Failed");
		return false;
	}

	FUniqueSocket NewSocket = SocketSubsystem->CreateUniqueSocket(NAME_Stream, TEXT("HttpListenerSocket"));
	if (!NewSocket)
	{
		UE_LOGF(LogHttpListener, Error, 
			"HttpListener - Unable to allocate stream socket");
		return false;
	}

	NewSocket->SetNonBlocking(true);

	// Bind to config-driven address
	TSharedRef<FInternetAddr> BindAddress = SocketSubsystem->CreateInternetAddr();
	Config = FHttpServerConfig::GetListenerConfig(ListenPort);
	if (0 == Config.BindAddress.Compare(TEXT("any"), ESearchCase::IgnoreCase))
	{
		BindAddress->SetAnyAddress();
	}
	else if (0 == Config.BindAddress.Compare(TEXT("localhost"), ESearchCase::IgnoreCase))
	{
		BindAddress->SetLoopbackAddress();
	}
	else
	{
		bool bIsValidAddress = false;
		BindAddress->SetIp(*(Config.BindAddress), bIsValidAddress);
		if (!bIsValidAddress)
		{
			UE_LOGF(LogHttpListener, Error,
				"HttpListener detected invalid bind address (%ls:%u)",
				*Config.BindAddress, ListenPort);
			return false;
		}
	}

	if (!BindAddress->IsPortValid(ListenPort))
	{
		UE_LOGF(LogHttpListener, Error,
			"HttpListener detected invalid port %u",
			ListenPort);
		return false;
	}
	BindAddress->SetPort(ListenPort);

	if (Config.bReuseAddressAndPort)
	{
		NewSocket->SetReuseAddr(true);
	}

	if (!NewSocket->Bind(*BindAddress))
	{
		UE_LOGF(LogHttpListener, Error, 
		"HttpListener unable to bind to %ls",
			*BindAddress->ToString(true));
		return false;
	}

	int32 ActualBufferSize;
	NewSocket->SetSendBufferSize(Config.BufferSize, ActualBufferSize);
	if (ActualBufferSize < Config.BufferSize)
	{
		UE_LOGF(LogHttpListener, Log, 
			"HttpListener unable to set desired buffer size (%d): Limited to %d",
			Config.BufferSize, ActualBufferSize);
	}

	if (!NewSocket->Listen(Config.ConnectionsBacklogSize))
	{
		UE_LOGF(LogHttpListener, Error, 
			"HttpListener unable to listen on socket");
		return false;
	}

	bIsListening = true;
	ListenSocket = MoveTemp(NewSocket);
	UE_LOGF(LogHttpListener, Log, 
		"Created new HttpListener on %ls", 
		*BindAddress->ToString(true));
	return true;
}

void FHttpListener::StopListening()
{
	check(bIsListening);

	// Tear down our top-level listener first
	if (ListenSocket)
	{
		UE_LOGF(LogHttpListener, Log,
			"HttpListener stopping listening on Port %u", ListenPort);

		ListenSocket.Reset();
	}
	bIsListening = false;

	const bool bRequestGracefulExit = true;
	for (const auto& Connection : Connections)
	{
		Connection->RequestDestroy(bRequestGracefulExit);
	}
}

void FHttpListener::Tick(float DeltaTime)
{
	if (bIsListening)
	{
		// Accept new connections
		AcceptConnections();

		// Tick Connections
		TickConnections(DeltaTime);

		// Remove any destroyed connections
		RemoveDestroyedConnections();
	}
}

bool FHttpListener::HasPendingConnections() const 
{
	for (const auto& Connection : Connections)
	{
		switch (Connection->GetState())
		{
		case EHttpConnectionState::Reading:
		case EHttpConnectionState::AwaitingProcessing:
		case EHttpConnectionState::Writing:
			return true;
		}
	}
	return false;
}

// --------------------------------------------------------------------------------------------
// Private Implementation
// --------------------------------------------------------------------------------------------
void FHttpListener::AcceptConnections()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpListener_AcceptConnections);
	check(ListenSocket);

	for (int32 i = 0; i < Config.MaxConnectionsAcceptPerFrame; ++i)
	{
		// Check pending prior to Accept()ing
		bool bHasPendingConnection = false;
		if (!ListenSocket->HasPendingConnection(bHasPendingConnection))
		{
			UE_LOGF(LogHttpListener, 
				Error, "ListenSocket failed to query pending connection");
			return;
		}

		if (bHasPendingConnection)
		{
			FSocket* IncomingConnection = ListenSocket->Accept(TEXT("HttpRequest"));

			if (nullptr == IncomingConnection)
			{
				ESocketErrors ErrorCode = ESocketErrors::SE_NO_ERROR;
				FString ErrorStr = "SocketSubsystem Unavialble";

				ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
				if (SocketSubsystem)
				{
					ErrorCode = SocketSubsystem->GetLastErrorCode();
					ErrorStr = SocketSubsystem->GetSocketError();
				}
				UE_LOGF(LogHttpListener, Error,
					"Error accepting expected connection [%d] %ls", (int32)ErrorCode, *ErrorStr);
				return;
			}

			IncomingConnection->SetNonBlocking(true);
			TSharedPtr<FHttpConnection> Connection = 
				MakeShared<FHttpConnection>(IncomingConnection, Router, ListenPort, NumConnectionsAccepted++);
			Connections.Add(Connection);
		}
	}
}

void FHttpListener::TickConnections(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpListener_TickConnections);
	for (const auto& Connection : Connections)
	{
		check(Connection.IsValid());

		switch (Connection->GetState())
		{
		case EHttpConnectionState::AwaitingRead:
		case EHttpConnectionState::Reading:
			Connection->Tick(DeltaTime);
			break;
		}
	}

	for (const auto& Connection : Connections)
	{
		check(Connection.IsValid());

		switch (Connection->GetState())
		{
		case EHttpConnectionState::Writing:
			Connection->Tick(DeltaTime);
			break;
		}
	}
}

void FHttpListener::RemoveDestroyedConnections()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpListener_RemoveDestroyedConnections);
	for (auto ConnectionsIter = Connections.CreateIterator(); ConnectionsIter; ++ConnectionsIter)
	{
		// Remove any destroyed connections
		if (EHttpConnectionState::Destroyed == ConnectionsIter->Get()->GetState())
		{
			ConnectionsIter.RemoveCurrent();
			continue;
		}
	}
}
