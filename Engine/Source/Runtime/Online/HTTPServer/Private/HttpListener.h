// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HttpConnectionTypes.h"
#include "HttpRouter.h"
#include "HttpServerConfig.h"
#include "SocketSubsystem.h"

struct FHttpConnection;
struct FHttpPath;
class FSocket;

DECLARE_LOG_CATEGORY_EXTERN(LogHttpListener, Log, All);

class FHttpListener final 
{

public:

	/**
	 * Constructor
	 *
	 *  @param InListenPort The port on which to listen for incoming connections
	 */
	FHttpListener(uint32 InListenPort);

	/**
	 * Destructor
	 */
	~FHttpListener();

	/**
	 * Starts listening for and accepting incoming connections
	 * 
	 * @return true if the listener was able to start listening, false otherwise
	 */
	bool StartListening();

	/**
	 * Stops listening for and accepting incoming connections
	 */
	void StopListening();

	/**
	 * Tick the listener to otherwise commence connection flows
	 *
	 * @param DeltaTime The elapsed time since the last tick
	 */
	void Tick(float DeltaTime);

	/**
	 * Determines whether this listener has pending connections in-flight
	 *
	 * @return true if there are pending connections, false otherwise
	 */
	bool HasPendingConnections() const;

	/**
	 * Determines whether the listener has been initialized
	 * @return true if this listener is bound and listening, false otherwise
	 */
	FORCEINLINE bool IsListening() const
	{
		return bIsListening;
	}

	/**
	 * Gets the respective router
	 * @return The respective router
	 */
	FORCEINLINE TSharedPtr<IHttpRouter> GetRouter() const
	{
		return Router;
	}

private:

	/**
	 * Accepts available connection(s)
	 */
	void AcceptConnections();

	/**
	 * Ticks connections in reading/writing phases
	 * 
	 * @param DeltaTime The elapsed time since the last tick
	 */
	void TickConnections(float DeltaTime);

	/**
	 * Removes connections that have been destroyed
	 */
	void RemoveDestroyedConnections();

private:
	
	/** Whether this listeners has begun listening */
	bool bIsListening = false;

	/** The port on which the binding socket listens */
	uint32 ListenPort = 0;

	/** The binding socket which accepts incoming connections */
	FUniqueSocket ListenSocket;

	/** The mechanism that routes requests to respective handlers  */
	TSharedPtr<FHttpRouter> Router = nullptr;

	/** The collection of unique connections */
	FHttpConnectionPool Connections;

	/** The total number of connections accepted by this listener */
	uint32 NumConnectionsAccepted = 0;

	/** Listener configuration data */
	FHttpServerListenerConfig Config;
};