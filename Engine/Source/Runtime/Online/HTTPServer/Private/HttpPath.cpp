// Copyright Epic Games, Inc. All Rights Reserved.
#include "HttpPath.h"

FHttpPath::FHttpPath()
{
}

FHttpPath::FHttpPath(FString InPath)
	:Path(MoveTemp(InPath))
{
	NormalizePath();
}

const FString& FHttpPath::GetPath() const
{
	return Path;
}

uint32 FHttpPath::ParsePathTokens(TArray<FString>& OutPathTokens) const
{
	return Path.ParseIntoArray(OutPathTokens, TEXT("/"), true);
}

void FHttpPath::SetPath(FString NewPath)
{
	Path = NewPath;
	NormalizePath();
}

bool FHttpPath::IsValidPath() const
{
	if (IsRoot())
	{
		return false;
	}
	if (!Path.StartsWith(TEXT("/")))
	{
		return false;
	}

	auto IsInvalidUriChar = [](TCHAR C) 
	{ 
		return 
			(C <= 32) || (C >= 127) ||
			(C == TEXT(' ')) || (C == TEXT(',')) || (C == TEXT('\\')) ||
			(C == TEXT('<')) || (C == TEXT('>')) || (C == TEXT(']')) ||
			(C == TEXT('[')) || (C == TEXT('}')) || (C == TEXT('{')) ||
			(C == TEXT('#')) || (C == TEXT('|')) || (C == TEXT('^'));
	}; 

	return (INDEX_NONE == Path.FindLastCharByPredicate(IsInvalidUriChar));
}

bool FHttpPath::IsRoot() const
{
	return Path == TEXT("/");
}

void FHttpPath::MakeRelative(const FString& OtherPath)
{
	if (OtherPath == TEXT("/"))
	{
		return;
	}

	if (Path == OtherPath)
	{
		Path.RemoveAt(1, Path.Len() - 1, EAllowShrinking::No);
	}

	if (Path.StartsWith(OtherPath))
	{
		Path.RemoveAt(0, OtherPath.Len(), EAllowShrinking::No);
	}
}

void FHttpPath::NormalizePath()
{
	if (!IsRoot() &&  Path.EndsWith(TEXT("/")))
	{
		Path.RemoveFromEnd(TEXT("/"));
	}
}

