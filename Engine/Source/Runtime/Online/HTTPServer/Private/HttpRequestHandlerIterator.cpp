// Copyright Epic Games, Inc. All Rights Reserved.
#include "HttpRequestHandlerIterator.h"
#include "HttpServerRequest.h"
#include "HttpRouteHandle.h"


FHttpRequestHandlerIterator::FHttpRequestHandlerIterator(
	TSharedPtr<FHttpServerRequest> InRequest, 
	const FHttpRequestHandlerRegistrar& InRequestHandlerRegistrar,
	TArray<FHttpRequestHandler> InRequestPreprocessors)
	: HttpPathIterator(InRequest->RelativePath)
	, Request(MoveTemp(InRequest))
	, RequestHandlerRegistrar(InRequestHandlerRegistrar)
	, RequestPreprocessors(MoveTemp(InRequestPreprocessors)) 
{
}

const FHttpRequestHandler* const FHttpRequestHandlerIterator::Next()
{
	if (RequestPreprocessors.IsValidIndex(CurrentPreprocessorIndex))
	{
		return &RequestPreprocessors[CurrentPreprocessorIndex++];
	}
	
	while (HttpPathIterator.HasNext())
	{
		// Determine if we have a matching handler for the next route
  		const auto& NextRoute = HttpPathIterator.Next();

		// Filter by http route
		FRouteQueryResult QueryResult = RequestHandlerRegistrar.QueryRoute(NextRoute, Request->Verb, &HttpPathIterator.ParsedTokens);
		if (!QueryResult)
		{
			// Not a matching route
			continue;
		}
		const FHttpRouteHandle& RouteHandle = QueryResult.RouteHandle;

		// Make request path relative to the respective handler
		Request->RelativePath.MakeRelative(NextRoute);
		Request->PathParams = MoveTemp(QueryResult.PathParams);

		return &(RouteHandle->Handler);
	}
	return nullptr;
}

FHttpRequestHandlerIterator::FHttpPathIterator::FHttpPathIterator(const FHttpPath& HttpPath)
{
	NextPath = HttpPath.GetPath();
}

bool FHttpRequestHandlerIterator::FHttpPathIterator::HasNext() const
{
	return !bLastIteration;
}

const FString& FHttpRequestHandlerIterator::FHttpPathIterator::Next()
{
	// Callers should always test HasNext() first!
	check(!bLastIteration); 

	if (!bFirstIteration)
	{
		int32 SlashIndex = 0;
		if (NextPath.FindLastChar(TCHAR('/'), SlashIndex))
		{
			ParsedTokens.Insert(NextPath.RightChop(SlashIndex + 1), 0);
			NextPath.RemoveAt(SlashIndex, NextPath.Len() - SlashIndex, EAllowShrinking::No);

			if (0 == NextPath.Len())
			{
				NextPath.AppendChar(TCHAR('/'));
				bLastIteration = true;
			}
		}
		else
		{
			bLastIteration = true;
		}
	}
	bFirstIteration = false;
	return NextPath;
}

