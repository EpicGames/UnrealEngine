// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IHttpRouter.h"
#include "HttpRequestHandler.h"
#include "HttpRequestHandlerRegistrar.h"

struct FHttpPath;
struct FHttpServerRequest;

/**
 * Utility to iterate qualified FHttpRequestHandlers
 */
struct FHttpRequestHandlerIterator final
{

	/**
	 *  Utility (internal) to iterate FHttpPaths in-place
	 */
	struct FHttpPathIterator final
	{
	public:
		/**
		 * Constructor
		 * @param HttpPath The iterator starting path
		 */
		FHttpPathIterator(const FHttpPath& HttpPath);

		/**
		 * Gets the next route 
		 * @return The base path of the current path
		 */
		const FString& Next();

		/**
		 * Determines whether there is a Next() path to get
		 * @return true if there is a next path, false otherwise
		 */
		FORCEINLINE bool HasNext() const;

		/**
		 * Holds the tokens that were removed from the end of the path.
		 */
		TArray<FString> ParsedTokens;
	private:
		FString NextPath;
		bool bFirstIteration = true;
		bool bLastIteration = false;
	};


public:

	/**
	 * Constructor
	 */
	FHttpRequestHandlerIterator(TSharedPtr<FHttpServerRequest> InRequest, const FHttpRequestHandlerRegistrar& InRequestHandlerRegistrar, TArray<FHttpRequestHandler> InRequestPreprocessors = {});

	/** 
	* Determines the next registered request handler
	* @return The next registered request handler if found, nullptr otherwise
	*/
	const FHttpRequestHandler* const Next();

private:

	/** Utility to iterate FHttpRoutes in-place */
	FHttpPathIterator HttpPathIterator;

	/** The basis request */
	const TSharedPtr<FHttpServerRequest> Request;

	/** The associative route/handler registration  */
	const FHttpRequestHandlerRegistrar& RequestHandlerRegistrar;

	/** Handlers that should be queried before routing the http request */
	TArray<FHttpRequestHandler> RequestPreprocessors;

	/** The index to the next preprocessor to execute with the request. */
	int32 CurrentPreprocessorIndex = 0;
};
