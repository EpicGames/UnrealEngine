// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpRequestHandlerRegistrar.h"
#include "Misc/StringBuilder.h"
#include "Algo/Transform.h"
#include "GenericPlatform/GenericPlatformHttp.h"

namespace HttpPathUtils
{
	/**
	 * @return The invariant part of the parameterized path, or an empty string if the path is not parameterized.
	 */
	FString GetStaticPathSection(const FString& InPath)
	{
		FString Path;
		int32 Index = INDEX_NONE;
		if (InPath.FindChar(TEXT(':'), Index))
		{
			if (Index != 1)
			{
				Path = InPath.Left(Index - 1);
			}
			else
			{
				// Handle case where first token is parameterized.
				Path = TEXT("/");
			}
		}
		
		return Path;
	}

	/**
	 * @return The variable part of the parameterized path, or an empty string if the path is not parameterized.
	 */
	FString GetDynamicPathSection(const FString& InPath)
	{
		FString Path;
		int32 Index = INDEX_NONE;
		if (InPath.FindChar(TEXT(':'), Index))
		{
			Path = InPath.RightChop(Index);
		}

		return Path;
	}
}

void FHttpRequestHandlerRegistrar::AddRoute(const FHttpRouteHandle& Handle)
{
	FString StaticPathSection = HttpPathUtils::GetStaticPathSection(Handle->Path);

	if (!StaticPathSection.IsEmpty())
	{
		TArray<FDynamicRouteHandleWrapper>& Handles = DynamicRoutes.FindOrAdd(StaticPathSection);
		Handles.Emplace(Handle);
	}
	else
	{
		TArray<FHttpRouteHandle>& Handles = StaticRoutes.FindOrAdd(Handle->Path);
		Handles.Add(Handle);
	}
}

void FHttpRequestHandlerRegistrar::RemoveRoute(const FHttpRouteHandle& Handle)
{
	if (!RemoveStaticRoute(Handle->Path, Handle->Verbs))
	{
		RemoveDynamicRoute(Handle->Path, Handle->Verbs);
	}
}

bool FHttpRequestHandlerRegistrar::ContainsRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb)
{
	if (ContainsStaticRoute(RoutePath.GetPath(), Verb))
	{
		return true;
	}

	return ContainsDynamicRoute(RoutePath.GetPath(), Verb);
}

FRouteQueryResult FHttpRequestHandlerRegistrar::QueryRoute(const FString& Query, EHttpServerRequestVerbs Verb, const TArray<FString>* ParsedTokens) const
{
	if (ParsedTokens && ParsedTokens->Num())
	{
		if (const TArray<FDynamicRouteHandleWrapper>* DynamicPathHandles = DynamicRoutes.Find(Query))
		{
			for (const FDynamicRouteHandleWrapper& Wrapper : *DynamicPathHandles) 
			{
				const TArray<FString>& DynamicTemplateTokens = Wrapper.DynamicPathSectionTokens;
				if (DynamicTemplateTokens.Num() != ParsedTokens->Num())
				{
					continue;
				}

				// Parse the path parameters once a path is found.
				if (MatchesPath(*ParsedTokens, DynamicTemplateTokens) && (Verb & Wrapper.Handle->Verbs) != EHttpServerRequestVerbs::VERB_NONE)
				{
					return { Wrapper.Handle, ParsePathParameters(*ParsedTokens, DynamicTemplateTokens)};
				}
			}
		}
	}

	// Try finding in StaticRoutes directly.
	if (const TArray<FHttpRouteHandle>* Handles = StaticRoutes.Find(Query))
	{
		for (auto HandleIt = Handles->CreateConstIterator(); HandleIt; ++HandleIt)
		{
			if (((*HandleIt)->Verbs & Verb) != EHttpServerRequestVerbs::VERB_NONE)
			{
				return FRouteQueryResult{ *HandleIt };
			}
		}
	}

	return FRouteQueryResult();
}

TTuple<TArray<FHttpRouteHandle>*, int64> FHttpRequestHandlerRegistrar::FindStaticRoute(const FString& RoutePath, EHttpServerRequestVerbs Verb)
{
	TTuple<TArray<FHttpRouteHandle>*, int64> RouteLocation;
	RouteLocation.Key = nullptr;
	RouteLocation.Value = -1;

	if (TArray<FHttpRouteHandle>* Handles = StaticRoutes.Find(RoutePath))
	{
		RouteLocation.Key = Handles;
		for (auto HandleIt = Handles->CreateConstIterator(); HandleIt; ++HandleIt)
		{
			if ((*HandleIt)->Path == RoutePath && ((*HandleIt)->Verbs & Verb) != EHttpServerRequestVerbs::VERB_NONE)
			{
				RouteLocation.Value = HandleIt.GetIndex();
			}
		}
	}

	return RouteLocation;
}

TTuple<TArray<FHttpRequestHandlerRegistrar::FDynamicRouteHandleWrapper>*, int64> FHttpRequestHandlerRegistrar::FindDynamicRoute(const FString& RoutePath, EHttpServerRequestVerbs Verb)
{
	TTuple<TArray<FDynamicRouteHandleWrapper>*, int64> RouteLocation;
	RouteLocation.Key = nullptr;
	RouteLocation.Value = -1;

	if (TArray<FDynamicRouteHandleWrapper>* Wrappers = DynamicRoutes.Find(HttpPathUtils::GetStaticPathSection(RoutePath)))
	{
		RouteLocation.Key = Wrappers;

		FString DynamicPathSection = HttpPathUtils::GetDynamicPathSection(RoutePath);
		for (auto WrapperIt = Wrappers->CreateConstIterator(); WrapperIt; ++WrapperIt)
		{
			if (WrapperIt->Handle->Path == DynamicPathSection
				&& (WrapperIt->Handle->Verbs & Verb) != EHttpServerRequestVerbs::VERB_NONE)
			{
				RouteLocation.Value = WrapperIt.GetIndex();
			}
		}
	}

	return RouteLocation;
}

bool FHttpRequestHandlerRegistrar::ContainsStaticRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb)
{
	TTuple<TArray<FHttpRouteHandle>*, int64> RouteLocation = FindStaticRoute(RoutePath.GetPath(), Verb);
	return RouteLocation.Key && RouteLocation.Value >= 0;
}

bool FHttpRequestHandlerRegistrar::ContainsDynamicRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb)
{
	TTuple<TArray<FDynamicRouteHandleWrapper>*, int64> RouteLocation = FindDynamicRoute(RoutePath.GetPath(), Verb);
	return RouteLocation.Key && RouteLocation.Value >= 0;
}

bool FHttpRequestHandlerRegistrar::RemoveStaticRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb)
{
	TTuple<TArray<FHttpRouteHandle>*, int64> RouteLocation = FindStaticRoute(RoutePath.GetPath(), Verb);
	if (RouteLocation.Key && RouteLocation.Value >= 0)
	{
		RouteLocation.Key->RemoveAt(RouteLocation.Value);
		return true;
	}
	return false;
}

bool FHttpRequestHandlerRegistrar::RemoveDynamicRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb)
{
	TTuple<TArray<FDynamicRouteHandleWrapper>*, int64> RouteLocation = FindDynamicRoute(RoutePath.GetPath(), Verb);
	if (RouteLocation.Key && RouteLocation.Value >= 0)
	{
		return true;
	}
	return false;
}

bool FHttpRequestHandlerRegistrar::MatchesPath(const TArray<FString>& InPath, const TArray<FString>& TemplatePath) const
{
	auto PathIt = InPath.CreateConstIterator();
	auto TemplateIt = TemplatePath.CreateConstIterator();

	while (PathIt && TemplateIt)
	{
		const FString& TestPathToken = *(PathIt++);
		const FString& PathToMatchToken = *(TemplateIt++);

		if (PathToMatchToken[0] != TEXT(':') && TestPathToken != PathToMatchToken)
		{
			return false;
		}
	}

	// It's not a match if tokens remain in either path.
	if (PathIt || TemplateIt)
	{
		return false;
	}

	return true;
}

TMap<FString, FString> FHttpRequestHandlerRegistrar::ParsePathParameters(const TArray<FString>& InPath, const TArray<FString>& TemplatePath) const
{
	TMap<FString, FString> PathParameters;

	auto PathIt = InPath.CreateConstIterator();
	auto TemplateIt = TemplatePath.CreateConstIterator();

	while (PathIt && TemplateIt)
	{
		const FString& TestPathToken = *(PathIt++);
		const FString& PathToMatchToken = *(TemplateIt++);

		if (PathToMatchToken[0] == TEXT(':'))
		{
			PathParameters.Add(PathToMatchToken.RightChop(1), FGenericPlatformHttp::UrlDecode(TestPathToken));
		}
	}

	return PathParameters;
}

FHttpRequestHandlerRegistrar::FDynamicRouteHandleWrapper::FDynamicRouteHandleWrapper(FHttpRouteHandle InHandle)
	: Handle(MoveTemp(InHandle))
{
	FString DynamicPathSection = HttpPathUtils::GetDynamicPathSection(Handle->Path);
	DynamicPathSection.ParseIntoArray(DynamicPathSectionTokens, TEXT("/"));
}
