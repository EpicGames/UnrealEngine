// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HttpRouteHandle.h"
#include "Templates/UnrealTypeTraits.h"

enum class EHttpServerRequestVerbs : uint16;

/**
 * The result of a Registrar query.
 */
struct FRouteQueryResult
{
	operator bool()
	{
		return !!RouteHandle;
	}

	/**
	 * The query's matching route handle if found, nullptr otherwise.
	 */
	FHttpRouteHandle RouteHandle = nullptr;

	/**
	 * The parsed parameters, if the query was successful.
	 */
	TMap<FString, FString> PathParams;
};

/**
 * FHttpRequestHandlerRegistrar
 * Represents the associative relationship between Http request paths and respective route handles
 */
class FHttpRequestHandlerRegistrar
{
public:
	/**
	 * Add a route to the registrar.
	 * @param Handle The new route's handle.
	 */
	void AddRoute(const FHttpRouteHandle& Handle);

	/**
	 * Remove a route from the registrar.
	 * @param Handle The handle of the route to delete.
	 */
	void RemoveRoute(const FHttpRouteHandle& Handle);

	/**
	 * Check if the registrar contains a route.
	 * @param RoutePath the HttpPath of the target route.
	 * @param Verb the verb associated with the target route.
	 * @return Whether the route was found.
	 */
	bool ContainsRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb);

	/**
	 * Query the registrar for a route, parsing the path parameters if any.
	 * @param Query the path to search for in the registry.
	 * @param Verb the handler's verb to look for. (A route might have multiple handlers each with a different verb.)
	 * @param ParsedTokens An optional list of tokens that were removed from the end of the path.
	 * @return The query's result, indicating if a match was found and parsed path parameters.
	 */
	FRouteQueryResult QueryRoute(const FString& Query, EHttpServerRequestVerbs Verb, const TArray<FString>* Parsedtokens = nullptr) const;
private:
	struct FDynamicRouteHandleWrapper
	{
		FDynamicRouteHandleWrapper(FHttpRouteHandle Handle);

		/** The handler for thi;s route. */
		FHttpRouteHandle Handle;
		/** Path tokens starting from the first colon in the path */
		TArray<FString> DynamicPathSectionTokens;
	};
private:
	/** 
	 * Find a static route in the registrar. 
	 * @return The route's container and index.
	 */
	TTuple<TArray<FHttpRouteHandle>*, int64> FindStaticRoute(const FString& RoutePath, EHttpServerRequestVerbs Verb);

	/** 
	 * Find a dynamic route in the registrar. 
	 * @return The route's container and index.
	 */
	TTuple<TArray<FDynamicRouteHandleWrapper>*, int64> FindDynamicRoute(const FString& RoutePath, EHttpServerRequestVerbs Verb);

	/** Checks if the registrar contains a static route given a path and verb. */
	bool ContainsStaticRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb);

	/** Checks if the registrar contains a dynamic route given a path and verb. */
	bool ContainsDynamicRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb);

	/** Removes a static route given a path and verb. */
	bool RemoveStaticRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb);

	/** Removes a dynamic route given a path and verb. */
	bool RemoveDynamicRoute(const FHttpPath& RoutePath, EHttpServerRequestVerbs Verb);

	/** Checks whether a given path template matches an input path. */
	bool MatchesPath(const TArray<FString>& InPath, const TArray<FString>& TemplatePath) const;
	/** Extract the path parameters from the path. */
	TMap<FString, FString> ParsePathParameters(const TArray<FString>& InPath, const TArray<FString>& TemplatePath) const;

private:
	/** Map of static routes to a list of handlers for different HTTP verbs for this route. */
	TMap<const FString, TArray<FHttpRouteHandle>> StaticRoutes;
	/**
	 * Map of subgroups of dynamic path handlers.
	 * A dynamic path is a path containing a path parameter.
	 * The subgroup consists of the portion of the dynamic path that does not contain path parameters, identified by ':'. 
	 */
	TMap<const FString, TArray<FDynamicRouteHandleWrapper>> DynamicRoutes;
};
