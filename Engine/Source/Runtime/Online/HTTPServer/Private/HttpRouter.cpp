// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpRouter.h"
#include "HttpRouteHandle.h"
#include "HttpPath.h"
#include "HttpConnection.h"
#include "HttpServerRequest.h"
#include "HttpServerResponse.h"
#include "HttpRequestHandler.h"
#include "HttpRequestHandlerIterator.h"

DEFINE_LOG_CATEGORY_STATIC(LogHttpRouter, Log, All);

bool FHttpRouter::Query(const TSharedPtr<FHttpServerRequest>& Request, const FHttpResultCallback& OnProcessingComplete)
{
	bool bRequestHandled = false;

	TArray<FHttpRequestHandler> PreprocessorsArray;
	RequestPreprocessors.GenerateValueArray(PreprocessorsArray);
	FHttpRequestHandlerIterator Iterator(Request, RequestHandlerRegistrar, MoveTemp(PreprocessorsArray));
	while (const FHttpRequestHandler* RequestHandlerPtr = Iterator.Next())
	{
		if (!RequestHandlerPtr->IsBound())
		{
			UE_LOGF(LogHttpRouter, Verbose, "Skipping an unbound FHttpRequestHandler.");
			continue;
		}

		bRequestHandled = RequestHandlerPtr->Execute(*Request, OnProcessingComplete);
		if (bRequestHandled)
		{
			break;
		}
	}

	return bRequestHandled;
}

FHttpRouteHandle FHttpRouter::BindRoute(const FHttpPath& HttpPath,  const EHttpServerRequestVerbs& HttpVerbs,  const FHttpRequestHandler& Handler)
{
	check(HttpPath.IsValidPath());
	check(EHttpServerRequestVerbs::VERB_NONE != HttpVerbs);

	if (RequestHandlerRegistrar.ContainsRoute(HttpPath, HttpVerbs))
	{
		return nullptr;
	}

	auto RouteHandle = MakeShared<FHttpRouteHandleInternal>(HttpPath.GetPath(), HttpVerbs, Handler);
	RequestHandlerRegistrar.AddRoute(RouteHandle);

	return RouteHandle;
}

void FHttpRouter::UnbindRoute(const FHttpRouteHandle& RouteHandle)
{
	if (!ensure(RouteHandle.IsValid()))
	{
		return;
	}

	if (FRouteQueryResult QueryResult = RequestHandlerRegistrar.QueryRoute(RouteHandle->Path, RouteHandle->Verbs))
	{
		// Ensure caller is unbinding a route they actually own
		check(QueryResult.RouteHandle == RouteHandle);
		RequestHandlerRegistrar.RemoveRoute(RouteHandle);
	}
}

FDelegateHandle FHttpRouter::RegisterRequestPreprocessor(FHttpRequestHandler RequestPreprocessor)
{
	FDelegateHandle Handle = FDelegateHandle(FDelegateHandle::GenerateNewHandle);
	RequestPreprocessors.Add(Handle, MoveTemp(RequestPreprocessor));
	return Handle;
}

void FHttpRouter::UnregisterRequestPreprocessor(const FDelegateHandle& RequestPreprocessorHandle)
{
	RequestPreprocessors.Remove(RequestPreprocessorHandle);
}

FHttpRequestHandlerIterator FHttpRouter::CreateRequestHandlerIterator(const TSharedPtr<FHttpServerRequest>& Request) const
{
	TArray<FHttpRequestHandler> PreprocessorsArray;
	RequestPreprocessors.GenerateValueArray(PreprocessorsArray);
	FHttpRequestHandlerIterator Iterator(Request, RequestHandlerRegistrar, MoveTemp(PreprocessorsArray));
	return Iterator;
}

