// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IHttpRouter.h"
#include "HttpRequestHandler.h"
#include "HttpRouteHandle.h"
#include "HttpRequestHandlerRegistrar.h"

struct FHttpPath;
struct FHttpRequestHandlerIterator;

class FHttpRouter final : public IHttpRouter
{
public:

	// IHttpRouter Overrides
	bool Query(const TSharedPtr<FHttpServerRequest>& Request, const FHttpResultCallback& OnProcessingComplete) override;
	FHttpRouteHandle BindRoute(const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler) override;
	void UnbindRoute(const FHttpRouteHandle& RouteHandle) override;
	FDelegateHandle RegisterRequestPreprocessor(FHttpRequestHandler RequestPreprocessor) override;
	void UnregisterRequestPreprocessor(const FDelegateHandle& RequestPreprocessorHandle) override;
	
	/**
	 * Creates a request handler iterator to facilitate http routing
	 * @param Request The basis request for which matching request handlers are found
	 * @return The instantiated FHttpRequestHandlerIterator
	 */
	FHttpRequestHandlerIterator CreateRequestHandlerIterator(const TSharedPtr<FHttpServerRequest>& Request) const;

private:

	/** The associative pairing of Http routes to respective request handlers */
	FHttpRequestHandlerRegistrar RequestHandlerRegistrar;
	
	/** Holds the list of request preprocessors. */
	TMap<FDelegateHandle, FHttpRequestHandler> RequestPreprocessors;
};


