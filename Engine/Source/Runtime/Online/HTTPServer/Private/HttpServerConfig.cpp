// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpServerConfig.h"
#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY(LogHttpServerConfig)

TAutoConsoleVariable<bool> CVarHttpServerDefaultReuseAddressAndPortEnabled(
	TEXT("HTTPServer.DefaultReuseAddressAndPortEnabled"),
	false,
	TEXT("When enabled, http server will reuse address and port without raising error")
);

// TODO: static FString will cause mem leaks warnings in Insights, try to use macro or member ptr in http server module instead
namespace FHttpServerConfigCache
{
	static FDelegateHandle ConfigSectionChangedHandle;
	static bool bConnectionsCacheDirty = true;
	static bool bListenersCacheDirty = true;
	static FHttpServerConnectionConfig CacheConnectionConfig;
	static FHttpServerListenerConfig CacheListenerConfig;
	static TArray<FString> CacheListenerConfigs;
	static const FString IniSectionNameHTTPServerConnections("HTTPServer.Connections");
	static const FString IniSectionNameHTTPServerListeners("HTTPServer.Listeners");
}

void FHttpServerConfig::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	using namespace FHttpServerConfigCache;

	if (IniFilename == GEngineIni)
	{
		if (SectionNames.Contains(IniSectionNameHTTPServerConnections))
		{
			bConnectionsCacheDirty = true;
		}
		if (SectionNames.Contains(IniSectionNameHTTPServerListeners))
		{
			bListenersCacheDirty = true;
		}
	}
}

const FHttpServerListenerConfig FHttpServerConfig::GetListenerConfig(uint32 Port) 
{
	using namespace FHttpServerConfigCache;

	if (!ConfigSectionChangedHandle.IsValid())
	{
		ConfigSectionChangedHandle = FCoreDelegates::TSOnConfigSectionsChanged().AddStatic(FHttpServerConfig::OnConfigSectionsChanged);
	}

	// Update values from Config if necessary
	if (bListenersCacheDirty)
	{
		// Apply default ini configuration
		GConfig->GetString(*IniSectionNameHTTPServerListeners, TEXT("DefaultBindAddress"), CacheListenerConfig.BindAddress, GEngineIni);
		GConfig->GetInt(*IniSectionNameHTTPServerListeners, TEXT("DefaultBufferSize"), CacheListenerConfig.BufferSize, GEngineIni);
		GConfig->GetInt(*IniSectionNameHTTPServerListeners, TEXT("DefaultConnectionsBacklogSize"), CacheListenerConfig.ConnectionsBacklogSize, GEngineIni);
		GConfig->GetInt(*IniSectionNameHTTPServerListeners, TEXT("DefaultMaxConnectionsAcceptPerFrame"), CacheListenerConfig.MaxConnectionsAcceptPerFrame, GEngineIni);
		if (!GConfig->GetBool(*IniSectionNameHTTPServerListeners, TEXT("DefaultReuseAddressAndPort"), CacheListenerConfig.bReuseAddressAndPort, GEngineIni))
		{
			CacheListenerConfig.bReuseAddressAndPort = CVarHttpServerDefaultReuseAddressAndPortEnabled.GetValueOnAnyThread();
		}

		GConfig->GetArray(*IniSectionNameHTTPServerListeners, TEXT("ListenerOverrides"), CacheListenerConfigs, GEngineIni);

		bListenersCacheDirty = false;
	}
	
	// Setup default values
	FHttpServerListenerConfig Config(CacheListenerConfig);

	// Apply per-port ini overrides
	for(FString ListenerConfigStr : CacheListenerConfigs)
	{
		ListenerConfigStr.TrimStartAndEndInline();
		ListenerConfigStr.ReplaceInline(TEXT("("), TEXT(""));
		ListenerConfigStr.ReplaceInline(TEXT(")"), TEXT(""));

		// Listener config overrides must specify a port
		uint32 ConfiguredPort = 0;
		if (!FParse::Value(*ListenerConfigStr, TEXT("Port="), ConfiguredPort))
		{
			UE_LOGF(LogHttpServerConfig, Error,
				"ListenerOverride: %ls does not specify required Port parameter",
				*ListenerConfigStr);
			continue;
		}

		if (Port == ConfiguredPort)
		{
			// override defaults with config values
			FParse::Value(*ListenerConfigStr, TEXT("BindAddress="), Config.BindAddress);
			FParse::Value(*ListenerConfigStr, TEXT("BufferSize="), Config.BufferSize);
			FParse::Value(*ListenerConfigStr, TEXT("ConnectionsBacklogSize="), Config.ConnectionsBacklogSize);
			FParse::Value(*ListenerConfigStr, TEXT("MaxConnectionsAcceptPerFrame="), Config.MaxConnectionsAcceptPerFrame);
			FParse::Bool(*ListenerConfigStr, TEXT("ReuseAddressAndPort="), Config.bReuseAddressAndPort);
			break;
		}
	}
	return Config;
}

const FHttpServerConnectionConfig FHttpServerConfig::GetConnectionConfig()
{
	using namespace FHttpServerConfigCache;

	if (!ConfigSectionChangedHandle.IsValid())
	{
		ConfigSectionChangedHandle = FCoreDelegates::TSOnConfigSectionsChanged().AddStatic(FHttpServerConfig::OnConfigSectionsChanged);
	}

	if (bConnectionsCacheDirty)
	{
		// Apply default ini configuration
		GConfig->GetFloat(*IniSectionNameHTTPServerConnections, TEXT("BeginReadWaitTimeMS"), CacheConnectionConfig.BeginReadWaitTimeMS, GEngineIni);
		bConnectionsCacheDirty = false;
	}

	return CacheConnectionConfig;
}
