// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Misc/Timespan.h"

DECLARE_LOG_CATEGORY_EXTERN(LogHttpServerConfig, Log, All);

struct FHttpServerListenerConfig
{
	/** The address to bind */
	FString BindAddress = FString("localhost");
	/** Maximum send buffer size */
	int32 BufferSize = 512 * 1024;
	/** Number of pending connections to queue */
	int32 ConnectionsBacklogSize = 16;
	/** Max Number of connections to accept per frame */
	int32 MaxConnectionsAcceptPerFrame = 1;
	/** If true, call FSocket::SetReuseAddr when binding to allow the use of an already bound address/port */
	bool bReuseAddressAndPort = false;
};

struct FHttpServerConnectionConfig
{
	/** Time in milliseconds to wait for data to be available when ticking the connection. */
	float BeginReadWaitTimeMS = 1;
};

struct FHttpServerConfig
{
	/** 
	* Gets the listener configuration for the caller-supplied Port
	*
	* @param Port The respective listener port
	* @return The per-port configuration if configured, or default configuration
	*/
	static const FHttpServerListenerConfig GetListenerConfig(uint32 Port);

	/**
	* Gets the connection configuration
	*
	* @return The configuration if configured, or default configuration
	*/
	static const FHttpServerConnectionConfig GetConnectionConfig();

	/**
	* Callback when the IniConfig is updated so that the internal cache can be updated
	*/
	static void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);
};