// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#define UE_HTTP_SERVER_HEADER_KEYS_CONTENT_TYPE TEXT("content-type")
#define UE_HTTP_SERVER_HEADER_KEYS_CONTENT_LENGTH TEXT("content-length")
#define UE_HTTP_SERVER_HEADER_KEYS_CONNECTION TEXT("connection")
#define UE_HTTP_SERVER_HEADER_KEYS_KEEP_ALIVE TEXT("keep-alive")

// Connection
#define UE_HTTP_SERVER_ERROR_STR_SOCKET_CLOSED_FAILURE TEXT("errors.com.epicgames.httpserver.socket_closed_failure")
#define UE_HTTP_SERVER_ERROR_STR_SOCKET_RECV_FAILURE TEXT("errors.com.epicgames.httpserver.socket_recv_failure")
#define UE_HTTP_SERVER_ERROR_STR_SOCKET_SEND_FAILURE TEXT("errors.com.epicgames.httpserver.socket_send_failure")

// Routing
#define UE_HTTP_SERVER_ERROR_STR_ROUTE_HANDLER_NOT_FOUND TEXT("errors.com.epicgames.httpserver.route_handler_not_found")

// Serialization
#define UE_HTTP_SERVER_ERROR_STR_MALFORMED_REQUEST_SIZE TEXT("errors.com.epicgames.httpserver.malformed_request_size")
#define UE_HTTP_SERVER_ERROR_STR_MALFORMED_REQUEST_HEADER TEXT("errors.com.epicgames.httpserver.malformed_request_header")
#define UE_HTTP_SERVER_ERROR_STR_MISSING_REQUEST_HEADERS TEXT("errors.com.epicgames.httpserver.missing_request_headers")
#define UE_HTTP_SERVER_ERROR_STR_MALFORMED_REQUEST_BODY TEXT("errors.com.epicgames.httpserver.malformed_request_body")
#define UE_HTTP_SERVER_ERROR_STR_UNKNOWN_REQUEST_VERB TEXT("errors.com.epicgames.httpserver.unknown_request_verb")
#define UE_HTTP_SERVER_ERROR_STR_UNSUPPORTED_HTTP_VERSION TEXT("errors.com.epicgames.httpserver.unsupported_http_version")
#define UE_HTTP_SERVER_ERROR_STR_INVALID_CONTENT_LENGTH_HEADER TEXT("errors.com.epicgames.httpserver.invalid_content_length_header")
#define UE_HTTP_SERVER_ERROR_STR_MISSING_CONTENT_LENGTH_HEADER TEXT("errors.com.epicgames.httpserver.missing_content_length_header")
#define UE_HTTP_SERVER_ERROR_STR_MISMATCHED_CONTENT_LENGTH_BODY_TOO_LARGE TEXT("errors.com.epicgames.httpserver.mismatched_content_length_body_too_large")
