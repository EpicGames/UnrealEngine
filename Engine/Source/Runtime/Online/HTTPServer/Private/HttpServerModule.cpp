// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpServerModule.h"
#include "HttpListener.h"
#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"

class FHttpServerModuleImpl
{
public:
	/** The association of port bindings and respective HTTP listeners */
	TMap<uint32, TUniquePtr<FHttpListener>> Listeners;

	/** Whether listeners can be started */
	bool bHttpListenersEnabled = false;
};

DEFINE_LOG_CATEGORY(LogHttpServerModule);

// FHttpServerModule 
IMPLEMENT_MODULE(FHttpServerModule, HTTPServer);

FHttpServerModule* FHttpServerModule::Singleton = nullptr;

FHttpServerModule::FHttpServerModule()
	: Impl(new FHttpServerModuleImpl)
{
}

FHttpServerModule::~FHttpServerModule()
{
	delete Impl;
	Impl = nullptr;
}

void FHttpServerModule::StartupModule()
{
	Singleton = this;
}

void FHttpServerModule::ShutdownModule()
{
	// stop all listeners
	StopAllListeners();

	// destroy all listeners
	Impl->Listeners.Empty();

	Singleton = nullptr;
}

void FHttpServerModule::StartAllListeners()
{
	Impl->bHttpListenersEnabled = true;

	UE_LOGF(LogHttpServerModule, Log,
		"Starting all listeners...");

	for (const auto& Listener : Impl->Listeners)
	{
		if (!Listener.Value->IsListening())
		{
			Listener.Value->StartListening();
		}
	}
	UE_LOGF(LogHttpServerModule, Log,
		"All listeners started");
}

void FHttpServerModule::StopAllListeners()
{
	UE_LOGF(LogHttpServerModule, Log,
		"Stopping all listeners...");

	Impl->bHttpListenersEnabled = false;

	for (const auto& Listener : Impl->Listeners)
	{
		if (Listener.Value->IsListening())
		{
			Listener.Value->StopListening();
		}
	}

	UE_LOGF(LogHttpServerModule, Log,
		"All listeners stopped");

}

bool FHttpServerModule::HasPendingListeners() const
{
	for (const auto& Listener : Impl->Listeners)
	{
		if (Listener.Value->HasPendingConnections())
		{
			return true;
		}
	}
	return false;
}

bool FHttpServerModule::IsAvailable()
{
	return nullptr != Singleton;
}

FHttpServerModule& FHttpServerModule::Get()
{
	if (nullptr == Singleton)
	{
		check(IsInGameThread());
		FModuleManager::LoadModuleChecked<FHttpServerModule>("HTTPServer");
	}
	check(Singleton);
	return *Singleton;
}

TSharedPtr<IHttpRouter> FHttpServerModule::GetHttpRouter(uint32 Port, bool bFailOnBindFailure /* = false */)
{
	check(Singleton == this);

	bool bFailedToListen = false;

	// We may already have a listener for this port
	TUniquePtr<FHttpListener>* ExistingListener = Impl->Listeners.Find(Port);
	if (ExistingListener)
	{
		if (Impl->bHttpListenersEnabled)
		{
			// if listeners are enabled, the existing listener for this port
			// should always be listening (IsListening() will only be true
			// if it fully initialized/successfully bound and actually started listening)
			if (ExistingListener->Get()->IsListening())
			{
				UE_LOGF(LogHttpServerModule, Verbose, "[%ls] found an existing, active listener for port %d", ANSI_TO_TCHAR(__FUNCTION__), Port);
				return ExistingListener->Get()->GetRouter();
			}
			else
			{
				// get rid of it and create a new one for now
				UE_LOGF(LogHttpServerModule, Error, "[%ls] the existing listener for port %d is not listening/bound and listeners are still enabled",
					ANSI_TO_TCHAR(__FUNCTION__), Port);
				Impl->Listeners.Remove(Port);
			}
		}
		else
		{
			UE_LOGF(LogHttpServerModule, Verbose, "[%ls] found an existing listener for port %d but listeners are currently disabled", ANSI_TO_TCHAR(__FUNCTION__), Port);
			return ExistingListener->Get()->GetRouter();
		}
	}

	// Otherwise create a new one
	UE_LOGF(LogHttpServerModule, VeryVerbose, "[%ls] creating a new listener for port %d", ANSI_TO_TCHAR(__FUNCTION__), Port);
	TUniquePtr<FHttpListener> NewListener = MakeUnique<FHttpListener>(Port);

	// Try to start this listener now
	if (Impl->bHttpListenersEnabled)
	{
		if (!NewListener->StartListening())
		{
			UE_LOGF(LogHttpServerModule, Warning, "[%ls] failed to start listening on port %d! (bFailOnBindFailure? %ls)", ANSI_TO_TCHAR(__FUNCTION__), Port, *LexToString(bFailOnBindFailure));
			bFailedToListen = true;
		}
	}

	if (bFailedToListen && bFailOnBindFailure)
	{
		// if we actually failed to listen on a port for whatever reason
		// (i.e. listeners are enabled and we attempted to do so) then
		// let the one we created fall out of scope and try again next time
		return nullptr;
	}
	else
	{
		// the legacy behavior returns the router regardless of listener success
		const auto& NewListenerRef = Impl->Listeners.Add(Port, MoveTemp(NewListener));
		return NewListenerRef->GetRouter();
	}
}

bool FHttpServerModule::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FHttpServerModule_Tick);
	check(Singleton == this);

	if (Impl->bHttpListenersEnabled)
	{
		for (const auto& Listener : Impl->Listeners)
		{
			Listener.Value->Tick(DeltaTime);
		}
	}
	return true;
}
