// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpServerResponse.h"

#include "Containers/Utf8String.h"
#include "HttpServerConstants.h"
#include "HttpServerConstantsPrivate.h"

TUniquePtr<FHttpServerResponse> FHttpServerResponse::Create(const FString& Text, const FString& ContentType)
{
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::Ok;

	FTCHARToUTF8 ConvertToUtf8(*Text);
	const uint8* ConvertToUtf8Bytes = (reinterpret_cast<const uint8*>(ConvertToUtf8.Get()));
	Response->Body.Append(ConvertToUtf8Bytes, ConvertToUtf8.Length());

	FString Utf8CharsetContentType = FString::Printf(TEXT("%s;charset=utf-8"), *ContentType);
	TArray<FString> ContentTypeValue = { MoveTemp(Utf8CharsetContentType) };
	Response->Headers.Add(UE_HTTP_SERVER_HEADER_KEYS_CONTENT_TYPE, MoveTemp(ContentTypeValue));

	return Response;
}

TUniquePtr<FHttpServerResponse> FHttpServerResponse::Create(const FUtf8String& Text, const FString& ContentType)
{
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::Ok;

	const uint8* Utf8Bytes = (reinterpret_cast<const uint8*>(Text.GetCharArray().GetData()));
	Response->Body.Append(Utf8Bytes, Text.Len()); // exclude null terminator

	FString Utf8CharsetContentType = FString::Printf(TEXT("%s;charset=utf-8"), *ContentType);
	TArray<FString> ContentTypeValue = { MoveTemp(Utf8CharsetContentType) };
	Response->Headers.Add(UE_HTTP_SERVER_HEADER_KEYS_CONTENT_TYPE, MoveTemp(ContentTypeValue));

	return Response;
}

TUniquePtr<FHttpServerResponse> FHttpServerResponse::Create(FUtf8String&& Text, const FString& ContentType)
{
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::Ok;

	Response->Body = TArray<uint8>(MoveTemp(Text.GetCharArray()));
	Text.Empty();

	// exclude null terminator
	if (!Response->Body.IsEmpty() && Response->Body.Last() == 0)
	{
		Response->Body.Pop(EAllowShrinking::No);
	}

	FString Utf8CharsetContentType = FString::Printf(TEXT("%s;charset=utf-8"), *ContentType);
	TArray<FString> ContentTypeValue = { MoveTemp(Utf8CharsetContentType) };
	Response->Headers.Add(UE_HTTP_SERVER_HEADER_KEYS_CONTENT_TYPE, MoveTemp(ContentTypeValue));

	return Response;
}

TUniquePtr<FHttpServerResponse> FHttpServerResponse::Create(TArray<uint8>&& RawBytes, FString ContentType)
{
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>(MoveTemp(RawBytes));
	Response->Code = EHttpServerResponseCodes::Ok;

	TArray<FString> ContentTypeValue = { MoveTemp(ContentType) };
	Response->Headers.Add(UE_HTTP_SERVER_HEADER_KEYS_CONTENT_TYPE, MoveTemp(ContentTypeValue));
	return Response;
}

TUniquePtr<FHttpServerResponse> FHttpServerResponse::Create(const TArrayView<uint8>& RawBytes, FString ContentType)
{
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::Ok;
	Response->Body.Append(RawBytes.GetData(), RawBytes.Num());

	TArray<FString> ContentTypeValue = { MoveTemp(ContentType) };
	Response->Headers.Add(UE_HTTP_SERVER_HEADER_KEYS_CONTENT_TYPE, MoveTemp(ContentTypeValue));
	return Response;
}

TUniquePtr<FHttpServerResponse> FHttpServerResponse::Ok()
{
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::NoContent;
	return Response;
}

TUniquePtr<FHttpServerResponse> FHttpServerResponse::Error(EHttpServerResponseCodes ResponseCode, 
	const FString& ErrorCode /*=TEXT("")*/, const FString& ErrorMessage /*=TEXT("")*/)
{
	const FString ErrorCodeEscaped = ErrorCode.ReplaceCharWithEscapedChar();
	const FString ErrorMessageEscaped = ErrorMessage.ReplaceCharWithEscapedChar();
	const FString ResponseBody = FString::Printf(TEXT("{\"errorCode\": \"%s\", \"errorMessage\": \"%s\"}"), *ErrorCodeEscaped, *ErrorMessageEscaped);
	auto Response = Create(ResponseBody, TEXT("application/json"));
	Response->Code = ResponseCode;
	return Response;
}
