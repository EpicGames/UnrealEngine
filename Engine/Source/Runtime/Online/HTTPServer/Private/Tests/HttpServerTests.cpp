// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Math/NumericLimits.h"
#include "HttpServerModule.h"
#include "IHttpRouter.h"
#include "HttpRouteHandle.h"
#include "HttpRequestHandler.h"
#include "HttpServerRequest.h"
#include "HttpServerResponse.h"
#include "HttpPath.h"
#include "HttpRequestHandlerRegistrar.h"
#include "HttpRequestHandlerIterator.h"
#include "HttpConnectionResponseWriteContext.h"
#include "HttpConnection.h"
#include "HttpRouter.h"
#include "Sockets.h"
#include "SocketSubsystem.h"


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerIntegrationTest, "System.Online.HttpServer.Integration", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerIntegrationTest::RunTest(const FString& Parameters)
{
	const uint32 HttpRouterPort = 8888;
	const uint32 InvalidHttpRouterPort = TNumericLimits<uint16>::Max() + 1; // 65536
	const FHttpPath HttpPath(TEXT("/TestHttpServer"));

	// Ensure router creation
	TSharedPtr<IHttpRouter> HttpRouter = FHttpServerModule::Get().GetHttpRouter(HttpRouterPort);
	TestTrue(TEXT("HttpRouter.IsValid()"), HttpRouter.IsValid());

	// Ensure unique routers per-port
	TSharedPtr<IHttpRouter> DuplicateHttpRouter = FHttpServerModule::Get().GetHttpRouter(HttpRouterPort);
	TestEqual(TEXT("HttpRouter Duplicates"), HttpRouter, DuplicateHttpRouter);

	// Ensure failed port binds still return a valid router if not explicitly requested to fail (and by default)
	TSharedPtr<IHttpRouter> ValidHttpRouterOnFail = FHttpServerModule::Get().GetHttpRouter(InvalidHttpRouterPort /*, bFailOnBindFailure = false */);
	TestTrue(TEXT("HttpRouter is NOT null on bind failure by default"), ValidHttpRouterOnFail.IsValid());

	// Ensure we can create route bindings
	const FHttpRequestHandler RequestHandler = FHttpRequestHandler::CreateLambda([](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
	{
		return true;
	});
	FHttpRouteHandle HttpRouteHandle = HttpRouter->BindRoute(HttpPath, EHttpServerRequestVerbs::VERB_GET, RequestHandler);
	TestTrue(TEXT("HttpRouteHandle.IsValid()"), HttpRouteHandle.IsValid());

	// Disallow duplicate route bindings
	FHttpRouteHandle DuplicateHandle = HttpRouter->BindRoute(HttpPath, EHttpServerRequestVerbs::VERB_GET, RequestHandler);
	TestFalse(TEXT("HttpRouteHandle Duplicated"), DuplicateHandle.IsValid());

	// Because of the ValidHttpRouterOnFail was created by FHttpServerModule::Get().GetHttpRouter(InvalidHttpRouterPort...), it will fail to listen in StartAllListeners
	// Also after bHttpListenersEnabled got set to true by StartAllListeners, when call GetHttpRouter(InvalidHttpRouterPort...) again, it will call StartListening again in there and fail
	AddExpectedError(TEXT("HttpListener detected invalid port"), EAutomationExpectedErrorFlags::Contains, 2);
	FHttpServerModule::Get().StartAllListeners();

	// Because of the ValidHttpRouterOnFail was created by FHttpServerModule::Get().GetHttpRouter(InvalidHttpRouterPort...)
	AddExpectedError(TEXT("is not listening/bound and listeners are still enabled"), EAutomationExpectedErrorFlags::Contains, 1);
	// Ensure failed port binds result in a null router instance if requested (and listeners are enabled)
	TSharedPtr<IHttpRouter> InvalidHttpRouterOnFail = FHttpServerModule::Get().GetHttpRouter(InvalidHttpRouterPort, /* bFailOnBindFailure = */ true);
	TestFalse(TEXT("HttpRouter is null on bind failure if requested"), InvalidHttpRouterOnFail.IsValid());

	// Make a request
	/*
	ADD_LATENT_AUTOMATION_COMMAND(FExecStringLatentCommand(TEXT("HTTP TEST 1 http://localhost:8888/TestHttpServer")));
	*/

	FHttpServerModule::Get().StopAllListeners();

	HttpRouter->UnbindRoute(HttpRouteHandle);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerPathParametersTest, "System.Online.HttpServer.PathParameters", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerPathParametersTest::RunTest(const FString& Parameters)
{
	enum EExpectedQueryResult
	{
		ShouldMatch,
		ShouldNotMatch
	};

	using EVerb = EHttpServerRequestVerbs;

	struct FHttpPathTest
	{
		FHttpPathTest(EVerb InQueryVerb, FString InTestQuery, EVerb InTargetRouteVerb, FString InTargetRoute, EExpectedQueryResult InExpectedResult)
			: QueryVerb(InQueryVerb)
			, TestQuery(MoveTemp(InTestQuery))
			, TargetRouteVerb(InTargetRouteVerb)
			, TargetRoute(MoveTemp(InTargetRoute))
			, bRouteQueried(false)
			, bExpectedResult(InExpectedResult)
		{
		}

		void SetHandler()
		{
			Handler = FHttpRequestHandler::CreateLambda([this](const FHttpServerRequest&, const FHttpResultCallback&)
			{
				bRouteQueried = true;
				return true;
			});
		}

		EVerb QueryVerb;
		FString TestQuery;
		EVerb TargetRouteVerb;
		FHttpPath TargetRoute;
		FHttpRequestHandler Handler;

		bool bRouteQueried;
		EExpectedQueryResult bExpectedResult;
	};

	auto Callback = [](TUniquePtr<FHttpServerResponse>) {};

	TArray<FHttpPathTest> Tests;

	// Test simple static route.
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/staticroute"),				    EVerb::VERB_GET, TEXT("/staticroute"),				       ShouldMatch);
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/"),							    EVerb::VERB_GET, TEXT("/"),						      	   ShouldMatch);

	// Test verbs.
	Tests.Emplace(EVerb::VERB_PUT,     TEXT("/putroute"),				        EVerb::VERB_PUT, TEXT("/putroute"),					       ShouldMatch);
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/otherputroute"),					EVerb::VERB_PUT, TEXT("/otherputroute"),		       	   ShouldNotMatch);
	Tests.Emplace(EVerb::VERB_PUT,     TEXT("/putdelete"),						EVerb::VERB_PUT | EVerb::VERB_DELETE, TEXT("/putdelete"),  ShouldMatch);
	Tests.Emplace(EVerb::VERB_DELETE,  TEXT("/putdelete2"),						EVerb::VERB_PUT | EVerb::VERB_DELETE, TEXT("/putdelete2"), ShouldMatch);

	// Test path parameters.
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/test/parm1value"),				EVerb::VERB_GET, TEXT("/test/:parm1"),					   ShouldMatch);
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/test/p1v/test2/p2v"),				EVerb::VERB_GET, TEXT("/test/:parm1/test2/:parm2"),		   ShouldMatch);
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/test/p1v/test2/p2v"),				EVerb::VERB_GET, TEXT("/test/:parm1/test2/other"),         ShouldNotMatch);
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/onevalue/twovalue/threevalue"),	EVerb::VERB_GET, TEXT("/:a/:b/:c"),					       ShouldMatch);
	
	// Special case: static routes should match before dynamic routes.
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/route/static"),					EVerb::VERB_GET, TEXT("/route/static"),				       ShouldMatch);
	Tests.Emplace(EVerb::VERB_GET,     TEXT("/route/static"),					EVerb::VERB_GET, TEXT("/route/:dynamic"),			 	   ShouldNotMatch);

	// Test path parameter with verbs
	Tests.Emplace(EVerb::VERB_PUT,	 TEXT("/rt/static"),						EVerb::VERB_GET | EVerb::VERB_PUT, TEXT("/rt/:dynamic"),   ShouldMatch);

	FHttpRequestHandlerRegistrar Registrar;

	for (FHttpPathTest& Test : Tests)
	{
		Test.SetHandler();
		Registrar.AddRoute(MakeShared<FHttpRouteHandleInternal>(Test.TargetRoute.GetPath(), Test.TargetRouteVerb, Test.Handler));
	}

	for (FHttpPathTest& Test : Tests)
	{
		// In case this route was queried by some other test.
		Test.bRouteQueried = false;

		TSharedPtr<FHttpServerRequest> Request = MakeShared<FHttpServerRequest>();
		Request->Verb = Test.QueryVerb;
		Request->RelativePath = Test.TestQuery;

		FHttpRequestHandlerIterator Iterator(Request, Registrar);
		if (const FHttpRequestHandler* RequestHandlerPtr = Iterator.Next())
		{
			[[maybe_unused]] bool bHandled = RequestHandlerPtr->Execute(*Request, Callback);
		}

		if ((Test.bExpectedResult == ShouldMatch && !Test.bRouteQueried) || (Test.bExpectedResult == ShouldNotMatch && Test.bRouteQueried))
		{
			AddError(FString::Printf(TEXT("Expected query %s to have result: %s when targeting route %s"), *Test.TestQuery, Test.bExpectedResult ? TEXT("true") : TEXT("false"), *Test.TargetRoute.GetPath()), 1);
		}
	}

	return true;
}

// ---------------------------------------------------------------------------
// SSE / Streaming Tests
// ---------------------------------------------------------------------------

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerResponseStreamingConstructionTest, "System.Online.HttpServer.SSE.ResponseStreamingConstruction", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerResponseStreamingConstructionTest::RunTest(const FString& Parameters)
{
	// Verify individual flag values are distinct
	TestEqual(TEXT("None is 0"), static_cast<uint8>(EHttpServerResponseFlags::None), (uint8)0);
	TestEqual(TEXT("MultipleWriteStream is 1"), static_cast<uint8>(EHttpServerResponseFlags::MultipleWriteStream), (uint8)1);
	TestEqual(TEXT("HasAdditionalWrites is 2"), static_cast<uint8>(EHttpServerResponseFlags::HasAdditionalWrites), (uint8)2);
	TestEqual(TEXT("SkipHeaderWrite is 4"), static_cast<uint8>(EHttpServerResponseFlags::SkipHeaderWrite), (uint8)4);

	// Verify EnumHasAnyFlags/EnumHasAllFlags
	const EHttpServerResponseFlags Combined = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites;
	TestTrue(TEXT("HasAnyFlags(Combined, MultipleWriteStream)"), EnumHasAnyFlags(Combined, EHttpServerResponseFlags::MultipleWriteStream));
	TestTrue(TEXT("HasAnyFlags(Combined, HasAdditionalWrites)"), EnumHasAnyFlags(Combined, EHttpServerResponseFlags::HasAdditionalWrites));
	TestTrue(TEXT("HasAllFlags(Combined, both)"), EnumHasAllFlags(Combined, Combined));
	TestFalse(TEXT("HasAllFlags(MultipleWriteStream alone, both)"), EnumHasAllFlags(EHttpServerResponseFlags::MultipleWriteStream, Combined));

	// Verify streaming response construction
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::Ok;
	Response->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites;
	Response->StreamingBodyQueue = MakeShared<TQueue<TArray<uint8>, EQueueMode::Spsc>>();
	Response->StreamingBodyComplete = MakeShared<TAtomic<bool>>(false);

	TestTrue(TEXT("StreamingBodyQueue is valid"), Response->StreamingBodyQueue.IsValid());
	TestTrue(TEXT("StreamingBodyComplete is valid"), Response->StreamingBodyComplete.IsValid());
	TestFalse(TEXT("StreamingBodyComplete initially false"), Response->StreamingBodyComplete->Load(EMemoryOrder::Relaxed));

	// Verify queue enqueue/dequeue round-trip
	const FString EventPayload = TEXT("data: hello\n\n");
	FTCHARToUTF8 Utf8(*EventPayload);
	TArray<uint8> EnqueuedData(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
	Response->StreamingBodyQueue->Enqueue(EnqueuedData);

	TArray<uint8> DequeuedData;
	TestTrue(TEXT("Dequeue succeeds"), Response->StreamingBodyQueue->Dequeue(DequeuedData));
	TestEqual(TEXT("Dequeued data matches enqueued"), DequeuedData, EnqueuedData);

	// Verify completion signal
	Response->StreamingBodyComplete->Store(true, EMemoryOrder::Relaxed);
	TestTrue(TEXT("StreamingBodyComplete is true after store"), Response->StreamingBodyComplete->Load(EMemoryOrder::Relaxed));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerWriteContextFlagQueriesTest, "System.Online.HttpServer.SSE.WriteContextFlagQueries", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerWriteContextFlagQueriesTest::RunTest(const FString& Parameters)
{
	// Construct with nullptr socket - flag queries never touch the socket
	FHttpConnectionResponseWriteContext WriteContext(nullptr);

	// Before ResetContext: no response set
	TestFalse(TEXT("IsMultipleWriteStream before reset"), WriteContext.IsMultipleWriteStream());
	TestFalse(TEXT("HasAdditionalWrites before reset"), WriteContext.HasAdditionalWrites());

	// No flags
	{
		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Flags = EHttpServerResponseFlags::None;
		WriteContext.ResetContext(MoveTemp(Response));
		TestFalse(TEXT("IsMultipleWriteStream with None"), WriteContext.IsMultipleWriteStream());
		TestFalse(TEXT("HasAdditionalWrites with None"), WriteContext.HasAdditionalWrites());
	}

	// MultipleWriteStream only
	{
		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Flags = EHttpServerResponseFlags::MultipleWriteStream;
		WriteContext.ResetContext(MoveTemp(Response));
		TestTrue(TEXT("IsMultipleWriteStream with MultipleWriteStream"), WriteContext.IsMultipleWriteStream());
		TestFalse(TEXT("HasAdditionalWrites with MultipleWriteStream only"), WriteContext.HasAdditionalWrites());
	}

	// MultipleWriteStream | HasAdditionalWrites
	{
		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites;
		WriteContext.ResetContext(MoveTemp(Response));
		TestTrue(TEXT("IsMultipleWriteStream with both"), WriteContext.IsMultipleWriteStream());
		TestTrue(TEXT("HasAdditionalWrites with both"), WriteContext.HasAdditionalWrites());
	}

	// HasAdditionalWrites alone (without MultipleWriteStream) - should be false
	{
		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Flags = EHttpServerResponseFlags::HasAdditionalWrites;
		WriteContext.ResetContext(MoveTemp(Response));
		TestFalse(TEXT("IsMultipleWriteStream with HasAdditionalWrites alone"), WriteContext.IsMultipleWriteStream());
		TestFalse(TEXT("HasAdditionalWrites without MultipleWriteStream"), WriteContext.HasAdditionalWrites());
	}

	// All three flags
	{
		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites | EHttpServerResponseFlags::SkipHeaderWrite;
		WriteContext.ResetContext(MoveTemp(Response));
		TestTrue(TEXT("IsMultipleWriteStream with all flags"), WriteContext.IsMultipleWriteStream());
		TestTrue(TEXT("HasAdditionalWrites with all flags"), WriteContext.HasAdditionalWrites());
	}

	return true;
}

// Helper: create a loopback TCP socket pair for integration testing
namespace HttpServerTestHelpers
{
	struct FLoopbackSocketPair
	{
		FSocket* ServerSocket = nullptr;
		FSocket* ClientSocket = nullptr;
		FSocket* ListenerSocket = nullptr;
		ISocketSubsystem* SocketSubsystem = nullptr;
		bool bValid = false;

		bool Create()
		{
			SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			if (!SocketSubsystem)
			{
				return false;
			}

			// Create listener
			ListenerSocket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("TestListener"), false);
			if (!ListenerSocket)
			{
				return false;
			}

			TSharedRef<FInternetAddr> ListenerAddr = SocketSubsystem->CreateInternetAddr();
			ListenerAddr->SetIp(0x7F000001); // 127.0.0.1
			ListenerAddr->SetPort(0); // OS-assigned

			if (!ListenerSocket->Bind(*ListenerAddr))
			{
				Destroy();
				return false;
			}
			if (!ListenerSocket->Listen(1))
			{
				Destroy();
				return false;
			}

			// Get the assigned port
			TSharedRef<FInternetAddr> BoundAddr = SocketSubsystem->CreateInternetAddr();
			ListenerSocket->GetAddress(*BoundAddr);
			int32 Port = BoundAddr->GetPort();

			// Create client and connect
			ClientSocket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("TestClient"), false);
			if (!ClientSocket)
			{
				Destroy();
				return false;
			}

			TSharedRef<FInternetAddr> ConnectAddr = SocketSubsystem->CreateInternetAddr();
			ConnectAddr->SetIp(0x7F000001);
			ConnectAddr->SetPort(Port);
			if (!ClientSocket->Connect(*ConnectAddr))
			{
				Destroy();
				return false;
			}

			// Accept server-side connection
			bool bHasPendingConnection = false;
			ListenerSocket->WaitForPendingConnection(bHasPendingConnection, FTimespan::FromSeconds(2.0));
			if (!bHasPendingConnection)
			{
				Destroy();
				return false;
			}
			ServerSocket = ListenerSocket->Accept(TEXT("TestServerConn"));
			if (!ServerSocket)
			{
				Destroy();
				return false;
			}

			// Set non-blocking
			ServerSocket->SetNonBlocking(true);
			ClientSocket->SetNonBlocking(true);

			bValid = true;
			return true;
		}

		void Destroy()
		{
			if (SocketSubsystem)
			{
				if (ServerSocket)
				{
					SocketSubsystem->DestroySocket(ServerSocket);
					ServerSocket = nullptr;
				}
				if (ClientSocket)
				{
					SocketSubsystem->DestroySocket(ClientSocket);
					ClientSocket = nullptr;
				}
				if (ListenerSocket)
				{
					SocketSubsystem->DestroySocket(ListenerSocket);
					ListenerSocket = nullptr;
				}
			}
			bValid = false;
		}

		~FLoopbackSocketPair()
		{
			Destroy();
		}
	};

	// Read all available data from socket with a small wait
	static FString ReadAllFromSocket(FSocket* Socket, float WaitSeconds = 0.5f)
	{
		// Wait for data to arrive
		Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromSeconds(WaitSeconds));

		TArray<uint8> Buffer;
		uint8 ReadBuf[4096];
		int32 BytesRead = 0;

		while (Socket->Recv(ReadBuf, sizeof(ReadBuf), BytesRead))
		{
			if (BytesRead > 0)
			{
				Buffer.Append(ReadBuf, BytesRead);
			}
			else
			{
				break;
			}
		}

		FUTF8ToTCHAR Converter(reinterpret_cast<const ANSICHAR*>(Buffer.GetData()), Buffer.Num());
		return FString(Converter.Length(), Converter.Get());
	}

	// Drive WriteStream in a loop until Done or max iterations
	static EHttpConnectionContextState DriveWriteStream(FHttpConnectionResponseWriteContext& WriteContext, int32 MaxIterations = 1000)
	{
		EHttpConnectionContextState LastState = EHttpConnectionContextState::Continue;
		for (int32 i = 0; i < MaxIterations; ++i)
		{
			LastState = WriteContext.WriteStream(0.016f);
			if (LastState != EHttpConnectionContextState::Continue)
			{
				return LastState;
			}
		}
		return LastState;
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerWriteContextStreamingTest, "System.Online.HttpServer.SSE.WriteContextStreaming", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerWriteContextStreamingTest::RunTest(const FString& Parameters)
{
	using namespace HttpServerTestHelpers;

	// --- Sub-test A: Normal response includes Content-Length ---
	{
		FLoopbackSocketPair Sockets;
		if (!TestTrue(TEXT("A: Socket pair created"), Sockets.Create()))
		{
			return false;
		}

		FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Code = EHttpServerResponseCodes::Ok;
		Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		const FString BodyStr = TEXT("hello");
		FTCHARToUTF8 BodyUtf8(*BodyStr);
		Response->Body.Append(reinterpret_cast<const uint8*>(BodyUtf8.Get()), BodyUtf8.Length());

		WriteContext.ResetContext(MoveTemp(Response));
		EHttpConnectionContextState State = DriveWriteStream(WriteContext);
		TestEqual(TEXT("A: WriteStream returns Done"), State, EHttpConnectionContextState::Done);

		FString Output = ReadAllFromSocket(Sockets.ClientSocket);
		TestTrue(TEXT("A: Response contains content-length"), Output.Contains(TEXT("content-length: 5")));
		TestTrue(TEXT("A: Response contains body"), Output.Contains(TEXT("hello")));
	}

	// --- Sub-test B: MultipleWriteStream suppresses Content-Length ---
	{
		FLoopbackSocketPair Sockets;
		if (!TestTrue(TEXT("B: Socket pair created"), Sockets.Create()))
		{
			return false;
		}

		FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Code = EHttpServerResponseCodes::Ok;
		Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		Response->Flags = EHttpServerResponseFlags::MultipleWriteStream;
		const FString EventStr = TEXT("event: ping\n\n");
		FTCHARToUTF8 EventUtf8(*EventStr);
		Response->Body.Append(reinterpret_cast<const uint8*>(EventUtf8.Get()), EventUtf8.Length());

		WriteContext.ResetContext(MoveTemp(Response));
		EHttpConnectionContextState State = DriveWriteStream(WriteContext);
		TestEqual(TEXT("B: WriteStream returns Done"), State, EHttpConnectionContextState::Done);

		FString Output = ReadAllFromSocket(Sockets.ClientSocket);
		TestFalse(TEXT("B: No content-length header for stream"), Output.Contains(TEXT("content-length")));
		TestTrue(TEXT("B: Body present"), Output.Contains(TEXT("event: ping")));
	}

	// --- Sub-test C: StreamingBodyQueue also suppresses Content-Length ---
	{
		FLoopbackSocketPair Sockets;
		if (!TestTrue(TEXT("C: Socket pair created"), Sockets.Create()))
		{
			return false;
		}

		FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

		TSharedPtr<TQueue<TArray<uint8>, EQueueMode::Spsc>> Queue = MakeShared<TQueue<TArray<uint8>, EQueueMode::Spsc>>();
		// Enqueue a single chunk, then mark complete — the queue path requires at least one
		// produced chunk to un-defer headers (producer always writes before signaling complete).
		{
			const FString Chunk = TEXT("data: only\n\n");
			FTCHARToUTF8 Utf8(*Chunk);
			TArray<uint8> ChunkData(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
			Queue->Enqueue(MoveTemp(ChunkData));
		}

		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Code = EHttpServerResponseCodes::Ok;
		Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		Response->StreamingBodyQueue = Queue;
		Response->StreamingBodyComplete = MakeShared<TAtomic<bool>>(true); // already complete after the single enqueue

		WriteContext.ResetContext(MoveTemp(Response));
		EHttpConnectionContextState State = DriveWriteStream(WriteContext);
		TestEqual(TEXT("C: WriteStream returns Done"), State, EHttpConnectionContextState::Done);

		FString Output = ReadAllFromSocket(Sockets.ClientSocket);
		TestFalse(TEXT("C: No content-length for queue-based streaming"), Output.Contains(TEXT("content-length")));
		TestTrue(TEXT("C: Body present"), Output.Contains(TEXT("data: only")));
	}

	// --- Sub-test D: SkipHeaderWrite produces body-only output ---
	{
		FLoopbackSocketPair Sockets;
		if (!TestTrue(TEXT("D: Socket pair created"), Sockets.Create()))
		{
			return false;
		}

		FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Code = EHttpServerResponseCodes::Ok;
		Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		Response->Flags = EHttpServerResponseFlags::SkipHeaderWrite;
		const FString ChunkStr = TEXT("data: chunk2\n\n");
		FTCHARToUTF8 ChunkUtf8(*ChunkStr);
		Response->Body.Append(reinterpret_cast<const uint8*>(ChunkUtf8.Get()), ChunkUtf8.Length());

		WriteContext.ResetContext(MoveTemp(Response));
		EHttpConnectionContextState State = DriveWriteStream(WriteContext);
		TestEqual(TEXT("D: WriteStream returns Done"), State, EHttpConnectionContextState::Done);

		FString Output = ReadAllFromSocket(Sockets.ClientSocket);
		TestFalse(TEXT("D: No HTTP status line"), Output.StartsWith(TEXT("HTTP/")));
		TestTrue(TEXT("D: Body content present"), Output.Contains(TEXT("data: chunk2")));
	}

	// --- Sub-test E: StreamingBodyQueue draining and completion ---
	{
		FLoopbackSocketPair Sockets;
		if (!TestTrue(TEXT("E: Socket pair created"), Sockets.Create()))
		{
			return false;
		}

		FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

		TSharedPtr<TQueue<TArray<uint8>, EQueueMode::Spsc>> Queue = MakeShared<TQueue<TArray<uint8>, EQueueMode::Spsc>>();
		TSharedPtr<TAtomic<bool>> Complete = MakeShared<TAtomic<bool>>(false);

		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Code = EHttpServerResponseCodes::Ok;
		Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		Response->StreamingBodyQueue = Queue;
		Response->StreamingBodyComplete = Complete;

		WriteContext.ResetContext(MoveTemp(Response));

		// Write headers, should Continue (queue empty, not complete)
		EHttpConnectionContextState State = WriteContext.WriteStream(0.016f);
		// May need a few iterations to finish writing headers
		for (int32 i = 0; i < 100 && State == EHttpConnectionContextState::Continue; ++i)
		{
			State = WriteContext.WriteStream(0.016f);
		}
		// After headers written and body empty, should still Continue since not complete
		TestEqual(TEXT("E: Continue while stream incomplete"), State, EHttpConnectionContextState::Continue);

		// Enqueue first event
		{
			const FString Event1 = TEXT("data: event1\n\n");
			FTCHARToUTF8 Utf8(*Event1);
			TArray<uint8> EventData(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
			Queue->Enqueue(MoveTemp(EventData));
		}

		// Drive a few writes to flush first event
		for (int32 i = 0; i < 100; ++i)
		{
			State = WriteContext.WriteStream(0.016f);
			if (State != EHttpConnectionContextState::Continue)
			{
				break;
			}
		}
		TestEqual(TEXT("E: Still Continue after first event"), State, EHttpConnectionContextState::Continue);

		// Enqueue second event and signal completion
		{
			const FString Event2 = TEXT("data: event2\n\n");
			FTCHARToUTF8 Utf8(*Event2);
			TArray<uint8> EventData(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
			Queue->Enqueue(MoveTemp(EventData));
		}
		Complete->Store(true, EMemoryOrder::Relaxed);

		State = DriveWriteStream(WriteContext);
		TestEqual(TEXT("E: Done after completion signaled"), State, EHttpConnectionContextState::Done);

		FString Output = ReadAllFromSocket(Sockets.ClientSocket);
		TestTrue(TEXT("E: Contains event1"), Output.Contains(TEXT("data: event1")));
		TestTrue(TEXT("E: Contains event2"), Output.Contains(TEXT("data: event2")));
	}

	// --- Sub-test F: Connection stays alive while stream incomplete ---
	{
		FLoopbackSocketPair Sockets;
		if (!TestTrue(TEXT("F: Socket pair created"), Sockets.Create()))
		{
			return false;
		}

		FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

		TSharedPtr<TQueue<TArray<uint8>, EQueueMode::Spsc>> Queue = MakeShared<TQueue<TArray<uint8>, EQueueMode::Spsc>>();
		TSharedPtr<TAtomic<bool>> Complete = MakeShared<TAtomic<bool>>(false);

		// Pre-enqueue a chunk so there's data to write
		{
			const FString Chunk = TEXT("data: keepalive\n\n");
			FTCHARToUTF8 Utf8(*Chunk);
			TArray<uint8> ChunkData(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
			Queue->Enqueue(MoveTemp(ChunkData));
		}

		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Code = EHttpServerResponseCodes::Ok;
		Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		Response->StreamingBodyQueue = Queue;
		Response->StreamingBodyComplete = Complete;

		WriteContext.ResetContext(MoveTemp(Response));

		// Drive writes until chunk is flushed
		EHttpConnectionContextState State = EHttpConnectionContextState::Continue;
		for (int32 i = 0; i < 1000 && State == EHttpConnectionContextState::Continue; ++i)
		{
			State = WriteContext.WriteStream(0.016f);
		}
		// Queue is drained but not marked complete - should be Continue
		TestEqual(TEXT("F: Continue when stream not complete"), State, EHttpConnectionContextState::Continue);

		// Now mark complete
		Complete->Store(true, EMemoryOrder::Relaxed);
		State = WriteContext.WriteStream(0.016f);
		TestEqual(TEXT("F: Done after marking complete"), State, EHttpConnectionContextState::Done);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerConnectionSSEStateTransitionTest, "System.Online.HttpServer.SSE.ConnectionStateTransition", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerConnectionSSEStateTransitionTest::RunTest(const FString& Parameters)
{
	using namespace HttpServerTestHelpers;

	FLoopbackSocketPair Sockets;
	if (!TestTrue(TEXT("Socket pair created"), Sockets.Create()))
	{
		return false;
	}

	// Create router with a route that responds with SSE flags
	TSharedPtr<FHttpRouter> Router = MakeShared<FHttpRouter>();
	FHttpRouteHandle RouteHandle = Router->BindRoute(
		FHttpPath(TEXT("/test")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateLambda([](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
		{
			TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
			Response->Code = EHttpServerResponseCodes::Ok;
			Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
			Response->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites;
			const FString EventStr = TEXT("data: sse-event\n\n");
			FTCHARToUTF8 Utf8(*EventStr);
			Response->Body.Append(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
			OnComplete(MoveTemp(Response));
			return true;
		})
	);
	TestTrue(TEXT("Route handle valid"), RouteHandle.IsValid());

	// Create connection using the server-side accepted socket
	TSharedPtr<FHttpConnection> Connection = MakeShared<FHttpConnection>(Sockets.ServerSocket, Router, 0, 0);
	// Prevent FLoopbackSocketPair destructor from double-destroying the socket
	Sockets.ServerSocket = nullptr;

	TestEqual(TEXT("Initial state is AwaitingRead"), Connection->GetState(), EHttpConnectionState::AwaitingRead);

	// Write a valid HTTP/1.1 GET request from the client side
	const FString HttpRequest = TEXT("GET /test HTTP/1.1\r\nHost: localhost\r\nConnection: keep-alive\r\n\r\n");
	FTCHARToUTF8 RequestUtf8(*HttpRequest);
	int32 BytesSent = 0;
	Sockets.ClientSocket->Send(reinterpret_cast<const uint8*>(RequestUtf8.Get()), RequestUtf8.Length(), BytesSent);

	// Tick the connection until it transitions through the full cycle
	// AwaitingRead -> Reading -> AwaitingProcessing -> Writing -> CompleteWrite
	bool bReachedExpectedState = false;
	for (int32 i = 0; i < 5000; ++i)
	{
		Connection->Tick(0.016f);
		EHttpConnectionState CurrentState = Connection->GetState();

		if (CurrentState == EHttpConnectionState::AwaitingProcessing)
		{
			// Check if this is post-write (not pre-write)
			// Pre-write AwaitingProcessing happens before we enter Writing.
			// Post-write AwaitingProcessing is the SSE re-invocation state.
			// We detect this by checking if the client received response data.
			FString Output = ReadAllFromSocket(Sockets.ClientSocket, 0.05f);
			if (Output.Contains(TEXT("data: sse-event")))
			{
				bReachedExpectedState = true;
				break;
			}
		}

		if (CurrentState == EHttpConnectionState::AwaitingRead)
		{
			// If we went back to AwaitingRead after writing, the SSE flag didn't work
			// But only check after we've had time to process
			if (i > 100)
			{
				FString Output = ReadAllFromSocket(Sockets.ClientSocket, 0.05f);
				if (Output.Contains(TEXT("data: sse-event")))
				{
					AddError(TEXT("Connection returned to AwaitingRead instead of AwaitingProcessing after SSE write"));
					break;
				}
			}
		}

		if (CurrentState == EHttpConnectionState::Destroyed)
		{
			AddError(TEXT("Connection was destroyed unexpectedly"));
			break;
		}
	}

	TestTrue(TEXT("Connection reached AwaitingProcessing after SSE write (re-invocation path)"), bReachedExpectedState);

	// Cleanup: destroy the connection
	Connection->RequestDestroy(false);

	Router->UnbindRoute(RouteHandle);

	return true;
}

// ---------------------------------------------------------------------------
// SSE / Streaming Tests — callback re-invocation flag patterns
//
// Server-sent events can be driven without a StreamingBodyQueue by stashing
// the FHttpResultCallback and re-invoking it for each frame. Three flag
// combinations cover the full lifecycle; these tests lock in the wire-level
// behavior each one must produce:
//   1. Open stream:  MWS|HAW with custom SSE headers, empty body
//   2. Mid-stream:   MWS|HAW|SkipHeaderWrite with SSE-formatted body
//   3. Final frame:  MWS|SkipHeaderWrite (no HAW) with SSE-formatted body
// ---------------------------------------------------------------------------

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerSSEOpenStreamHeadersOnlyTest, "System.Online.HttpServer.SSE.OpenStreamHeadersOnly", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerSSEOpenStreamHeadersOnlyTest::RunTest(const FString& Parameters)
{
	using namespace HttpServerTestHelpers;

	FLoopbackSocketPair Sockets;
	if (!TestTrue(TEXT("Socket pair created"), Sockets.Create()))
	{
		return false;
	}

	FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

	// Use case: open an SSE stream — MWS|HAW, text/event-stream content type,
	// Connection: keep-alive, Cache-Control: no-cache, empty body.
	// Must emit all headers verbatim and omit Content-Length.
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(FString(TEXT("")), TEXT("text/event-stream"));
	Response->Code = EHttpServerResponseCodes::Ok;
	Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
	Response->Headers.Add(TEXT("Connection"), { TEXT("keep-alive") });
	Response->Headers.Add(TEXT("Cache-Control"), { TEXT("no-cache") });
	Response->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites;

	WriteContext.ResetContext(MoveTemp(Response));
	EHttpConnectionContextState State = DriveWriteStream(WriteContext);
	TestEqual(TEXT("WriteStream returns Done for empty-body open stream"), State, EHttpConnectionContextState::Done);

	// HAW must be observable post-write so the connection knows to re-invoke OnComplete
	TestTrue(TEXT("HasAdditionalWrites() returns true with MWS|HAW"), WriteContext.HasAdditionalWrites());
	TestTrue(TEXT("IsMultipleWriteStream() returns true"), WriteContext.IsMultipleWriteStream());

	const FString Output = ReadAllFromSocket(Sockets.ClientSocket);
	TestTrue(TEXT("Starts with HTTP/1.1 200"), Output.StartsWith(TEXT("HTTP/1.1 200")));
	TestTrue(TEXT("Content-Type: text/event-stream on wire"), Output.Contains(TEXT("content-type: text/event-stream")));
	TestTrue(TEXT("Connection: keep-alive on wire"), Output.Contains(TEXT("connection: keep-alive")));
	TestTrue(TEXT("Cache-Control: no-cache on wire"), Output.Contains(TEXT("cache-control: no-cache")));
	TestFalse(TEXT("No content-length for streaming response"), Output.Contains(TEXT("content-length")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerSSEMidStreamEventTest, "System.Online.HttpServer.SSE.MidStreamEvent", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerSSEMidStreamEventTest::RunTest(const FString& Parameters)
{
	using namespace HttpServerTestHelpers;

	FLoopbackSocketPair Sockets;
	if (!TestTrue(TEXT("Socket pair created"), Sockets.Create()))
	{
		return false;
	}

	FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

	// Use case: mid-stream event on an already-open SSE stream —
	// MWS|HAW|SkipHeaderWrite, SSE body, no HTTP status line or headers on the wire.
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::Ok;
	Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
	Response->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites | EHttpServerResponseFlags::SkipHeaderWrite;
	const FString EventStr = TEXT("data: mid-chunk\n\n");
	FTCHARToUTF8 EventUtf8(*EventStr);
	Response->Body.Append(reinterpret_cast<const uint8*>(EventUtf8.Get()), EventUtf8.Length());

	WriteContext.ResetContext(MoveTemp(Response));
	EHttpConnectionContextState State = DriveWriteStream(WriteContext);
	TestEqual(TEXT("WriteStream returns Done for mid-stream event"), State, EHttpConnectionContextState::Done);
	TestTrue(TEXT("HasAdditionalWrites() returns true"), WriteContext.HasAdditionalWrites());

	const FString Output = ReadAllFromSocket(Sockets.ClientSocket);
	TestFalse(TEXT("No HTTP status line for SkipHeaderWrite"), Output.StartsWith(TEXT("HTTP/")));
	TestFalse(TEXT("No content-type header for SkipHeaderWrite"), Output.Contains(TEXT("content-type")));
	TestFalse(TEXT("No content-length for SkipHeaderWrite"), Output.Contains(TEXT("content-length")));
	TestTrue(TEXT("Body emitted verbatim"), Output.Contains(TEXT("data: mid-chunk")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerSSEFinalFrameTest, "System.Online.HttpServer.SSE.FinalFrame", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerSSEFinalFrameTest::RunTest(const FString& Parameters)
{
	using namespace HttpServerTestHelpers;

	FLoopbackSocketPair Sockets;
	if (!TestTrue(TEXT("Socket pair created"), Sockets.Create()))
	{
		return false;
	}

	FHttpConnectionResponseWriteContext WriteContext(Sockets.ServerSocket);

	// Use case: final frame on an SSE stream — MWS|SkipHeaderWrite with no HAW.
	// Body only, then the connection state machine transitions out of the stream.
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	Response->Code = EHttpServerResponseCodes::Ok;
	Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
	Response->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::SkipHeaderWrite;
	const FString EventStr = TEXT("data: final-chunk\n\n");
	FTCHARToUTF8 EventUtf8(*EventStr);
	Response->Body.Append(reinterpret_cast<const uint8*>(EventUtf8.Get()), EventUtf8.Length());

	WriteContext.ResetContext(MoveTemp(Response));
	EHttpConnectionContextState State = DriveWriteStream(WriteContext);
	TestEqual(TEXT("WriteStream returns Done for final frame"), State, EHttpConnectionContextState::Done);

	// No HAW — signals the connection state machine this is the last write
	TestFalse(TEXT("HasAdditionalWrites() returns false without HAW flag"), WriteContext.HasAdditionalWrites());
	TestTrue(TEXT("IsMultipleWriteStream() still true"), WriteContext.IsMultipleWriteStream());

	const FString Output = ReadAllFromSocket(Sockets.ClientSocket);
	TestFalse(TEXT("No HTTP status line for SkipHeaderWrite"), Output.StartsWith(TEXT("HTTP/")));
	TestTrue(TEXT("Body emitted verbatim"), Output.Contains(TEXT("data: final-chunk")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerSSECallbackReinvocationTest, "System.Online.HttpServer.SSE.CallbackReinvocation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerSSECallbackReinvocationTest::RunTest(const FString& Parameters)
{
	using namespace HttpServerTestHelpers;

	FLoopbackSocketPair Sockets;
	if (!TestTrue(TEXT("Socket pair created"), Sockets.Create()))
	{
		return false;
	}

	// The route handler is invoked once (on the first request). It stashes OnComplete
	// for the test to re-invoke with subsequent SSE frames, exercising the full
	// open → mid-stream → final-frame lifecycle on a single connection.
	TSharedPtr<FHttpResultCallback> StashedCallback = MakeShared<FHttpResultCallback>();

	TSharedPtr<FHttpRouter> Router = MakeShared<FHttpRouter>();
	FHttpRouteHandle RouteHandle = Router->BindRoute(
		FHttpPath(TEXT("/sse")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateLambda([StashedCallback](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
		{
			*StashedCallback = OnComplete;

			// Stage 1: open stream with MWS|HAW, SSE content type, custom headers, empty body.
			TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(FString(TEXT("")), TEXT("text/event-stream"));
			Response->Code = EHttpServerResponseCodes::Ok;
			Response->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
			Response->Headers.Add(TEXT("Connection"), { TEXT("keep-alive") });
			Response->Headers.Add(TEXT("Cache-Control"), { TEXT("no-cache") });
			Response->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites;
			OnComplete(MoveTemp(Response));
			return true;
		})
	);
	TestTrue(TEXT("Route handle valid"), RouteHandle.IsValid());

	TSharedPtr<FHttpConnection> Connection = MakeShared<FHttpConnection>(Sockets.ServerSocket, Router, 0, 0);
	// Prevent the socket-pair destructor from double-destroying the server socket now owned by the connection.
	Sockets.ServerSocket = nullptr;

	const FString HttpRequest = TEXT("GET /sse HTTP/1.1\r\nHost: localhost\r\nConnection: keep-alive\r\n\r\n");
	FTCHARToUTF8 RequestUtf8(*HttpRequest);
	int32 BytesSent = 0;
	Sockets.ClientSocket->Send(reinterpret_cast<const uint8*>(RequestUtf8.Get()), RequestUtf8.Length(), BytesSent);

	auto TickUntil = [&](EHttpConnectionState DesiredState, const TCHAR* StageName, int32 MaxIterations = 5000) -> bool
	{
		for (int32 i = 0; i < MaxIterations; ++i)
		{
			Connection->Tick(0.016f);
			const EHttpConnectionState Current = Connection->GetState();
			if (Current == DesiredState)
			{
				return true;
			}
			if (Current == EHttpConnectionState::Destroyed)
			{
				AddError(FString::Printf(TEXT("%s: connection destroyed while waiting for expected state"), StageName));
				return false;
			}
		}
		AddError(FString::Printf(TEXT("%s: never reached expected state"), StageName));
		return false;
	};

	// Stage 1: open stream
	if (!TickUntil(EHttpConnectionState::AwaitingProcessing, TEXT("Open")))
	{
		Connection->RequestDestroy(false);
		Router->UnbindRoute(RouteHandle);
		return false;
	}

	FString OpenOutput = ReadAllFromSocket(Sockets.ClientSocket, 0.1f);
	TestTrue(TEXT("Open: has HTTP status"), OpenOutput.StartsWith(TEXT("HTTP/1.1 200")));
	TestTrue(TEXT("Open: has text/event-stream content-type"), OpenOutput.Contains(TEXT("content-type: text/event-stream")));
	TestTrue(TEXT("Open: has keep-alive"), OpenOutput.Contains(TEXT("connection: keep-alive")));
	TestTrue(TEXT("Open: has cache-control no-cache"), OpenOutput.Contains(TEXT("cache-control: no-cache")));
	TestFalse(TEXT("Open: no content-length"), OpenOutput.Contains(TEXT("content-length")));
	TestTrue(TEXT("Stashed callback is bound"), static_cast<bool>(*StashedCallback));

	// Stage 2: mid-stream event (re-invoke stashed callback)
	{
		TUniquePtr<FHttpServerResponse> Mid = MakeUnique<FHttpServerResponse>();
		Mid->Code = EHttpServerResponseCodes::Ok;
		Mid->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		Mid->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites | EHttpServerResponseFlags::SkipHeaderWrite;
		const FString Chunk = TEXT("data: mid-chunk\n\n");
		FTCHARToUTF8 Utf8(*Chunk);
		Mid->Body.Append(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
		(*StashedCallback)(MoveTemp(Mid));
	}

	if (!TickUntil(EHttpConnectionState::AwaitingProcessing, TEXT("Mid")))
	{
		Connection->RequestDestroy(false);
		Router->UnbindRoute(RouteHandle);
		return false;
	}

	FString MidOutput = ReadAllFromSocket(Sockets.ClientSocket, 0.1f);
	TestFalse(TEXT("Mid: no HTTP status line"), MidOutput.StartsWith(TEXT("HTTP/")));
	TestTrue(TEXT("Mid: body present on wire"), MidOutput.Contains(TEXT("data: mid-chunk")));

	// Stage 3: final frame — no HAW, connection should return to AwaitingRead (keep-alive) after write.
	{
		TUniquePtr<FHttpServerResponse> Final = MakeUnique<FHttpServerResponse>();
		Final->Code = EHttpServerResponseCodes::Ok;
		Final->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		Final->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::SkipHeaderWrite;
		const FString Chunk = TEXT("data: final-chunk\n\n");
		FTCHARToUTF8 Utf8(*Chunk);
		Final->Body.Append(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
		(*StashedCallback)(MoveTemp(Final));
	}

	const bool bReachedAwaitingRead = TickUntil(EHttpConnectionState::AwaitingRead, TEXT("Final"));
	TestTrue(TEXT("Final: connection returned to AwaitingRead after non-HAW write"), bReachedAwaitingRead);

	FString FinalOutput = ReadAllFromSocket(Sockets.ClientSocket, 0.1f);
	TestFalse(TEXT("Final: no HTTP status line"), FinalOutput.StartsWith(TEXT("HTTP/")));
	TestTrue(TEXT("Final: body present on wire"), FinalOutput.Contains(TEXT("data: final-chunk")));

	Connection->RequestDestroy(false);
	Router->UnbindRoute(RouteHandle);

	return true;
}

// Regression: a synchronous handler that invokes `OnComplete` twice on the same stack
// crashed with `check(AwaitingProcessing == State)` at HttpConnection.cpp:184 when the
// connection had transitioned out of `AwaitingProcessing` between the two invocations.
//
// This is exactly the pattern an MCP `tools/call` produces when the tool's `RunAsync`
// falls back to the default `Run`-then-`OnComplete` path: the SSE handshake response
// and the tool result are emitted on the same call stack. Production hit `Destroyed`
// when the first write failed (`HandleWriteError` -> `Destroy`); this test reproduces
// the same end state deterministically by tearing the connection down inside the
// handler between the two `OnComplete` calls.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHttpServerSSECallbackSyncReentryAfterDestroyTest, "System.Online.HttpServer.SSE.CallbackSyncReentryAfterDestroy", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FHttpServerSSECallbackSyncReentryAfterDestroyTest::RunTest(const FString& Parameters)
{
	using namespace HttpServerTestHelpers;

	FLoopbackSocketPair Sockets;
	if (!TestTrue(TEXT("Socket pair created"), Sockets.Create()))
	{
		return false;
	}

	TSharedPtr<FHttpRouter> Router = MakeShared<FHttpRouter>();
	TSharedPtr<FHttpConnection> Connection = MakeShared<FHttpConnection>(Sockets.ServerSocket, Router, 0, 0);
	// Prevent the socket-pair destructor from double-destroying the server socket now owned by the connection.
	Sockets.ServerSocket = nullptr;

	TWeakPtr<FHttpConnection> WeakConn = Connection;
	bool bFirstOnCompleteInvoked = false;
	bool bSecondOnCompleteReturned = false;

	FHttpRouteHandle RouteHandle = Router->BindRoute(
		FHttpPath(TEXT("/sse")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateLambda([WeakConn, &bFirstOnCompleteInvoked, &bSecondOnCompleteReturned](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete) -> bool
		{
			// Stage 1: open the SSE stream (MWS|HAW, empty body, custom headers).
			TUniquePtr<FHttpServerResponse> Open = FHttpServerResponse::Create(FString(TEXT("")), TEXT("text/event-stream"));
			Open->Code = EHttpServerResponseCodes::Ok;
			Open->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
			Open->Headers.Add(TEXT("Connection"), { TEXT("keep-alive") });
			Open->Headers.Add(TEXT("Cache-Control"), { TEXT("no-cache") });
			Open->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites;
			OnComplete(MoveTemp(Open));
			bFirstOnCompleteInvoked = true;

			// Force the connection into `Destroyed` between the two invocations. This is the
			// state the production `HandleWriteError` -> `Destroy` path leaves the connection in
			// when the first write fails on the wire.
			if (TSharedPtr<FHttpConnection> Pinned = WeakConn.Pin())
			{
				Pinned->RequestDestroy(false);
			}

			// Stage 2: emit the second frame on the same stack. Pre-fix this asserted in the
			// connection's `OnProcessingComplete` lambda; post-fix it must return cleanly.
			TUniquePtr<FHttpServerResponse> Second = MakeUnique<FHttpServerResponse>();
			Second->Code = EHttpServerResponseCodes::Ok;
			Second->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
			Second->Flags = EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites | EHttpServerResponseFlags::SkipHeaderWrite;
			const FString Chunk = TEXT("data: tool-result\n\n");
			FTCHARToUTF8 Utf8(*Chunk);
			Second->Body.Append(reinterpret_cast<const uint8*>(Utf8.Get()), Utf8.Length());
			OnComplete(MoveTemp(Second));
			bSecondOnCompleteReturned = true;

			return true;
		})
	);
	TestTrue(TEXT("Route handle valid"), RouteHandle.IsValid());

	const FString HttpRequest = TEXT("GET /sse HTTP/1.1\r\nHost: localhost\r\nConnection: keep-alive\r\n\r\n");
	FTCHARToUTF8 RequestUtf8(*HttpRequest);
	int32 BytesSent = 0;
	Sockets.ClientSocket->Send(reinterpret_cast<const uint8*>(RequestUtf8.Get()), RequestUtf8.Length(), BytesSent);

	// Tick until the connection enters `Destroyed`. The handler runs inside the tick that
	// promotes the connection to `AwaitingProcessing`, so the destroyed state is observed
	// on the same tick the handler returns. Stop ticking immediately to avoid the
	// `ensure(false)` guard for `Destroyed` in `FHttpConnection::Tick`.
	bool bReachedDestroyed = false;
	for (int32 i = 0; i < 5000; ++i)
	{
		Connection->Tick(0.016f);
		if (Connection->GetState() == EHttpConnectionState::Destroyed)
		{
			bReachedDestroyed = true;
			break;
		}
	}

	TestTrue(TEXT("Handler invoked first OnComplete"), bFirstOnCompleteInvoked);
	TestTrue(TEXT("Second OnComplete returned without crashing"), bSecondOnCompleteReturned);
	TestTrue(TEXT("Connection ended in Destroyed state"), bReachedDestroyed);

	Router->UnbindRoute(RouteHandle);

	return true;
}