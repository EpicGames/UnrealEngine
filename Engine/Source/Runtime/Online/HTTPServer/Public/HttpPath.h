// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HttpRequestHandler.h"
#include "Misc/CString.h"

struct FHttpPath
{

public:

	/**
	 * Constructor
	 */
	HTTPSERVER_API FHttpPath();

	/**
	 * Constructor
	 *
	 * @param Path - The http path
	 */
	HTTPSERVER_API FHttpPath(FString InPath);

	/**
     * Gets the respective http path as a string
     */
	HTTPSERVER_API const FString& GetPath() const;

	/**
	 * Parses the respective http path into a caller-supplied array
	 * 
	 * @param OutPathTokens The caller-allocated tokens array
	 * @return The number of tokens parsed
	 */
	HTTPSERVER_API uint32 ParsePathTokens(TArray<FString>& OutPathTokens) const;

	/**
	 * Sets the server-relative http path
	 *
	 * @param Path the new path to assign
	 */
	HTTPSERVER_API void SetPath(FString Path);

	/**
	 * Determines if the path is valid
	 */
	HTTPSERVER_API bool IsValidPath() const;

	/**
	 * Determines if the path is /
	 */
	HTTPSERVER_API bool IsRoot() const;

	/**
	 * Re-path this path sans the path of another
	 * MakeRelative(/a/b/c/d, /a/b) => /c/d  
	 *
	 * @param OtherPath The path to make relative against
	 */
	HTTPSERVER_API void MakeRelative(const FString& OtherPath);

	// TMap<> Comparer
	bool operator==(const FHttpPath& Other) const
	{
		bool bEqual = (0 == Path.Compare(Other.Path, ESearchCase::CaseSensitive));
		return bEqual;
	}

	friend uint32 GetTypeHash(const FHttpPath& InPath)
	{
		return GetTypeHash(InPath.GetPath());
	}

private:

	/**
	 * Normalizes the internally stored Path
	 */
	void NormalizePath();

	/** The respective Http path */
	FString Path = "/";

};


