// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HttpResultCallback.h"

struct FHttpServerRequest;

/**
 * FHttpRequestHandler 
 *
 *  NOTE - Returning true implies that the delegate will eventually invoke OnComplete
 *  NOTE - Returning false implies that the delegate will never invoke OnComplete
 * 
 * @param Request The incoming http request to be handled
 * @param OnComplete The callback to invoke to write an http response
 * @return True if the request has been handled, false otherwise
 */
using FHttpRequestHandler = TDelegate<bool(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)>;

