// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HttpServerResponse.h"

/**
* FHttpResultCallback
* This callback is intended to be invoked exclusively by FHttpRequestHandler delegates.
*
* Synchronous reentry contract: a streaming handler using `MultipleWriteStream | HasAdditionalWrites`
* may invoke this callback up to twice on the same call stack (e.g. SSE open + a single follow-up
* frame). The second invocation is captured into the connection's single-slot pending response and
* drained once the in-flight write finishes. A third synchronous invocation asserts via `checkf`.
* Cross-tick invocations are not bounded.
*
* @param Response The response to write
*/
typedef TFunction<void(TUniquePtr<FHttpServerResponse>&& Response)> FHttpResultCallback;


