// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "HttpServerRequest.h"

struct FHttpRouteHandleInternal
{
	FHttpRouteHandleInternal(FString InPath, EHttpServerRequestVerbs InVerbs, FHttpRequestHandler InHandler)
		: Path(MoveTemp(InPath))
		, Verbs(InVerbs)
		, Handler(MoveTemp(InHandler))
	{ };

	/** The respective bound http path */
	const FString Path;
	/** The qualifying http verbs for which this route is valid */
	const EHttpServerRequestVerbs Verbs;
	/** The respective handler to be invoked */
	const FHttpRequestHandler Handler;
};

/**
 * FHttpRouteHandle 
 *  Returned by IHttpRouter BindRoute() invocations. 
 *  Clients may test this value to validate and relinquish http router bindings
*/
typedef TSharedPtr<const FHttpRouteHandleInternal> FHttpRouteHandle;
