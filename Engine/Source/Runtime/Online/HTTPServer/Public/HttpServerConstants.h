// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

enum class EHttpServerResponseCodes : uint16
{
	/**
	 * Response codes that can come back from an Http request
	 */
	 // status code not set yet
	Unknown = 0,
	// the request can be continued.
	Continue = 100,
	// the server has switched protocols in an upgrade header.
	SwitchProtocol = 101,
	// the request completed successfully.
	Ok = 200,
	// the request has been fulfilled and resulted in the creation of a new resource.
	Created = 201,
	// the request has been accepted for processing, but the processing has not been completed.
	Accepted = 202,
	// the returned meta information in the entity-header is not the definitive set available from the origin server.
	Partial = 203,
	// the server has fulfilled the request, but there is no new information to send back.
	NoContent = 204,
	// the request has been completed, and the client program should reset the document view that caused the request to be sent to allow the user to easily initiate another input action.
	ResetContent = 205,
	// the server has fulfilled the partial get request for the resource.
	PartialContent = 206,
	// the server couldn't decide what to return.
	Ambiguous = 300,
	// the requested resource has been assigned to a new permanent uri (uniform resource identifier), and any future references to this resource should be done using one of the returned uris.
	Moved = 301,
	// the requested resource resides temporarily under a different uri (uniform resource identifier).
	Redirect = 302,
	// the response to the request can be found under a different uri (uniform resource identifier) and should be retrieved using a get http verb on that resource.
	RedirectMethod = 303,
	// the requested resource has not been modified.
	NotModified = 304,
	// the requested resource must be accessed through the proxy given by the location field.
	UseProxy = 305,
	// the redirected request keeps the same http verb. http/1.1 behavior.
	RedirectKeepVerb = 307,
	// the request could not be processed by the server due to invalid syntax.
	BadRequest = 400,
	// the requested resource requires user authentication.
	Denied = 401,
	// not currently implemented in the http protocol.
	PaymentReq = 402,
	// the server understood the request, but is refusing to fulfill it.
	Forbidden = 403,
	// the server has not found anything matching the requested uri (uniform resource identifier).
	NotFound = 404,
	// the http verb used is not allowed.
	BadMethod = 405,
	// no responses acceptable to the client were found.
	NoneAcceptable = 406,
	// proxy authentication required.
	ProxyAuthReq = 407,
	// the server timed out waiting for the request.
	RequestTimeout = 408,
	// the request could not be completed due to a conflict with the current state of the resource. the user should resubmit with more information.
	Conflict = 409,
	// the requested resource is no longer available at the server, and no forwarding address is known.
	Gone = 410,
	// the server refuses to accept the request without a defined content length.
	LengthRequired = 411,
	// the precondition given in one or more of the request header fields evaluated to false when it was tested on the server.
	PrecondFailed = 412,
	// the server is refusing to process a request because the request entity is larger than the server is willing or able to process.
	RequestTooLarge = 413,
	// the server is refusing to service the request because the request uri (uniform resource identifier) is longer than the server is willing to interpret.
	UriTooLong = 414,
	// the server is refusing to service the request because the entity of the request is in a format not supported by the requested resource for the requested method.
	UnsupportedMedia = 415,
	// too many requests, the server is throttling
	TooManyRequests = 429,
	// the request should be retried after doing the appropriate action.
	RetryWith = 449,
	// the server encountered an unexpected condition that prevented it from fulfilling the request.
	ServerError = 500,
	// the server does not support the functionality required to fulfill the request.
	NotSupported = 501,
	// the server, while acting as a gateway or proxy, received an invalid response from the upstream server it accessed in attempting to fulfill the request.
	BadGateway = 502,
	// the service is temporarily overloaded.
	ServiceUnavail = 503,
	// the request was timed out waiting for a gateway.
	GatewayTimeout = 504,
	// the server does not support, or refuses to support, the http protocol version that was used in the request message.
	VersionNotSup = 505
};

