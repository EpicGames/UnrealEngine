// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace HttpVersion
{
	enum class EHttpServerHttpVersion : uint8
	{
		HTTP_VERSION_UNKNOWN,
		HTTP_VERSION_1_0,
		HTTP_VERSION_1_1
	};

	static bool FromString(const FString& HttpVersionStr, EHttpServerHttpVersion& OutHttpVersion)
	{
		if (0 == HttpVersionStr.Compare(TEXT("Unknown")))
		{
			OutHttpVersion = EHttpServerHttpVersion::HTTP_VERSION_UNKNOWN;
			return true;
		}
		if (0 == HttpVersionStr.Compare(TEXT("HTTP/1.0")))
		{
			OutHttpVersion = EHttpServerHttpVersion::HTTP_VERSION_1_0;
			return true;
		}
		if (0 == HttpVersionStr.Compare(TEXT("HTTP/1.1")))
		{
			OutHttpVersion = EHttpServerHttpVersion::HTTP_VERSION_1_1;
			return true;
		}
		return false;
	}

	static FString ToString(EHttpServerHttpVersion& InHttpVersion)
	{
		FString Result;
		switch (InHttpVersion)
		{
		case EHttpServerHttpVersion::HTTP_VERSION_UNKNOWN:
			Result = TEXT("Unknown");
			break;
		case EHttpServerHttpVersion::HTTP_VERSION_1_0:
			Result = TEXT("HTTP/1.0");
			break;
		case EHttpServerHttpVersion::HTTP_VERSION_1_1:
			Result = TEXT("HTTP/1.1");
			break;
		default:
			ensure(false);
		}
		return Result;
	}

}

