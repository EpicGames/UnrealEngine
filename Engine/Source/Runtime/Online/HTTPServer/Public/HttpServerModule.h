// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FHttpListener;
class IHttpRouter;
class FHttpServerModuleImpl;

DECLARE_LOG_CATEGORY_EXTERN(LogHttpServerModule, Log, All);

/**
 * Module for HtttpServer Implementation
 */
class FHttpServerModule : 
	public IModuleInterface
	,public FTSTickerObjectBase
{

public:
	FHttpServerModule();
	~FHttpServerModule();

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	HTTPSERVER_API static bool IsAvailable();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	HTTPSERVER_API static FHttpServerModule& Get();

	/**
	 * Per-port-binding access to an http router
	 *
	 * @param  Port The listener's bound port
	 * @param  bFailOnBindFailure if true, return nullptr if we fail to bind/listen on the given port
	 * @return An IHttpRouter instance that can be leveraged to respond to HTTP requests
	 */
	HTTPSERVER_API TSharedPtr<IHttpRouter> GetHttpRouter(uint32 Port, bool bFailOnBindFailure = false);

	/**
	 * FTSTicker callback
	 * 
	 * @param DeltaTime  The time in seconds since the last tick
	 * @return           false if no longer needs ticking, true otherwise
	 */
	 bool Tick(float DeltaTime) override;

	 /**
	  * Starts all listeners
	  */
	 HTTPSERVER_API void StartAllListeners();

	 /**
	  * Stops all listeners
	  */
	 HTTPSERVER_API void StopAllListeners();

	 /**
	  * Determines if any listeners are pending operations
	  *
	  * @return true if there are pending listeners, false otherwise
	  */
	 HTTPSERVER_API bool HasPendingListeners() const;

	// IModuleInterface

	/**
	 * Called when voice module is loaded
	 * Initialize platform specific parts of template handling
	 */
	virtual void StartupModule() override;
	
	/**
	 * Called when voice module is unloaded
	 * Shutdown platform specific parts of template handling
	 */
	virtual void ShutdownModule() override;

private:

	/** Singleton Instance */
	static FHttpServerModule* Singleton;

	FHttpServerModuleImpl* Impl;
};

