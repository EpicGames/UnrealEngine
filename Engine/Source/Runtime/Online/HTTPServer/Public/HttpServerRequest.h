// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HttpPath.h"
#include "HttpServerHttpVersion.h"

class FInternetAddr;

enum class EHttpServerRequestVerbs : uint16
{
	VERB_NONE   = 0,
	VERB_GET    = 1 << 0,
	VERB_POST   = 1 << 1,
	VERB_PUT    = 1 << 2,
	VERB_PATCH  = 1 << 3,
	VERB_DELETE = 1 << 4,
	VERB_OPTIONS = 1 << 5,
	VERB_HEAD    = 1 << 6,
	VERB_TRACE   = 1 << 7,
	VERB_CONNECT = 1 << 8,
};

ENUM_CLASS_FLAGS(EHttpServerRequestVerbs);

struct FHttpServerRequest
{
public:

	/** Constructor */
	FHttpServerRequest() { };

	/** The IP address of the peer that initiated the request. */
	TSharedPtr<FInternetAddr> PeerAddress;

	/** The full HTTP path as originally received */
	FHttpPath FullPath;

	/** The handler-route-relative HTTP path */
	FHttpPath RelativePath;

	/** The HTTP-compliant verb  */
	EHttpServerRequestVerbs Verb = EHttpServerRequestVerbs::VERB_NONE;

	/** The request HTTP protocol version */
	HttpVersion::EHttpServerHttpVersion HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;

	/** The HTTP headers */
	TMap<FString, TArray<FString>> Headers;

	/** The query parameters */
	TMap<FString, FString> QueryParams;

	/** The path parameters */
	TMap<FString, FString> PathParams;

	/** The raw body contents */
	TArray<uint8> Body;

};

