// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HttpServerConstants.h"
#include "HttpServerHttpVersion.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

class FUtf8String;

enum class EHttpServerResponseFlags : uint8
{
	None = 0,

	/** Used to indicate the response body will consist of multiple writes before another read/close (possibly an SSE stream) */
	MultipleWriteStream = 1 << 0,

	/** Used to indicate if additional stream writes are expected, used with MultipleWriteStream */
	HasAdditionalWrites = 1 << 1,

	/** Used to indicate if the next response write should skip including headers, used with MultipleWriteStream */
	SkipHeaderWrite		= 1 << 2,

	/**
	 * Destroy the connection immediately after this response is written, bypassing the
	 * keep-alive return to `AwaitingRead`. Required for streaming responses (`MultipleWriteStream`)
	 * that emit neither `Content-Length` nor `Transfer-Encoding: chunked` framing: HTTP/1.1
	 * clients fall back to read-to-EOF semantics for unframed bodies, so the socket close is
	 * the only end-of-message signal they receive. Without this flag, the socket idles until
	 * `FHttpConnection::ConnectionKeepAliveTimeout` (15s) elapses, masking the response delivery.
	 * Set on the final write of a streaming sequence.
	 */
	CloseAfterWrite		= 1 << 3,
};

ENUM_CLASS_FLAGS(EHttpServerResponseFlags);

struct FHttpServerResponse final
{
public:
	/**
	 * Constructor
	 */
	FHttpServerResponse() 
	{ }

	/**
	 * Constructor
	 * Facilitates in-place body construction
     *
	 * @param InBody The r-value body data
	 */
	FHttpServerResponse(TArray<uint8>&& InBody)
		: Body(MoveTemp(InBody))
	{ }

	/** Http protocol version */
	HttpVersion::EHttpServerHttpVersion HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_UNKNOWN;

	/** Http Response Code */
	EHttpServerResponseCodes Code = EHttpServerResponseCodes::Unknown;

	/** Http Headers */
	TMap<FString, TArray<FString>> Headers;

	/** Http Body Content */
	TArray<uint8> Body;

	/** Response flags */
	EHttpServerResponseFlags Flags = EHttpServerResponseFlags::None;

	/**
	 * Optional: streaming body queue. When set, WriteStream will drain this queue chunk-by-chunk
	 * in addition to (after) the initial Body content.  Must be an SPSC queue: one producer thread
	 * enqueues chunks, the game thread (WriteStream) dequeues.
	 * Set bStreamingBodyComplete to true after all chunks have been enqueued.
	 */
	TSharedPtr<TQueue<TArray<uint8>, EQueueMode::Spsc>> StreamingBodyQueue;
	TSharedPtr<TAtomic<bool>> StreamingBodyComplete;

public:

	/**
	 * Creates an FHttpServerResponse from a string
	 * 
	 * @param  Text         The text to serialize
	 * @param  ContentType  The HTTP response content type
	 * @return              A unique pointer to an initialized response object
	 */
	HTTPSERVER_API static TUniquePtr<FHttpServerResponse> Create(const FString& Text, const FString& ContentType);

	/**
	 * Creates an FHttpServerResponse from a string
	 *
	 * @param  Text         The text to serialize
	 * @param  ContentType  The HTTP response content type
	 * @return              A unique pointer to an initialized response object
	 */
	HTTPSERVER_API static TUniquePtr<FHttpServerResponse> Create(const FUtf8String& Text, const FString& ContentType);

	/**
	 * Creates an FHttpServerResponse from a string
	 *
	 * @param  Text         The text to serialize
	 * @param  ContentType  The HTTP response content type
	 * @return              A unique pointer to an initialized response object
	 */
	HTTPSERVER_API static TUniquePtr<FHttpServerResponse> Create(FUtf8String&& Text, const FString& ContentType);

	/**
	 * Creates an FHttpServerResponse from a raw byte buffer
	 *
	 * @param  RawBytes     The byte buffer to serialize
	 * @param  ContentType  The HTTP response content type
	 * @return              A unique pointer to an initialized response object
	 */
	HTTPSERVER_API static TUniquePtr<FHttpServerResponse> Create(TArray<uint8>&& RawBytes, FString ContentType);

	/**
	 * Creates an FHttpServerResponse from a raw byte buffer
	 *
	 * @param  RawBytes     The byte buffer view to serialize
	 * @param  ContentType  The HTTP response content type
	 * @return              A unique pointer to an initialized response object
	 */
	HTTPSERVER_API static TUniquePtr<FHttpServerResponse> Create(const TArrayView<uint8>& RawBytes, FString ContentType);

	/**
	 * Creates an FHttpServerResponse 204
	 * 
	 * @return A unique pointer to an initialized response object
	 */
	HTTPSERVER_API static TUniquePtr<FHttpServerResponse> Ok();

	/**
    * Creates an FHttpServerResponse with the caller-supplied response and error codes
	*
	* @param ResponseCode The HTTP response code
	* @param ErrorCode    The machine-readable error code
	* @param ErrorMessage The contextually descriptive error message
    * @return A unique pointer to an initialized response object
    */
	HTTPSERVER_API static TUniquePtr<FHttpServerResponse> Error(EHttpServerResponseCodes ResponseCode, const FString& ErrorCode = TEXT(""), const FString& ErrorMessage = TEXT(""));
};


