// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "HttpRequestHandler.h"
#include "HttpResultCallback.h"
#include "HttpRouteHandle.h"
#include "HttpServerRequest.h"
#include "Templates/SharedPointer.h"

struct FHttpPath;

DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnPreprocessHttpRequest, const TSharedPtr<FHttpServerRequest>& /*Request*/, const FHttpResultCallback& /*OnProcessingComplete*/);

class IHttpRouter : public TSharedFromThis<IHttpRouter>
{
public:

	/**
	 * Query the router with a request.
	 * @param Request the request to route.
	 * @param OnComplete the callback called when 
	 * @return	 An FHttpRouteHandle on success, nullptr otherwise.
	 */
	virtual bool Query(const TSharedPtr<FHttpServerRequest>& Request, const FHttpResultCallback& OnProcessingComplete) = 0;

	/**
	 * Binds the caller-supplied Uri to the caller-supplied handler
	 *  @param  HttpPath   The respective http path to bind
	 *  @param  HttpVerbs  The respective HTTP verbs to bind
	 *  @param  Handler    The caller-defined closure to execute when the binding is invoked
	 *  @return            An FHttpRouteHandle on success, nullptr otherwise. 
	 */
	virtual FHttpRouteHandle BindRoute(const FHttpPath& HttpPath, const EHttpServerRequestVerbs& HttpVerbs, const FHttpRequestHandler& Handler) = 0;

	/**
	 * Unbinds the caller-supplied Route 
	 *
	 *  @param  RouteHandle The handle to the route to unbind (callers must retain from BindRoute)
	 */
	virtual void UnbindRoute(const FHttpRouteHandle& RouteHandle) = 0;
	
	/**
     * Register a request preprocessor.
     * Useful for cases where you want to drop or handle incoming requests before they are dispatched to their respective handler.
     * @param RequestPreprocessor The function called to process the incoming request.
     * @return FDelegateHandle The handle to the delegate, used for unregistering preprocessors. 
     */
    virtual FDelegateHandle RegisterRequestPreprocessor(FHttpRequestHandler RequestPreprocessor) = 0;

	/**
	 * Unregister a request preprocessor.
	 * @param RequestPreprocessorHandle The handle to the preprocessor delegate to unregister.
	 */
	virtual void UnregisterRequestPreprocessor(const FDelegateHandle& RequestPreprocessorHandle) = 0;

protected:

	/**
	 * Destructor
	 */
	virtual ~IHttpRouter();
};


