// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Icmp : ModuleRules
{
	public Icmp(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDefinitions.Add("ICMP_PACKAGE=1");

		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core", 
				"CoreUObject",
                "Sockets",
			}
		);
	}
}
