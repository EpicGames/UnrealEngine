// Copyright Epic Games, Inc. All Rights Reserved.

#include "Icmp.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Ticker.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "IcmpPrivate.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "Misc/ConfigCacheIni.h"

FIcmpEchoResult IcmpEchoImpl(ISocketSubsystem* SocketSub, const FString& TargetAddress, float Timeout);

#if !PLATFORM_SUPPORTS_ICMP
FIcmpEchoResult IcmpEchoImpl(ISocketSubsystem* SocketSub, const FString& TargetAddress, float Timeout)
{
	FIcmpEchoResult Result;
	Result.Status = EIcmpResponseStatus::NotImplemented;
	return Result;
}
#endif

// Calculate one's complement checksum
int CalculateChecksum(uint8* Address, int Length)
{
	uint16* Paired = reinterpret_cast<uint16*>(Address);
	int Sum = 0;

	while (Length > 1)
	{
		Sum += *Paired++;
		Length -= 2;
	}

	if (Length == 1)
	{
		// Add the last odd byte
		Sum += *reinterpret_cast<uint8*>(Paired);
	}

	// Carry over overflow back to the LSB
	Sum = (Sum >> 16) + (Sum & 0xFFFF);
	// And in case the overflow caused another overflow, add it back again
	Sum += (Sum >> 16);

	return ~Sum;
}

bool ResolveIp(ISocketSubsystem* SocketSub, const FString& HostName, FString& OutIp)
{
	//ISocketSubsystem* SocketSub = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	if (SocketSub)
	{
		TSharedRef<FInternetAddr> HostAddr = SocketSub->CreateInternetAddr();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ESocketErrors HostResolveError = SocketSub->GetHostByName(TCHAR_TO_ANSI(*HostName), *HostAddr);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (HostResolveError == SE_NO_ERROR || HostResolveError == SE_EWOULDBLOCK)
		{
			OutIp = HostAddr->ToString(false);
			return true;
		}
	}

	return false;
}

class FIcmpAsyncResult
	: public FTSTickerObjectBase
{
public:

	FIcmpAsyncResult(ISocketSubsystem* InSocketSub, const FString& TargetAddress, float Timeout, uint32 StackSize, FIcmpEchoResultCallback InCallback)
		: FTSTickerObjectBase(0)
		, SocketSub(InSocketSub)
		, Callback(InCallback)
		, bThreadCompleted(false)
	{
		if (SocketSub)
		{
			TFunction<FIcmpEchoResult()> Task = [this, TargetAddress, Timeout]()
			{
				auto Result = IcmpEchoImpl(SocketSub, TargetAddress, Timeout);
				bThreadCompleted = true;
				return Result;
			};
			FutureResult = AsyncThread(Task, StackSize);
		}
		else
		{
			bThreadCompleted = true;
		}
	}

	virtual ~FIcmpAsyncResult()
	{
		check(IsInGameThread());
		if (FutureResult.IsValid())
		{
			FutureResult.Wait();
		}
	}

private:
	virtual bool Tick(float DeltaTime) override
	{
        QUICK_SCOPE_CYCLE_COUNTER(STAT_Icmp_Tick);

		if (bThreadCompleted)
		{
			FIcmpEchoResult Result;
			if (FutureResult.IsValid())
			{
				Result = FutureResult.Get();
			}

			Callback(Result);

			delete this;
			return false;
		}
		return true;
	}

	/** Reference to the socket subsystem */
	ISocketSubsystem* SocketSub;
	/** Callback when the icmp result returns */
	FIcmpEchoResultCallback Callback;
	/** Thread task complete */
	FThreadSafeBool bThreadCompleted;
	/** Async result future */
	TFuture<FIcmpEchoResult> FutureResult;
};

void FIcmp::IcmpEcho(const FString& TargetAddress, float Timeout, FIcmpEchoResultCallback HandleResult)
{
	int32 StackSize = 0;

#if PING_ALLOWS_CUSTOM_THREAD_SIZE
	GConfig->GetInt(TEXT("Ping"), TEXT("StackSize"), StackSize, GEngineIni);

	// Sanity clamp
	if (StackSize != 0)
	{
		StackSize = FMath::Max<int32>(FMath::Min<int32>(StackSize, 2 * 1024 * 1024), 32 * 1024);
	}
#endif

	ISocketSubsystem* SocketSub = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	new FIcmpAsyncResult(SocketSub, TargetAddress, Timeout, StackSize, HandleResult);
}
