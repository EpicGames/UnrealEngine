// Copyright Epic Games, Inc. All Rights Reserved.

#include "IcmpModule.h"
#include "Icmp.h"
#include "HAL/IConsoleManager.h"

IMPLEMENT_MODULE(FIcmpModule, Icmp);

DEFINE_LOG_CATEGORY(LogIcmp);

void FIcmpModule::StartupModule()
{
}

void FIcmpModule::ShutdownModule()
{
}


bool FIcmpModule::Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Ignore any execs that don't start with Icmp
	if (FParse::Command(&Cmd, TEXT("Icmp")))
	{
		if (FParse::Command(&Cmd, TEXT("Ping")))
		{
			FString Url;
			FParse::Token(Cmd, Url, true);
			if (Url.IsEmpty())
			{
				UE_LOGF(LogConsoleResponse, Warning, "Url is missing, console command is Icmp Ping URL, defaulting to 127.0.0.1");
				Url = TEXT("127.0.0.1");
			}
			FIcmp::IcmpEcho(Url, 1.0f, [](FIcmpEchoResult Result)
				{
					if (Result.Status == EIcmpResponseStatus::Success)
					{
						const double CurPing = static_cast<double>(Result.Time) * 1000.0;
						UE_LOGF(LogConsoleResponse, Display, "Ping success: Reply from %ls time=%.02f ms", *Result.ReplyFrom, CurPing);
					}
					else
					{
						UE_LOGF(LogConsoleResponse, Display, "Ping failure");
					}
				});
			return true;
		}
	}
	return false;
}
