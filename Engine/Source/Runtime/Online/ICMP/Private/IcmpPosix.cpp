// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "Misc/ScopeLock.h"
#include "IcmpPrivate.h"
#include "Icmp.h"

#if PLATFORM_USES_POSIX_ICMP


#include <arpa/inet.h>
#include <netinet/in.h>
#include <netinet/ip.h>
#include <netinet/ip_icmp.h>
#include <netdb.h>
#include <poll.h>
#include <unistd.h>

namespace IcmpPosix
{
	// 32 bytes is the default size for the windows ping utility, and windows has problems with sending < 18 bytes.
	static constexpr SIZE_T IcmpPayloadSize = 32;
	static const uint8 IcmpPayload[IcmpPayloadSize] = ">>>>This string is 32 bytes<<<<";

	// A critical section that ensures we only have a single ping in flight at once.
	static FCriticalSection gPingCS;

	// Returns the ip address as string
	static FString IpToString(const struct sockaddr_in* const Address)
	{
		ANSICHAR Buffer[INET_ADDRSTRLEN];
		return ANSI_TO_TCHAR(inet_ntop(AF_INET, &Address->sin_addr, Buffer, INET_ADDRSTRLEN));
	}
}

uint16 NtoHS(uint16 val)
{
	return ntohs(val);
}

uint16 HtoNS(uint16 val)
{
	return htons(val);
}

uint32 NtoHL(uint32 val)
{
	return ntohl(val);
}

uint32 HtoNL(uint32 val)
{
	return htonl(val);
}

FIcmpEchoResult IcmpEchoImpl(ISocketSubsystem* SocketSub, const FString& TargetAddress, float Timeout)
{
	// Android implementation has some particularities:
	//   - Data received when using recvfrom does not include the ip header
	//   - Sent value for icmp_id is ignored and set and increased by the os, so we include this value in the payload
#if PLATFORM_ANDROID
	static const SIZE_T IpHeaderFixedPartSize = 0;
	static const SIZE_T IpHeaderVariablePartMaxSize = 0;
#else
	static const SIZE_T IpHeaderFixedPartSize = sizeof(struct ip);
	static const SIZE_T IpHeaderVariablePartMaxSize = 40;
#endif
	static const SIZE_T IcmpHeaderSize = ICMP_MINLEN;

	// The packet we send is just the ICMP header plus our payload
	static const SIZE_T IcmpPacketSize = IcmpHeaderSize + IcmpPosix::IcmpPayloadSize;

	// The result read back will need a room for the IP header as well the icmp echo reply packet
	static const SIZE_T MaxReceivedPacketSize = IpHeaderFixedPartSize + IpHeaderVariablePartMaxSize + IcmpPacketSize;
	static int PingSequence = 0;

	FIcmpEchoResult Result;
	Result.Status = EIcmpResponseStatus::InternalError;

	int IcmpSocket = socket(AF_INET, SOCK_DGRAM, IPPROTO_ICMP);
	if (IcmpSocket < 0)
	{
		return Result;
	}

	FString ResolvedAddress;
	if (!ResolveIp(SocketSub, TargetAddress, ResolvedAddress))
	{
		Result.Status = EIcmpResponseStatus::Unresolvable;
		return Result;
	}
	Result.ResolvedAddress = ResolvedAddress;

	struct sockaddr_in DestAddress = {};
	DestAddress.sin_family = AF_INET;
	if ( 1 != inet_pton(AF_INET, TCHAR_TO_UTF8(*ResolvedAddress), &DestAddress.sin_addr) )
	{
		Result.Status = EIcmpResponseStatus::InternalError;
		return Result;
	}

	alignas(struct icmp) uint8 PacketStorage[IcmpPacketSize] = {};
	
	struct icmp *SentPacket = new (PacketStorage) struct icmp;
	SentPacket->icmp_type = ICMP_ECHO;
	SentPacket->icmp_code = 0;

	// Put some data into the packet payload
	FMemory::Memcpy(SentPacket->icmp_data, IcmpPosix::IcmpPayload, IcmpPosix::IcmpPayloadSize);

	uint16 SentId = getpid() & 0xFFFF;
#if PLATFORM_ANDROID
	// Embed icmp_id into payload by overwriting initial 2 bytes
	static_assert(IcmpPosix::IcmpPayloadSize >= sizeof(SentId), "There should be enough space to include the icmp id");
	FMemory::Memcpy(SentPacket->icmp_data, &SentId, sizeof(SentId));
#else
	SentPacket->icmp_id = SentId;
#endif
	SentPacket->icmp_seq = PingSequence++;

	// Calculate the IP checksum
	SentPacket->icmp_cksum = 0; // chksum must be zero when calculating it
	SentPacket->icmp_cksum = CalculateChecksum(PacketStorage, IcmpPacketSize);

	// We can only have one ping in flight at once, as otherwise we risk swapping echo replies between requests
	{
		FScopeLock PingLock(&IcmpPosix::gPingCS);
		double TimeLeft = Timeout;
		double StartTime = FPlatformTime::Seconds();
		if (0 < sendto(IcmpSocket, PacketStorage, IcmpPacketSize, 0, reinterpret_cast<struct sockaddr*>(&DestAddress), sizeof(DestAddress)))
		{
			struct pollfd PollData[1] = { {.fd = IcmpSocket, .events = POLLIN} };

			bool bDone = false;

			while (!bDone)
			{
				int NumReady = poll(PollData, 1, int(TimeLeft * 1000.0));
				if (NumReady == 0)
				{
					// timeout - if we've received an 'Unreachable' result earlier, return that result instead.
					if (Result.Status != EIcmpResponseStatus::Unreachable)
					{
						Result.Status = EIcmpResponseStatus::Timeout;
						Result.Time = Timeout;
					}
					bDone = true;
				}
				else if (NumReady == 1)
				{
					struct sockaddr FromAddressUnion = {};
					socklen_t FromAddressUnionSize = sizeof(FromAddressUnion);
					uint8 ReceiveBuffer[MaxReceivedPacketSize];
					int ReadSize = recvfrom(IcmpSocket, ReceiveBuffer, MaxReceivedPacketSize, 0, &FromAddressUnion, &FromAddressUnionSize);

					double EndTime = FPlatformTime::Seconds();

					// Estimate elapsed time
					Result.Time = EndTime - StartTime;

					TimeLeft = FPlatformMath::Max(0.0, (double)Timeout - Result.Time);

					if (ReadSize == -1)
					{
						Result.Status = EIcmpResponseStatus::InternalError;
						bDone = true;
					}
					else if (ReadSize > IpHeaderFixedPartSize)
					{
						if (FromAddressUnion.sa_family != AF_INET)
						{
							// We got response from a non-IPv4 address??!
							continue;
						}
						const struct sockaddr_in* const FromAddress = reinterpret_cast<struct sockaddr_in*>(&FromAddressUnion);
						Result.ReplyFrom = IcmpPosix::IpToString(FromAddress);
						
#if PLATFORM_ANDROID
						// recvfrom data received on Android does not contain the ip header
						// Also we assume we received an IPPROTO_ICMP without checking the header since the socket was created using IPPROTO_ICMP
						const SIZE_T ReceivedIpHeaderSize = 0;
#else
						const struct ip* const ReceivedIpHeader = reinterpret_cast<struct ip*>(ReceiveBuffer);
						if (ReceivedIpHeader->ip_p != IPPROTO_ICMP)
						{
							// We got a non-ICMP packet back??!
							continue;
						}
						const SIZE_T ReceivedIpHeaderSize = SIZE_T(ReceivedIpHeader->ip_hl) << 2;
#endif
						const SIZE_T ReceivedHeadersSize = ReceivedIpHeaderSize + IcmpHeaderSize;
						
						if (ReadSize >= ReceivedHeadersSize)
						{
							const struct icmp* const ReceivedPacket = reinterpret_cast<struct icmp*>(ReceiveBuffer + ReceivedIpHeaderSize);
							const SIZE_T ReceivedPayloadSize = ReadSize - ReceivedHeadersSize;

							switch (ReceivedPacket->icmp_type)
							{
							case ICMP_ECHOREPLY:
								if (DestAddress.sin_addr.s_addr == FromAddress->sin_addr.s_addr  &&
									ReceivedPayloadSize == IcmpPosix::IcmpPayloadSize &&
#if PLATFORM_ANDROID
									// icmp_id sent is not controlled by us on Android. We embedded it on the icmp_data
#else
									SentPacket->icmp_id == ReceivedPacket->icmp_id &&
#endif
									SentPacket->icmp_seq == ReceivedPacket->icmp_seq &&
									0 == FMemory::Memcmp(SentPacket->icmp_data, ReceivedPacket->icmp_data, IcmpPosix::IcmpPayloadSize))
								{
									Result.Status = EIcmpResponseStatus::Success;
									bDone = true;
								}
								break;
							case ICMP_UNREACH:
								Result.Status = EIcmpResponseStatus::Unreachable;
								// If there is still time left, try waiting for another result.
								// If we run out of time, we'll return Unreachable instead of Timeout.
								break;
							default:
								break;
							}
						}
						else
						{
							Result.Status = EIcmpResponseStatus::InternalError;
							bDone = true;
						}
					}
				}
			}
		}

		close(IcmpSocket);
	}

	return Result;
}

#endif //PLATFORM_USES_POSIX_ICMP
