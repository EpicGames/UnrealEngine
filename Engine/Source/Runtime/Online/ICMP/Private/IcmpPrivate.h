// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Module includes

#ifndef PLATFORM_SUPPORTS_ICMP

	#if PLATFORM_WINDOWS

		#define PLATFORM_SUPPORTS_ICMP 1
		#define PLATFORM_USES_POSIX_ICMP 0
		#define PING_ALLOWS_CUSTOM_THREAD_SIZE 1

	#elif PLATFORM_MAC

		#define PLATFORM_SUPPORTS_ICMP 1
		#define PLATFORM_USES_POSIX_ICMP 1
		#define PING_ALLOWS_CUSTOM_THREAD_SIZE 0

	#elif PLATFORM_UNIX

		#define PLATFORM_SUPPORTS_ICMP 1
		#define PLATFORM_USES_POSIX_ICMP 1
		#define PING_ALLOWS_CUSTOM_THREAD_SIZE 0

	#elif PLATFORM_ANDROID

		#define PLATFORM_SUPPORTS_ICMP 1
		#define PLATFORM_USES_POSIX_ICMP 1
		#define PING_ALLOWS_CUSTOM_THREAD_SIZE 0

	#elif PLATFORM_IOS

		#define PLATFORM_SUPPORTS_ICMP 1
		#define PLATFORM_USES_POSIX_ICMP 1
		#define PING_ALLOWS_CUSTOM_THREAD_SIZE 1

	#else

		#define PLATFORM_SUPPORTS_ICMP 0
		#define PLATFORM_USES_POSIX_ICMP 0
		#define PING_ALLOWS_CUSTOM_THREAD_SIZE 0

	#endif

#endif // PLATFORM_SUPPORTS_ICMP

class ISocketSubsystem;

/** 
 * Calculate the 16-bit one's complement of the one's complement 
 * sum of the ICMP message starting at the beginning of the IcmpHeader
 * 
 * @param Address byte array to calculate the checksum from
 * @param Length length of the input byte array
 */
int CalculateChecksum(uint8* Address, int Length);

/**
 * Convert a string based hostname (ipv4) to a valid ip address
 *
 * @param HostName the name of the host to look up
 * @param OutIp [out] the string to copy the IP address to
 *
 * @return true if conversion was successful, false otherwise
 */
bool ResolveIp(ISocketSubsystem* SocketSub, const FString& HostName, FString& OutIp);
