// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Ticker.h"
#include "Async/Future.h"
#include "Async/Async.h"
#include "IcmpPrivate.h"
#include "Icmp.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"

#include "Sockets.h"

DEFINE_LOG_CATEGORY_STATIC(LogPing, Log, All)

extern uint16 NtoHS(uint16 val);
extern uint16 HtoNS(uint16 val);
extern uint32 NtoHL(uint32 val);
extern uint32 HtoNL(uint32 val);

// ----------------------------------------------------------------------------
// UDPEchoMany private interface
// ----------------------------------------------------------------------------

namespace UDPPing
{
	struct FUdpPingHeader
	{
		uint16 Id;
		uint16 Sequence;
		uint16 Checksum;

		void HostToNetwork();
		void NetworkToHost();
	};

	struct FUdpPingBody
	{
		uint64 TimeCode;
		uint32 Data[2];

		void HostToNetwork();
		void NetworkToHost();
	};

	struct FUdpPingPacket
	{
		FUdpPingHeader Header;
		FUdpPingBody Body;

		void HostToNetwork();
		void NetworkToHost();

		bool Validate(uint16 ExpectedId, uint16 ExpectedSequenceNum) const;

		static FUdpPingPacket Create(uint16 Id, uint16 SequenceNum);
	};

	static const SIZE_T HeaderSize = sizeof(FUdpPingHeader);
	static const SIZE_T BodySize = sizeof(FUdpPingBody);
	static const SIZE_T SizePacked = HeaderSize + BodySize;

	bool Pack(uint8* OutBuf, const SIZE_T BufSize, const FUdpPingPacket& Packet);
	bool Unpack(FUdpPingPacket& OutPacket, uint8* const Buf, const SIZE_T BufSize);
	bool UpdatePacketChecksum(uint8* const PingPacketBuf, const SIZE_T BufSize, const bool ToNetworkByteOrder);
	uint16 CalculatePacketChecksum(uint8* const PingPacketBuf, const SIZE_T BufSize);

} // namespace UDPPing

// ----------------------------------------------------------------------------

namespace
{
	typedef TFunction<void(FIcmpEchoManyResult)> FGotResultCallback;
	DECLARE_DELEGATE_OneParam(FGotResultDelegate, FIcmpEchoManyResult);

	typedef TFunction<void(EIcmpEchoManyStatus)> FStatusCallback;
	DECLARE_DELEGATE_OneParam(FStatusDelegate, EIcmpEchoManyStatus);

	static constexpr uint32 PingDataHigh = 0xaaaaaaaa;
	static constexpr uint32 PingDataLow = 0xbbbbbbbb;

	FSocket* CreateDatagramSocket(ISocketSubsystem& SocketSub, const FName& ProtocolType, bool blocking);

	uint64 NtoHLL(uint64 Val);
	uint64 HtoNLL(uint64 Val);

	int32 CalcStackSize();
} // namespace

// ----------------------------------------------------------------------------

class FUdpPingWorker
	: public FRunnable
{
	struct FProgress
	{
		/* Unresolved target address */
		FString Address;

		/* Target port */
		int32 Port;

		/* Time at which ping was sent to address (used by timeout check) */
		double StartTime;

		/* Duration after sending that the ping times out */
		double TimeoutDuration;

		/* Identifying number for the ping, to distinguish between multiple pings to same host (or over entire target list) */
		uint16 EchoId;

		/* Sequence number, i.e. which send attempt sent the ping associated with this progress data */
		uint16 SequenceNum;

		/* Ping result */
		FIcmpEchoResult Result;

		/* The shared socket used to send/receive the ping */
		FSocket* SocketPtr;

		/* The socket IP address for sending to the target host */
		TSharedPtr<FInternetAddr> ToAddr;

	public:
		FProgress();

	}; // struct FProgress

	struct FProgressKey
	{
		/* Resolved IP address if resolvable, for faster lookup on receiving replies */
		FString Address;

		/* Identifier used when sending to same address multiple times */
		int UniqueId;

		FProgressKey(const FString& InAddress, int InUniqueId);
		bool operator ==(const FProgressKey& Rhs) const;
		uint32 CalcHash() const;

		friend uint32 GetTypeHash(const FProgressKey& Key) { return Key.CalcHash(); }

	}; // struct FProgressKey

	enum class ESendStatus
	{
		None,
		Ok,
		Busy,
		NoTarget,
		BadTarget,
		NoSocket,
		BadBuffer,
		SocketSendFail,
		BadSendSize
	};

public:
	FUdpPingWorker(ISocketSubsystem *const InSocketSub, const TArray<FIcmpTarget>& InTargets, float Timeout,
		const FGotResultDelegate& InGotResultDelegate, const FStatusDelegate& InCompletionDelegate);

	virtual ~FUdpPingWorker();

	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

	bool IsCanceled() const;

private:
	bool InitProgressTable(const TArray<FIcmpTarget>& PingTargets);

	FSocket* GetOrCreateSocket(const FName& ProtocolType);
	bool DestroySockets();

	bool SendPings(ISocketSubsystem& SocketSub);

	int CheckAwaitReplies();

	ESendStatus SendPing(ISocketSubsystem& SocketSub, uint8* SendBuf, const uint32 BufSize, FProgress& Progress);

	static ESendStatus SendPing(FSocket& Socket, uint8* SendBuf, const uint32 BufSize,
		const FInternetAddr& ToAddr, UDPPing::FUdpPingPacket& Packet, double& OutSendTimeSecs);

	static bool HasReplyData(FSocket& Socket, const uint32 MinSize);

	static bool RecvReplyFrom(FSocket& Socket, uint8* RecvBuf, const uint32 RecvSize, FInternetAddr& OutRecvFrom, uint64& OutRecvTimeCode);

	static FProgress* ProcessReply(const FInternetAddr& FromAddr, uint8* const RecvBuf, const uint32 RecvSize,
		const uint64 RecvTimeCode, TMap<FProgressKey, FProgress>& ProgressTable);

	static bool CalculateTripTime(const uint64 ReplyTimeCode, const uint64 CurrentTimeCode, const double TimeoutSecs,
		float& OutTime, EIcmpResponseStatus& OutStatus);

	static void ResetSequenceNum();
	static int NextSequenceNum();

private:
	float Timeout;
	int NumAwaitingReplies;
	int NumValidRepliesReceived;
	bool bCancelOperation;

	double MinPingSendWaitTimeMs;

	ISocketSubsystem* SocketSubsystem;

	TArray<FIcmpTarget> Targets;
	TMap<FProgressKey, FProgress> ProgressTable;

	TMap<FName, FSocket *> SocketTable;

	FGotResultDelegate GotResultDelegate;
	FStatusDelegate CompletionDelegate;

	static int SequenceNum;

	static const SIZE_T SendDataSize;
	static const SIZE_T RecvDataSize;

	static const FTimespan RecvWaitTime;
	static const FTimespan SendWaitTime;

	static const double MaxTripTimeSecs;
	static const double MaxInactivityWaitTimeSecs;

}; // class FUdpPingWorker

// ----------------------------------------------------------------------------

class FUdpPingManyAsync
	: public FTSTickerObjectBase
{
public:
	FUdpPingManyAsync(ISocketSubsystem* const InSocketSub, const FIcmpEchoManyCompleteDelegate& InCompletionDelegate);

	virtual ~FUdpPingManyAsync();

	void Start(const TArray<FIcmpTarget>& Targets, float Timeout, uint32 StackSize);

protected:
	virtual bool Tick(float DeltaTime) override;

private:
	static FIcmpEchoManyCompleteResult RunWorker(ISocketSubsystem* const SocketSub, const TArray<FIcmpTarget>& Targets, float Timeout);

private:
	ISocketSubsystem* SocketSub;

	FIcmpEchoManyCompleteDelegate CompletionDelegate;

	FThreadSafeBool bThreadCompleted;

	TFuture<FIcmpEchoManyCompleteResult> FutureResult;

}; // class FUdpPingManyAsync

// ----------------------------------------------------------------------------
// UDPEchoMany implementation
// ----------------------------------------------------------------------------

const SIZE_T FUdpPingWorker::SendDataSize = UDPPing::SizePacked;
const SIZE_T FUdpPingWorker::RecvDataSize = UDPPing::SizePacked;

const FTimespan FUdpPingWorker::RecvWaitTime = FTimespan::FromMilliseconds(0.25);
const FTimespan FUdpPingWorker::SendWaitTime = FTimespan::FromMilliseconds(0.25);

const double FUdpPingWorker::MaxTripTimeSecs = 60.0 * 1000.0; // 1000 minutes
const double FUdpPingWorker::MaxInactivityWaitTimeSecs = 60.0 * 10.0; // 10 minutes

int FUdpPingWorker::SequenceNum = 0;

FUdpPingWorker::FUdpPingWorker(ISocketSubsystem *const InSocketSub, const TArray<FIcmpTarget>& InTargets, float InTimeout,
	const FGotResultDelegate& InGotResultDelegate, const FStatusDelegate& InCompletionDelegate)
	: FRunnable()
	, Timeout(InTimeout)
	, NumAwaitingReplies(0)
	, NumValidRepliesReceived(0)
	, bCancelOperation(false)
	, MinPingSendWaitTimeMs(0)
	, SocketSubsystem(InSocketSub)
	, Targets(InTargets)
	, GotResultDelegate(InGotResultDelegate)
	, CompletionDelegate(InCompletionDelegate)
{
}

FUdpPingWorker::~FUdpPingWorker()
{
	DestroySockets();
	SocketTable.Empty();
}

bool FUdpPingWorker::Init()
{
	ProgressTable.Empty();

	DestroySockets();
	SocketTable.Empty();

	NumAwaitingReplies = 0;
	NumValidRepliesReceived = 0;
	bCancelOperation = false;

	MinPingSendWaitTimeMs = FUdpPingWorker::SendWaitTime.GetTotalMilliseconds();
	GConfig->GetDouble(TEXT("Ping"), TEXT("MinPingSendWaitTimeMs"), MinPingSendWaitTimeMs, GEngineIni);

	return true;
}

uint32 FUdpPingWorker::Run()
{
	bool bOk = true;

	// Setup

	if (bOk && !SocketSubsystem)
	{
		UE_LOGF(LogPing, Warning, "Run: no socket subsystem");
		bOk = false;
	}

	if (bOk && (Targets.Num() == 0))
	{
		UE_LOGF(LogPing, Warning, "Run; no ping targets");
		bOk = false;
	}

	if (bOk && !InitProgressTable(Targets))
	{
		UE_LOGF(LogPing, Warning, "Run; cannot init progress table");
		bOk = false;
	}

	// Loop

	bOk = bOk && !IsCanceled() && SendPings(*SocketSubsystem);

	// Completion

	EIcmpEchoManyStatus CompletionStatus = EIcmpEchoManyStatus::Invalid;
	if (IsCanceled())
	{
		CompletionStatus = EIcmpEchoManyStatus::Canceled;
	}
	else if (bOk)
	{
		CompletionStatus = EIcmpEchoManyStatus::Success;
	}
	else
	{
		CompletionStatus = EIcmpEchoManyStatus::Failure;
	}

	CompletionDelegate.ExecuteIfBound(CompletionStatus);

	return 0;
}

void FUdpPingWorker::Stop()
{
	bCancelOperation = true;
}

void FUdpPingWorker::Exit()
{
}

bool FUdpPingWorker::IsCanceled() const
{
	return bCancelOperation;
}

bool FUdpPingWorker::InitProgressTable(const TArray<FIcmpTarget>& PingTargets)
{
	if (!SocketSubsystem)
	{
		return false;
	}

	ProgressTable.Empty();

	int Index = 0;

	for (const FIcmpTarget& Target : PingTargets)
	{
		const int EchoId = ++Index;

		FProgress Progress;
		Progress.Address = Target.Address;
		Progress.Port = Target.Port;
		Progress.TimeoutDuration = Timeout;
		Progress.EchoId = static_cast<uint16>(EchoId);

		FString ResolvedAddress;
		const bool bResolveOk = ResolveIp(SocketSubsystem, Target.Address, ResolvedAddress);

		if (bResolveOk)
		{
			Progress.Result.ResolvedAddress = ResolvedAddress;

			// Add an entry to table, ready to hold reply result.
			// Use resolved address in the table key - needed for constant-time progress lookup when receiving replies.
			const FProgressKey Key(ResolvedAddress, EchoId);
			ProgressTable.Add(Key, Progress);
		}
		else
		{
			Progress.Result.Status = EIcmpResponseStatus::Unresolvable;

			// Add an entry to table, pre-filled with unresolvable status.
			// Use unresolved address in the table key.
			const FProgressKey Key(Target.Address, EchoId);
			ProgressTable.Add(Key, Progress);
		}
	}

	return true;
}

FSocket* FUdpPingWorker::GetOrCreateSocket(const FName& ProtocolType)
{
	if (!SocketSubsystem)
	{
		return nullptr;
	}

	FSocket** FoundSocket = SocketTable.Find(ProtocolType);
	if (FoundSocket)
	{
		return *FoundSocket;
	}

	FSocket* Socket = CreateDatagramSocket(*SocketSubsystem, ProtocolType, false);
	if (!Socket)
	{
		return nullptr;
	}

	return SocketTable.Add(ProtocolType, Socket);
}

bool FUdpPingWorker::DestroySockets()
{
	if (!SocketSubsystem)
	{
		return false;
	}

	for (auto& Elem : SocketTable)
	{
		FSocket* & Socket = Elem.Value;
		if (Socket)
		{
			SocketSubsystem->DestroySocket(Socket);
			Socket = nullptr;
		}
	}

	return true;
}

bool FUdpPingWorker::SendPings(ISocketSubsystem& SocketSub)
{
	if (ProgressTable.Num() == 0)
	{
		UE_LOGF(LogPing, Warning, "SendPings; empty progress table");
		return false;
	}

	typedef TMap<FProgressKey, FProgress>::TIterator TProgressIterator;
	TProgressIterator ItSend = ProgressTable.CreateIterator();

	uint8 SendBuf[SendDataSize]{ 0 };

	uint64 RecvTimeCode = 0;
	uint8 RecvBuf[RecvDataSize]{ 0 };
	TSharedRef<FInternetAddr> RecvFromAddr = SocketSub.CreateInternetAddr();

	NumAwaitingReplies = 0;
	NumValidRepliesReceived = 0;

	bool bDone = false;

	uint64 LastSendActivityTimeCycles = FPlatformTime::Cycles64();
	bool bSentLastPing = false;
	int NumSkippedPings = -1;

	while (!bDone && !IsCanceled())
	{
		// Receiving Stage

		// Iterate over each socket created (sockets are shared based on protocol type)
		for (TPair<FName, FSocket*>& Elem : SocketTable)
		{
			if (IsCanceled())
			{
				break;
			}

			FSocket* SocketPtr = Elem.Value;
			if (!SocketPtr)
			{
				// Skip invalid Socket pointers.
				continue;
			}

			if (SocketPtr->Wait(ESocketWaitConditions::WaitForRead, RecvWaitTime) && !IsCanceled())
			{
				// Deal with any pending incoming data on the socket.
				// Inner loop deals with possibility that there are multiple incoming replies on socket.
				while (HasReplyData(*SocketPtr, RecvDataSize))
				{
					if (!RecvReplyFrom(*SocketPtr, RecvBuf, RecvDataSize, *RecvFromAddr, RecvTimeCode))
					{
						// Failed to get a complete reply packet from socket this time,
						// so exit this inner loop to allow more pings to be sent out
						// and replies on other sockets to be checked.
						UE_LOGF(LogPing, Verbose, "SendPings; recv failed");

						break;
					}

					UE_LOGF(LogPing, VeryVerbose, "SendPings; recv reply from %ls", *RecvFromAddr->ToString(false));

					FProgress* const Progress = ProcessReply(*RecvFromAddr, RecvBuf, RecvDataSize, RecvTimeCode, ProgressTable);
					if (Progress)
					{
						Progress->SocketPtr = nullptr;

						// Notify result listeners (e.g. so results can be aggregated).
						const FIcmpTarget OriginalSendInfo(Progress->Address, Progress->Port);
						GotResultDelegate.ExecuteIfBound(FIcmpEchoManyResult(Progress->Result, OriginalSendInfo));

						++NumValidRepliesReceived;
					}
				} // while (socket has reply data)

				const ESocketErrors LastRecvError = SocketSub.GetLastErrorCode();
				if (LastRecvError != ESocketErrors::SE_NO_ERROR)
				{
					UE_LOGF(LogPing, Verbose, "SendPings; recv error: %u", LastRecvError);
				}
			}
		} // for (socket table)

		// Sending Stage

		if (!IsCanceled())
		{
			const bool bMoreToSend = (bool)ItSend;

			// Check if we are waiting on any more replies, and timeout any that have expired.
			NumAwaitingReplies = CheckAwaitReplies();
			bDone = !bMoreToSend && (NumAwaitingReplies == 0);

			if (bMoreToSend)
			{
				// More pings to send out.

				// Check for inactivity.
				const double DeltaSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastSendActivityTimeCycles);
				if (DeltaSeconds > MaxInactivityWaitTimeSecs)
				{
					// Waiting too long for availability to send any more pings, so abort the whole send/recv loop.
					// Allows task thread to complete with "canceled" status.
					UE_LOGF(LogPing, Warning, "SendPings; aborting due to inactivity after %.4f seconds",
						DeltaSeconds);

					bCancelOperation = true;
					break;
				}

				const double DeltaMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - LastSendActivityTimeCycles);
				if (bSentLastPing && (DeltaMs < MinPingSendWaitTimeMs && NumSkippedPings >= 0))
				{
					// Skip sending pings for a bit (but continue receiving) to not spam traffic.
					++NumSkippedPings;
					continue;
				}

				if (NumSkippedPings > 0)
				{
					UE_LOGF(LogPing, VeryVerbose, "SendPings: paused sending pings for %d iterations (for %.4f ms, configured wait time is %.4f ms); resuming sends", NumSkippedPings, DeltaMs, MinPingSendWaitTimeMs);
				}

				// Send ping for current target in progress table.
				// Only send a maximum of one ping request per iteration; checking for replies takes priority.
				FProgress& Progress = ItSend->Value;

				const ESendStatus Status = SendPing(SocketSub, SendBuf, SendDataSize, Progress);
				NumSkippedPings = 0;
				bSentLastPing = false;

				if ((ESendStatus::Ok == Status) || (ESendStatus::NoTarget == Status))
				{
					// Don't wait/block here; poll for is-reply-data-available in loop until received or timeout.
					if (ESendStatus::Ok == Status)
					{
						// actually kick in the wait timer if we did anything on the network this time
						bSentLastPing = true;
						LastSendActivityTimeCycles = FPlatformTime::Cycles64();
					}
					
					// Advance to next send target address.
					++ItSend;
				}
				else if (ESendStatus::Busy == Status)
				{
					// Socket wasn't ready to write, so don't advance through the send targets list;
					// attempt to send again on next iteration.

					// Double-check that socket didn't generate a fatal send error, as retrying would be pointless.
					const ESocketErrors LastSendError = SocketSub.GetLastErrorCode();
					const bool bInternalError = (LastSendError != ESocketErrors::SE_NO_ERROR)
						&& (LastSendError != ESocketErrors::SE_EWOULDBLOCK)
						&& (LastSendError != ESocketErrors::SE_EINPROGRESS);

					if (bInternalError)
					{
						UE_LOGF(LogPing, Verbose, "SendPings; (busy) internal socket error: %u", LastSendError);

						// Send failed, mark the failure in the results and advance to next target address.
						Progress.Result.Status = EIcmpResponseStatus::Unresolvable;
						++ItSend;
					}
				}
				else
				{
					// Status is one of: BadTarget, NoSocket, BadBuffer, SocketSendFail, BadSendSize

					UE_LOGF(LogPing, Verbose, "SendPings; send error, status: %u", int(Status));

					// Send failed, mark the failure in the results and advance to next target address.
					Progress.Result.Status = EIcmpResponseStatus::Unresolvable;
					++ItSend;
				} // send status
			} // send
		} // not canceled (send)
	} // while (send/recv loop)

	// Cleanup.
	DestroySockets();
	SocketTable.Empty();
	// if we have any unresolvable results in the progress table, notify everyone of their failure now that we're all done
	for (TPair<FProgressKey, FProgress>& Row : ProgressTable)
	{
		FProgress& Progress = Row.Value;
		FIcmpEchoResult& Result = Progress.Result;

		if (Result.Status != EIcmpResponseStatus::Success && Result.Status != EIcmpResponseStatus::Timeout)
		{
			UE_LOGF(LogPing, Verbose, "SendPings(Done): %ls:%d (id: %d, seq: %d) had non-successful status '%ls' after send loop", *Progress.Address, Progress.Port, Progress.EchoId, Progress.SequenceNum, LexToString(Result.Status));
			Result.Time = Progress.TimeoutDuration;
			Result.ReplyFrom.Empty();
			const FIcmpTarget SendInfo(Progress.Address, Progress.Port);
			GotResultDelegate.ExecuteIfBound(FIcmpEchoManyResult(Progress.Result, SendInfo));
		}
	}

	return true;
}

int FUdpPingWorker::CheckAwaitReplies()
{
	int NumAwaitingReply = 0;

	for (TPair<FProgressKey, FProgress>& Row : ProgressTable)
	{
		FProgress& Progress = Row.Value;
		FIcmpEchoResult& Result = Progress.Result;

		bool bShouldNotifyFailure = false;

		// InternalError status means no result yet.
		if ((EIcmpResponseStatus::InternalError == Result.Status) && (Progress.StartTime > 0))
		{
			const double ReplyWaitDuration = FPlatformTime::Seconds() - Progress.StartTime;

			if (ReplyWaitDuration >= Progress.TimeoutDuration)
			{
				UE_LOGF(LogPing, Log, "CheckAwaitReplies: %ls:%d (id: %d, seq: %d) timed out after %.4f seconds", *Progress.Address, Progress.Port, Progress.EchoId, Progress.SequenceNum, Progress.TimeoutDuration);
				bShouldNotifyFailure = true;

				Result.Time = Progress.TimeoutDuration;
				Result.Status = EIcmpResponseStatus::Timeout;
			}
			else
			{
				++NumAwaitingReply;
			}
		}

		if (bShouldNotifyFailure)
		{
			Result.ReplyFrom.Empty();
			Progress.SocketPtr = nullptr;

			// Notify result listeners since we failed
			const FIcmpTarget SendInfo(Progress.Address, Progress.Port);
			GotResultDelegate.ExecuteIfBound(FIcmpEchoManyResult(Progress.Result, SendInfo));
		}
	}

	return NumAwaitingReply;
}

FUdpPingWorker::ESendStatus FUdpPingWorker::SendPing(ISocketSubsystem& SocketSub,
	uint8* SendBuf, const uint32 BufSize, FProgress& Progress)
{
	if (EIcmpResponseStatus::Unresolvable == Progress.Result.Status)
	{
		return ESendStatus::NoTarget;
	}

	FIcmpEchoResult& Result = Progress.Result;

	if (!Progress.ToAddr.IsValid())
	{
		TSharedRef<FInternetAddr> IpAddr = SocketSub.CreateInternetAddr();

		bool bIsValidAddress = false;
		IpAddr->SetIp(*Result.ResolvedAddress, bIsValidAddress);
		if (!bIsValidAddress)
		{
			UE_LOGF(LogPing, Verbose, "SendPing; unresolvable IP address");
			Result.Status = EIcmpResponseStatus::Unresolvable;

			return ESendStatus::BadTarget;
		}

		IpAddr->SetPort(Progress.Port);

		Progress.ToAddr = TSharedPtr<FInternetAddr>(IpAddr);
	}

	check(Progress.ToAddr.IsValid());
	TSharedPtr<FInternetAddr> ToAddr = Progress.ToAddr;

	if (!Progress.SocketPtr)
	{
		// Get/create a socket appropriate for the remote IP address.
		FSocket* NewSocketPtr = GetOrCreateSocket(ToAddr->GetProtocolType());
		if (!NewSocketPtr)
		{
			UE_LOGF(LogPing, Verbose, "SendPing; failed to get socket for IP address: %ls", *Result.ResolvedAddress);

			return ESendStatus::NoSocket;
		}

		Progress.SocketPtr = NewSocketPtr;
	}

	FSocket *const SocketPtr = Progress.SocketPtr;
	check(SocketPtr);
	if (!SocketPtr->Wait(ESocketWaitConditions::WaitForWrite, SendWaitTime))
	{
		// Socket not writable at this time, try again later.
		return ESendStatus::Busy;
	}

	// Create the packet (in host byte order).
	const uint16 SeqNum = static_cast<uint16>(NextSequenceNum());
	Progress.SequenceNum = SeqNum;
	UDPPing::FUdpPingPacket PingPacket = UDPPing::FUdpPingPacket::Create(Progress.EchoId, SeqNum);

	// Pack and send out the packet.
	return SendPing(*SocketPtr, SendBuf, BufSize, *ToAddr, PingPacket, Progress.StartTime);
}

FUdpPingWorker::ESendStatus FUdpPingWorker::SendPing(FSocket& Socket, uint8* SendBuf, const uint32 BufSize,
	const FInternetAddr& ToAddr, UDPPing::FUdpPingPacket& Packet, double& OutSendTimeSecs)
{
	UE_LOGF(LogPing, VeryVerbose, "SendPing; %ls  id=%u (%#06x)",
		*ToAddr.ToString(true), Packet.Header.Id, Packet.Header.Id);

	check(BufSize >= SendDataSize);
	if (BufSize < SendDataSize)
	{
		return ESendStatus::BadBuffer;
	}

	Packet.HostToNetwork();

	// Pack ping packet into send buffer and compute checksum.
	FMemory::Memset(SendBuf, 0, SendDataSize);
	UDPPing::Pack(SendBuf, SendDataSize, Packet);
	UDPPing::UpdatePacketChecksum(SendBuf, SendDataSize, true);

	// Send ping packet
	int32 BytesSent = 0;
	OutSendTimeSecs = FPlatformTime::Seconds();

	if (!Socket.SendTo(SendBuf, SendDataSize, BytesSent, ToAddr))
	{
		UE_LOGF(LogPing, Verbose, "SendPing: failed to send datagram");
		return ESendStatus::SocketSendFail;
	}

	if (BytesSent != SendDataSize)
	{
		UE_LOG(LogPing, Verbose, TEXT("SendPing: failed, sent %d/%" SIZE_T_FMT " bytes"), BytesSent, SendDataSize);
		return ESendStatus::BadSendSize;
	}

	UE_LOG(LogPing, VeryVerbose, TEXT("SendPing; %s  id=%u (%#06x)  seq=%#06x  chk=%#06x  sendtime=%.4f  ok (%d/%" SIZE_T_FMT " bytes)"),
		*ToAddr.ToString(true), NtoHS(Packet.Header.Id), NtoHS(Packet.Header.Id), NtoHS(Packet.Header.Sequence),
		NtoHS(reinterpret_cast<UDPPing::FUdpPingHeader* const>(SendBuf)->Checksum),
		OutSendTimeSecs, BytesSent, SendDataSize);

	return ESendStatus::Ok;
}

bool FUdpPingWorker::HasReplyData(FSocket& Socket, const uint32 MinSize)
{
	uint32 DataSize(0u);
	const bool bHasData = Socket.HasPendingData(DataSize)
		&& (DataSize >= MinSize);
	return bHasData;
}

bool FUdpPingWorker::RecvReplyFrom(FSocket& Socket, uint8* RecvBuf, const uint32 RecvSize,
	FInternetAddr& OutRecvFrom, uint64& OutRecvTimeCode)
{
	FMemory::Memset(RecvBuf, 0, RecvSize);

	int32 BytesRead = 0;
	const bool bRecvOk = Socket.RecvFrom(RecvBuf, RecvSize, BytesRead, OutRecvFrom);

	if (bRecvOk)
	{
		OutRecvTimeCode = FPlatformTime::Cycles64();
	}

	UE_LOGF(LogPing, VeryVerbose, "RecvReplyFrom: recv %d/%d bytes", BytesRead, RecvSize);

	return bRecvOk && (BytesRead == RecvSize);
}

FUdpPingWorker::FProgress* FUdpPingWorker::ProcessReply(const FInternetAddr& FromAddr, uint8* const RecvBuf,
	const uint32 RecvSize, const uint64 RecvTimeCode, TMap<FProgressKey, FProgress>& ProgressTable)
{
	// Unpack reply data from buffer into packet.
	UDPPing::FUdpPingPacket RecvPacket;
	FMemory::Memset(&RecvPacket, 0, sizeof(UDPPing::FUdpPingPacket));
	UDPPing::Unpack(RecvPacket, RecvBuf, RecvSize);

	// Convert to host byte-ordering, except for checksum which is manipulated below.
	RecvPacket.NetworkToHost();

	// Get checksum from packet, converted into host byte order.
	const uint16 RecvChecksum = NtoHS(RecvPacket.Header.Checksum);

	// Calculate checksum for packet data (excluding the checksum data within the packet)
	const uint16 LocalChecksum = UDPPing::CalculatePacketChecksum(RecvBuf, RecvSize);

	if (RecvChecksum != LocalChecksum)
	{
		UE_LOGF(LogPing, Verbose, "ProcessReply; checksum mismatch: recv{%#06x} != local{%#06x}",
			RecvChecksum, LocalChecksum);
		return nullptr;
	}

	// Find corresponding send progress information for this reply.
	const FProgressKey LookupKey(FromAddr.ToString(false), RecvPacket.Header.Id);

	FProgress* const FoundProgress = ProgressTable.Find(LookupKey);
	if (!FoundProgress)
	{
		UE_LOGF(LogPing, Verbose, "ProcessReply; progress info not found for %ls", *LookupKey.Address);
		return nullptr;
	}

	FProgress& Progress = *FoundProgress;
	FIcmpEchoResult& Result = Progress.Result;

	check(LookupKey.Address == Result.ResolvedAddress);
	if (LookupKey.Address != Result.ResolvedAddress)
	{
		// If we get here, the initial progress table setup is wrong, as the resolved address
		// of the remote target is also part of the table element key.
		UE_LOGF(LogPing, Verbose, "ProcessReply; address mismatch  %ls != %ls",
			*LookupKey.Address, *Result.ResolvedAddress);
		return nullptr;
	}

	if (EIcmpResponseStatus::InternalError != Result.Status)
	{
		// Already have a status for this ping-reply pair (e.g. timeout).
		return nullptr;
	}

	// Check that reply data makes sense, and also corresponds to the ID and sequence
	// number of the ping originally sent out.
	if (!RecvPacket.Validate(Progress.EchoId, Progress.SequenceNum))
	{
		UE_LOGF(LogPing, Verbose, "ProcessReply; data mismatch: recv{id=%u  seq=%u} != expect{id=%u  seq=%u}",
			Progress.EchoId, Progress.SequenceNum, RecvPacket.Header.Id, RecvPacket.Header.Sequence);
		return nullptr;
	}

	// Reply checks out, so update the progress result with the remote address,
	// round-trip time, and success/fail/etc. status.

	const double TimeoutSecs = Progress.TimeoutDuration;

	Result.ReplyFrom = LookupKey.Address;
	CalculateTripTime(RecvPacket.Body.TimeCode, RecvTimeCode, TimeoutSecs, Result.Time, Result.Status);

	UE_LOGF(LogPing, VeryVerbose, "ProcessReply; ok: %ls:%u (%ls)  id=%u (%#06x)  seq=%#06x  ping=%.4f s  status=%d",
				 *Progress.Address, Progress.Port, *Result.ResolvedAddress, Progress.EchoId, Progress.EchoId,
				 Progress.SequenceNum, Result.Time, int(Result.Status));

	return FoundProgress;
}

bool FUdpPingWorker::CalculateTripTime(const uint64 ReplyTimeCode, const uint64 CurrentTimeCode, const double TimeoutSecs,
	float& OutTime, EIcmpResponseStatus& OutStatus)
{
	const double DurationSecs = FPlatformTime::ToSeconds64(CurrentTimeCode - ReplyTimeCode);
	const bool bIsValid = (DurationSecs > 0.0) && (DurationSecs < MaxTripTimeSecs);

	if (bIsValid)
	{
		OutTime = static_cast<float>(DurationSecs);
		OutStatus = EIcmpResponseStatus::Success;
	}
	else
	{
		OutTime = -1;
		OutStatus = EIcmpResponseStatus::InternalError;
	}

	UE_LOGF(LogPing, VeryVerbose, "CalculateTripTime; time=%.4f s  status=%d", DurationSecs, int(OutStatus));

	return bIsValid;
}

void FUdpPingWorker::ResetSequenceNum()
{
	SequenceNum = 0;
}

int FUdpPingWorker::NextSequenceNum()
{
	const int Seq = SequenceNum;
	++SequenceNum;
	return Seq;
}

// ----------------------------------------------------------------------------

FUdpPingWorker::FProgress::FProgress()
	: Port(0)
	, StartTime(0.0)
	, TimeoutDuration(0.0)
	, EchoId(0u)
	, SequenceNum(0u)
	, SocketPtr(nullptr)
{
	Result.Status = EIcmpResponseStatus::InternalError;
}

// ----------------------------------------------------------------------------

FUdpPingWorker::FProgressKey::FProgressKey(const FString& InAddress, int InUniqueId)
	: Address(InAddress)
	, UniqueId(InUniqueId)
{}

bool FUdpPingWorker::FProgressKey::operator ==(const FProgressKey& Rhs) const
{
	return (Address == Rhs.Address) && (UniqueId == Rhs.UniqueId);
}

uint32 FUdpPingWorker::FProgressKey::CalcHash() const
{
	return HashCombine(GetTypeHash(Address), GetTypeHash(UniqueId));
}

// ----------------------------------------------------------------------------

FUdpPingManyAsync::FUdpPingManyAsync(ISocketSubsystem* const InSocketSub, const FIcmpEchoManyCompleteDelegate& InCompletionDelegate)
	: FTSTickerObjectBase(0)
	, SocketSub(InSocketSub)
	, CompletionDelegate(InCompletionDelegate)
	, bThreadCompleted(false)
{}

FUdpPingManyAsync::~FUdpPingManyAsync()
{
	check(IsInGameThread());

	if (FutureResult.IsValid())
	{
		FutureResult.Wait();
	}
}

void FUdpPingManyAsync::Start(const TArray<FIcmpTarget>& Targets, float Timeout, uint32 StackSize)
{
	if (!SocketSub)
	{
		bThreadCompleted = true;
		return;
	}

	bThreadCompleted = false;
	TFunction<FIcmpEchoManyCompleteResult()> WorkerTask = [this, Targets, Timeout]()
	{
		auto Result = FUdpPingManyAsync::RunWorker(SocketSub, Targets, Timeout);
		bThreadCompleted = true;
		return Result;
	};

	FutureResult = AsyncThread(WorkerTask, StackSize);
}

bool FUdpPingManyAsync::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UDPPing_Tick);

	if (bThreadCompleted)
	{
		FIcmpEchoManyCompleteResult Result;
		if (FutureResult.IsValid())
		{
			Result = FutureResult.Get();
		}

		CompletionDelegate.ExecuteIfBound(Result);

		delete this;
		return false;
	}

	return true;
}

FIcmpEchoManyCompleteResult FUdpPingManyAsync::RunWorker(ISocketSubsystem* const SocketSub, const TArray<FIcmpTarget>& Targets, float Timeout)
{
	FIcmpEchoManyCompleteResult EndResult;

	FGotResultDelegate OnGotResult;
	OnGotResult.BindLambda([&EndResult](FIcmpEchoManyResult Result)
	{
		// Push result into total.
		EndResult.AllResults.Add(Result);
	});

	FStatusDelegate OnCompleted;
	OnCompleted.BindLambda([&EndResult](EIcmpEchoManyStatus EndStatus)
	{
		// Done, so just need the status of the completed work.
		EndResult.Status = EndStatus;
	});

	FUdpPingWorker PingWorker(SocketSub, Targets, Timeout, OnGotResult, OnCompleted);
	PingWorker.Init();
	PingWorker.Run();

	return EndResult;
}

// ----------------------------------------------------------------------------

namespace UDPPing
{
	void FUdpPingHeader::HostToNetwork()
	{
		Id = HtoNS(Id);
		Sequence = HtoNS(Sequence);
	}

	void FUdpPingHeader::NetworkToHost()
	{
		Id = NtoHS(Id);
		Sequence = NtoHS(Sequence);
	}

	void FUdpPingBody::HostToNetwork()
	{
		TimeCode = HtoNLL(TimeCode);
		Data[0] = HtoNL(Data[0]);
		Data[1] = HtoNL(Data[1]);
	}

	void FUdpPingBody::NetworkToHost()
	{
		TimeCode = NtoHLL(TimeCode);
		Data[0] = NtoHL(Data[0]);
		Data[1] = NtoHL(Data[1]);
	}

	void FUdpPingPacket::HostToNetwork()
	{
		Header.HostToNetwork();
		Body.HostToNetwork();
	}

	void FUdpPingPacket::NetworkToHost()
	{
		Header.NetworkToHost();
		Body.NetworkToHost();
	}

	bool FUdpPingPacket::Validate(uint16 ExpectedId, uint16 ExpectedSequenceNum) const
	{
		// Assumes that the packet is in host byte order.
		return (Header.Id == ExpectedId) && (Header.Sequence == ExpectedSequenceNum) &&
			(Body.Data[0] == PingDataHigh) && (Body.Data[1] == PingDataLow) && (Body.TimeCode != 0u);
	}

	FUdpPingPacket FUdpPingPacket::Create(uint16 Id, uint16 SequenceNum)
	{
		// Create the packet (in host byte order).
		FUdpPingPacket Packet;
		FMemory::Memset(&Packet, 0, sizeof(FUdpPingPacket));
		FUdpPingHeader& Header = Packet.Header;
		Header.Id = Id;
		Header.Sequence = SequenceNum;

		Packet.Body.Data[0] = PingDataHigh;
		Packet.Body.Data[1] = PingDataLow;
		Packet.Body.TimeCode = FPlatformTime::Cycles64();

		return Packet;
	}

	bool Pack(uint8* OutBuf, const SIZE_T BufSize, const FUdpPingPacket& InPacket)
	{
		if (!OutBuf || (BufSize < SizePacked))
		{
			return false;
		}

		FUdpPingHeader& OutHeader = *reinterpret_cast<FUdpPingHeader*>(OutBuf);
		OutHeader.Id = InPacket.Header.Id;
		OutHeader.Sequence = InPacket.Header.Sequence;
		OutHeader.Checksum = InPacket.Header.Checksum;

		FUdpPingBody& OutBody = *reinterpret_cast<FUdpPingBody*>(OutBuf + HeaderSize);
		OutBody.TimeCode = InPacket.Body.TimeCode;
		OutBody.Data[0] = InPacket.Body.Data[0];
		OutBody.Data[1] = InPacket.Body.Data[1];

		return true;
	}

	bool Unpack(FUdpPingPacket& OutPacket, uint8* const InBuf, const SIZE_T BufSize)
	{
		if (!InBuf || (BufSize < SizePacked))
		{
			return false;
		}

		FUdpPingHeader& InHeader = *reinterpret_cast<FUdpPingHeader*>(InBuf);
		OutPacket.Header.Id = InHeader.Id;
		OutPacket.Header.Sequence = InHeader.Sequence;
		OutPacket.Header.Checksum = InHeader.Checksum;

		FUdpPingBody& InBody = *reinterpret_cast<FUdpPingBody*>(InBuf + HeaderSize);
		OutPacket.Body.TimeCode = InBody.TimeCode;
		OutPacket.Body.Data[0] = InBody.Data[0];
		OutPacket.Body.Data[1] = InBody.Data[1];

		return true;
	}

	bool UpdatePacketChecksum(uint8* const PingPacketBuf, const SIZE_T BufSize, const bool ToNetworkByteOrder)
	{
		if (BufSize < SizePacked)
		{
			return false;
		}

		FUdpPingHeader& Header = *reinterpret_cast<FUdpPingHeader*>(PingPacketBuf);
		Header.Checksum = 0;
		const uint16 Checksum = static_cast<uint16>(CalculateChecksum(PingPacketBuf, SizePacked));
		Header.Checksum = ToNetworkByteOrder ? HtoNS(Checksum) : Checksum;
		return true;
	}

	uint16 CalculatePacketChecksum(uint8* const PingPacketBuf, const SIZE_T BufSize)
	{
		check(PingPacketBuf);
		check(BufSize > 0);

		FUdpPingHeader& Header = *reinterpret_cast<FUdpPingHeader*>(PingPacketBuf);
		const uint16 OldValue = Header.Checksum;

		Header.Checksum = 0;
		const uint16 ChecksumResult = static_cast<uint16>(CalculateChecksum(PingPacketBuf, SizePacked));
		Header.Checksum = OldValue;

		return ChecksumResult;
	}

} // namespace UDPPing

// ----------------------------------------------------------------------------

namespace
{
	FSocket* CreateDatagramSocket(ISocketSubsystem& SocketSub, const FName& ProtocolType, bool blocking)
	{
		FSocket* const Socket = SocketSub.CreateSocket(NAME_DGram, TEXT("UDPPing"), ProtocolType);
		if (!Socket)
		{
			return nullptr;
		}

		if (!blocking && !Socket->SetNonBlocking(true))
		{
			SocketSub.DestroySocket(Socket);
			return nullptr;
		}

		return Socket;
	}

	uint64 NtoHLL(uint64 Val)
	{
		static const bool bNoConvert = (NtoHL(1) == 1);
		return bNoConvert ? Val : ((static_cast<uint64>(NtoHL(Val)) << 32) + NtoHL(Val >> 32));
	}

	uint64 HtoNLL(uint64 Val)
	{
		static const bool bNoConvert = (HtoNL(1) == 1);
		return bNoConvert ? Val : ((static_cast<uint64>(HtoNL(Val)) << 32) + HtoNL(Val >> 32));
	}

	int32 CalcStackSize()
	{
		int32 StackSize = 0;

#if PING_ALLOWS_CUSTOM_THREAD_SIZE
		static const int32 MinSize = 32 * 1024;
		static const int32 MaxSize = 2 * 1024 * 1024;

		GConfig->GetInt(TEXT("Ping"), TEXT("StackSize"), StackSize, GEngineIni);

		// Sanity clamp
		if (StackSize != 0)
		{
			StackSize = FMath::Clamp(StackSize, MinSize, MaxSize);
		}
#endif // PING_ALLOWS_CUSTOM_THREAD_SIZE

		return StackSize;
	}
} // namespace (anonymous)

// ----------------------------------------------------------------------------

void FUDPPing::UDPEchoMany(const TArray<FIcmpTarget>& Targets, float Timeout,
	FIcmpEchoManyCompleteCallback CompletionCallback)
{
	FIcmpEchoManyCompleteDelegate CompletionDelegate;
	CompletionDelegate.BindLambda(CompletionCallback);

	UDPEchoMany(Targets, Timeout, CompletionDelegate);
}

void FUDPPing::UDPEchoMany(const TArray<FIcmpTarget>& Targets, float Timeout,
	FIcmpEchoManyCompleteDelegate CompletionDelegate)
{
	const int32 StackSize = CalcStackSize();

	ISocketSubsystem *const SocketSub = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	FUdpPingManyAsync *const PingMany = new FUdpPingManyAsync(SocketSub, CompletionDelegate);
	check(PingMany);
	if (PingMany)
	{
		PingMany->Start(Targets, Timeout, StackSize);
	}
}


// ----------------------------------------------------------------------------
// UDPEcho (single)
// ----------------------------------------------------------------------------

namespace UDPPing
{
	// 2 uint32 magic numbers + 64bit timecode
	const SIZE_T PayloadSize = 4 * sizeof(uint32);
}

FIcmpEchoResult UDPEchoImpl(ISocketSubsystem* SocketSub, const FString& TargetAddress, float Timeout)
{
	struct FUDPPingHeader
	{
		uint16 Id;
		uint16 Sequence;
		uint16 Checksum;
	};

	// Size of the udp header sent/received
	static const SIZE_T UDPPingHeaderSize = sizeof(FUDPPingHeader);

	// The packet we send is just the header plus our payload
	static const SIZE_T PacketSize = UDPPingHeaderSize + UDPPing::PayloadSize;

	// The result read back is just the header plus our payload;
	static const SIZE_T ResultPacketSize = PacketSize;

	// Location of the timecode in the buffer
	static const SIZE_T TimeCodeOffset = UDPPingHeaderSize;

	// Location of the payload in the buffer
	static const SIZE_T MagicNumberOffset = TimeCodeOffset + sizeof(uint64);

	static int PingSequence = 0;

	FIcmpEchoResult Result;
	Result.Status = EIcmpResponseStatus::InternalError;

	FString PortStr;

	TArray<FString> IpParts;
	int32 NumTokens = TargetAddress.ParseIntoArray(IpParts, TEXT(":"));

	FString Address = TargetAddress;
	if (NumTokens == 2)
	{
		Address = IpParts[0];
		PortStr = IpParts[1];
	}

	FString ResolvedAddress;
	if (!ResolveIp(SocketSub, Address, ResolvedAddress))
	{
		Result.Status = EIcmpResponseStatus::Unresolvable;
		return Result;
	}

	int32 Port = 0;
	LexFromString(Port, *PortStr);

	//ISocketSubsystem* SocketSub = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	if (SocketSub)
	{
		TSharedRef<FInternetAddr> ToAddr = SocketSub->CreateInternetAddr();

		bool bIsValid = false;
		ToAddr->SetIp(*ResolvedAddress, bIsValid);
		ToAddr->SetPort(Port);

		if (bIsValid)
		{
			uint8 SendBuffer[PacketSize];
			// Clear packet buffer
			FMemory::Memset(SendBuffer, 0, PacketSize);
			Result.ResolvedAddress = ResolvedAddress;

			FSocket* Socket = SocketSub->CreateSocket(NAME_DGram, TEXT("UDPPing"), ToAddr->GetProtocolType());
			if (Socket)
			{
				uint16 SentId = (uint16)FPlatformProcess::GetCurrentProcessId();
				uint16 SentSeq = PingSequence++;

				FUDPPingHeader* PacketHeader = reinterpret_cast<FUDPPingHeader*>(SendBuffer);
				PacketHeader->Id = HtoNS(SentId);
				PacketHeader->Sequence = HtoNS(SentSeq);
				PacketHeader->Checksum = 0;

				// Put some data into the packet payload
				uint32* PayloadStart = (uint32*)(SendBuffer + MagicNumberOffset);
				PayloadStart[0] = HtoNL(PingDataHigh);
				PayloadStart[1] = HtoNL(PingDataLow);

				// Calculate the time packet is to be sent
				uint64* TimeCodeStart = (uint64*)(SendBuffer + TimeCodeOffset);
				uint64 TimeCode = FPlatformTime::Cycles64();
				TimeCodeStart[0] = TimeCode;

				// Calculate the packet checksum
				PacketHeader->Checksum = CalculateChecksum(SendBuffer, PacketSize);

				uint8 ResultBuffer[ResultPacketSize];

				double TimeLeft = Timeout;
				double StartTime = FPlatformTime::Seconds();

				int32 BytesSent = 0;
				if (Socket->SendTo(SendBuffer, PacketSize, BytesSent, *ToAddr))
				{
					bool bDone = false;
					while (!bDone)
					{
						if (Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromSeconds(TimeLeft)))
						{
							double EndTime = FPlatformTime::Seconds();
							TimeLeft = FPlatformMath::Max(0.0, (double)Timeout - (EndTime - StartTime));

							int32 BytesRead = 0;
							TSharedRef<FInternetAddr> RecvAddr = SocketSub->CreateInternetAddr();
							if (Socket->RecvFrom(ResultBuffer, ResultPacketSize, BytesRead, *RecvAddr))
							{
								if (BytesRead == ResultPacketSize)
								{
									uint64 NowTime = FPlatformTime::Cycles64();

									Result.ReplyFrom = RecvAddr->ToString(false);
									FUDPPingHeader* RecvHeader = reinterpret_cast<FUDPPingHeader*>(ResultBuffer);

									// Validate the packet checksum
									const uint16 RecvChecksum = RecvHeader->Checksum;
									RecvHeader->Checksum = 0;
									const uint16 LocalChecksum = (uint16)CalculateChecksum((uint8*)RecvHeader, PacketSize);

									if (RecvChecksum == LocalChecksum)
									{
										// Convert values back from network byte order
										RecvHeader->Id = NtoHS(RecvHeader->Id);
										RecvHeader->Sequence = NtoHS(RecvHeader->Sequence);

										uint32* MagicNumberPtr = (uint32*)(ResultBuffer + MagicNumberOffset);
										if (MagicNumberPtr[0] == PingDataHigh && MagicNumberPtr[1] == PingDataLow)
										{
											// Estimate elapsed time
											uint64* TimeCodePtr = (uint64*)(ResultBuffer + TimeCodeOffset);
											uint64 PrevTime = *TimeCodePtr;
											double DeltaTime = (NowTime - PrevTime) * FPlatformTime::GetSecondsPerCycle64();

											if (Result.ReplyFrom == Result.ResolvedAddress &&
												RecvHeader->Id == SentId && RecvHeader->Sequence == SentSeq &&
												DeltaTime >= 0.0 && DeltaTime < (60.0 * 1000.0))
											{
												Result.Time = DeltaTime;
												Result.Status = EIcmpResponseStatus::Success;
											}
										}
									}

									bDone = true;
								}
							}
							else
							{
								// error reading from socket
								bDone = true;
							}
						}
						else
						{
							// timeout
							Result.Status = EIcmpResponseStatus::Timeout;
							Result.ReplyFrom.Empty();
							Result.Time = Timeout;

							bDone = true;
						}
					}
				}
			}

			SocketSub->DestroySocket(Socket);
		}
	}

	return Result;
}

class FUDPPingAsyncResult
	: public FTSTickerObjectBase
{
public:

	FUDPPingAsyncResult(ISocketSubsystem* InSocketSub, const FString& TargetAddress, float Timeout, uint32 StackSize, FIcmpEchoResultCallback InCallback)
		: FTSTickerObjectBase(0)
		, SocketSub(InSocketSub)
		, Callback(InCallback)
		, bThreadCompleted(false)
	{
		if (SocketSub)
		{
			bThreadCompleted = false;
			TFunction<FIcmpEchoResult()> Task = [this, TargetAddress, Timeout]()
			{
				auto Result = UDPEchoImpl(SocketSub, TargetAddress, Timeout);
				bThreadCompleted = true;
				return Result;
			};

			// if we don't need a special stack size, use the task graph
			if (StackSize == 0)
			{
				FutureResult = Async(EAsyncExecution::ThreadPool, Task);
			}
			else
			{
				FutureResult = AsyncThread(Task, StackSize);
			}
		}
		else
		{
			bThreadCompleted = true;
		}
	}

	virtual ~FUDPPingAsyncResult()
	{
		check(IsInGameThread());

		if (FutureResult.IsValid())
		{
			FutureResult.Wait();
		}
	}

private:
	virtual bool Tick(float DeltaTime) override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UDPPing_Tick);

		if (bThreadCompleted)
		{
			FIcmpEchoResult Result;
			if (FutureResult.IsValid())
			{
				Result = FutureResult.Get();
			}

			Callback(Result);

			delete this;
			return false;
		}
		return true;
	}

	/** Reference to the socket subsystem */
	ISocketSubsystem* SocketSub;
	/** Callback when the ping result returns */
	FIcmpEchoResultCallback Callback;
	/** Thread task complete */
	FThreadSafeBool bThreadCompleted;
	/** Async result future */
	TFuture<FIcmpEchoResult> FutureResult;
};

void FUDPPing::UDPEcho(const FString& TargetAddress, float Timeout, FIcmpEchoResultCallback HandleResult)
{
	int32 StackSize = 0;

#if PING_ALLOWS_CUSTOM_THREAD_SIZE
	GConfig->GetInt(TEXT("Ping"), TEXT("StackSize"), StackSize, GEngineIni);

	// Sanity clamp
	if (StackSize != 0)
	{
		StackSize = FMath::Max<int32>(FMath::Min<int32>(StackSize, 2 * 1024 * 1024), 32 * 1024);
	}
#endif

	ISocketSubsystem* SocketSub = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	new FUDPPingAsyncResult(SocketSub, TargetAddress, Timeout, StackSize, HandleResult);
}
