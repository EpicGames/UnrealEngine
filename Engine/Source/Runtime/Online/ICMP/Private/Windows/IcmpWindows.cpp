// Copyright Epic Games, Inc. All Rights Reserved.

#include "Icmp.h"
#include "IcmpPrivate.h"
#include "IcmpModule.h"
#include "SocketSubsystem.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#define _WINSOCK_DEPRECATED_NO_WARNINGS
#include "WinSock2.h"
#include "Ws2tcpip.h"
#include "iphlpapi.h"
#include "IcmpAPI.h"
#undef _WINSOCK_DEPRECATED_NO_WARNINGS
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

typedef ICMP_ECHO_REPLY32 FIcmpEchoReply;

#define ICMP_MAX_REPLIES 10

uint16 NtoHS(uint16 val)
{
	return ntohs(val);
}

uint16 HtoNS(uint16 val)
{
	return htons(val);
}

uint32 NtoHL(uint32 val)
{
	return ntohl(val);
}

uint32 HtoNL(uint32 val)
{
	return htonl(val);
}

namespace IcmpWindows
{
	// 32 bytes is the default size for the windows ping utility, and windows has problems with sending < 18 bytes.
	const SIZE_T IcmpPayloadSize = 32;
	const uint8 IcmpPayload[IcmpPayloadSize] = ">>>>This string is 32 bytes<<<<";

	// Returns the ip address as string
	FString IpToString(void* Address)
	{
		TCHAR Buffer[16];
		PCWSTR Ret = InetNtop(AF_INET, Address, Buffer, 16);
		if (Ret == nullptr)
		{
			UE_LOGF(LogIcmp, Warning, "Error converting Ip Address: 0x%08x", static_cast<int32>(GetLastError()));
		}
		return Buffer;
	}
}

FIcmpEchoResult IcmpEchoImpl(ISocketSubsystem* SocketSub, const FString& TargetAddress, float Timeout)
{
	FIcmpEchoResult Result;

	FString ResolvedAddress;
	if (!ResolveIp(SocketSub, TargetAddress, ResolvedAddress))
	{
		Result.Status = EIcmpResponseStatus::Unresolvable;
		return Result;
	}
	Result.ResolvedAddress = ResolvedAddress;

	// Allow for up to ICMP_MAX_REPLIES replies in case we get any bogus replies from other nodes
	static const SIZE_T ReplyBufferSize = (sizeof(FIcmpEchoReply) + IcmpWindows::IcmpPayloadSize) * ICMP_MAX_REPLIES;
	uint8 ReplyBuffer[ReplyBufferSize];

	uint32 Ip = inet_addr(TCHAR_TO_UTF8(*ResolvedAddress));
	if (Ip == INADDR_NONE)
	{
		// Invalid address returned from resolver
		return Result;
	}

	HANDLE IcmpSocket = IcmpCreateFile();
	if (IcmpSocket == INVALID_HANDLE_VALUE)
	{
		// Internal error opening icmp handle
		return Result;
	}

	uint32 RetVal = IcmpSendEcho(IcmpSocket, Ip, const_cast<uint8*>(&IcmpWindows::IcmpPayload[0]), IcmpWindows::IcmpPayloadSize, nullptr, ReplyBuffer, ReplyBufferSize, int(Timeout * 1000));
	if (RetVal > 0)
	{
		FIcmpEchoReply* EchoReplies = reinterpret_cast<FIcmpEchoReply*>(ReplyBuffer);
		bool bDone = false;

		// Default to Timeout, unless other statuses are seen in the replies
		Result.Status = EIcmpResponseStatus::Timeout;
		for(uint32 I=0; I<RetVal && !bDone; ++I)
		{
			Result.Time = float(EchoReplies[I].RoundTripTime) / 1000.0;
			Result.ReplyFrom = IcmpWindows::IpToString(reinterpret_cast<void*>(&EchoReplies[I].Address));
			switch (EchoReplies[I].Status)
			{
				case IP_SUCCESS:
					// Only accept a successful reply comming from the resolved IP address, otherwise keep looping through the results
					if (Result.ReplyFrom == Result.ResolvedAddress)
					{
						Result.Status = EIcmpResponseStatus::Success;
						bDone = true;
					}
					break;
				case IP_DEST_HOST_UNREACHABLE:
				case IP_DEST_NET_UNREACHABLE:
				case IP_DEST_PROT_UNREACHABLE:
				case IP_DEST_PORT_UNREACHABLE:
					Result.Status = EIcmpResponseStatus::Unreachable;
					// Keep looping through the results, in case we received both an unreachable error and a valid reply
					break;
				case IP_REQ_TIMED_OUT:
					// Simply ignore, if there are other reply packets use the status from them instead, otherwise we already default to Timeout.
					// (Is this actually possible btw?)
					break;
				default:
					// Treating all other errors as internal ones, but keep looking in case there are multiple replies
					Result.Status = EIcmpResponseStatus::InternalError;
					break;
			}
		}
	}
	else if (GetLastError() == IP_REQ_TIMED_OUT)
	{
		Result.Status = EIcmpResponseStatus::Timeout;
	}
	IcmpCloseHandle(IcmpSocket);
	return Result;
}
