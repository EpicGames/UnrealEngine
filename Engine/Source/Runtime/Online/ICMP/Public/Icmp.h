// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"

enum class EIcmpResponseStatus
{
	/** We did receive a valid Echo reply back from the target host */
	Success,
	/** We did not receive any results within the time limit */
	Timeout,
	/** We got an unreachable error from another node on the way */
	Unreachable,
	/** We could not resolve the target address to a valid IP address */
	Unresolvable,
	/** Some internal error happened during setting up or sending the ping packet */
	InternalError,
	/** not implemented - used to indicate we haven't implemented ICMP ping on this platform */
	NotImplemented,
};

inline const TCHAR* LexToString(EIcmpResponseStatus ResponseStatus)
{
	switch (ResponseStatus)
	{
		case EIcmpResponseStatus::Success: return TEXT("Success");
		case EIcmpResponseStatus::Timeout: return TEXT("Timeout");
		case EIcmpResponseStatus::Unreachable: return TEXT("Unreachable");
		case EIcmpResponseStatus::Unresolvable: return TEXT("Unresolvable");
		case EIcmpResponseStatus::InternalError: return TEXT("InternalError");
		case EIcmpResponseStatus::NotImplemented: return TEXT("NotImplemented");
		default: return TEXT("Unknown");
	}
}

struct FIcmpEchoResult
{
	/** Status of the final response */
	EIcmpResponseStatus Status;
	/** Addressed resolved by GetHostName */
	FString ResolvedAddress;
	/** Reply received from this address */
	FString ReplyFrom;
	/** Total round trip time */
	float Time;

	FIcmpEchoResult()
		: Status(EIcmpResponseStatus::InternalError)
		, ResolvedAddress()
		, ReplyFrom()
		, Time(-1)
	{}
};

typedef TFunction<void(FIcmpEchoResult)> FIcmpEchoResultCallback;
DECLARE_DELEGATE_OneParam(FIcmpEchoResultDelegate, FIcmpEchoResult);

struct FIcmpTarget
{
	FString Address;
	int32 Port;

	FIcmpTarget()
		: Port(0)
	{}

	FIcmpTarget(const FString& InAddress, int32 InPort)
		: Address(InAddress)
		, Port(InPort)
	{}
}; // struct FIcmpTarget

struct FIcmpEchoManyResult
{
	FIcmpEchoResult EchoResult;
	FIcmpTarget Target;

	FIcmpEchoManyResult() = default;
	FIcmpEchoManyResult(const FIcmpEchoResult& InEchoResult, const FIcmpTarget& InTarget)
		: EchoResult(InEchoResult)
		, Target(InTarget)
	{}

}; // struct FIcmpEchoManyResult

enum class EIcmpEchoManyStatus : uint8
{
	Invalid,
	Success,
	Failure,
	Canceled
};

struct FIcmpEchoManyCompleteResult
{
	TArray<FIcmpEchoManyResult> AllResults;
	EIcmpEchoManyStatus Status;

	FIcmpEchoManyCompleteResult()
		: Status(EIcmpEchoManyStatus::Invalid)
	{}

}; // struct FIcmpEchoManyCompleteResult

typedef TFunction<void(FIcmpEchoManyCompleteResult)> FIcmpEchoManyCompleteCallback;
DECLARE_DELEGATE_OneParam(FIcmpEchoManyCompleteDelegate, FIcmpEchoManyCompleteResult);


// Simple ping interface that sends an ICMP packet to the given address and returns timing info for the reply if reachable
class FIcmp
{
	public:

	/** Send an ICMP echo packet and wait for a reply.
	 *
	 * The name resolution and ping send/receive will happen on a separate thread.
	 * The third argument is a callback function that will be invoked on the game thread after the
	 * a reply has been received from the target address, the timeout has expired, or if there
	 * was an error resolving the address or delivering the ICMP message to it.
	 *
	 * Multiple pings can be issued concurrently and this function will ensure they're executed in
	 * turn in order not to mix ping replies from different nodes.
	 *
	 * @param TargetAddress the target address to ping
	 * @param Timeout max time to wait for a reply
	 * @param HandleResult a callback function that will be called when the result is ready
	 */
	static ICMP_API void IcmpEcho(const FString& TargetAddress, float Timeout, FIcmpEchoResultCallback HandleResult);

	/** Send an ICMP echo packet and wait for a reply.
	 *
	 * This is a wrapper around the above function, taking a delegate instead of a function argument.
	 *
	 * @param TargetAddress the target address to ping
	 * @param Timeout max time to wait for a reply
	 * @param ResultDelegate a delegate that will be called when the result is ready
	 */
	static void IcmpEcho(const FString& TargetAddress, float Timeout, FIcmpEchoResultDelegate ResultDelegate)
	{
		IcmpEcho(TargetAddress, Timeout, [ResultDelegate](FIcmpEchoResult Result)
		{
			ResultDelegate.ExecuteIfBound(Result);
		});
	}
};

// Simple ping interface that sends an ICMP packet over UDP to the given address and returns timing info for the reply if reachable
class FUDPPing
{
public:

	/** Send an ICMP echo packet and wait for a reply.
	 *
	 * The name resolution and ping send/receive will happen on a separate thread.
	 * The third argument is a callback function that will be invoked on the game thread after the
	 * a reply has been received from the target address, the timeout has expired, or if there
	 * was an error resolving the address or delivering the ICMP message to it.
	 *
	 * Multiple pings can be issued concurrently and this function will ensure they're executed in
	 * turn in order not to mix ping replies from different nodes.
	 *
	 * @param TargetAddress the target address to ping
	 * @param Timeout max time to wait for a reply
	 * @param HandleResult a callback function that will be called when the result is ready
	 */
	static ICMP_API void UDPEcho(const FString& TargetAddress, float Timeout, FIcmpEchoResultCallback HandleResult);

	/** Send an ICMP echo packet and wait for a reply.
	 *
	 * This is a wrapper around the above function, taking a delegate instead of a function argument.
	 *
	 * @param TargetAddress the target address to ping
	 * @param Timeout max time to wait for a reply
	 * @param ResultDelegate a delegate that will be called when the result is ready
	 */
	static void UDPEcho(const FString& TargetAddress, float Timeout, FIcmpEchoResultDelegate ResultDelegate)
	{
		UDPEcho(TargetAddress, Timeout, [ResultDelegate](FIcmpEchoResult Result)
		{
			ResultDelegate.ExecuteIfBound(Result);
		});
	}

	/** Send multiple ICMP echo packets and wait for replies.
	 *
	 * Creates a new thread in which name resolution and ping send/receive are handled for all
	 * supplied target addresses.  These are non-blocking requests, sent and received on a single
	 * network socket.
	 *
	 * The third argument is a callback function that will be invoked on the game thread after
	 * all replies are received, timed out, or otherwise terminated (e.g. remote address could
	 * not be resolved, host unreachable, send failed, etc.).  It passes a FIcmpEchoManyCompleteResult
	 * instance, containing the overall completion status, and the ping results obtained so far
	 * for all of the supplied targets.
	 *
	 * @param Targets the target addresses to ping
	 * @param Timeout max time to wait for a replies
	 * @param CompletionCallback callback function that is invoked when the final results are ready
	 */
	static ICMP_API void UDPEchoMany(const TArray<FIcmpTarget>& Targets, float Timeout, FIcmpEchoManyCompleteCallback CompletionCallback);

	/** Send multiple ICMP echo packets and wait for replies.
	 *
	 * Creates a new thread in which name resolution and ping send/receive are handled for all
	 * supplied target addresses.  These are non-blocking requests, sent and received on a single
	 * network socket.
	 *
	 * The third argument is a callback function that will be invoked on the game thread after
	 * all replies are received, timed out, or otherwise terminated (e.g. remote address could
	 * not be resolved, host unreachable, send failed, etc.).  It passes a FIcmpEchoManyCompleteResult
	 * instance, containing the overall completion status, and the ping results obtained so far
	 * for all of the supplied targets.
	 *
	 * @param Targets the target addresses to ping
	 * @param Timeout max time to wait for a replies
	 * @param CompletionDelegate delegate that is invoked when the final results are ready
	 */
	static ICMP_API void UDPEchoMany(const TArray<FIcmpTarget>& Targets, float Timeout, FIcmpEchoManyCompleteDelegate CompletionDelegate);
};


#define EnumCase(Name) case EIcmpResponseStatus::Name : return TEXT(#Name)
static const TCHAR* ToString(EIcmpResponseStatus Status)
{
	switch (Status)
	{
		EnumCase(Success);
		EnumCase(Timeout);
		EnumCase(Unreachable);
		EnumCase(Unresolvable);
		EnumCase(InternalError);
		EnumCase(NotImplemented);
		default:
			return TEXT("Unknown");
	}
}

#undef EnumCase
