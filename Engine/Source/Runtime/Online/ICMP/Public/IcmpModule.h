// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FOutputDevice;
class UWorld;

/** Logging related to parties */
ICMP_API DECLARE_LOG_CATEGORY_EXTERN(LogIcmp, Display, All);

/**
 * Module for Icmp service utilities
 */
class FIcmpModule : 
	public IModuleInterface, public FSelfRegisteringExec
{

public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FIcmpModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FIcmpModule>("Icmp");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("Icmp");
	}

private:

	// FSelfRegisteringExec

	/**
	 * Handle exec commands starting with "Icmp"
	 *
	 * @param InWorld	the world context
	 * @param Cmd		the exec command being executed
	 * @param Ar		the archive to log results to
	 *
	 * @return true if the handler consumed the input, false to continue searching handlers
	 */
	virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

	// IModuleInterface

	/**
	 * Called when icmp module is loaded
	 * Initialize platform specific parts of template handling
	 */
	virtual void StartupModule() override;
	
	/**
	 * Called when icmp module is unloaded
	 * Shutdown platform specific parts of template handling
	 */
	virtual void ShutdownModule() override;
};

