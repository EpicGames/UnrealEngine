// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ImageDownload : ModuleRules
{
	public ImageDownload(ReadOnlyTargetRules Target) : base(Target)
    {
        PublicDefinitions.Add("IMAGEDOWNLOAD_PACKAGE=1");

        PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"HTTP",
				"SlateCore",
				"ImageCore",
				"ImageWrapper",
			}
			);
    }
}
