// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ImageDownloadPrivate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogImageDownload);

/**
 * Image Download module
 */
class FImageDownloadModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface

	virtual void StartupModule( ) override { }
	virtual void ShutdownModule( ) override { }
};

IMPLEMENT_MODULE(FImageDownloadModule, ImageDownload);

