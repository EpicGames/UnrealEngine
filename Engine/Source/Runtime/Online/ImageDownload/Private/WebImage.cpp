// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebImage.h"

#include "IImageWrapperModule.h"
#include "HttpModule.h"
#include "Modules/ModuleManager.h"
#include "Styling/CoreStyle.h"

#include "ImageDownloadPrivate.h"


FWebImage::FWebImage()
	: StandInBrush(FCoreStyle::Get().GetDefaultBrush())
	, DownloadedBrushDrawType(ESlateBrushDrawType::Image)
	, bDownloadSuccess(false)
{
}

FWebImage::~FWebImage()
{
	CancelDownload();
}

void FWebImage::SetDownloadedBrushDrawType(ESlateBrushDrawType::Type DrawType)
{
	DownloadedBrushDrawType = DrawType;
	if (DownloadedBrush)
	{
		DownloadedBrush->DrawAs = DownloadedBrushDrawType;
	}
}

void FWebImage::SetDownloadedBrushOutlineSettings(FSlateBrushOutlineSettings OutlineSettings)
{
	DownloadedBrushOutlineSettings = MoveTemp(OutlineSettings);
	if (DownloadedBrush)
	{
		DownloadedBrush->OutlineSettings = DownloadedBrushOutlineSettings;
	}
}

TAttribute< const FSlateBrush* > FWebImage::Attr() const
{
	return TAttribute< const FSlateBrush* >(AsShared(), &FWebImage::GetBrush);
}

bool FWebImage::BeginDownload(const FString& UrlIn, const TOptional<FString>& StandInETag, const FOnImageDownloaded& DownloadCb)
{
	CancelDownload();

	// store the url
	Url = UrlIn;

	// make a new request
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Accept"), TEXT("image/png, image/x-png, image/jpeg; q=0.8, image/vnd.microsoft.icon, image/x-icon, image/bmp, image/*; q=0.5, image/webp; q=0.0"));
	HttpRequest->OnProcessRequestComplete().BindSP(this, &FWebImage::HttpRequestComplete);

	if (StandInETag.IsSet())
	{
		HttpRequest->SetHeader(TEXT("If-None-Match"), *StandInETag.GetValue());
	}

	// queue the request
	if (!HttpRequest->ProcessRequest())
	{
		return false;
	}
	else
	{
		PendingRequest = HttpRequest;
		PendingCallback = DownloadCb;
		return true;
	}
}

void FWebImage::HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	// clear our handle to the request
	PendingRequest.Reset();

	// get the request URL
	check(HttpRequest.IsValid()); // this should be valid, we did just send a request...
	if (HttpRequest->OnProcessRequestComplete().IsBound())
	{
		HttpRequest->OnProcessRequestComplete().Unbind();
	}
	bool bSuccess = ProcessHttpResponse(HttpRequest->GetURL(), bSucceeded ? HttpResponse : FHttpResponsePtr());

	// save this info
	bDownloadSuccess = bSuccess;
	DownloadTimeUtc = FDateTime::UtcNow();

	// fire the response delegate
	if (PendingCallback.IsBound())
	{
		PendingCallback.Execute(bSuccess);
		PendingCallback.Unbind();
	}
}

bool FWebImage::ProcessHttpResponse(const FString& RequestUrl, FHttpResponsePtr HttpResponse)
{
	// check for successful response
	if (!HttpResponse.IsValid())
	{
		UE_LOGF(LogImageDownload, Error, "Image Download: Connection Failed. url=%ls", *RequestUrl);
		return false;
	}

	ETag = HttpResponse->GetHeader("ETag");

	// check status code
	int32 StatusCode = HttpResponse->GetResponseCode();
	if (StatusCode / 100 != 2)
	{
		if ( StatusCode == 304)
		{
			// Not modified means that the image is identical to the placeholder image.
			return true;
		}
		UE_LOGF(LogImageDownload, Error, "Image Download: HTTP response %d. url=%ls", StatusCode, *RequestUrl);
		return false;
	}

	// build an image wrapper for this type
	static const FName MODULE_IMAGE_WRAPPER("ImageWrapper");
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(MODULE_IMAGE_WRAPPER);

	const TArray<uint8>& Content = HttpResponse->GetContent();
	FImage DownloadedImage;
	if (!ImageWrapperModule.DecompressImage(Content.GetData(), Content.Num(), DownloadedImage))
	{
		FString ContentType = HttpResponse->GetContentType();
		UE_LOGF(LogImageDownload, Error, "Image Download: Could not recognize file type of image downloaded from url %ls, server-reported content type: %ls", *RequestUrl, *ContentType);
		return false;
	}

	DownloadedImage.ChangeFormat(ERawImageFormat::BGRA8, EGammaSpace::sRGB);

	// make a dynamic image
	FName ResourceName(*RequestUrl);
	// FImage.RawData is a TArray64<uint8> while FSlateDynamicImageBrush::CreateWithImageData expects a TArray<uint8>. We need to copy the image data to adapt
	TArray<uint8> RawImageCopy(DownloadedImage.RawData);
	DownloadedBrush = FSlateDynamicImageBrush::CreateWithImageData(ResourceName, FVector2D((float)DownloadedImage.SizeX, (float)DownloadedImage.SizeY), RawImageCopy);
	if (DownloadedBrush)
	{
		DownloadedBrush->DrawAs = DownloadedBrushDrawType;
		DownloadedBrush->OutlineSettings = DownloadedBrushOutlineSettings;
	}
	return DownloadedBrush.IsValid();
}

void FWebImage::CancelDownload()
{
	if (PendingRequest.IsValid())
	{
		if (PendingRequest->OnProcessRequestComplete().IsBound())
		{
			PendingRequest->OnProcessRequestComplete().Unbind();
		}
		PendingRequest->CancelRequest();
		PendingRequest.Reset();
	}
	if (PendingCallback.IsBound())
	{
		PendingCallback.Unbind();
	}
	bDownloadSuccess = false;
}
