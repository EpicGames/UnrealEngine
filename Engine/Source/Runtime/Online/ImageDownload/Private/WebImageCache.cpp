// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebImageCache.h"
#include "Styling/CoreStyle.h"

FWebImageCache::FWebImageCache()
: DefaultStandInBrush(FCoreStyle::Get().GetDefaultBrush())
{
}

void FWebImageCache::PreUnload()
{
	for (const TSharedRef<FWebImage>& WebImage : StrongRefCache)
	{
		WebImage->CancelDownload();
	}
}

void FWebImageCache::Empty()
{
	for (const TSharedRef<FWebImage>& WebImage : StrongRefCache)
	{
		WebImage->CancelDownload();
	}

	UrlToImageMap.Empty();
	StrongRefCache.Empty();
}

TSharedRef<const FWebImage> FWebImageCache::Download(const FString& Url, const TOptional<FString>& DefaultImageUrl)
{
	TAttribute<const FSlateBrush*> StandInBrush;
	TOptional<FString> StandInEtag;
	// If the optional DefaultImageUrl is set, use its brush as the stand-in image, falling back to the default brush if not downloaded yet.
	if (DefaultImageUrl.IsSet())
	{
		TSharedRef<const FWebImage> DefaultImage = Download(DefaultImageUrl.GetValue());
		StandInBrush = DefaultImage->Attr();

		// If already downloaded, the etag will prevent further downloads, if the current URLs contents are identical to the default image.
		StandInEtag = DefaultImage->GetETag();
	}
	else
	{
		StandInBrush = DefaultStandInBrush;
	}

	// canonicalize URL (we don't currently have code to do this so just treat the URL as opaque)
	const FString& CanonicalUrl = Url;

	// see if there's a cached copy
	TWeakPtr<FWebImage>* ImageFind = UrlToImageMap.Find(CanonicalUrl);
	TSharedPtr<FWebImage> ImagePtr;
	if (ImageFind != nullptr)
	{
		ImagePtr = ImageFind->Pin();
		if (!ImagePtr.IsValid())
		{
			UrlToImageMap.Remove(CanonicalUrl);
		}
	}

	// return any pinned version
	if (ImagePtr.IsValid())
	{
		// if it is done and we failed, and it's being requested again, queue up another try
		if (ImagePtr->DidDownloadFail())
		{
			ImagePtr->SetStandInBrush(StandInBrush);
			ImagePtr->BeginDownload(ImagePtr->GetUrl(), StandInEtag);
		}

		// return the image ptr
		TSharedRef<FWebImage> ImageRef = ImagePtr.ToSharedRef();
		StrongRefCache.AddUnique(ImageRef);
		return ImageRef;
	}

	// make a new one
	TSharedRef<FWebImage> WebImage = MakeShareable(new FWebImage());
	WebImage->SetStandInBrush(StandInBrush);
	WebImage->BeginDownload(CanonicalUrl, StandInEtag);

	// add it to the cache
	StrongRefCache.Add(WebImage);
	UrlToImageMap.Add(CanonicalUrl, WebImage);

	return WebImage;
}

void FWebImageCache::RelinquishUnusedImages()
{
	StrongRefCache.Empty();
	for (auto It = UrlToImageMap.CreateConstIterator(); It;)
	{
		if (!It.Value().IsValid())
		{
			auto RemoveMe = It;
			++It;
			UrlToImageMap.Remove(RemoveMe.Key());
		}
		else
		{
			++It;
		}
	}
}
