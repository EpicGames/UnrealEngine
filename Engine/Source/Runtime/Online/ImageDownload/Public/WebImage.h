// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/DateTime.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "IImageWrapper.h"

class IHttpRequest;
struct FSlateBrush;

typedef TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> FHttpRequestPtr;
typedef TSharedPtr<class IHttpResponse, ESPMode::ThreadSafe> FHttpResponsePtr;

/** 
 * This class manages downloading an image and swapping it out a standin once it's done.
 *
 * Example (not cached):
 *   FWebImage Image;
 *   Image.SetStandInBrush(FStyle::Get()->GetBrush("Foo"));
 *   Image.BeginDownload(Url, OptionalCallback);
 *
 *   SNew(SImage)
 *   .Image(Image.Attr())
 *
 * You may want to get your FWebImage from a FWebImageCache so we're not re-downloading the same URL all the time.
 *
 * Example (cached):
 *   FWebImageCache ImageCache; // usually global
 *   ImageCache.SetStandInBrush(FStyle::Get()->GetBrush("Foo"));
 *
 *   SNew(SImage)
 *   .Image(ImageCache.Download(Url)->Attr())
 */
class FWebImage 
	: public TSharedFromThis<FWebImage>
{
public:
	IMAGEDOWNLOAD_API FWebImage();
	IMAGEDOWNLOAD_API ~FWebImage();

	/**
	 * Fired when the image finishes downloading or is canceled.
	 * @param Success True if the downloaded image will now be returned by GetBrush()
	 */
	DECLARE_DELEGATE_OneParam(FOnImageDownloaded, bool);

	/** Set the brush that is currently being returned (this will be overridden when any async download completes) */
	inline FWebImage& SetStandInBrush(TAttribute<const FSlateBrush*> StandInBrushIn)
	{
		StandInBrush = StandInBrushIn;
		DownloadedBrush.Reset();
		return *this;
	}

	/** Set draw type for the downloaded image brush */
	IMAGEDOWNLOAD_API void SetDownloadedBrushDrawType(ESlateBrushDrawType::Type DrawType);

	/** Set outline settings for the downloaded image brush */
	IMAGEDOWNLOAD_API void SetDownloadedBrushOutlineSettings(FSlateBrushOutlineSettings OutlineSettings);

	/** Begin downloading an image. This will automatically set the current brush to the downloaded image when it completes (if successful) */
	IMAGEDOWNLOAD_API bool BeginDownload(const FString& InUrl, const TOptional<FString>& StandInETag = TOptional<FString>(), const FOnImageDownloaded& DownloadCallback = FOnImageDownloaded());

	/** Begin downloading an image. */
	inline bool BeginDownload(const FString& InUrl, const FOnImageDownloaded& DownloadCallback)
	{
		return BeginDownload(InUrl, TOptional<FString>(), DownloadCallback);
	}

	/** Cancel any download in progress */
	IMAGEDOWNLOAD_API void CancelDownload();

public:

	/** Use .Attr() to pass this brush into a slate attribute */
	IMAGEDOWNLOAD_API TAttribute< const FSlateBrush* > Attr() const;

	/** Get the current brush displayed (will automatically change when download completes) */
	inline const FSlateBrush* GetBrush() const { return DownloadedBrush.IsValid() ? DownloadedBrush.Get() : StandInBrush.Get(); }

	/** Only returns the downloaded brush. May be null if the download hasn't finished or was unsuccessful */
	inline const FSlateBrush* GetDownloadedBrush() const { return DownloadedBrush.IsValid() ? DownloadedBrush.Get() : nullptr; }

	/** Is there a pending HTTP request */
	inline bool IsDownloadPending() const { return PendingRequest.IsValid(); }

	/** Has the download finished AND was it successful */
	inline bool DidDownloadSucceed() const { return bDownloadSuccess; }

	/** Has the download finished AND did it fail */
	inline bool DidDownloadFail() const { return !IsDownloadPending() && !DidDownloadSucceed(); }

	/** What URL was requested */
	inline const FString& GetUrl() const { return Url; }

	/** What is the ETag of the downloaded resource */
	inline const TOptional<FString>& GetETag() const { return ETag; }

private:
	/** request complete callback */
	IMAGEDOWNLOAD_API void HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
	IMAGEDOWNLOAD_API bool ProcessHttpResponse(const FString& RequestUrl, FHttpResponsePtr HttpResponse);

private:
	/** The Url being downloaded */
	FString Url;

	/** The image resource to show */
	TAttribute< const FSlateBrush* > StandInBrush;

	/** The most recently downloaded and generated brush */
	TSharedPtr<FSlateDynamicImageBrush> DownloadedBrush;

	/** Outline settings for the downloaded image brush */
	FSlateBrushOutlineSettings DownloadedBrushOutlineSettings;

	/** Draw type for the downloaded image brush */
	ESlateBrushDrawType::Type DownloadedBrushDrawType;

	/** Any pending request */
	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> PendingRequest;

	/** Callback to call upon completion */
	FOnImageDownloaded PendingCallback;

	/** Have we successfully downloaded the URL we asked for */
	bool bDownloadSuccess;

	/** When did the download complete */
	FDateTime DownloadTimeUtc;

	/** The ETag of the downloaded image */
	TOptional<FString> ETag;
};
