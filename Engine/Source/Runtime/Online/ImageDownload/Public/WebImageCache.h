// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "WebImage.h"

struct FSlateBrush;

/** 
 * This class is designed to facilitate caching of web images and setting a global stand-in so we don't
 * re-download the same image every time the UI shows it again.
 *
 * See WebImage.h for example usage.
 */
class FWebImageCache
{
public:
	IMAGEDOWNLOAD_API FWebImageCache();

	/** Signifies the module is being unloaded and to perform any actions that depend on other modules which may be unloaded as well */
	IMAGEDOWNLOAD_API void PreUnload();

	/** Removes all cached images */
	IMAGEDOWNLOAD_API void Empty();

	/** Find or create a WebImage object for this URL (you probably just want to call ->Attr() on this) */
	IMAGEDOWNLOAD_API TSharedRef<const FWebImage> Download(const FString& Url, const TOptional<FString>& DefaultImageUrl = TOptional<FString>());

	/** Set the brush that will be returned until the download completes (only affects future downloads). */
	inline void SetDefaultStandInBrush(TAttribute<const FSlateBrush*> StandInBrushIn) { DefaultStandInBrush = StandInBrushIn; }

	/* This function causes the web image cache to stop holding on to strong references to images. Normally
	 * once downloaded, an image is cached forever. This allows us to release images that are not currently
	 * being displayed (those would have Strong pointers existing external to this class) to be released.
	 */
	IMAGEDOWNLOAD_API void RelinquishUnusedImages();

private:
	/** Map of canonical URL to web images (weak pointer so we don't affect lifetime) */
	TMap<FString, TWeakPtr<FWebImage> > UrlToImageMap;
	
	/** Strong references to keep images cached when not in use. Can be flushed manually */
	TArray< TSharedRef<FWebImage> > StrongRefCache;

	/** The image resource to show */
	TAttribute< const FSlateBrush* > DefaultStandInBrush;
};
