// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPlatformSslCertificateManager.h"

#if WITH_SSL

#include "Ssl.h"
#include "SslError.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/LocalTimestampDirectoryVisitor.h"

void FAndroidPlatformSslCertificateManager::BuildRootCertificateArray()
{
	FSslCertificateManager::BuildRootCertificateArray();

	bool bUsePlatformProvidedCertificates = true;
	if (GConfig->GetBool(TEXT("SSL"), TEXT("bUsePlatformProvidedCertificates"), bUsePlatformProvidedCertificates, GEngineIni) && !bUsePlatformProvidedCertificates)
	{
		return;
	}

	// gather all the files in system certificates directory
	TArray<FString> DirectoriesToIgnoreAndNotRecurse;
	FLocalTimestampDirectoryVisitor Visitor(FPlatformFileManager::Get().GetPlatformFile(), DirectoriesToIgnoreAndNotRecurse, DirectoriesToIgnoreAndNotRecurse, false);
	IFileManager::Get().IterateDirectory(TEXT("/system/etc/security/cacerts"), Visitor);

	for (const TPair<FString, FDateTime>& FileTimePair : Visitor.FileTimes)
	{
		const FString& CertFilename = FileTimePair.Key;
		AddPEMFileToRootCertificateArray(CertFilename);
	}
}

#endif