// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_SSL

#include "SslCertificateManager.h"

class FAndroidPlatformSslCertificateManager : public FSslCertificateManager
{
public:
	virtual void BuildRootCertificateArray() override;
};

using FPlatformSslCertificateManager = FAndroidPlatformSslCertificateManager;

#endif
