// Copyright Epic Games, Inc. All Rights Reserved.

// @todo platplug: Replace all of these includes with a call to COMPILED_PLATFORM_HEADER(SslCertificateManager.h)

#pragma once

#include "CoreMinimal.h"

#if WITH_SSL

#if PLATFORM_WINDOWS
#include "Windows/WindowsPlatformSslCertificateManager.h"
#elif PLATFORM_ANDROID
#include "Android/AndroidPlatformSslCertificateManager.h"
#elif PLATFORM_UNIX
#include "Unix/UnixPlatformSslCertificateManager.h"
#elif USE_DEFAULT_SSLCERT
#include "SslCertificateManager.h"
using FPlatformSslCertificateManager = FSslCertificateManager;
#else
#error Unknown platform
#endif

#endif // WITH_SSL
