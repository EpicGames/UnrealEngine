// Copyright Epic Games, Inc. All Rights Reserved.

#include "SslCertificateManager.h"
#include "Ssl.h"
#include "SslError.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformFile.h"
#include "HAL/FileManager.h"
#include "Templates/UniquePtr.h"
#include "Misc/Base64.h"
#include "Misc/CommandLine.h"
#include <Algo/Count.h>

FSslCertificateDelegates::FVerifySslCertificates FSslCertificateDelegates::VerifySslCertificates;

#if WITH_SSL

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <openssl/ssl.h>
#include <openssl/x509v3.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

namespace
{
	FString GetCertificateName(X509* const Certificate)
	{
		char StaticBuffer[2048];
		// We do not have to free the return value of get_subject_name
		X509_NAME_oneline(X509_get_subject_name(Certificate), StaticBuffer, sizeof(StaticBuffer));

		return FString(ANSI_TO_TCHAR(StaticBuffer));
	}

	FString GetCertificateIssuer(X509* const Certificate)
	{
		char StaticBuffer[2048];
		// We do not have to free the return value of get_subject_name
		X509_NAME_oneline(X509_get_issuer_name(Certificate), StaticBuffer, sizeof(StaticBuffer));

		return FString(ANSI_TO_TCHAR(StaticBuffer));
	}
}

void FSslCertificateManager::AddCertificatesToSslContext(SSL_CTX* SslContextPtr) const
{
	X509_STORE* CertStore = SSL_CTX_get_cert_store(SslContextPtr);
	for (int i = 0; i < RootCertificateArray.Num(); ++i)
	{
		if (X509_STORE_add_cert(CertStore, RootCertificateArray[i]) == 0)
		{
			UE_LOGF(LogSsl, Log, "Unable to add certificate: %ls", *GetSslErrorString());
		}
	}
}

bool FSslCertificateManager::HasCertificatesAvailable() const
{
	return RootCertificateArray.Num() > 0;
}

void FSslCertificateManager::ClearAllPinnedPublicKeys()
{
	PinnedPublicKeys.Empty();
}

bool FSslCertificateManager::HasPinnedPublicKeys() const
{
	return PinnedPublicKeys.Num() > 0;
}

// Compare function to order domains by exact matches, then from most specific to least specific subdomain matches
// For example: { "a.b.c.d", ".b.c.d", ".c.d", ".d" }
static bool DomainLessThan(const FString& DomainA, const FString& DomainB)
{
	const bool bDomainAIncludesSubdomains = DomainA[0] == TEXT('.');
	const bool bDomainBIncludesSubdomains = DomainB[0] == TEXT('.');

	if (bDomainAIncludesSubdomains == bDomainBIncludesSubdomains)
	{
		if (bDomainAIncludesSubdomains)
		{
			// both start with '.', sort by number of '.'s
			const SIZE_T DomainAPeriods = Algo::Count(DomainA, TEXT('.'));
			const SIZE_T DomainBPeriods = Algo::Count(DomainB, TEXT('.'));
			if (DomainAPeriods == DomainBPeriods)
			{
				// sort alphabetically
				return DomainA < DomainB;
			}
			else
			{
				// sort from most specific to least specific
				return DomainAPeriods > DomainBPeriods;
			}
		}
		else
		{
			// sort alphabetically
			return DomainA < DomainB;
		}
	}
	else
	{
		// exact matches come first
		return bDomainBIncludesSubdomains;
	}
}

bool FSslCertificateManager::IsDomainPinned(const FString& Domain)
{
	bool bWasDomainFound = false;

	FString DomainWithoutPort = Domain;
	int PortStart = Domain.Find(TEXT(":"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (PortStart >= 0)
	{
		int PortLength = DomainWithoutPort.Len() - PortStart;
		DomainWithoutPort.RemoveAt(PortStart, PortLength);
	}

	const TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>* PinnedKeys = nullptr;
	for (const TPair<FString, TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>>& PinnedKeyPair : PinnedPublicKeys)
	{
		const FString& PinnedDomain = PinnedKeyPair.Key;
		if ((PinnedDomain[0] == TEXT('.') && DomainWithoutPort.EndsWith(PinnedDomain))
			|| DomainWithoutPort == PinnedDomain)
		{
			bWasDomainFound = true;
			break;
		}
	}

	return bWasDomainFound;
}

void FSslCertificateManager::SetPinnedPublicKeys(const FString& Domain, const FString& PinnedKeyDigests)
{
	if (Domain.Len() == 0)
	{
		return;
	}

	if (PinnedKeyDigests.IsEmpty())
	{
		PinnedPublicKeys.RemoveAll([&Domain](const TPair<FString, TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>>& Pair) { return Pair.Key == Domain; });
	}
	else
	{
		int32 FoundIndex = INDEX_NONE;
		for (int Index = 0; Index < PinnedPublicKeys.Num(); ++Index)
		{
			const FString& ElementDomain = PinnedPublicKeys[Index].Key;
			if (ElementDomain == Domain)
			{
				FoundIndex = Index;
				break;
			}
			else if (DomainLessThan(Domain, ElementDomain))
			{
				FoundIndex = Index;
				PinnedPublicKeys.EmplaceAt(Index, Domain, TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>());
				break;
			}
		}
		if (FoundIndex == INDEX_NONE)
		{
			FoundIndex = PinnedPublicKeys.Emplace(Domain, TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>());
		}

		TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>& PinnedDigests = PinnedPublicKeys[FoundIndex].Value;
		PinnedDigests.Reset();
		TArray<FString> Digests;
		PinnedKeyDigests.ParseIntoArray(Digests, TEXT(";"));
		for (const FString& Digest : Digests)
		{
			if (FBase64::GetDecodedDataSize(Digest) == PUBLIC_KEY_DIGEST_SIZE)
			{
				TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>> DecodedDigest;
				DecodedDigest.AddUninitialized(PUBLIC_KEY_DIGEST_SIZE);
				if (FBase64::Decode(*Digest, Digest.Len(), DecodedDigest.GetData()))
				{
					PinnedDigests.Add(DecodedDigest);
				}
			}
		}
	}
}

bool FSslCertificateManager::VerifySslCertificates(X509_STORE_CTX* Context, const FString& Domain) const
{
#if !UE_BUILD_SHIPPING || WITH_SERVER_CODE
	static const bool bPinningDisabled = FParse::Param(FCommandLine::Get(), TEXT("DisableSSLCertificatePinning"));
	if (bPinningDisabled)
	{
		return true;
	}
#endif

	STACK_OF(X509)* Chain = X509_STORE_CTX_get_chain(Context);
	const int NumCertsInChain = sk_X509_num(Chain);
	if (NumCertsInChain <= 0)
	{
		return false;
	}

	TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>> CertDigests;
	TArray<FSslCertificateDelegates::FCertInfo> CertInfoList;
	const bool bCollectCertInfo = FSslCertificateDelegates::VerifySslCertificates.IsBound();

	for (int CertIndex = 0; CertIndex < NumCertsInChain; ++CertIndex)
	{
		X509* Certificate = sk_X509_value(Chain, CertIndex);
		int Length = i2d_X509_PUBKEY(X509_get_X509_PUBKEY(Certificate), nullptr);
		if (Length <= 0)
		{
			// no key
			continue;
		}

		TArray<uint8> PubKey;
		PubKey.AddUninitialized(Length);
		uint8* PubKeyPtr = PubKey.GetData();
		i2d_X509_PUBKEY(X509_get_X509_PUBKEY(Certificate), &PubKeyPtr);

		TArray<uint8, TFixedAllocator<SHA256_DIGEST_LENGTH>> Digest;
		Digest.AddUninitialized(SHA256_DIGEST_LENGTH);
		SHA256_CTX ShaContext;
		SHA256_Init(&ShaContext);
		SHA256_Update(&ShaContext, PubKey.GetData(), PubKey.Num());
		SHA256_Final(Digest.GetData(), &ShaContext);

		CertDigests.Add(Digest);

		if (bCollectCertInfo)
		{
			FSslCertificateDelegates::FCertInfo CertInfo;
			CertInfo.KeyDigest = Digest;
			CertInfo.Issuer = GetCertificateIssuer(Certificate);
			CertInfo.Subject = GetCertificateName(Certificate);

			const EVP_MD* CertDigest = EVP_get_digestbyname("sha1");
			if (CertDigest)
			{
				unsigned int DummySize = 0;
				CertInfo.Thumbprint.AddZeroed(FSslCertificateDelegates::FCertInfo::CERT_DIGEST_SIZE);
				X509_digest(Certificate, CertDigest, CertInfo.Thumbprint.GetData(), &DummySize);
			}

			CertInfoList.Add(CertInfo);
		}
	}

	bool bFoundMatch = false;

	bool bFoundMatchDelegate = FSslCertificateDelegates::VerifySslCertificates.IsBound() ? FSslCertificateDelegates::VerifySslCertificates.Execute(Domain, CertInfoList) : true;
	if (bFoundMatchDelegate)
	{
		bFoundMatch = VerifySslCertificates(CertDigests, Domain);
	}

    if (!bFoundMatch)
    {
        X509_STORE_CTX_set_error(Context, X509_V_ERR_CERT_UNTRUSTED);
    }
    return bFoundMatch;
}

bool FSslCertificateManager::VerifySslCertificates(TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>& Digests, const FString& Domain) const
{
#if !UE_BUILD_SHIPPING || WITH_SERVER_CODE
	static const bool bPinningDisabled = FParse::Param(FCommandLine::Get(), TEXT("DisableSSLCertificatePinning"));
	if (bPinningDisabled)
	{
		return true;
	}
#endif
	const TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>* PinnedKeys = nullptr;
	for (const TPair<FString, TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>>& PinnedKeyPair : PinnedPublicKeys)
	{
		const FString& PinnedDomain = PinnedKeyPair.Key;
		if ((PinnedDomain[0] == TEXT('.') && Domain.EndsWith(PinnedDomain))
			|| Domain == PinnedDomain)
		{
			PinnedKeys = &PinnedKeyPair.Value;
			break;
		}
	}
	if (!PinnedKeys)
	{
		// No keys pinned for this domain
		UE_LOGF(LogSsl, Verbose, "no pinned key digests found for domain '%ls'", *Domain);
		return true;
	}
	bool bFoundMatch = false;
	for (const TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>& CurrentDigest: Digests)
	{
		UE_LOGF(LogSsl, VeryVerbose, "checking digest. Base64: '%ls'", *FBase64::Encode(CurrentDigest.GetData(), CurrentDigest.Num()));
		
		if (PinnedKeys->Contains(CurrentDigest))
		{
			UE_LOGF(LogSsl, Verbose, "found public key digest in request that matches a pinned key for '%ls'", *Domain);
			bFoundMatch = true;
			break;
		}
	}
	return bFoundMatch;
}

void FSslCertificateManager::BuildRootCertificateArray()
{
	FString CertificateBundlePath;
#if !UE_BUILD_SHIPPING
	FString OverrideCertificateBundlePath;
	if (GConfig->GetString(TEXT("SSL"), TEXT("OverrideCertificateBundlePath"), OverrideCertificateBundlePath, GEngineIni) && OverrideCertificateBundlePath.Len() > 0)
	{
		if (FPaths::FileExists(OverrideCertificateBundlePath))
		{
			CertificateBundlePath = OverrideCertificateBundlePath;
		}
	}
#endif

	if (CertificateBundlePath.IsEmpty())
	{
		const FString PerPlatformBundlePath = FString::Printf(TEXT("Certificates/%s/cacert.pem"), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()));

		const FString SearchPaths[]
		{
			FPaths::ProjectContentDir() + PerPlatformBundlePath,
			FPaths::ProjectContentDir() + TEXT("Certificates/cacert.pem"),
			FPaths::EngineContentDir() + TEXT("Certificates/ThirdParty/cacert.pem")
		};

		for (const FString& SearchPath : SearchPaths)
		{
			if (FPaths::FileExists(SearchPath))
			{
				CertificateBundlePath = SearchPath;
				break;
			}
		}
	}

	if (!CertificateBundlePath.IsEmpty())
	{
		AddPEMFileToRootCertificateArray(CertificateBundlePath);
	}

	FString DebuggingCertificatePath;
	if (GConfig->GetString(TEXT("SSL"), TEXT("DebuggingCertificatePath"), DebuggingCertificatePath, GEngineIni) && DebuggingCertificatePath.Len() > 0)
	{
		if (FPaths::FileExists(DebuggingCertificatePath))
		{
			TUniquePtr<FArchive> DebuggingCertificateArchive(IFileManager::Get().CreateFileReader(*DebuggingCertificatePath, 0));
			int64 CertificateBufferSize = DebuggingCertificateArchive->TotalSize();
			char* CertificateBuffer = new char[CertificateBufferSize + 1];
			DebuggingCertificateArchive->Serialize(CertificateBuffer, CertificateBufferSize);
			CertificateBuffer[CertificateBufferSize] = '\0';
			BIO* CertificateBio = BIO_new_mem_buf(CertificateBuffer, -1);
			X509* Certificate = PEM_read_bio_X509(CertificateBio, NULL, 0, NULL);
			if (Certificate)
			{
				AddCertificateToRootCertificateArray(Certificate);
			}
			else
			{
				UE_LOGF(LogSsl, Warning, "Error loading debugging certificate: %ls", *GetSslErrorString());
			}
			BIO_free(CertificateBio);
			delete[] CertificateBuffer;
			CertificateBuffer = nullptr;
		}
	}
}

void FSslCertificateManager::EmptyRootCertificateArray()
{
	for (int i = 0; i < RootCertificateArray.Num(); ++i)
	{
		X509_free(RootCertificateArray[i]);
	}
	RootCertificateArray.Reset();
}

void FSslCertificateManager::AddPEMFileToRootCertificateArray(const FString& Path)
{
	int64 CertificateBundleBufferSize = 0;
	TUniquePtr<char[]> CertificateBundleBuffer;

	if (TUniquePtr<FArchive> CertificateBundleArchive = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*Path, 0)))
	{
		CertificateBundleBufferSize = CertificateBundleArchive->TotalSize();
		CertificateBundleBuffer.Reset(new char[CertificateBundleBufferSize + 1]);
		CertificateBundleArchive->Serialize(CertificateBundleBuffer.Get(), CertificateBundleBufferSize);
		CertificateBundleBuffer[CertificateBundleBufferSize] = '\0';
	}

	if (CertificateBundleBufferSize > 0 && CertificateBundleBuffer != nullptr)
	{
		static const char BeginCertificateString[] = "-----BEGIN CERTIFICATE-----";
		static const char EndCertificateString[] = "-----END CERTIFICATE-----";

		const char* FoundString = CertificateBundleBuffer.Get();
		while (nullptr != (FoundString = FPlatformString::Strstr(FoundString, BeginCertificateString)))
		{
			const char* EndString = FPlatformString::Strstr(FoundString, EndCertificateString);
			if (EndString != nullptr)
			{
				size_t LengthOfCertificateData = EndString - FoundString + sizeof(EndCertificateString);
				BIO* CertificateBio = BIO_new_mem_buf(const_cast<char*>(FoundString), LengthOfCertificateData);
				X509* Certificate = PEM_read_bio_X509(CertificateBio, NULL, 0, NULL);
				if (Certificate)
				{
					AddCertificateToRootCertificateArray(Certificate);
				}
				else
				{
					UE_LOGF(LogSsl, Log, "Error loading certificate from bundle: %ls", *GetSslErrorString());
				}
				BIO_free(CertificateBio);
			}
			FoundString = EndString;
		}
	}
}

void FSslCertificateManager::AddCertificateToRootCertificateArray(X509* Certificate)
{
	bool bValidateRootCertificates = true;
	GConfig->GetBool(TEXT("SSL"), TEXT("bValidateRootCertificates"), bValidateRootCertificates, GEngineIni);
	if (bValidateRootCertificates)
	{
		ASN1_TIME* NotBefore = X509_get_notBefore(Certificate);
		ASN1_TIME* NotAfter = X509_get_notAfter(Certificate);
		if (X509_cmp_current_time(NotAfter) < 0)
		{
			UE_LOGF(LogSsl, Log, "Ignoring expired certificate: %ls", *GetCertificateName(Certificate));
			X509_free(Certificate);
			return;
		}
		if (X509_cmp_current_time(NotBefore) > 0)
		{
			UE_LOGF(LogSsl, Log, "Ignoring not yet valid certificate: %ls", *GetCertificateName(Certificate));
			X509_free(Certificate);
			return;
		}
	}

	const bool bFound = RootCertificateArray.ContainsByPredicate(
		[Certificate](X509* Other)
		{
			return X509_cmp(Other, Certificate) == 0;
		});

	if (bFound)
	{
		UE_LOGF(LogSsl, VeryVerbose, "Ignoring duplicate certificate: %ls", *GetCertificateName(Certificate));
		X509_free(Certificate);
	}
	else
	{
		UE_LOGF(LogSsl, Verbose, "Adding certificate: %ls", *GetCertificateName(Certificate));
		RootCertificateArray.Add(Certificate);
	}
}

#endif // #if WITH_SSL
