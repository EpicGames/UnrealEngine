// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_SSL

#include "Interfaces/ISslCertificateManager.h"

struct x509_st;
typedef struct x509_st X509;

class FSslCertificateManager : public ISslCertificateManager
{
public:
	virtual void AddCertificatesToSslContext(SSL_CTX* SslContextPtr) const override;
	virtual bool HasCertificatesAvailable() const override;

	virtual void ClearAllPinnedPublicKeys() override;
	virtual bool HasPinnedPublicKeys() const override;
	virtual bool IsDomainPinned(const FString& Domain) override;
	virtual void SetPinnedPublicKeys(const FString& Domain, const FString& PinnedKeyDigests) override;
	virtual bool VerifySslCertificates(X509_STORE_CTX* Context, const FString& Domain) const override;
	virtual bool VerifySslCertificates(TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>& Digests, const FString& Domain) const override;

	virtual void BuildRootCertificateArray();
	virtual void EmptyRootCertificateArray();

protected:
	void AddPEMFileToRootCertificateArray(const FString& Path);
	void AddCertificateToRootCertificateArray(X509* Certificate);

	TArray<X509*> RootCertificateArray;
	TArray<TPair<FString, TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>>> PinnedPublicKeys;
};

#endif
