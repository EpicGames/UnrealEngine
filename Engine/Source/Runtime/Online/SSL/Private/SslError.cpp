// Copyright Epic Games, Inc. All Rights Reserved.

#include "SslError.h"

#if WITH_SSL

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <openssl/ssl.h>
#include <openssl/err.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

FString GetSslErrorString()
{
	FString SslErrorString;
	unsigned long Error = ERR_get_error();
	if (Error != SSL_ERROR_NONE)
	{
		char AnsiErrorBuffer[256];
		ERR_error_string_n(Error, AnsiErrorBuffer, UE_ARRAY_COUNT(AnsiErrorBuffer) - 1);
		AnsiErrorBuffer[UE_ARRAY_COUNT(AnsiErrorBuffer) - 1] = '\0';

		SslErrorString = ANSI_TO_TCHAR(AnsiErrorBuffer);
	}
	return SslErrorString;
}

#endif // WITH_SSL