// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_SSL

#include "CoreMinimal.h"

// Helper to get ssl/crypto error string
FString GetSslErrorString();

#endif
