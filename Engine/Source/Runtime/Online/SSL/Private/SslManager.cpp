// Copyright Epic Games, Inc. All Rights Reserved.

#include "SslManager.h"
#include "Ssl.h"
#include "SslError.h"

#if WITH_SSL

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <openssl/ssl.h>
#include <openssl/conf.h>
#include <openssl/err.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

FSslManager::FSslManager()
	: InitCount(0)
{
}

bool FSslManager::InitializeSsl()
{
	bool bSuccessful = false;
	// Only actually do SSL initialization in monolithic builds
	// while we are statically linking OpenSSL in various libraries (such as libcurl), when the
	// SSL module is non-monolithic, OpenSSL would only get initialized in the SSL module's 
	// scope (not the caller)
#if IS_MONOLITHIC || UE_MERGED_MODULES
	++ InitCount;
	if (InitCount == 1)
	{
		UE_LOGF(LogSsl, Log, "Initializing SSL");
		
		OPENSSL_load_builtin_modules();

		// Per libcurl: OPENSSL_config(NULL) may call exit(), so just do the heart of the work and call CONF_modules_load_file
		CONF_modules_load_file(NULL, NULL, CONF_MFLAGS_DEFAULT_SECTION | CONF_MFLAGS_IGNORE_MISSING_FILE);
		SSL_load_error_strings();

		SSLeay_add_ssl_algorithms();
		OpenSSL_add_all_algorithms();
		bSuccessful = true;
	}
	else
	{
		bSuccessful = true;
	}
#endif // IS_MONOLITHIC || UE_MERGED_MODULES
	return bSuccessful;
}

void FSslManager::ShutdownSsl()
{
#if IS_MONOLITHIC || UE_MERGED_MODULES
	-- InitCount;
	check(InitCount >= 0);
	if (InitCount == 0)
	{
		UE_LOGF(LogSsl, Log, "Shutting down SSL");
		EVP_cleanup();
		CRYPTO_cleanup_all_ex_data();
		ERR_free_strings();
#if OPENSSL_VERSION_NUMBER < 0x10100000L
		ERR_remove_thread_state(NULL);
#endif // OPENSSL_VERSION_NUMBER < 0x10100000L
		CONF_modules_free();
	}
#endif // IS_MONOLITHIC || UE_MERGED_MODULES
}

SSL_CTX* FSslManager::CreateSslContext(const FSslContextCreateOptions& CreateOptions)
{
	SSL_CTX* SslContext = nullptr;
#if IS_MONOLITHIC || UE_MERGED_MODULES
	check(InitCount > 0);
	if (InitCount > 0)
	{
		const SSL_METHOD* SslMethod = SSLv23_client_method();
		if (SslMethod)
		{
			SslContext = SSL_CTX_new(SslMethod);
			if (SslContext)
			{
				// Restrict protocols we do not want
				int32 RestrictedProtocols = 0;
#define RESTRICT_SSL_TLS_PROTOCOL(EnumVal, OpenSSLBit) \
				if (CreateOptions.MinimumProtocol > ESslTlsProtocol::EnumVal || CreateOptions.MaximumProtocol < ESslTlsProtocol::EnumVal) \
				{\
					RestrictedProtocols |= OpenSSLBit; \
				}

				RESTRICT_SSL_TLS_PROTOCOL(SSLv2, SSL_OP_NO_SSLv2);
				RESTRICT_SSL_TLS_PROTOCOL(SSLv3, SSL_OP_NO_SSLv3);
				RESTRICT_SSL_TLS_PROTOCOL(TLSv1, SSL_OP_NO_TLSv1);
				RESTRICT_SSL_TLS_PROTOCOL(TLSv1_1, SSL_OP_NO_TLSv1_1);
				RESTRICT_SSL_TLS_PROTOCOL(TLSv1_2, SSL_OP_NO_TLSv1_2);
				static_assert(ESslTlsProtocol::Maximum == ESslTlsProtocol::TLSv1_2, "Implement new protocols above");

#undef RESTRICT_SSL_TLS_PROTOCOL

				int32 SslContextFlags = 0;
				if (!CreateOptions.bAllowCompression)
				{
					SslContextFlags |= SSL_OP_NO_COMPRESSION;
				}

				SSL_CTX_set_options(SslContext, SslContextFlags | RestrictedProtocols);
				
				if (CreateOptions.bAddCertificates)
				{
					FSslModule::Get().GetCertificateManager().AddCertificatesToSslContext(SslContext);
				}
			}
			else
			{
				UE_LOGF(LogSsl, Warning, "FSslManager::CreateSslContext: Failed to create the SSL context: %ls", *GetSslErrorString());
			}
		}
		else
		{
			UE_LOGF(LogSsl, Warning, "FSslManager::CreateSslContext: Failed to create method SSLv23_client_method: %ls", *GetSslErrorString());
		}
	}
#endif // IS_MONOLITHIC || UE_MERGED_MODULES
	return SslContext;
}

void FSslManager::DestroySslContext(SSL_CTX* SslContext)
{
#if IS_MONOLITHIC || UE_MERGED_MODULES
	check(InitCount > 0);
	if (InitCount > 0)
	{
		SSL_CTX_free(SslContext);
	}
#endif // IS_MONOLITHIC || UE_MERGED_MODULES
}

#endif
