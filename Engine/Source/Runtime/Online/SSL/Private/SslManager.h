// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_SSL

#include "CoreMinimal.h"
#include "CoreTypes.h"

#include "Interfaces/ISslManager.h"

/**
 * Manager of the ssl library
 */
class FSslManager : ISslManager
{
public:

	//~ Begin ISslManager Interface
	SSL_API virtual bool InitializeSsl() override;
	SSL_API virtual void ShutdownSsl() override;
	SSL_API virtual SSL_CTX* CreateSslContext(const FSslContextCreateOptions& CreateOptions) override;
	SSL_API virtual void DestroySslContext(SSL_CTX* SslContext) override;
	//~ End ISslManager Interface

protected:
	/** Default constructor */
	SSL_API FSslManager();
	/** Disable copies */
	UE_NONCOPYABLE(FSslManager)
	/** SSL ref count */
	int32 InitCount;

	friend class FSslModule;
};

#endif // WITH_SSL
