// Copyright Epic Games, Inc. All Rights Reserved.

#include "SslModule.h"
#include "PlatformSslCertificateManager.h"
#include "SslManager.h"
#include "Ssl.h"
#include "Misc/Parse.h"
#include "Misc/ConfigCacheIni.h"

DEFINE_LOG_CATEGORY(LogSsl);

// FHttpModule

IMPLEMENT_MODULE(FSslModule, SSL);

FSslModule* FSslModule::Singleton = NULL;

FSslModule::FSslModule()
	: CertificateManagerPtr(nullptr)
	, SslManagerPtr(nullptr)
{
}

void FSslModule::StartupModule()
{	
	Singleton = this;

#if WITH_SSL
	SslManagerPtr = new FSslManager();

	CertificateManagerPtr = new FPlatformSslCertificateManager();
	{
		SCOPED_BOOT_TIMING("BuildRootCertificateArray");
		static_cast<FPlatformSslCertificateManager*>(CertificateManagerPtr)->BuildRootCertificateArray();
	}

	// Load pinned public keys from Engine.ini. For example to pin epicgames.com and its subdomains to require Amazon Root CA 1 or Starfield Services Root Certificate Authority - G2 in the cert chain,
	// [SSL]
	// +PinnedPublicKeys="epicgames.com:++MBgDH5WGvL9Bcn5Be30cRcL0f5O+NyoXuWtQdX1aI=;KwccWaCgrnaw6tsrrSO61FgLacNgG2MMLq8GE6+oP5I="
	// +PinnedPublicKeys=".epicgames.com:++MBgDH5WGvL9Bcn5Be30cRcL0f5O+NyoXuWtQdX1aI=;KwccWaCgrnaw6tsrrSO61FgLacNgG2MMLq8GE6+oP5I="
	TArray<FString> PinnedPublicKeys;
	if (GConfig->GetArray(TEXT("SSL"), TEXT("PinnedPublicKeys"), PinnedPublicKeys, GEngineIni))
	{
		for (const FString& PinnedPublicKey : PinnedPublicKeys)
		{
			TArray<FString> DomainAndKeys;
			PinnedPublicKey.ParseIntoArray(DomainAndKeys, TEXT(":"));
			if (DomainAndKeys.Num() == 2)
			{
				CertificateManagerPtr->SetPinnedPublicKeys(DomainAndKeys[0], DomainAndKeys[1]);
			}
		}
	}
#endif //#if WITH_SSL
}

void FSslModule::ShutdownModule()
{
#if WITH_SSL
	static_cast<FPlatformSslCertificateManager*>(CertificateManagerPtr)->EmptyRootCertificateArray();
	delete CertificateManagerPtr;

	delete SslManagerPtr;
#endif // #if WITH_SSL

	Singleton = nullptr;
}

FSslModule& FSslModule::Get()
{
	if (Singleton == NULL)
	{
		check(IsInGameThread());
		FModuleManager::LoadModuleChecked<FSslModule>("SSL");
	}
	check(Singleton != NULL);
	return *Singleton;
}
