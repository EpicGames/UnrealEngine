// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnixPlatformSslCertificateManager.h"

#include "Ssl.h"
#include "SslError.h"

#include "Misc/ConfigCacheIni.h"

#if WITH_SSL

#include <openssl/x509.h>

void FUnixPlatformSslCertificateManager::BuildRootCertificateArray()
{
	FSslCertificateManager::BuildRootCertificateArray();

	bool bUsePlatformProvidedCertificates = true;
	if (GConfig->GetBool(TEXT("SSL"), TEXT("bUsePlatformProvidedCertificates"), bUsePlatformProvidedCertificates, GEngineIni) && !bUsePlatformProvidedCertificates)
	{
		return;
	}

	static const TCHAR* KnownBundlePaths[] =
	{
		TEXT("/etc/pki/tls/certs/ca-bundle.crt"),
		TEXT("/etc/ssl/certs/ca-certificates.crt"),
		TEXT("/etc/ssl/ca-bundle.pem"),
	};

	for (const TCHAR* CurrentBundle : KnownBundlePaths)
	{
		FString FileName(CurrentBundle);
		UE_LOGF(LogSsl, Log, "Checking if '%ls' exists", *FileName);

		if (FPaths::FileExists(FileName))
		{
			UE_LOGF(LogSsl, Log, "Loading certificates from %ls", *FileName);
			AddPEMFileToRootCertificateArray(FileName);
			return;
		}
	}

	if (RootCertificateArray.Num() > 0)
	{
		UE_LOGF(LogSsl, Warning, "Unable to find a cert bundle in any of known locations. Platform provided certificates will not be used");
	}
	else
	{
		UE_LOGF(LogSsl, Warning, "Unable to find a cert bundle in any of known locations. TLS may not work.");
	}
}

#endif