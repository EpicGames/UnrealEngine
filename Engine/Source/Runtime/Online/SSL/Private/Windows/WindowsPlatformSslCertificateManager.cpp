// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsPlatformSslCertificateManager.h"

#include "Ssl.h"
#include "SslError.h"

#include "Misc/ConfigCacheIni.h"

#if WITH_SSL

#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
#include <wincrypt.h>
#include <openssl/x509.h>
#include <openssl/err.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

void FWindowsPlatformSslCertificateManager::BuildRootCertificateArray()
{
	FSslCertificateManager::BuildRootCertificateArray();

	bool bUsePlatformProvidedCertificates = true;
	if (GConfig->GetBool(TEXT("SSL"), TEXT("bUsePlatformProvidedCertificates"), bUsePlatformProvidedCertificates, GEngineIni) && !bUsePlatformProvidedCertificates)
	{
		return;
	}

	// Add certificates from the windows root certificate store
	HCERTSTORE SystemRootStore = CertOpenSystemStoreW(0, L"ROOT");
	if (SystemRootStore)
	{
		PCCERT_CONTEXT CertContext = nullptr;
		while (nullptr != (CertContext = CertEnumCertificatesInStore(SystemRootStore, CertContext)))
		{
			if ((CertContext->dwCertEncodingType & X509_ASN_ENCODING) != 0)
			{
				if (X509* Certificate = d2i_X509(nullptr, (const unsigned char**)&CertContext->pbCertEncoded, CertContext->cbCertEncoded))
				{
					AddCertificateToRootCertificateArray(Certificate);
				}
				else
				{
					TCHAR Name[128];
					CertGetNameString(CertContext, CERT_NAME_SIMPLE_DISPLAY_TYPE, 0, nullptr, Name, UE_ARRAY_COUNT(Name));
					UE_LOGF(LogSsl, Log, "Unable to convert certificate: name:%ls error:%ls", Name, *GetSslErrorString());
				}
			}
			else
			{
				TCHAR Name[128];
				CertGetNameString(CertContext, CERT_NAME_SIMPLE_DISPLAY_TYPE, 0, nullptr, Name, UE_ARRAY_COUNT(Name));
				UE_LOGF(LogSsl, Log, "Unhandled certificate encoding: name:%ls encodingType:0x%08x", Name, CertContext->dwCertEncodingType);
			}
		}
		CertCloseStore(SystemRootStore, 0);
	}
	else if (RootCertificateArray.Num() > 0)
	{
		UE_LOGF(LogSsl, Warning, "Unable to open ROOT certificate store. Platform provided certificates will not be used");
	}
	else
	{
		UE_LOGF(LogSsl, Warning, "Unable to open ROOT certificate store. TLS may not work.");
	}
}

#endif