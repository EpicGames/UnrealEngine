// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "CoreFwd.h"
#include "Delegates/Delegate.h"

struct ssl_ctx_st;
typedef struct ssl_ctx_st SSL_CTX;

struct x509_store_ctx_st;
typedef struct x509_store_ctx_st X509_STORE_CTX;

class ISslCertificateManager
{
public:
	virtual ~ISslCertificateManager() = default;
    
	static constexpr int PUBLIC_KEY_DIGEST_SIZE = 32;
    
	/**
	 * Add trusted root certificates to the SSL context
	 *
	 * @param SslContextPtr Ssl context
	 */
	virtual void AddCertificatesToSslContext(SSL_CTX* SslContextPtr) const = 0;

	/**
	 * @return true if certificates are available
	 */
	virtual bool HasCertificatesAvailable() const = 0;

	/**
	 * Clear all pinned keys
	 */
	virtual void ClearAllPinnedPublicKeys() = 0;

	/**
	 * Check if keys have been pinned yet
	 */
	virtual bool HasPinnedPublicKeys() const = 0;

	/**
	 * Check if the domain is currently pinned
	 */
	virtual bool IsDomainPinned(const FString& Domain) = 0;

	/**
	* Set digests for pinned certificate public key for a domain
	*
	* @param Domain Domain the pinned keys are valid for. If Domain starts with a '.' it will match any subdomain
	* @param PinnedKeyDigests Semicolon separated base64 encoded SHA256 digests of pinned public keys
	*/
	virtual void SetPinnedPublicKeys(const FString& Domain, const FString& PinnedKeyDigests) = 0;

	/**
	 * Performs additional ssl validation (certificate pinning)
	 *
	 * @param Context Pointer to the x509 context containing a certificate chain
	 * @param Domain Domain we are connected to
	 *
	 * @return false if validation fails
	 */
	virtual bool VerifySslCertificates(X509_STORE_CTX* Context, const FString& Domain) const = 0;
    
	/**
	 * Performs additional ssl validation (certificate pinning)
	 *
	 * @param Digests Array of public key digests to check against pinned key digests
	 * @param Domain Domain we are connected to
	 *
	 * @return false if validation fails
	 */
	virtual bool VerifySslCertificates(TArray<TArray<uint8, TFixedAllocator<PUBLIC_KEY_DIGEST_SIZE>>>& Digests, const FString& Domain) const = 0;
};

class FSslCertificateDelegates
{
public:
	struct FCertInfo
	{
		static constexpr int CERT_DIGEST_SIZE = 20;

		TArray<uint8, TInlineAllocator<ISslCertificateManager::PUBLIC_KEY_DIGEST_SIZE>> KeyDigest;
		TArray<uint8, TInlineAllocator<CERT_DIGEST_SIZE>> Thumbprint;
		FString Issuer;
		FString Subject;
	};

	using FVerifySslCertificates = TTSDelegate<bool(const FString&, const TArray<FCertInfo>&)>;
	static SSL_API FVerifySslCertificates VerifySslCertificates;
};
