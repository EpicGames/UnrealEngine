// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"

struct ssl_ctx_st;
typedef struct ssl_ctx_st SSL_CTX;

/**
 * SSL/TLS protocol
 */
enum class ESslTlsProtocol : uint8
{
	/** Start (used for specifying a protocol range) */
	Minimum = 0,
	/** SSLv2 */
	SSLv2 = Minimum,
	/** SSLv3 */
	SSLv3,
	/** TLSv1 */
	TLSv1,
	/** TLSv1.1 */
	TLSv1_1,
	/** TLSv1.2 */
	TLSv1_2,
	
	// INSERT NEW VALUES ABOVE THIS LINE (and keep Maximum up to date)

	/** End (used for specifying a protocol range) */
	Maximum = TLSv1_2
};

/**
 * Options for creating an SSL context using FSslManager::CreateSslContext
 */
struct FSslContextCreateOptions
{
	/** Minimum version of SSL/TLS to allow */
	ESslTlsProtocol MinimumProtocol = ESslTlsProtocol::Minimum;
	/** Maximum version of SSL/TLS to allow */
	ESslTlsProtocol MaximumProtocol = ESslTlsProtocol::Maximum;
	/** Do we want to allow compression? */
	bool bAllowCompression = true;
	/** Automatically add certificates from the certificate manager? */
	bool bAddCertificates = true;
};

/**
 * Manager of the ssl library
 */
class ISslManager
{
public:
	/** Destructor */
	virtual ~ISslManager() = default;
	/**
	 * Initialize the ssl library.  Can be called multiple times (may not do anything beyond first call).
	 * ShutdownSsl must be called once for each call to InitializeSsl
	 *
	 * @return true if ssl was successfully initialized, false if not
	 */
	virtual bool InitializeSsl() = 0;

	/**
	 * Shutdown the ssl library.  Must be called once per call to InitializeSsl
	 */
	virtual void ShutdownSsl() = 0;

	/**
	 * Create an SSL context
	 * @return an SSL context, may be null if an error occurred
	 */
	virtual SSL_CTX* CreateSslContext(const FSslContextCreateOptions& CreateOptions) = 0;

	/**
	 * Destroy an SSL context
	 * @param SslContext SSL context to be destroyed
	 */
	virtual void DestroySslContext(SSL_CTX* SslContext) = 0;
};
