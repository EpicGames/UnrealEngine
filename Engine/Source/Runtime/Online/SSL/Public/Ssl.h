// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Module dependencies
#include "Interfaces/ISslCertificateManager.h"
#include "Interfaces/ISslManager.h"

// Module includes
#include "SslModule.h"

SSL_API DECLARE_LOG_CATEGORY_EXTERN(LogSsl, Warning, All);
