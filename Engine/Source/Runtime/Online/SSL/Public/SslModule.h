// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FOutputDevice;
class ISslCertificateManager;
class ISslManager;
class UWorld;

/**
 * Module for SSL/TLS certificate management
 */
class FSslModule : 
	public IModuleInterface
{

public:

	// FSslModule

	/** Constructor */
	FSslModule();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	SSL_API static FSslModule& Get();

	/**
	 * Accessor for the SSL certificate manager
	 *
	 * @return SSL certificate manager manager
	 */
	inline ISslCertificateManager& GetCertificateManager()
	{
		check(CertificateManagerPtr != nullptr);
		return *CertificateManagerPtr;
	}

	inline ISslManager& GetSslManager()
	{
		check(SslManagerPtr != nullptr);
		return *SslManagerPtr;
	}

private:

	// IModuleInterface

	/**
	 * Called when Http module is loaded
	 * Initialize platform specific parts of Http handling
	 */
	virtual void StartupModule() override;
	
	/**
	 * Called when Http module is unloaded
	 * Shutdown platform specific parts of Http handling
	 */
	virtual void ShutdownModule() override;

	/** Keeps track of SSL certificates */
	ISslCertificateManager* CertificateManagerPtr;

	/** Manager of the SSL library */
	ISslManager* SslManagerPtr;

	/** singleton for the module while loaded and available */
	static FSslModule* Singleton;
};
