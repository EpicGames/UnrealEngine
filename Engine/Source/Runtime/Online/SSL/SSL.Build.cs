// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SSL : ModuleRules
{
	protected virtual bool bPlatformSupportsSSL
	{
		get
		{
			return
				Target.Platform == UnrealTargetPlatform.Mac ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) ||
				Target.Platform == UnrealTargetPlatform.IOS ||
				Target.Platform == UnrealTargetPlatform.Android;
		}
	}

	protected virtual bool bUseDefaultSSLCert
	{
		get
		{
			return
				Target.Platform == UnrealTargetPlatform.Mac ||
				Target.Platform == UnrealTargetPlatform.IOS;
		}
	}

	public SSL(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDefinitions.Add("SSL_PACKAGE=1");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		if (bPlatformSupportsSSL)
		{
			PublicDefinitions.Add("WITH_SSL=1");
			PrivateDefinitions.Add("USE_DEFAULT_SSLCERT=" + (bUseDefaultSSLCert ? "1" : "0"));

			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");

			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
			{
				PublicSystemLibraries.Add("crypt32.lib");
			}
		}
		else
		{
			PublicDefinitions.Add("WITH_SSL=0");
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
