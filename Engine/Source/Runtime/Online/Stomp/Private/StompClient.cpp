// Copyright Epic Games, Inc. All Rights Reserved.

#include "StompClient.h"
#include "StompLog.h"

#if WITH_STOMP
#include "Stats/Stats.h"

#include "StompMessage.h"
#include "StompFrame.h"

#include "WebSocketsModule.h"

namespace
{
const FTimespan RequestTimeout = FTimespan::FromMinutes(5);
const int MissedServerPongsBeforeError = 5;
}

FStompClient::FStompClient(const FString& Url, const FTimespan& InPingInterval, const FTimespan& InPongInterval, const FString& OptAuthToken)
	: IdCounter(0)
	, PingInterval(InPingInterval)
	, PongInterval(InPongInterval)
	, bReportedNoHeartbeatError(false)
	, bIsConnected(false)
{
	TArray<FString> Protocols;
	Protocols.Add(TEXT("v10.stomp"));
	Protocols.Add(TEXT("v11.stomp"));
	Protocols.Add(TEXT("v12.stomp"));

	TMap<FString, FString> UpgradeHeaders;
	if (!OptAuthToken.IsEmpty())
	{
		AuthStrategy = EAuthStrategy::OnConnectionUpgrade;
		UpgradeHeaders.Add(TEXT("Authorization"), OptAuthToken);
	}

	WebSocket = FWebSocketsModule::Get().CreateWebSocket(Url, Protocols, UpgradeHeaders);
}

FStompClient::~FStompClient()
{
	WebSocket->OnConnected().RemoveAll(this);
	WebSocket->OnConnectionError().RemoveAll(this);
	WebSocket->OnClosed().RemoveAll(this);
	WebSocket->OnRawMessage().RemoveAll(this);
}

void FStompClient::Connect(const FStompHeader& Header)
{
	static const FName LoginHeader("login");
	static const FName PasscodeHeader("passcode");
	static const FName HeartbeatHeader("heart-beat");
	ConnectHeader = Header;
	if (!ConnectHeader.Contains(HeartbeatHeader))
	{
		FString HeartbeatValue = FString::FromInt((int32)PingInterval.GetTotalMilliseconds()) + TEXT(",") + FString::FromInt((int32)PongInterval.GetTotalMilliseconds());
		ConnectHeader.Emplace(HeartbeatHeader, HeartbeatValue);
	}

	if (ConnectHeader.Contains(LoginHeader))
	{
		checkf(ConnectHeader.Contains(PasscodeHeader) && AuthStrategy != EAuthStrategy::OnConnectionUpgrade, TEXT("Stomp: invalid to pass auth to both constructor and Connect"));
		AuthStrategy = EAuthStrategy::OnStompConnect;
	}

	ConnectHeader.Emplace(TEXT("accept-version"), TEXT("1.0,1.1,1.2"));
	WebSocket->OnConnected().AddSP(this, &FStompClient::HandleWebSocketConnected);
	WebSocket->OnConnectionError().AddSP(this, &FStompClient::HandleWebSocketConnectionError);
	WebSocket->OnClosed().AddSP(this, &FStompClient::HandleWebSocketConnectionClosed);
	WebSocket->OnRawMessage().AddSP(this, &FStompClient::HandleWebSocketData);
	WebSocket->Connect();
}

void FStompClient::Disconnect(const FStompHeader& Header)
{
	FStompFrame DisconnectFrame (DisconnectCommand, Header);
	WriteFrame(DisconnectFrame, FStompRequestCompleted::CreateSP(this, &FStompClient::HandleDisconnectCompleted));
	// TODO: We should prevent new commands fom being scheduled after sending the DISCONNECT command.
}

void FStompClient::HandleDisconnectCompleted(bool bIsSuccess, const FString& Error)
{
	if (bIsSuccess)
	{
		UE_LOGF(LogStomp, Verbose, "Successfully disconnected from server");
	}
	else
	{
		UE_LOGF(LogStomp, Warning, "Error when disconnecting from Stomp server: %ls", *Error);
		OnError().Broadcast(Error);
	}
	WebSocket->Close();
}

bool FStompClient::IsConnected() const
{
	return bIsConnected;
}

FString FStompClient::MakeId(const FStompFrame& Frame)
{
	return FString::Printf(TEXT("%s-%llu%s%s"),*(Frame.GetCommand().ToString().Left(3).ToLower()), IdCounter++, SessionId.IsEmpty()?TEXT(""):TEXT("-"), *SessionId);
}

FStompSubscriptionId FStompClient::Subscribe(const FString& Destination, const FStompSubscriptionEvent& EventCallback, const FStompRequestCompleted& CompletionCallback )
{
	FStompFrame SubscribeFrame (SubscribeCommand);
	FStompSubscriptionId Id = MakeId(SubscribeFrame);
	Subscriptions.Add(Id, EventCallback);

	SubscribeFrame.GetHeader().Add(TEXT("id"), Id);
	SubscribeFrame.GetHeader().Add(TEXT("destination"), Destination);
	WriteFrame(SubscribeFrame, CompletionCallback);
	return Id;
}

void FStompClient::Unsubscribe(FStompSubscriptionId Subscription, const FStompRequestCompleted& CompletionCallback)
{
	Subscriptions.Remove(Subscription);

	FStompFrame UnsubscribeFrame (UnsubscribeCommand);
	UnsubscribeFrame.GetHeader().Add(TEXT("id"), Subscription);
	WriteFrame(UnsubscribeFrame, CompletionCallback);
}

void FStompClient::Send(const FString& Destination, const FStompBuffer& Body, const FStompHeader& Header, const FStompRequestCompleted& CompletionCallback)
{
	FStompFrame Frame(SendCommand, Header, Body);
	Frame.GetHeader().Emplace(TEXT("destination"), Destination);

	WriteFrame(Frame, CompletionCallback);
}

void FStompClient::PingServer()
{
	FStompFrame EmptyFrame;
	WriteFrame(EmptyFrame);
}

void FStompClient::WriteFrame(FStompFrame& Frame, const FStompRequestCompleted& CompletionCallback)
{
	if (CompletionCallback.IsBound())
	{
		FString ReceiptId = MakeId(Frame);
		Frame.GetHeader().Emplace(TEXT("receipt"), ReceiptId);
		OutstandingRequests.Add(ReceiptId, {CompletionCallback, FDateTime::UtcNow()});
	}

	FStompBuffer FrameData;
	Frame.Encode(FrameData);
	check(FrameData.Num()>0);

	// Even though a Stomp frame is terminated with a 0 byte, they must be sent as text
	WebSocket->Send(FrameData.GetData(), FrameData.Num(), false);
	LastSent = FDateTime::UtcNow();
}

void FStompClient::HandleWebSocketConnected()
{
	FStompFrame ConnectFrame(ConnectCommand, ConnectHeader);
	WriteFrame(ConnectFrame);
}

void FStompClient::HandleWebSocketConnectionError(const FString& Error)
{
	OnConnectionError().Broadcast(Error);
}

void FStompClient::HandleWebSocketConnectionClosed(int32 Status, const FString& Reason, bool bWasClean)
{
	if(bIsConnected)
	{
		OnClosed().Broadcast(Reason);
		bIsConnected = false;
	}
}

void FStompClient::HandleWebSocketData(const void* Data, SIZE_T Length, SIZE_T BytesRemaining)
{
	LastReceivedPacket = FDateTime::UtcNow();
	if (BytesRemaining == 0 && ReceiveBuffer.Num() == 0)
	{
		// Skip the temporary buffer when the entire frame arrives in a single message. (common case)
		HandleIncomingFrame((const uint8*)Data, Length);
	}
	else
	{
		ReceiveBuffer.Append((const uint8*)Data, Length);
		if (BytesRemaining == 0)
		{
			HandleIncomingFrame(ReceiveBuffer.GetData(), ReceiveBuffer.Num());
			ReceiveBuffer.Empty();
		}
	}
}

void FStompClient::HandleIncomingFrame(const uint8* Data, SIZE_T Length)
{
	static const FName ReceiptHeader("receipt-id");
	static const FName HeartBeatHeader("heart-beat");
	static const FName MessageHeader("message");
	static const FName VersionHeader("version");
	static const FName SessionHeader("session");
	static const FName ServerHeader("server");

	LastReceivedFrame = FDateTime::UtcNow();
	bReportedNoHeartbeatError = false;

	TSharedRef<FStompFrame> Frame = MakeShareable(new FStompFrame(Data, Length));
	const FStompCommand& Command = Frame->GetCommand();
	const FStompHeader& Header = Frame->GetHeader();
	if(Command == ConnectedCommand)
	{

		FString Left, Right;
		if(Header.Contains(HeartBeatHeader) &&
			Header[HeartBeatHeader].Split(TEXT(","), &Left, &Right, ESearchCase::CaseSensitive, ESearchDir::FromStart) )
		{
			// Note that the server reply swaps the "ping" and "pong" intervals compared to the client CONNECT command header.
			int32 ServerPing = Right.IsNumeric() ? FCString::Atoi(*Right) : 0;
			int32 ServerPong = Left.IsNumeric() ? FCString::Atoi(*Left) : 0;

			int32 ClientPing = PingInterval.GetTotalMilliseconds();
			int32 ClientPong = PongInterval.GetTotalMilliseconds();

			PingInterval = (ServerPing == 0 || ClientPing == 0) ? FTimespan::Zero() : FTimespan::FromMilliseconds(FMath::Max(ServerPing, ClientPing));
			PongInterval = (ServerPong == 0 || ClientPong == 0) ? FTimespan::Zero() : FTimespan::FromMilliseconds(FMath::Max(ServerPong, ClientPong));
		}
		else // No or invalid heart-beat header received - disable heartbeating
		{
			PingInterval = FTimespan::Zero();
			PongInterval = FTimespan::Zero();
		}
		if(Header.Contains(VersionHeader))
		{
			// @TODO: change behavior for old stomp versions?
			ProtocolVersion = Header[VersionHeader];
		}
		if(Header.Contains(SessionHeader))
		{
			SessionId = Header[SessionHeader];
		}
		if(Header.Contains(ServerHeader))
		{
			ServerString = Header[ServerHeader];
		}

		bIsConnected = true;
		OnConnected().Broadcast(ProtocolVersion, SessionId, ServerString);
	}
	else if (Command == MessageCommand)
	{
		FStompMessage Message(SharedThis(this), Frame);
		FStompSubscriptionId Id = Message.GetSubscriptionId();
		if (!Subscriptions.Contains(Id))
		{
			UE_LOGF(LogStomp, Warning, "Received a message from %ls with an unknown or unhandled subscription id %ls", *Message.GetDestination(), *Id);
		}
		else
		{
			Subscriptions[Id].ExecuteIfBound(Message);
		}
	}
	else if (Command == ReceiptCommand)
	{
		const FString& ReceiptId = Header[ReceiptHeader];
		if (!OutstandingRequests.Contains(ReceiptId))
		{
			UE_LOGF(LogStomp, Warning, "Got a receipt with an unknown or unhandled recept id %ls", *ReceiptId);
		}
		else
		{
			OutstandingRequests[ReceiptId].Delegate.ExecuteIfBound(true, FString());
			OutstandingRequests.Remove(ReceiptId);
		}
	}
	else if (Command == ErrorCommand)
	{
		const FString& Message = Header[MessageHeader];
		// If bIsConnected is false, this error is from the CONNECT command
		if(! bIsConnected)
		{
			OnConnectionError().Broadcast(Message);
		}
		else
		{
			const FString& ReceiptId = Header.Contains(ReceiptHeader)?Header[ReceiptHeader]:FString();
			if (!ReceiptId.IsEmpty() && OutstandingRequests.Contains(ReceiptId))
			{
				OutstandingRequests[ReceiptId].Delegate.ExecuteIfBound(false, Message);
				OutstandingRequests.Remove(ReceiptId);
			}
			else
			{
				OnError().Broadcast(Message);
			}
		}
	}
	else if (Command != HeartbeatCommand)
	{
		UE_LOGF(LogStomp, Error, "Got an unknown command %ls", *Command.ToString());
		OnError().Broadcast(FString::Printf(TEXT("Unknown server command %s"), *Command.ToString()));
	}
}

bool FStompClient::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FStompClient_Tick);

	if (IsConnected())
	{
		FDateTime Now = FDateTime::UtcNow();
		if( PingInterval > FTimespan::Zero() && Now - LastSent >= PingInterval )
		{
			PingServer();
		}

		if (!bReportedNoHeartbeatError && PongInterval > FTimespan::Zero() &&
			Now - LastReceivedFrame >= PongInterval * (float)MissedServerPongsBeforeError)
		{
			bReportedNoHeartbeatError = true;
			UE_LOGF(LogStomp, Error, "No Stomp heartbeat for %.1f seconds", (Now - LastReceivedFrame).GetTotalSeconds());
			if (ReceiveBuffer.Num() != 0)
			{
				UE_LOGF(LogStomp, Log, "Stomp: %d bytes pending, received %.1f seconds ago", ReceiveBuffer.Num(), (Now - LastReceivedPacket).GetTotalSeconds());

			}
		}

		TArray<FString> ExpiredRequests;
		for (const auto& Item : OutstandingRequests)
		{
			if (Item.Value.StartTime - Now >= RequestTimeout)
			{
				ExpiredRequests.Add(Item.Key);
			}
		}
		for (const auto& Key : ExpiredRequests)
		{
			OutstandingRequests[Key].Delegate.ExecuteIfBound(false, TEXT("Request timed out"));
			OutstandingRequests.Remove(Key);
		}
	}
	return true;
}

#endif // #if WITH_STOMP
