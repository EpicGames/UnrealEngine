// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStompClient.h"
#include "Containers/Ticker.h"

#if WITH_STOMP

#include "IStompClient.h"
#include "IWebSocket.h"

class Error;

class FStompClient
	: public IStompClient
	, public FTSTickerObjectBase
	, public TSharedFromThis<FStompClient>
{
public:

	virtual ~FStompClient();

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

	// IStompClient
	
	/**
	 * Initiate a client connection to the server.
	 * Use this after setting up event handlers or to reconnect after connection errors.
	 *
	 * @param Header custom headers to send with the initial CONNECT command.
	 */
	virtual void Connect(const FStompHeader& Header=FStompHeader()) override;

	/**
	 * Disconnect from the server.
	 *
	 * @param Header custom headers to send with the DISCONNECT command.
	 */
	virtual void Disconnect(const FStompHeader& Header=FStompHeader()) override;

	/**
	 * Inquire if this instance is connected to a server.
	 */
	virtual bool IsConnected() const override;

	/**
	 * Subscribe to an event
	 */
	virtual FStompSubscriptionId Subscribe(const FString& Destination, const FStompSubscriptionEvent& EventCallback, const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) override;

	/**
	 * Unsubscribe from an event
	 */
	virtual void Unsubscribe(FStompSubscriptionId Subscription, const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) override;

	/**
	 * Emit an event to a destination
	 */
	virtual void Send(const FString& Destination, const FStompBuffer& Body, const FStompHeader& Header=FStompHeader(), const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) override;

	/**
	 * Delegate called when a connection been established successfully.
	 *
	 */
	DECLARE_DERIVED_EVENT(FStompClient, IStompClient::FStompClientConnectedEvent, FStompClientConnectedEvent);
	virtual FStompClientConnectedEvent& OnConnected() override
	{
		return ConnectedEvent;
	}

	/**
	 * Delegate called when a connection could not be established.
	 *
	 */
	DECLARE_DERIVED_EVENT(FStompClient, IStompClient::FStompClientConnectionErrorEvent, FStompClientConnectionErrorEvent);
	virtual FStompClientConnectionErrorEvent& OnConnectionError() override
	{
		return ConnectionErrorEvent;
	}

	/**
	 * Delegate called when an error is received from the server.
	 *
	 */
	DECLARE_DERIVED_EVENT(FStompClient, IStompClient::FStompClientErrorEvent, FStompClientErrorEvent);
	virtual FStompClientErrorEvent& OnError() override
	{
		return ErrorEvent;
	}

	/**
	 * Delegate called when a connection has been closed.
	 *
	 */
	DECLARE_DERIVED_EVENT(FStompClient, IStompClient::FStompClientClosedEvent, FStompClientClosedEvent);
	virtual FStompClientClosedEvent& OnClosed() override
	{
		return ClosedEvent;
	}
private:
	FStompClient(const FString& Url, const FTimespan& InPingInterval, const FTimespan& InPongInterval, const FString& OptAuthToken);

	FString MakeId(const class FStompFrame& Frame);
	void WriteFrame(class FStompFrame& Frame, const FStompRequestCompleted& CompletionCallback=FStompRequestCompleted());
	void PingServer();

	// WS event handlers
	void HandleWebSocketConnected();
	void HandleWebSocketConnectionError(const FString& Error);
	void HandleWebSocketConnectionClosed(int32 Status, const FString& Reason, bool bWasClean);
	void HandleWebSocketData(const void* Data, SIZE_T Length, SIZE_T BytesRemaining);

	void HandleIncomingFrame(const uint8* Data, SIZE_T Length);
	void HandleDisconnectCompleted(bool bSuccess, const FString& Error);


	FStompClientConnectedEvent ConnectedEvent;
	FStompClientConnectionErrorEvent ConnectionErrorEvent;
	FStompClientErrorEvent ErrorEvent;
	FStompClientClosedEvent ClosedEvent;

	TSharedPtr<IWebSocket> WebSocket;
	FStompBuffer ReceiveBuffer;
	FStompHeader ConnectHeader;
	uint64 IdCounter;

	struct FOutstantdingRequestInfo
	{
		FStompRequestCompleted Delegate;
		FDateTime StartTime;
	};
	TMap<FStompSubscriptionId, FStompSubscriptionEvent> Subscriptions;
	TMap<FString, FOutstantdingRequestInfo> OutstandingRequests;

	// "ping" is how often we should send an empty heartbeat packets to the server when inactive
	// "pong" is the minimum how often we should expect activity from the server (counting commands and empty heartbeat frames)
	FTimespan PingInterval, PongInterval;
	FDateTime LastSent;
	FDateTime LastReceivedPacket;
	FDateTime LastReceivedFrame;

	bool bReportedNoHeartbeatError;

	bool bIsConnected;
	FString SessionId;
	FString ServerString;
	FString ProtocolVersion;

	/** Enum to allow safety check to ensure consistent use of API for auth, effectivily a compile-time property */
	enum class EAuthStrategy
	{
		/** Not using any auth */
		None,

		/** Auth token is provided on FStompClient construction, and passed in HTTPS request that upgrades connection to WSS */
		OnConnectionUpgrade,

		/** Auth token is provided on Connect call as Stomp headers to pass in Connect frome */
		OnStompConnect
	};

	EAuthStrategy AuthStrategy = EAuthStrategy::None;

	friend class FStompModule;
	friend class FStompMessage;

};

#endif
