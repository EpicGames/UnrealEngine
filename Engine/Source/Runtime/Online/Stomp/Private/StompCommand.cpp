// Copyright Epic Games, Inc. All Rights Reserved.

#include "StompCommand.h"

#if WITH_STOMP

#define DEFINE_COMMAND(Name) const FLazyName Name ## Command = TEXT(#Name)

const FLazyName HeartbeatCommand = FLazyName();

// note: commands are converted to upper case to send over the wire
DEFINE_COMMAND(Connect);
DEFINE_COMMAND(Connected);
DEFINE_COMMAND(Send);
DEFINE_COMMAND(Subscribe);
DEFINE_COMMAND(Unsubscribe);
DEFINE_COMMAND(Begin);
DEFINE_COMMAND(Commit);
DEFINE_COMMAND(Abort);
DEFINE_COMMAND(Ack);
DEFINE_COMMAND(Nack);
DEFINE_COMMAND(Disconnect);
DEFINE_COMMAND(Message);
DEFINE_COMMAND(Receipt);
DEFINE_COMMAND(Error);

#undef DEFINE_COMMAND

#endif // #if WITH_STOMP
