// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class Error;

#if WITH_STOMP

typedef FName FStompCommand;

#define DECLARE_COMMAND(Name) extern const FLazyName Name ## Command

DECLARE_COMMAND(Heartbeat);
DECLARE_COMMAND(Connect);
DECLARE_COMMAND(Connected);
DECLARE_COMMAND(Send);
DECLARE_COMMAND(Subscribe);
DECLARE_COMMAND(Unsubscribe);
DECLARE_COMMAND(Begin);
DECLARE_COMMAND(Commit);
DECLARE_COMMAND(Abort);
DECLARE_COMMAND(Ack);
DECLARE_COMMAND(Nack);
DECLARE_COMMAND(Disconnect);
DECLARE_COMMAND(Message);
DECLARE_COMMAND(Receipt);
DECLARE_COMMAND(Error);

#undef DECLARE_COMMAND

#endif
