// Copyright Epic Games, Inc. All Rights Reserved.

#include "StompFrame.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "StompLog.h"

#if WITH_STOMP

// Anonymous name space for some frame parsing helpers
namespace
{
	const FLazyName ContentLengthHeader(TEXT("content-length"));

	const uint8* MatchDelimiter(uint8 Element, const char* Delimiters)
	{
		const uint8* Found = (const uint8*)FCStringAnsi::Strchr((const char*)Delimiters, Element);
		return Found;
	}

	static uint8 ReadValue(const uint8* In, SIZE_T Length, SIZE_T& Index, FStompBuffer& Buffer, const char* Delimiters="\n", bool bAllowEscaping = true)
	{
		bool bEscapeNext = false;
		uint8 Retval = '\0';
		for(; Index < Length; Index++)
		{
			if (!bEscapeNext)
			{
				if (bAllowEscaping && In[Index] == '\\')
				{
					bEscapeNext = true;
					continue;
				}
				const uint8* Found = MatchDelimiter(In[Index], Delimiters);
				if (Found != nullptr)
				{
					Retval = *Found;
					Index++;
					break;
				}
			}
			Buffer.Add(In[Index]);
			bEscapeNext = false;
		}

		// The stomp protocol also allows \r\n in addition to \n as line delimiter -- simply trim the \r off the end if present
		// (Unhandled edge case: In case the buffer contains an escaped CR followed by a terminating newline, the CR will be stripped off in any case)
		if (Retval == '\n' && Buffer.Num() > 0 && Buffer[Buffer.Num()-1] == '\r')
		{
			Buffer.RemoveAt(Buffer.Num()-1);
		}

		// Add string terminator at the end of the buffer
		Buffer.Add('\0');
		return Retval;
	}

	void SkipNewlines(const uint8* In, SIZE_T Length, SIZE_T& Index)
	{
		while (Index < Length && MatchDelimiter(In[Index], "\r\n") != nullptr)
		{
			Index++;
		}
	}

	void AppendArray(FStompBuffer& Out, uint8* In, SIZE_T Length, bool bShouldEscape)
	{
		if( bShouldEscape)
		{
			for (SIZE_T I=0; I<Length; I++)
			{
				if (In[I] == ':' || In[I] == '\\' || In[I] == '\n' || In[I] == '\r')
				{
					Out.Add('\\');
				}
				Out.Add(In[I]);
			}
		}
		else
		{
			Out.Append(In, Length);
		}
	}
}


FStompFrame::FStompFrame(const FStompCommand& InCommand, const FStompHeader& InHeader, const FStompBuffer& InBody)
	: Command(InCommand)
	, Header(InHeader)
	, Body(InBody)
{
	if (Body.Num() > 0 && !Header.Contains(ContentLengthHeader))
	{
		Header.Add(ContentLengthHeader, FString::FromInt(Body.Num()));
	}
}

FStompFrame::FStompFrame(const uint8* Data, SIZE_T Length)
	: FStompFrame()
{
	Decode(Data, Length);
}

void FStompFrame::Encode(FStompBuffer& Out) const
{
	// A heartbeat is just a newline and can't contain any data nor is it terminated with a \0 byte
	if (Command == HeartbeatCommand)
	{
		Out.Add('\n');

		if(Header.Num() > 0)
		{
			UE_LOGF(LogStomp, Warning, "Ignoring header fields for heartbeat frame.");
		}
		if(Body.Num() > 0)
		{
			UE_LOGF(LogStomp, Warning, "Ignoring body for heartbeat frame.");
		}
	}
	// Else output COMMAND\nHeaders\n\nBody\0
	else
	{
		// According to the spec, the CONNECT command should not escape metacharacters for backwards compatibility.
		bool bShouldEscapeFrameHeader = Command != ConnectCommand;

		FString CommandString = Command.ToString();
		CommandString.ToUpperInline();
		FTCHARToUTF8 CommandEncoded(*CommandString);
		AppendArray(Out, (uint8*)CommandEncoded.Get(), CommandEncoded.Length(), bShouldEscapeFrameHeader);
		Out.Add('\n');

		for (const TPair<FName, FString>& Element : Header)
		{
			FString ElementKeyString = Element.Key.ToString();
			ElementKeyString.ToLowerInline();
			FTCHARToUTF8 HeaderNameEncoded(*ElementKeyString);
			FTCHARToUTF8 HeaderValueEncoded(*Element.Value);
			AppendArray(Out, (uint8*)HeaderNameEncoded.Get(), HeaderNameEncoded.Length(), bShouldEscapeFrameHeader);
			Out.Add(':');
			AppendArray(Out, (uint8*)HeaderValueEncoded.Get(), HeaderValueEncoded.Length(), bShouldEscapeFrameHeader);
			Out.Add('\n');
		}

		Out.Add('\n');
		Out.Append(Body);
		Out.Add('\0');
	}
}


void FStompFrame::Decode(const uint8* In, SIZE_T Length)
{
	// Ignore terminating 0 if present
	if (Length > 0 && In[Length-1] == 0)
	{
		Length--;
	}

	FStompBuffer Buffer;
	SIZE_T Index = 0;
	// Trim off any initial newlines
	SkipNewlines(In, Length, Index);

	// Empty buffer after trimming newlines means this is a heartbeat packet
	if (Index >= Length)
	{
		Command = HeartbeatCommand;
		return;
	}

	// Read command
	ReadValue(In, Length, Index, Buffer);
	Command = UTF8_TO_TCHAR(Buffer.GetData());

	if (Index >= Length)
	{
		UE_LOGF(LogStomp, Warning, "Stomp command '%ls' received without any headers", *Command.ToString().ToUpper());
		return;
	}

	while(Index < Length)
	{
		const uint8* Junk = In+Index;
		Buffer.Empty();
		uint8 Delimiter = ReadValue(In, Length, Index, Buffer, "\n:");
		FName HeaderName = UTF8_TO_TCHAR(Buffer.GetData());

		if (Delimiter == ':')
		{
			Buffer.Empty();
			ReadValue(In, Length, Index, Buffer);
			Header.Add(HeaderName, UTF8_TO_TCHAR(Buffer.GetData()));
		}
		else if (HeaderName == FName())
		{
			// Empty line marks the end of headers
			break;
		}
		else
		{
			UE_LOGF(LogStomp, Warning, "Encountered header line with no colons, '%ls'.", *HeaderName.ToString())
			Header.Add(HeaderName, TEXT(""));
		}
	}

	// The remaining part, if any is the raw message body
	if (Header.Contains(ContentLengthHeader))
	{
		SIZE_T ContentLength = FCString::Atoi(*Header[ContentLengthHeader]);
		if (ContentLength > Length - Index)
		{
			ContentLength = Length - Index;
			UE_LOG(LogStomp, Warning, TEXT("Warning truncating body. Content-length says %s but only %" SIZE_T_FMT " bytes remain"), *Header[ContentLengthHeader], ContentLength);
		}
		Body.Append(In+Index, ContentLength);
		Index += ContentLength;
	}
	else
	{
		// When there is no content length header, we sould read body until the next zero byte in the stream
		ReadValue(In, Length, Index, Body, "", false); // 0 byte is always a delimiter, don't allow using '\' as escape character
	}

	// Update content length header to match what was actually read from the frame
	Header.Emplace(ContentLengthHeader, FString::FromInt(Body.Num()));

	// Trim off any padding newlines
	SkipNewlines(In, Length, Index);

	// Check for junk data after end of body
	if (Index < Length)
	{
		UE_LOG(LogStomp, Warning, TEXT("%" SIZE_T_FMT " bytes of junk data at end of frame. Was the content-length header missing or wrong?"), Length-Index);
	}
}

#endif // #if WITH_STOMP
