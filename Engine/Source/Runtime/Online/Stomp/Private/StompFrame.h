// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StompCommand.h"
#include "IStompClient.h"

#if WITH_STOMP

/**
 * Class for encoding and parsing stomp frames
 */
class FStompFrame
{
public:

	// Create a frame for sending to the server
	FStompFrame(const FStompCommand& Command=HeartbeatCommand, const FStompHeader& Header=FStompHeader(), const FStompBuffer& Body=FStompBuffer());

	// Parse a frame from the server
	FStompFrame(const uint8* Data, SIZE_T Length);

	const FStompCommand& GetCommand() const
	{
		return Command;
	}

	FStompHeader& GetHeader()
	{
		return Header;
	}

	FStompBuffer& GetBody()
	{
		return Body;
	}

	void Encode(FStompBuffer& Out) const;
private:
	void Decode(const uint8* In, SIZE_T Length);;

	FStompCommand Command;
	FStompHeader Header;
	FStompBuffer Body;
};

#endif
