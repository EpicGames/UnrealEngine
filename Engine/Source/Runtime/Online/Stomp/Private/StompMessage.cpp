// Copyright Epic Games, Inc. All Rights Reserved.

#include "StompMessage.h"
#include "StompClient.h"

#if WITH_STOMP

void FStompMessage::Ack(const FStompHeader& Header, const FStompRequestCompleted& CompletionCallback) const
{
	TSharedPtr<FStompClient> Client = ClientPtr.Pin();
	if (Client.IsValid())
	{
		FStompFrame AckFrame (AckCommand, Header);
		AckFrame.GetHeader().Emplace(TEXT("id"), GetAckId());
		Client->WriteFrame(AckFrame, CompletionCallback);
	}
}

void FStompMessage::Nack(const FStompHeader& Header, const FStompRequestCompleted& CompletionCallback) const
{
	TSharedPtr<FStompClient> Client = ClientPtr.Pin();
	if (Client.IsValid())
	{
		FStompFrame NackFrame (NackCommand, Header);
		NackFrame.GetHeader().Emplace(TEXT("id"), GetAckId());
		Client->WriteFrame(NackFrame, CompletionCallback);
	}
}

const FStompHeader& FStompMessage::GetHeader() const
{
	return Frame->GetHeader();
}

FString FStompMessage::GetBodyAsString() const
{
	// TODO get encoding from content-type header. Currently assuming utf8
	FStompBuffer& Body = Frame->GetBody();
	FUTF8ToTCHAR Convert((ANSICHAR*)Body.GetData(), Body.Num());
	return FString::ConstructFromPtrSize(Convert.Get(), Convert.Length());
}

const uint8* FStompMessage::GetRawBody() const
{
	return Frame->GetBody().GetData();
}

SIZE_T FStompMessage::GetRawBodyLength() const
{
	return Frame->GetBody().Num();
}

#define IMPL_HEADER_FIELD_GETTER(RetType, Name, InHeaderName) \
RetType FStompMessage::Get ## Name() const \
{ \
	static const FName HeaderName(TEXT(InHeaderName)); \
	const FStompHeader& Header = Frame->GetHeader(); \
	return Header.Contains(HeaderName)?Header[HeaderName]:RetType(); \
}

IMPL_HEADER_FIELD_GETTER(FStompSubscriptionId, SubscriptionId, "subscription")
IMPL_HEADER_FIELD_GETTER(FString, Destination, "destination")
IMPL_HEADER_FIELD_GETTER(FString, MessageId, "message-id")
IMPL_HEADER_FIELD_GETTER(FString, AckId, "ack")

#undef IMPL_HEADER_FIELD_GETTER

#endif // #if WITH_STOMP
