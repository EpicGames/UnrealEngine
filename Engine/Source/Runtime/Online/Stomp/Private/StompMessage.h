// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StompFrame.h"

#if WITH_STOMP

#include "IStompMessage.h"

class FStompClient;
class FStompFrame;

class FStompMessage
	: public IStompMessage
{
public:

	virtual ~FStompMessage() = default;

	virtual void Ack(const FStompHeader& Header=FStompHeader(), const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) const override;
	virtual void Nack(const FStompHeader& Header=FStompHeader(), const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) const override;

	virtual const FStompHeader& GetHeader() const override;
	virtual FString GetBodyAsString() const override;
	virtual const uint8* GetRawBody() const override;
	virtual SIZE_T GetRawBodyLength() const override;

	virtual FStompSubscriptionId GetSubscriptionId() const override;
	virtual FString GetDestination() const override;
	virtual FString GetMessageId() const override;
	virtual FString GetAckId() const override;

private:
	FStompMessage(const TSharedRef<FStompClient>& InClient, const TSharedRef<FStompFrame>& InFrame)
		: ClientPtr(InClient)
		, Frame(InFrame)
	{ }

	TWeakPtr<FStompClient> ClientPtr;
	TSharedRef<FStompFrame> Frame;
	friend class FStompClient;

};

#endif
