// Copyright Epic Games, Inc. All Rights Reserved.

#include "StompModule.h"
#include "Modules/ModuleManager.h"
#include "IStompClient.h"
#include "StompLog.h"

// FStompModule
IMPLEMENT_MODULE(FStompModule, Stomp);

#if WITH_STOMP

#include "StompModule.h"
#include "StompClient.h"
#include "WebSocketsModule.h"

#endif // #if WITH_STOMP

DEFINE_LOG_CATEGORY(LogStomp);

static const FTimespan DefaultPingInterval = FTimespan::FromSeconds(30);

FStompModule* FStompModule::Singleton = nullptr;

void FStompModule::StartupModule()
{
#if WITH_STOMP
	FModuleManager::LoadModuleChecked<FWebSocketsModule>("WebSockets");
	Singleton = this;
#endif // #if WITH_STOMP
}

void FStompModule::ShutdownModule()
{
	Singleton = nullptr;
}

FStompModule& FStompModule::Get()
{
	return *Singleton;
}

#if WITH_STOMP

TSharedRef<IStompClient> FStompModule::CreateClient(const FString& Url, const FString& OptAuthToken)
{
	// no server pong - wasteful having all the cloud websocket workers sending out heartbeats
	static const int RequestedServerHeartbeatInterval = 0;
	return MakeShareable(new FStompClient(Url, DefaultPingInterval, RequestedServerHeartbeatInterval, OptAuthToken));
}

#endif // #if WITH_STOMP
