// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IStompMessage;
typedef FString FStompSubscriptionId;
typedef TMap<FName,FString> FStompHeader;
typedef TArray<uint8> FStompBuffer;

DECLARE_DELEGATE_OneParam(FStompSubscriptionEvent, const IStompMessage& /*Message*/);
DECLARE_DELEGATE_TwoParams(FStompRequestCompleted, bool /*bSuccess*/, const FString& /*Error*/);

class IStompClient
{
public:

	virtual ~IStompClient() = default;

	/**
	 * Initiate a client connection to the server.
	 * Use this after setting up event handlers or to reconnect after connection errors.
	 *
	 * @param Header custom headers to send with the initial CONNECT command.
	 */
	virtual void Connect(const FStompHeader& Header=FStompHeader()) = 0;

	/**
	 * Disconnect from the server.
	 *
	 * @param Header custom headers to send with the DISCONNECT command.
	 */
	virtual void Disconnect(const FStompHeader& Header=FStompHeader()) = 0;

	/**
	 * Inquire if this instance is connected to a server.
	 */
	virtual bool IsConnected() const = 0;

	/**
	 * Subscribe to an event
	 * @param Destination Destination endpoint to subscribe to.
	 * @param EventCallback Delegate called when events arrive on this subscription.
	 * @param CompletionCallback Delegate called when the request has been acknowledged by the server or if there is an error.
	 * @return a handle to the active subscription. Can be passed to Unsubscribe to unsubscribe from the end point.
	 */
	virtual FStompSubscriptionId Subscribe(const FString& Destination, const FStompSubscriptionEvent& EventCallback, const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) = 0;

	/**
	 * Unsubscribe from an event
	 * @param Subscription The id returned from the call to Subscribe.
	 * @param CompletionCallback Delegate called when the request has been acknowledged by the server or if there is an error.
	 * @param ResponseCallback Delegate called when the request has been completed.
	 */
	virtual void Unsubscribe(FStompSubscriptionId Subscription, const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) = 0;

	/**
	 * Emit an event to a destination
	 * @param Destination The destination endoint of the event.
	 * @param Body The event body as string. It will be encoded as UTF8 before sending to the Stomp server.
	 * @param Header Custom header values to send along with the data.
	 * @param CompletionCallback Delegate called when the request has been acknowledged by the server or if there is an error.
	 */
	virtual void Send(const FString& Destination, const FString& Body, const FStompHeader& Header=FStompHeader(),  const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted())
	{
		FTCHARToUTF8 Convert(*Body);
		FStompBuffer Encoded;
		Encoded.Append((uint8*)Convert.Get(), Convert.Length());
		Send(Destination, Encoded, Header, CompletionCallback);
	}

	/**
	 * Emit an event to a destination
	 * @param Destination The destination endoint of the event.
	 * @param Body The event body as a binary blob.
	 * @param Header Custom header values to send along with the data.
	 * @param CompletionCallback Delegate called when the request has been acknowledged by the server or if there is an error.
	 */
	virtual void Send(const FString& Destination, const FStompBuffer& Body, const FStompHeader& Header=FStompHeader(),  const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) = 0;


	virtual void Send(const FString& Destination, const FString& Body, const FStompRequestCompleted& CompletionCallback)
	{
		Send(Destination, Body, FStompHeader(), CompletionCallback);
	}

	virtual void Send(const FString& Destination, const FStompBuffer& Body, const FStompRequestCompleted& CompletionCallback)
	{
		Send(Destination, Body, FStompHeader(), CompletionCallback);
	}

	/**
	 * Delegate called when a connection been established successfully.
	 * @param ProtocoVersion The protocol version supported by the server
	 * @param SessionId A unique connection identifier. Can be empty depending on the server implementation.
	 * @param ServerString A server version string if returned from the server, otherwise empty string.
	 *
	 */
	DECLARE_EVENT_ThreeParams(IStompClient, FStompClientConnectedEvent, const FString& /*ProtocolVersion*/, const FString& /*SessionId*/, const FString& /*ServerString*/);
	virtual FStompClientConnectedEvent& OnConnected() = 0;

	/**
	 * Delegate called when a connection could not be established.
	 *
	 */
	DECLARE_EVENT_OneParam(IStompClient, FStompClientConnectionErrorEvent, const FString& /* Error */);
	virtual FStompClientConnectionErrorEvent& OnConnectionError() = 0;

	/**
	 * Delegate called when an error is received from the server.
	 *
	 */
	DECLARE_EVENT_OneParam(IStompClient, FStompClientErrorEvent, const FString& /* Error */);
	virtual FStompClientErrorEvent& OnError() = 0;

	/**
	 * Delegate called when a connection has been closed.
	 *
	 */
	DECLARE_EVENT_OneParam(IStompClient, FStompClientClosedEvent, const FString& /* Reason */);
	virtual FStompClientClosedEvent& OnClosed() = 0;

};
