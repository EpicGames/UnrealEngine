// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStompClient.h"


#if WITH_STOMP

class IStompMessage
{
public:
	virtual void Ack(const FStompHeader& Header=FStompHeader(), const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) const = 0;
	virtual void Nack(const FStompHeader& Header=FStompHeader(), const FStompRequestCompleted& CompletionCallback = FStompRequestCompleted()) const = 0;
	virtual void Ack(const FStompRequestCompleted& CompletionCallback) const
	{
		Ack(FStompHeader(), CompletionCallback);
	}
	virtual void Nack(const FStompRequestCompleted& CompletionCallback) const
	{
		Nack(FStompHeader(), CompletionCallback);
	}

	virtual const FStompHeader& GetHeader() const = 0;
	virtual FString GetBodyAsString() const = 0;
	virtual const uint8* GetRawBody() const = 0;
	virtual SIZE_T GetRawBodyLength() const = 0;

	virtual FStompSubscriptionId GetSubscriptionId() const = 0;
	virtual FString GetDestination() const = 0;
	virtual FString GetMessageId() const = 0;
	virtual FString GetAckId() const = 0;

};

#endif
