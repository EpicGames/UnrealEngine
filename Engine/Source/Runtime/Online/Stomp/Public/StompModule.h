// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class IStompClient;

/**
 * Module for Stomp over WebSockets
 */
class FStompModule :
	public IModuleInterface
{

public:

	// FStompModule

	/**
	 * Singleton-like access to this module's interface - this is just for convenience!
	 * Beware of calling this during the shutdown phase, since yuor module might have been unloaded already.
	 *
	 * @return Returns singleton instance
	 */
	static STOMP_API FStompModule& Get();

#if WITH_STOMP
	/**
	 * Instantiates a new Stomp-over-websockets connection and returns it.
	 *
	 * @param Url The URL to which to connect; this should be the URL to which the WebSocket server will respond with Stomp protocol data.
	 * @return new IStompClient instance
	 */
	STOMP_API TSharedRef<IStompClient> CreateClient(const FString& Url, const FString& OptAuthToken = FString());
#endif // #if WITH_STOMP

private:

	// IModuleInterface

	/**
	 * Called when Stomp module is loaded
	 * Initialize implementation specific parts of Stomp handling
	 */
	STOMP_API virtual void StartupModule() override;

	/**
	 * Called when Stomp module is unloaded
	 * Shutdown implementation specific parts of Stomp handling
	 */
	STOMP_API virtual void ShutdownModule() override;


	/** singleton for the module while loaded and available */
	static FStompModule* Singleton;
};
