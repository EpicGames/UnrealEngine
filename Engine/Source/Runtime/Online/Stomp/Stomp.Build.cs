// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Stomp : ModuleRules
{
	protected virtual bool bPlatformSupportsStomp
	{
		get
		{
			return Target.Platform == UnrealTargetPlatform.Win64 ||
				Target.Platform == UnrealTargetPlatform.Mac ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
		}
	}
	public Stomp(ReadOnlyTargetRules Target) : base(Target)
    {
		bool bShouldUseModule = bPlatformSupportsStomp;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core"
			}
		);

		if (bShouldUseModule)
		{
			PublicDefinitions.Add("WITH_STOMP=1");

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"WebSockets"
				}
			);
		}
		else
		{
			PublicDefinitions.Add("WITH_STOMP=0");
		}
	}
}
