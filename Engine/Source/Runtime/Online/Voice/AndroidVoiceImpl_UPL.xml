<?xml version="1.0" encoding="utf-8"?>
<!--Android Voice additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Android Voice init"/>
		<setBoolFromProperty result="bAndroidVoiceModuleEnabled" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bAndroidVoiceEnabled" default="false"/>
		<setBoolFromProperty result="bRecordPermissionAtStartupEnabled" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bRecordPermissionAtStartupEnabled" default="false"/>
	</init>

	<!-- updates applied to AndroidManifest.xml -->
	<androidManifestUpdates>
		<if condition="bAndroidVoiceModuleEnabled">
			<true>
				<addPermission android:name="android.permission.RECORD_AUDIO"/>

				<if condition="bRecordPermissionAtStartupEnabled">
					<true>
						<!-- add RECORD_AUDIO permission to startup permissions for SplashActivity -->
						<loopElements tag="meta-data">
							<setStringFromAttribute result="metaName" tag="$" name="android:name"/>
							<setBoolIsEqual result="bStartupPerms" arg1="$S(metaName)" arg2="com.epicgames.unreal.GameActivity.StartupPermissions"/>
							<if condition="bStartupPerms">
								<true>
									<setStringFromAttribute result="metaValue" tag="$" name="android:value"/>
									<setIntLength result="metaValueLength" source="$S(metaValue)"/>
									<setBoolIsGreater result="bNeedComma" arg1="$I(metaValueLength)" arg2="0"/>
									<if condition="bNeedComma">
										<false>
											<setString result="metaComma" value=""/>
										</false>
										<true>
											<setString result="metaComma" value=","/>
										</true>
									</if>
									<setBoolContains result="bHasRecord" source="$S(metaValue)" find="android.permission.RECORD_AUDIO"/>
									<if condition="bHasRecord">
										<false>
											<setString result="metaValue" value="$S(metaValue)$S(metaComma)android.permission.RECORD_AUDIO"/>
											<setString result="metaComma" value=","/>
										</false>
									</if>
									<addAttribute tag="$" name="android:value" value="$S(metaValue)"/>
								</true>
							</if>
						</loopElements>
					</true>
				</if>
			</true>
		</if>
	</androidManifestUpdates>

</root>
