// Copyright Epic Games, Inc. All Rights Reserved.

/*
opensl_io.c:
Android OpenSL input/output module
Copyright (c) 2012, Victor Lazzarini
All rights reserved.

Redistribution and use in source and binary forms, with or without
modification, are permitted provided that the following conditions are met:
* Redistributions of source code must retain the above copyright
notice, this list of conditions and the following disclaimer.
* Redistributions in binary form must reproduce the above copyright
notice, this list of conditions and the following disclaimer in the
documentation and/or other materials provided with the distribution.
* Neither the name of the <organization> nor the
names of its contributors may be used to endorse or promote products
derived from this software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS" AND
ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE IMPLIED
WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL <COPYRIGHT HOLDER> BE LIABLE FOR ANY
DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND
ON ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "VoiceCodecOpus.h"
#include "Voice.h"
#include <SLES/OpenSLES.h>
#include <SLES/OpenSLES_Android.h>
#include <pthread.h>
#include <stdlib.h>
#include "AndroidPermissionFunctionLibrary.h"
#include "AndroidPermissionCallbackProxy.h"

#define BUFFERFRAMES 1024


// BEGIN code snippet from https://audioprograming.wordpress.com
// Circular buffer
typedef struct _circular_buffer 
{
	char *buffer;
	int wp;
	int rp;
	int size;
} circular_buffer;

circular_buffer* create_circular_buffer(int bytes)
{
	circular_buffer *p;
	if ((p = (circular_buffer*)calloc(1, sizeof(circular_buffer))) == NULL) 
	{
		return NULL;
	}
	p->size = bytes;
	p->wp = p->rp = 0;
 
	if ((p->buffer = (char*)calloc(bytes, sizeof(char))) == NULL) 
	{
		free (p);
		return NULL;
	}
	return p;
}

int checkspace_circular_buffer(circular_buffer *p, int writeCheck)
{
	int wp = p->wp, rp = p->rp, size = p->size;
	if (writeCheck) 
	{
		if (wp > rp)
		{
			return rp - wp + size - 1;
		}
		else if (wp < rp)
		{
			return rp - wp - 1;
		}
		else
		{
			return size - 1;
		}
	}
	else 
	{
		if (wp > rp)
		{
			return wp - rp;
		}
		else if (wp < rp)
		{
			return wp - rp + size;
		}
		else
		{
			return 0;
		}
	}
}

int read_circular_buffer_bytes(circular_buffer *p, char *out, int bytes)
{
	int remaining;
	int bytesread, size = p->size;
	int i=0, rp = p->rp;
	char *buffer = p->buffer;

	if ((remaining = checkspace_circular_buffer(p, 0)) == 0) 
	{
		return 0;
	}

	bytesread = bytes > remaining ? remaining : bytes;

	for(i=0; i < bytesread; i++)
	{
		out[i] = buffer[rp++];
		if (rp == size)
		{ 
			rp = 0;
		}
	}
	p->rp = rp;

	return bytesread;
}

int write_circular_buffer_bytes(circular_buffer *p, const char *in, int bytes)
{
	int remaining;
	int byteswrite, size = p->size;
	int i=0, wp = p->wp;
	char *buffer = p->buffer;
	
	if ((remaining = checkspace_circular_buffer(p, 1)) == 0) 
	{
		return 0;
	}

	byteswrite = bytes > remaining ? remaining : bytes;
	
	for(i=0; i < byteswrite; i++)
	{
		buffer[wp++] = in[i];
		if(wp == size)
		{ 
			wp = 0;
		}
	}
	p->wp = wp;

	return byteswrite;
}

void free_circular_buffer (circular_buffer *p)
{
	if (p == NULL) 
	{
		return;
	}
	free(p->buffer);
	free(p);
}
// END code snippet from https://audioprograming.wordpress.com


#if ANDROIDVOICE_SUPPORTED_PLATFORMS
/** All outstanding voice capture objects */
TArray<IVoiceCapture*> ActiveVoiceCaptures;

void OpenSLRecordBufferQueueCallback(SLAndroidSimpleBufferQueueItf bq, void *context);
void FreeVoiceCaptureObject(IVoiceCapture* VoiceCaptureObj);
void FreeAllVoiceCaptureObjects();
#endif


/**
 * Implementation of voice capture using OpenSLES
 */

class FVoiceCaptureOpenSLES : public IVoiceCapture
{
public:
#if ANDROIDVOICE_SUPPORTED_PLATFORMS
	SLAndroidSimpleBufferQueueItf SL_RecorderBufferQueue;
#endif
	short *inputBuffer;
	short *recBuffer;
	circular_buffer *inrb;
	int32 inBufSamples;
	/** State of capture device */
	mutable EVoiceCaptureState::Type VoiceCaptureState;
	
	FVoiceCaptureOpenSLES()
		: VoiceCaptureState(EVoiceCaptureState::UnInitialized)
		, inrb(NULL)
		, recBuffer(NULL)
		, inputBuffer(NULL)
		, inBufSamples(0)
#if ANDROIDVOICE_SUPPORTED_PLATFORMS
		, SL_RecorderBufferQueue(NULL)
		, SL_EngineObject(NULL)
		, SL_RecorderObject(NULL)
		, SL_EngineEngine(NULL)
#endif
	{
		
	}

	~FVoiceCaptureOpenSLES()
	{
		Shutdown();
	}
	
	
	bool InitializeHardware( void )
	{
#if ANDROIDVOICE_SUPPORTED_PLATFORMS
		SLresult result;
		
		UE_LOGF( LogVoiceCapture, Warning, "OpenSLES Initializing HW");
		
		SLEngineOption EngineOption[] = { {(SLuint32) SL_ENGINEOPTION_THREADSAFE, (SLuint32) SL_BOOLEAN_TRUE} };
		
		// create engine
		result = slCreateEngine( &SL_EngineObject, 1, EngineOption, 0, NULL, NULL);
		
		if (SL_RESULT_SUCCESS != result)
		{
			UE_LOGF( LogVoiceCapture, Error, "Engine create failed %d", int32(result));
		}		
		
		// realize the engine
		result = (*SL_EngineObject)->Realize(SL_EngineObject, SL_BOOLEAN_FALSE);
		check(SL_RESULT_SUCCESS == result);
		
		// get the engine interface, which is needed in order to create other objects
		result = (*SL_EngineObject)->GetInterface(SL_EngineObject, SL_IID_ENGINE, &SL_EngineEngine);
		check(SL_RESULT_SUCCESS == result);
		
		return true;
#else
		return false;
#endif
	}
	
	
	// IVoiceCapture
	virtual bool Init(const FString& DeviceName, int32 InSampleRate, int32 InNumChannels) override
	{		
#if ANDROIDVOICE_SUPPORTED_PLATFORMS
		UE_LOGF(LogVoiceCapture, Warning, "VoiceModuleAndroid Init");
		check(VoiceCaptureState == EVoiceCaptureState::UnInitialized);

		if (InSampleRate < 8000 || InSampleRate > 48000)
		{
			UE_LOGF(LogVoiceCapture, Warning, "Voice capture doesn't support %d hz", InSampleRate);
			return false;
		}
		
		if (InNumChannels < 0 || InNumChannels > 2)
		{
			UE_LOGF(LogVoiceCapture, Warning, "Voice capture only supports 1 or 2 channels");
			return false;
		}
		
		if ((inBufSamples = BUFFERFRAMES*InNumChannels) != 0)
		{
			if((inputBuffer = (short *) calloc(inBufSamples, sizeof(short))) == NULL)
			{
				return false;
			}
		}
				
		if ((inrb = create_circular_buffer(inBufSamples*sizeof(short)*4)) == NULL) 
		{
			return false;
		}

		//Check that RECORD_AUDIO Permission is included
		UAndroidPermissionFunctionLibrary::Initialize();
		FString Permission = "android.permission.RECORD_AUDIO";
		if(!UAndroidPermissionFunctionLibrary::CheckPermission(Permission))
		{
			UE_LOGF(LogVoiceCapture, Warning, "ANDROID PERMISSION: RECORD_AUDIO is not granted.");
			return false;
		}
		
		// Create the Engine
		InitializeHardware();

		// Set source and sink
		SLDataLocator_IODevice loc_dev = {SL_DATALOCATOR_IODEVICE, SL_IODEVICE_AUDIOINPUT, SL_DEFAULTDEVICEID_AUDIOINPUT, NULL};
		SLDataSource audioSrc = {&loc_dev, NULL};
				
		// data info
		SLDataLocator_AndroidSimpleBufferQueue LocationBuffer = { SL_DATALOCATOR_ANDROIDSIMPLEBUFFERQUEUE, 1 };
		
		// PCM Info
		SLDataFormat_PCM PCM_Format = { SL_DATAFORMAT_PCM, SLuint32(InNumChannels), SLuint32( InSampleRate * 1000 ),
										SL_PCMSAMPLEFORMAT_FIXED_16, SL_PCMSAMPLEFORMAT_FIXED_16,
										InNumChannels == 2 ? ( SL_SPEAKER_FRONT_LEFT | SL_SPEAKER_FRONT_RIGHT ) : SL_SPEAKER_FRONT_CENTER,
										SL_BYTEORDER_LITTLEENDIAN };
		
		SLDataSink audioSnk = { &LocationBuffer, &PCM_Format};
		
		// Create audio recorder
		// Requires the RECORD_AUDIO permission
		UE_LOGF(LogVoiceCapture, Warning, "Create audio recorder");
		SLresult result;
		const SLInterfaceID id[1] = {SL_IID_ANDROIDSIMPLEBUFFERQUEUE};
		const SLboolean req[1] = {SL_BOOLEAN_TRUE};
		result = (*SL_EngineEngine)->CreateAudioRecorder(SL_EngineEngine,
														 &(SL_RecorderObject), &audioSrc,
														 &audioSnk, 1, id, req);
		
		if (result != SL_RESULT_SUCCESS)
		{
			UE_LOGF(LogVoiceCapture, Warning, "FAILED OPENSL Create AudioRecorder 0x%x ", result);
			return false;
		}

		result = (*SL_RecorderObject)->Realize(SL_RecorderObject,
											   SL_BOOLEAN_FALSE);
		
		if (result != SL_RESULT_SUCCESS)
		{
			UE_LOGF(LogVoiceCapture, Warning, "FAILED OPENSL AudioRecorder Realize 0x%x ", result);
			return false;
		}

		result = (*SL_RecorderObject)->GetInterface(SL_RecorderObject,
													SL_IID_RECORD, &(SL_RecorderRecord));
		
		if (result != SL_RESULT_SUCCESS)
		{
			UE_LOGF(LogVoiceCapture, Warning, "FAILED OPENSL RECORD GetInterface 0x%x ", result);
			return false;
		}

		result = (*SL_RecorderObject)->GetInterface(SL_RecorderObject,
													SL_IID_ANDROIDSIMPLEBUFFERQUEUE, &(SL_RecorderBufferQueue));
		
		if(result != SL_RESULT_SUCCESS) 
		{
			UE_LOGF( LogVoiceCapture, Warning,"FAILED OPENSL BUFFER QUEUE GetInterface 0x%x ", result); 
			return false;
		}
		
		UE_LOGF( LogVoiceCapture, Warning, "OpenSLES SL_RecorderBufferQueue %lld", SL_RecorderBufferQueue);
		
		result = (*SL_RecorderBufferQueue)->RegisterCallback(SL_RecorderBufferQueue, OpenSLRecordBufferQueueCallback, (void*)this);

		if(result != SL_RESULT_SUCCESS)
		{ 
			UE_LOGF( LogVoiceCapture, Warning,"FAILED OPENSL BUFFER QUEUE RegisterCallback 0x%x ", result);
			return false;
		}
		
		result = (*SL_RecorderRecord)->SetRecordState(SL_RecorderRecord,SL_RECORDSTATE_RECORDING);
		
		if(result != SL_RESULT_SUCCESS) 
		{
			UE_LOGF( LogVoiceCapture, Warning,"FAILED OPENSL Start Recording 0x%x ", result);
			return false;
		}
		
		if((recBuffer = (short *) calloc(inBufSamples, sizeof(short))) == NULL) 
		{
			return false;
		}
		
		(*SL_RecorderBufferQueue)->Enqueue(SL_RecorderBufferQueue, recBuffer,inBufSamples*sizeof(short));
		
		return true;
#else
		return false;
#endif
	}	
	
	virtual void Shutdown() override
	{
		switch (VoiceCaptureState)
		{
			case EVoiceCaptureState::Ok:
			case EVoiceCaptureState::NoData:
			case EVoiceCaptureState::Stopping:
			case EVoiceCaptureState::BufferTooSmall:
			case EVoiceCaptureState::Error:
				Stop();
			case EVoiceCaptureState::NotCapturing:
				VoiceCaptureState = EVoiceCaptureState::UnInitialized;
				break;
			case EVoiceCaptureState::UnInitialized:
				break;
				
			default:
				check(false);
				break;
		}

		free_circular_buffer(inrb);

		if (inputBuffer != NULL)
		{
			free(inputBuffer);
			inputBuffer = NULL;
		}

		if (recBuffer != NULL)
		{
			free(recBuffer);
			recBuffer = NULL;
		}

#if ANDROIDVOICE_SUPPORTED_PLATFORMS
		if (SL_RecorderBufferQueue != NULL) {
			(*SL_RecorderBufferQueue)->Clear(SL_RecorderBufferQueue);
			SL_RecorderBufferQueue = NULL;
		}

		if (SL_EngineObject != NULL) {
			(*SL_EngineObject)->Destroy(SL_EngineObject);
			SL_EngineObject = NULL;
		}

		if (SL_RecorderObject != NULL) {
			(*SL_RecorderObject)->Destroy(SL_RecorderObject);
			SL_RecorderObject = NULL;
		}

		SL_EngineEngine = NULL;
		
		FreeVoiceCaptureObject(this);
#endif
	}

	virtual bool Start() override
	{
		VoiceCaptureState = EVoiceCaptureState::Ok;

		return true;
	}

	virtual void Stop() override
	{
		VoiceCaptureState = EVoiceCaptureState::NotCapturing;
		return;
	}

	virtual bool ChangeDevice(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
	{
		/** NYI */
		return false;
	}
	
	virtual bool IsCapturing() override
	{
		return VoiceCaptureState > EVoiceCaptureState::NotCapturing;
	}
	
	virtual EVoiceCaptureState::Type GetCaptureState(uint32& OutAvailableVoiceData) const override
	{
		if (VoiceCaptureState != EVoiceCaptureState::UnInitialized &&
			VoiceCaptureState != EVoiceCaptureState::NotCapturing)
		{
			OutAvailableVoiceData = checkspace_circular_buffer(inrb, 0);
			
		}
		else
		{
			OutAvailableVoiceData = 0;
		}
		
		return VoiceCaptureState;
	}
	
	virtual EVoiceCaptureState::Type GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData) override
	{
		EVoiceCaptureState::Type State = VoiceCaptureState;
		
		if (VoiceCaptureState != EVoiceCaptureState::UnInitialized &&
			VoiceCaptureState != EVoiceCaptureState::NotCapturing)
		{
			
			uint32 i, bytes = InVoiceBufferSize*sizeof(short);
			if(inBufSamples == 0) 
			{
				VoiceCaptureState = EVoiceCaptureState::NoData;
				State = EVoiceCaptureState::Ok;
				bytes = 0;
			} 
			else 
			{
				if(InVoiceBufferSize>2048)
				{ // Workaround for dealing with noise after stand-by
					int32 RemainingBytes = InVoiceBufferSize - bytes;
					if (RemainingBytes > 0)
					{
						memset(OutVoiceBuffer + bytes, 0, RemainingBytes);
						bytes = InVoiceBufferSize;
					}
					inrb->wp = 0;
					inrb->rp = 0;
				} 
				else 
				{				
					bytes = read_circular_buffer_bytes(inrb,(char *)OutVoiceBuffer,bytes);
				}
				
				OutAvailableVoiceData = bytes;
				VoiceCaptureState = EVoiceCaptureState::Ok;
				State = EVoiceCaptureState::Ok;
				
			}
		}
		else
		{
			OutAvailableVoiceData = 0;
		}
		return State;
	}

	virtual int32 GetBufferSize() const
	{
		//max raw data size
		return (inBufSamples * sizeof(short) * 4);
	}

	virtual void DumpState() const
	{
		/** NYI */
		UE_LOGF(LogVoiceCapture, Display, "NYI");
	}

private:

#if ANDROIDVOICE_SUPPORTED_PLATFORMS
	// engine interfaces
	SLObjectItf	SL_EngineObject;
	SLObjectItf	SL_RecorderObject;
	SLRecordItf	SL_RecorderRecord;
	SLEngineItf	SL_EngineEngine;

#endif

};


bool InitVoiceCapture()
{
	return true;
}

void ShutdownVoiceCapture()
{
#if ANDROIDVOICE_SUPPORTED_PLATFORMS
	FreeAllVoiceCaptureObjects();
#endif
}

IVoiceCapture* CreateVoiceCaptureObject(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
#if ANDROIDVOICE_SUPPORTED_PLATFORMS
	IVoiceCapture* Capture = new FVoiceCaptureOpenSLES;
	if (!Capture || !Capture->Init(DeviceName, SampleRate, NumChannels))
	{
		delete Capture;
		Capture = nullptr;
	}
	else
	{
		ActiveVoiceCaptures.Add(Capture);
	}
	return Capture;
#else
	return nullptr;
#endif 
}

// BEGIN code snippet from https://audioprograming.wordpress.com	
#if ANDROIDVOICE_SUPPORTED_PLATFORMS
void OpenSLRecordBufferQueueCallback(SLAndroidSimpleBufferQueueItf bq, void *context)
{
	FVoiceCaptureOpenSLES *p = (FVoiceCaptureOpenSLES *) context;
	int32 bytes = p->inBufSamples*sizeof(short);
	write_circular_buffer_bytes(p->inrb, (char *) p->recBuffer,bytes);
	SLresult result = (*p->SL_RecorderBufferQueue)->Enqueue(p->SL_RecorderBufferQueue,p->recBuffer,bytes);
}

void FreeVoiceCaptureObject(IVoiceCapture* VoiceCaptureObj)
{
	if (VoiceCaptureObj != nullptr)
	{
		int32 RemoveIdx = ActiveVoiceCaptures.Find(VoiceCaptureObj);
		if (RemoveIdx != INDEX_NONE)
		{
			ActiveVoiceCaptures.RemoveAtSwap(RemoveIdx);
		}
		else
		{
			UE_LOGF(LogVoiceCapture, Warning, "Trying to free unknown voice object");
		}
	}
}

void FreeAllVoiceCaptureObjects()
{
	// Close any active captures
	UE_LOGF(LogVoiceCapture, Warning, "FreeAllVoiceCaptureObjects");
	for (int32 CaptureIdx = 0; CaptureIdx < ActiveVoiceCaptures.Num(); CaptureIdx++)
	{
		IVoiceCapture* VoiceCapture = ActiveVoiceCaptures[CaptureIdx];
		VoiceCapture->Shutdown();
	}

	ActiveVoiceCaptures.Empty();
}

#endif
// END code snippet from https://audioprograming.wordpress.com