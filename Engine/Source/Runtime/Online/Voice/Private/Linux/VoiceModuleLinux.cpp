// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#if !VOICE_MODULE_WITH_CAPTURE
#include "Interfaces/VoiceCapture.h"
#include "Interfaces/VoiceCodec.h"

bool InitVoiceCapture()
{
	return false;
}

void ShutdownVoiceCapture()
{
}

IVoiceCapture* CreateVoiceCaptureObject(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	return nullptr;
}

#else // VOICE_MODULE_WITH_CAPTURE

#include "VoiceCodecOpus.h"
#include "Voice.h"

#include <SDL3/SDL.h>
#include <SDL3/SDL_audio.h>

class FVoiceCaptureSDL : public IVoiceCapture
{
public:
	FVoiceCaptureSDL();
	~FVoiceCaptureSDL();

	static void AudioCallback(void* UserData, Uint8* Stream, int Length);
	void ReadData(Uint8* Stream, int Length);

	// IVoiceCapture
	virtual bool Init(const FString& DeviceName, int32 SampleRate, int32 NumChannels) override;
	virtual void Shutdown() override;
	virtual bool Start() override;
	virtual void Stop() override;
	virtual bool ChangeDevice(const FString& DeviceName, int32 SampleRate, int32 NumChannels) override;
	virtual bool IsCapturing() override;
	virtual EVoiceCaptureState::Type GetCaptureState(uint32& OutAvailableVoiceData) const override;
	virtual EVoiceCaptureState::Type GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData) override;
	virtual int32 GetBufferSize() const override;
	virtual void DumpState() const override;

private:
	SDL_AudioStream* AudioStream;
	EVoiceCaptureState::Type VoiceCaptureState;

	/** Array to be used as a ring buffer */
	TArray<Uint8> AudioBuffer;
	int32 AudioBufferWritePos;
	int32 AudioBufferAvailableData;
};

FVoiceCaptureSDL::FVoiceCaptureSDL()
	: AudioStream(nullptr)
	, VoiceCaptureState(EVoiceCaptureState::UnInitialized)
	, AudioBufferWritePos(0)
	, AudioBufferAvailableData(0)
{
}

FVoiceCaptureSDL::~FVoiceCaptureSDL()
{
	Shutdown();
}

void FVoiceCaptureSDL::AudioCallback(void* UserData, Uint8* Stream, int Length)
{
	// SDL locks/unlocks the audio device around this callback
	static_cast<FVoiceCaptureSDL*>(UserData)->ReadData(Stream, Length);
}

void FVoiceCaptureSDL::ReadData(Uint8* Stream, int Length)
{
	if (AudioBuffer.Num() - AudioBufferWritePos < Length)
	{
		const int32 FirstCopySize = AudioBuffer.Num() - AudioBufferWritePos;
		const int32 SecondCopySize = Length - FirstCopySize;
		FMemory::Memcpy(&AudioBuffer[AudioBufferWritePos], Stream, FirstCopySize);
		FMemory::Memcpy(&AudioBuffer[0], Stream + FirstCopySize, SecondCopySize);
		AudioBufferWritePos = SecondCopySize;
	}
	else
	{
		FMemory::Memcpy(&AudioBuffer[AudioBufferWritePos], Stream, Length);
		AudioBufferWritePos += Length;
		if (AudioBufferWritePos == AudioBuffer.Num())
		{
			AudioBufferWritePos = 0;
		}
	}

#if UE_BUILD_DEBUG
	if (AudioBufferAvailableData + Length > AudioBuffer.Num())
	{
		UE_LOGF(LogVoiceCapture, Warning, "Discarding %i voice bytes\n", AudioBufferAvailableData + Length - AudioBuffer.Num());
	}
#endif

	AudioBufferAvailableData = FMath::Min(AudioBufferAvailableData + Length, AudioBuffer.Num());
}

static void SDLCALL VoiceCallbackWrapper(void *UserData, SDL_AudioStream *AudioStream, int AdditionalAmount, int TotalAmount)
{
	if (AdditionalAmount > 0)
	{
		Uint8 *Data = SDL_stack_alloc(Uint8, AdditionalAmount);
		if(Data)
		{
			int BytesRead = SDL_GetAudioStreamData(AudioStream, Data, AdditionalAmount);
			if (BytesRead > 0)
			{
				FVoiceCaptureSDL::AudioCallback(UserData, Data, BytesRead);
			}
			SDL_stack_free(Data);
		}
	}
}

bool FVoiceCaptureSDL::Init(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	check(VoiceCaptureState == EVoiceCaptureState::UnInitialized);

	if (SampleRate < 8000 || SampleRate > 48000)
	{
		UE_LOGF(LogVoiceCapture, Warning, "Voice capture doesn't support %d hz", SampleRate);
		return false;
	}

	if (NumChannels < 0 || NumChannels > 2)
	{
		UE_LOGF(LogVoiceCapture, Warning, "Voice capture only supports 1 or 2 channels");
		return false;
	}


	const SDL_AudioSpec DesiredAudioSpec = { SDL_AUDIO_S16LE, NumChannels, SampleRate };
	AudioStream = SDL_OpenAudioDeviceStream(SDL_AUDIO_DEVICE_DEFAULT_RECORDING, &DesiredAudioSpec, &VoiceCallbackWrapper, this);
	if (AudioStream == nullptr)
	{
		UE_LOGF(LogVoiceCapture, Error, "Unable to open Audio Stream: %ls", ANSI_TO_TCHAR(SDL_GetError()));
		return false;
	}

	int SampleSizeInBytes;
	
	switch(DesiredAudioSpec.format)
	{
		case SDL_AUDIO_U8:	// intentional fallthrough
		case SDL_AUDIO_S8:
			SampleSizeInBytes = 1;
			break;
		case SDL_AUDIO_S16LE:	// intentional fallthrough
		case SDL_AUDIO_S16BE:
			SampleSizeInBytes = 2;
			break;
		case SDL_AUDIO_S32LE:	// intentional fallthrough
		case SDL_AUDIO_S32BE:
		case SDL_AUDIO_F32LE:
		case SDL_AUDIO_F32BE:
			SampleSizeInBytes = 4;
			break;
		default: SampleSizeInBytes = -1;
			 break;
	}
	AudioBuffer.SetNum(DesiredAudioSpec.channels * DesiredAudioSpec.freq * SampleSizeInBytes);

	return true;
}

void FVoiceCaptureSDL::Shutdown()
{
	switch (VoiceCaptureState)
	{
		case EVoiceCaptureState::Ok:
		case EVoiceCaptureState::NoData:
		case EVoiceCaptureState::Stopping:
		case EVoiceCaptureState::BufferTooSmall:
		case EVoiceCaptureState::Error:
			Stop();
			// intentional fall-through
		case EVoiceCaptureState::NotCapturing:
			if (AudioStream != nullptr)
			{
				SDL_DestroyAudioStream(AudioStream);
				AudioStream = nullptr;
			}
			VoiceCaptureState = EVoiceCaptureState::UnInitialized;
			break;
		case EVoiceCaptureState::UnInitialized:
			break;

		default:
			check(false);
			break;
	}
}

bool FVoiceCaptureSDL::Start()
{
	AudioBufferWritePos = 0;
	AudioBufferAvailableData = 0;

	SDL_ResumeAudioStreamDevice(AudioStream);
	if (SDL_AudioStreamDevicePaused(AudioStream) == false)
	{
		VoiceCaptureState = EVoiceCaptureState::Ok;
		return true;
	}
	else
	{
		UE_LOGF(LogVoiceCapture, Warning, "Failed to start capture");
		return false;
	}
}

void FVoiceCaptureSDL::Stop()
{
	check(IsCapturing());
	SDL_PauseAudioStreamDevice(AudioStream);

	VoiceCaptureState = EVoiceCaptureState::NotCapturing;
	AudioBufferWritePos = 0;
	AudioBufferAvailableData = 0;
}

bool FVoiceCaptureSDL::ChangeDevice(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	/** stubbed */
	return false;
}

bool FVoiceCaptureSDL::IsCapturing()
{
	return AudioStream != nullptr && VoiceCaptureState > EVoiceCaptureState::NotCapturing;
}

EVoiceCaptureState::Type FVoiceCaptureSDL::GetCaptureState(uint32& OutAvailableVoiceData) const
{
	if (VoiceCaptureState == EVoiceCaptureState::Ok)
	{
		// may be better to use atomics to avoid locking here
		SDL_LockAudioStream(AudioStream);
		OutAvailableVoiceData = AudioBufferAvailableData;
		SDL_UnlockAudioStream(AudioStream);
	}
	else
	{
		OutAvailableVoiceData = 0;
	}

	return VoiceCaptureState;
}

EVoiceCaptureState::Type FVoiceCaptureSDL::GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData)
{
	EVoiceCaptureState::Type ReturnState = VoiceCaptureState;
	OutAvailableVoiceData = 0;

	if (VoiceCaptureState == EVoiceCaptureState::Ok)
	{
		SDL_LockAudioStream(AudioStream);

		uint32 AvailableVoiceData = AudioBufferAvailableData;

		OutAvailableVoiceData = FMath::Min(AvailableVoiceData, InVoiceBufferSize);
		if (OutAvailableVoiceData > 0)
		{
			if (AudioBufferAvailableData > AudioBufferWritePos)
			{
				const int32 FirstCopyAvailableData = AudioBufferAvailableData - AudioBufferWritePos;
				const int32 FirstCopyIndex = AudioBuffer.Num() - FirstCopyAvailableData;
				if (FirstCopyAvailableData >= OutAvailableVoiceData)
				{
					FMemory::Memcpy(OutVoiceBuffer, &AudioBuffer[FirstCopyIndex], OutAvailableVoiceData);
				}
				else
				{
					const int32 SecondCopySize = OutAvailableVoiceData - FirstCopyAvailableData;
					FMemory::Memcpy(OutVoiceBuffer, &AudioBuffer[FirstCopyIndex], FirstCopyAvailableData);
					FMemory::Memcpy(OutVoiceBuffer + FirstCopyAvailableData, &AudioBuffer[0], SecondCopySize);
				}
			}
			else
			{
				const int32 ReadIndex = AudioBufferWritePos - AudioBufferAvailableData;
				FMemory::Memcpy(OutVoiceBuffer, &AudioBuffer[ReadIndex], OutAvailableVoiceData);
			}

			AudioBufferAvailableData -= OutAvailableVoiceData;
		}

		SDL_UnlockAudioStream(AudioStream);
	}
	return ReturnState;
}

int32 FVoiceCaptureSDL::GetBufferSize() const
{
	return AudioBuffer.Num();
}

void FVoiceCaptureSDL::DumpState() const
{
	/** stubbed */
	UE_LOGF(LogVoiceCapture, Display, "FVoiceCaptureSDL::DumpState() is not implemented");
}

bool GVoiceCaptureNeedsToShutdownSDLAudio = false;

bool InitVoiceCapture()
{
	// AudioMixerSDL may interfere with this
	if (!SDL_WasInit(SDL_INIT_AUDIO))
	{
		GVoiceCaptureNeedsToShutdownSDLAudio = SDL_InitSubSystem(SDL_INIT_AUDIO);
		return GVoiceCaptureNeedsToShutdownSDLAudio;
	}

	GVoiceCaptureNeedsToShutdownSDLAudio = false;
	return true;
}

void ShutdownVoiceCapture()
{
	// assume reverse order of shutdown
	if (GVoiceCaptureNeedsToShutdownSDLAudio)
	{
		SDL_QuitSubSystem(SDL_INIT_AUDIO);
	}
}

IVoiceCapture* CreateVoiceCaptureObject(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	IVoiceCapture* Capture = new FVoiceCaptureSDL;
	if (!Capture->Init(DeviceName, SampleRate, NumChannels))
	{
		delete Capture;
		Capture = nullptr;
	}

	return Capture;
}
#endif // VOICE_MODULE_WITH_CAPTURE
