// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoiceCodecOpus.h"
#include "Voice.h"

#include "Mac/MacSystemIncludes.h"
#include <CoreAudio/CoreAudio.h>
#include <AudioToolbox/AudioToolbox.h>
#include <AudioUnit/AudioUnit.h>

/** Maximum buffer size for storing raw uncompressed audio from the system */
#define MAX_UNCOMPRESSED_VOICE_BUFFER_SIZE 30 * 1024

/** Number of hardware buffers per uncompressed CoreAudio buffer */
#define NUM_HARDWARE_BUFFERS_PER_UNCOMPRESSED 6

/**
 * Implementation of voice capture using CoreAudio
 */
class FVoiceCaptureCoreAudio : public IVoiceCapture
{
public:
	FVoiceCaptureCoreAudio()
	: StreamComponent(nullptr)
	, StreamConverter(nullptr)
	, VoiceCaptureState(EVoiceCaptureState::UnInitialized)
	, BufferSize(0)
	, InputLatency(0)
	, ReadableBytes(0)
	, WriteBuffer(0)
	, ReadBuffer(0)
	, ReadOffset(0)
	{
		
	}
	~FVoiceCaptureCoreAudio()
	{
		Shutdown();
	}
	
	static OSStatus InputProc(void* RefCon,
							  AudioUnitRenderActionFlags* IOActionFlags,
							  const AudioTimeStamp* InTimeStamp,
							  UInt32 InBusNumber,
							  UInt32 InNumberFrames,
							  AudioBufferList* IOData)
	{
		FVoiceCaptureCoreAudio* This = static_cast<FVoiceCaptureCoreAudio*>(RefCon);
		if ( This && This->IsCapturing() )
		{
			AudioBufferList BufferList;
			FMemory::Memzero(BufferList);
			BufferList.mNumberBuffers = 1;
			BufferList.mBuffers[0].mNumberChannels = This->OutputDesc.mChannelsPerFrame;
			
			AudioUnitRenderActionFlags Flags = 0;
			if ( AudioUnitRender(This->StreamComponent, &Flags, InTimeStamp, InBusNumber, InNumberFrames, &BufferList) == 0 )
			{
				if ( This->IsCapturing() && This->ReadBuffer != This->WriteBuffer )
				{
					AudioBuffer& WriteBuffer = This->BufferList[This->WriteBuffer];
					uint32 WritableSize = BufferList.mBuffers[0].mDataByteSize;
					uint8 const* InputData = (uint8 const*)BufferList.mBuffers[0].mData;
					while ( WritableSize )
					{
						uint32 MaxWritableSize = FMath::Min(This->BufferSize - WriteBuffer.mDataByteSize, WritableSize);
						uint8* WriteData = (uint8*)(WriteBuffer.mData) + WriteBuffer.mDataByteSize;
						FMemory::Memcpy(WriteData, InputData, MaxWritableSize);
						InputData += MaxWritableSize;
						WritableSize -= MaxWritableSize;
						
						WriteBuffer.mDataByteSize += MaxWritableSize;
						
						uint32 ReadIndex = (This->WriteBuffer + 1) % This->BufferList.Num();
						if( WriteBuffer.mDataByteSize == This->BufferSize )
						{
							This->WriteBuffer = ReadIndex;
							FPlatformAtomics::InterlockedAdd(&This->ReadableBytes, WriteBuffer.mDataByteSize);
							This->VoiceCaptureState = This->VoiceCaptureState == EVoiceCaptureState::NoData ? EVoiceCaptureState::Ok : This->VoiceCaptureState;
							if ( This->ReadBuffer == ReadIndex )
							{
								break; // Clip
							}
						}
					}
				}
			}
			else
			{
				This->VoiceCaptureState = EVoiceCaptureState::Error;
			}
		}
		
		return noErr;
	}
	
	// IVoiceCapture
	virtual bool Init(const FString& DeviceName, int32 InSampleRate, int32 InNumChannels) override
	{
		check(VoiceCaptureState == EVoiceCaptureState::UnInitialized);
		
		if (InSampleRate < 8000 || InSampleRate > 48000)
		{
			UE_LOGF(LogVoiceCapture, Warning, "Voice capture doesn't support %d hz", InSampleRate);
			return false;
		}
		
		if (InNumChannels < 0 || InNumChannels > 2)
		{
			UE_LOGF(LogVoiceCapture, Warning, "Voice capture only supports 1 or 2 channels");
			return false;
		}
		
		uint32 PropSize = sizeof(AudioDeviceID);
		AudioDeviceID InputDevice = 0;
		AudioObjectPropertyAddress Address = { kAudioHardwarePropertyDefaultInputDevice, kAudioObjectPropertyScopeGlobal, kAudioObjectPropertyElementMain };
		if ( AudioObjectGetPropertyData(kAudioObjectSystemObject, &Address, 0, nullptr, &PropSize, &InputDevice) != 0 || InputDevice == kAudioDeviceUnknown )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't get Default CoreAudio input device");
			return false;
		}
		
		StreamDesc.mSampleRate = InSampleRate;
		StreamDesc.mFormatID = kAudioFormatLinearPCM;
		StreamDesc.mFormatFlags = kAudioFormatFlagIsSignedInteger | kAudioFormatFlagsNativeEndian | kAudioFormatFlagIsPacked;
		StreamDesc.mBytesPerPacket = InNumChannels == 1 ? 2 : 4;
		StreamDesc.mBytesPerFrame = InNumChannels == 1 ? 2 : 4;
		StreamDesc.mFramesPerPacket = 1;
		StreamDesc.mBitsPerChannel = 16;
		StreamDesc.mChannelsPerFrame = InNumChannels;
		StreamDesc.mReserved = 0;
		
		AudioComponentDescription ComponentDesc;
		ComponentDesc.componentType = kAudioUnitType_Output;
		ComponentDesc.componentSubType = kAudioUnitSubType_HALOutput;
		ComponentDesc.componentManufacturer = kAudioUnitManufacturer_Apple;
		ComponentDesc.componentFlags = 0;
		ComponentDesc.componentFlagsMask = 0;
		
		AudioComponent Component = AudioComponentFindNext(nullptr, & ComponentDesc);
		if ( !Component || AudioComponentInstanceNew(Component, &StreamComponent) != 0 || !StreamComponent || AudioUnitInitialize(StreamComponent) != 0 )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't get CoreAudio input component");
			return false;
		}
		
		uint32 EnableInput = 1;
		uint32 EnableOutput = 0;
		
		OSStatus Error = AudioUnitSetProperty(StreamComponent, kAudioOutputUnitProperty_EnableIO, kAudioUnitScope_Output, 0, &EnableOutput, sizeof(EnableOutput));
		if ( Error != 0 )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't configure CoreAudio I/O settings");
			AudioComponentInstanceDispose(StreamComponent);
			return false;
		}
		
		Error = AudioUnitSetProperty(StreamComponent, kAudioOutputUnitProperty_EnableIO, kAudioUnitScope_Input, 1, &EnableInput, sizeof(EnableInput));
		if ( Error != 0 )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't configure CoreAudio I/O settings");
			AudioComponentInstanceDispose(StreamComponent);
			return false;
		}
		
		Error = AudioUnitSetProperty(StreamComponent, kAudioOutputUnitProperty_CurrentDevice, kAudioUnitScope_Global, 0, &InputDevice, sizeof(InputDevice));
		if ( Error != 0 )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't configure CoreAudio I/O settings");
			AudioComponentInstanceDispose(StreamComponent);
			return false;
		}
		
		AURenderCallbackStruct InputCb;
		InputCb.inputProc = &FVoiceCaptureCoreAudio::InputProc;
		InputCb.inputProcRefCon = this;
		PropSize = sizeof(NativeDesc);
		FMemory::Memzero(NativeDesc);
		if ( AudioUnitSetProperty(StreamComponent, kAudioOutputUnitProperty_SetInputCallback, kAudioUnitScope_Global, 0, &InputCb, sizeof(InputCb)) != 0
			|| AudioUnitGetProperty(StreamComponent, kAudioUnitProperty_StreamFormat, kAudioUnitScope_Input, 1, &NativeDesc, &PropSize) != 0)
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't configure CoreAudio input component");
			AudioComponentInstanceDispose(StreamComponent);
			return false;
		}
		
		OutputDesc = StreamDesc;
		OutputDesc.mSampleRate = NativeDesc.mSampleRate;
		if ( AudioUnitSetProperty(StreamComponent, kAudioUnitProperty_StreamFormat, kAudioUnitScope_Output, 1, (void *)&OutputDesc, sizeof(OutputDesc)) != 0 )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't configure CoreAudio input component format");
			AudioComponentInstanceDispose(StreamComponent);
			return false;
		}
		
		// create an audio converter
		if ( StreamDesc.mSampleRate != OutputDesc.mSampleRate )
		{
			if ( AudioConverterNew(&OutputDesc, &StreamDesc, &StreamConverter) != 0 || !StreamConverter )
			{
				UE_LOGF(LogVoiceCapture, Warning, "Couldn't configure CoreAudio input format converter");
				AudioComponentInstanceDispose(StreamComponent);
				return false;
			}
		}
		
		PropSize = sizeof(UInt32);
		UInt32 SafetyOffset = 0;
		Address = { kAudioDevicePropertySafetyOffset, kAudioDevicePropertyScopeInput, 0 };
		if ( AudioObjectGetPropertyData(InputDevice, &Address, 0, nullptr, &PropSize, &SafetyOffset) != 0 )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't get CoreAudio input latency");
			AudioComponentInstanceDispose(StreamComponent);
			if(StreamConverter)
			{
				AudioConverterDispose(StreamConverter);
				StreamConverter = nullptr;
			}
			return false;
		}
		SafetyOffset *= StreamDesc.mBytesPerFrame;
		
		PropSize = sizeof(UInt32);
		BufferSize = 0;
		Address = { kAudioDevicePropertyBufferFrameSize, kAudioDevicePropertyScopeInput, 0 };
		if ( AudioObjectGetPropertyData(InputDevice, &Address, 0, nullptr, &PropSize, &BufferSize) != 0 )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Couldn't get CoreAudio input latency");
			AudioComponentInstanceDispose(StreamComponent);
			if(StreamConverter)
			{
				AudioConverterDispose(StreamConverter);
				StreamConverter = nullptr;
			}
			return false;
		}
		
		// Make the buffer size sufficiently large to prevent underflow when decoding.
		BufferSize *= (StreamDesc.mBytesPerFrame * NUM_HARDWARE_BUFFERS_PER_UNCOMPRESSED);
		
		InputLatency = SafetyOffset + BufferSize;
		check(InputLatency < MAX_UNCOMPRESSED_VOICE_BUFFER_SIZE);
		
		uint32 NumBuffers = (uint32)ceilf(MAX_UNCOMPRESSED_VOICE_BUFFER_SIZE / (float)BufferSize);
		
		BufferList.AddZeroed(NumBuffers);
		WriteBuffer = 0;
		for ( uint32 i = 0; i < BufferList.Num(); i++ )
		{
			BufferList[i].mData = FMemory::Malloc(BufferSize);
			if ( !BufferList[i].mData )
			{
				UE_LOGF(LogVoiceCapture, Warning, "Couldn't allocate CoreAudio Buffer List backing store");
				for ( uint32 j = i; j > 0; j-- )
				{
					FMemory::Free(BufferList[j-1].mData);
				}
				if(StreamConverter)
				{
					AudioConverterDispose(StreamConverter);
					StreamConverter = nullptr;
				}
				AudioComponentInstanceDispose(StreamComponent);
				return false;
			}
		}
		
		VoiceCaptureState = EVoiceCaptureState::NotCapturing;
		
		return true;
	}

	virtual void Shutdown() override
	{
		switch (VoiceCaptureState)
		{
			case EVoiceCaptureState::Ok:
			case EVoiceCaptureState::NoData:
			case EVoiceCaptureState::Stopping:
			case EVoiceCaptureState::BufferTooSmall:
			case EVoiceCaptureState::Error:
				Stop();
			case EVoiceCaptureState::NotCapturing:
				check(StreamComponent);
				if(StreamConverter)
				{
					AudioConverterDispose(StreamConverter);
					StreamConverter = nullptr;
				}
				AudioComponentInstanceDispose(StreamComponent);
				StreamComponent = nullptr;
				for ( uint32 i = 0; i < BufferList.Num(); i++ )
				{
					FMemory::Free(BufferList[i].mData);
				}
				VoiceCaptureState = EVoiceCaptureState::UnInitialized;
				break;
			case EVoiceCaptureState::UnInitialized:
				break;
				
			default:
				check(false);
				break;
		}
	}

	virtual bool Start() override
	{
		check(StreamComponent && VoiceCaptureState == EVoiceCaptureState::NotCapturing);
		check(WriteBuffer == 0 && BufferList.Num());
		for ( uint32 i = 0; i < BufferList.Num(); i++ )
		{
			BufferList[i].mDataByteSize = 0;
			FMemory::Memzero(BufferList[i].mData, BufferSize);
		}
		WriteBuffer = 0;
		ReadBuffer = BufferList.Num() - 1;
		ReadOffset = 0;
		ReadableBytes = 0;
		OSStatus Error = AudioOutputUnitStart(StreamComponent);
		if ( Error == 0 )
		{
			VoiceCaptureState = EVoiceCaptureState::Ok;
			return true;
		}
		else
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to start capture %d", Error);
			return false;
		}
	}

	virtual void Stop() override
	{
		check(IsCapturing());
		OSStatus Error = AudioOutputUnitStop(StreamComponent);
		if ( Error != 0 )
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to stop capture %d", Error);
		}
		VoiceCaptureState = EVoiceCaptureState::NotCapturing;
		WriteBuffer = 0;
		ReadBuffer = 0;
		ReadOffset = 0;
		ReadableBytes = 0;
		for ( uint32 i = 0; i < BufferList.Num(); i++ )
		{
			BufferList[i].mDataByteSize = 0;
			FMemory::Memzero(BufferList[i].mData, BufferSize);
		}
	}

	virtual bool ChangeDevice(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
	{
		/** NYI */
		return false;
	}

	virtual bool IsCapturing() override
	{
		return StreamComponent && VoiceCaptureState > EVoiceCaptureState::NotCapturing;
	}

	virtual EVoiceCaptureState::Type GetCaptureState(uint32& OutAvailableVoiceData) const override
	{
		if (VoiceCaptureState != EVoiceCaptureState::UnInitialized &&
			VoiceCaptureState != EVoiceCaptureState::NotCapturing)
		{
			OutAvailableVoiceData = ReadableBytes >= InputLatency ? (ReadableBytes - InputLatency) : 0;
		}
		else
		{
			OutAvailableVoiceData = 0;
		}
		
		return VoiceCaptureState;
	}
	
	virtual EVoiceCaptureState::Type GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData) override
	{
		EVoiceCaptureState::Type State = VoiceCaptureState;
		
		if (VoiceCaptureState != EVoiceCaptureState::UnInitialized &&
			VoiceCaptureState != EVoiceCaptureState::NotCapturing)
		{
			GetCaptureState(OutAvailableVoiceData);
			
			if ( OutAvailableVoiceData > 0 )
			{
				uint32 ConvertedVoiceData = 0;
				if ( InVoiceBufferSize >= OutAvailableVoiceData )
				{
					check(ReadOffset <= BufferSize);
					uint32 BytesToRead = OutAvailableVoiceData;
					uint8* Data = OutVoiceBuffer;
					while (BytesToRead)
					{
						uint32 CurrentRead = FMath::Min(FMath::Min(BufferList[ReadBuffer].mDataByteSize - ReadOffset, BytesToRead), BufferSize);
						uint32 CurrentWrite = 0;
						State = CopyBuffer(ReadBuffer, Data, ReadOffset, &CurrentRead, &CurrentWrite);
						
						if ( State == EVoiceCaptureState::Ok )
						{
							Data += CurrentWrite;
							ConvertedVoiceData += CurrentWrite;
							BytesToRead = ( CurrentRead != 0 ) ? BytesToRead - CurrentRead : 0;
							ReadOffset += CurrentRead;
							if( ReadOffset == BufferList[ReadBuffer].mDataByteSize )
							{
								BufferList[ReadBuffer].mDataByteSize = 0;
								FMemory::Memzero(BufferList[ReadBuffer].mData, BufferSize);
								ReadBuffer = (ReadBuffer + 1) % BufferList.Num();
								ReadOffset = 0;
							}
							check(ReadOffset <= BufferSize);
						}
						else
						{
							break;
						}
					}
					int32 BytesRead = OutAvailableVoiceData;
					FPlatformAtomics::InterlockedAdd(&ReadableBytes, -BytesRead);
					
					OutAvailableVoiceData  = ConvertedVoiceData;
				}
				else
				{
					State = EVoiceCaptureState::BufferTooSmall;
				}
			}
			else
			{
				VoiceCaptureState = EVoiceCaptureState::NoData;
				State = EVoiceCaptureState::Ok;
			}
		}
		else
		{
			OutAvailableVoiceData = 0;
		}
		return State;
	}

	virtual int32 GetBufferSize() const
	{
		/** NYI */
		return 0;
	}

	virtual void DumpState() const
	{
		/** NYI */
		UE_LOGF(LogVoiceCapture, Display, "NYI");
	}

private:

	struct FAudioFileIO
	{
		uint32 SrcReadFrames;
		AudioBufferList* SrcBuffer;
		uint32 SrcBufferSize;
		uint32 SrcBytesPerFrame;
	};

	static OSStatus ConvertInputFormat(AudioConverterRef AudioConverter,
										UInt32* IONumberDataPackets,
										AudioBufferList* IOData,
										AudioStreamPacketDescription** OutDataPacketDescription,
										void* InUserData)
	{
		FAudioFileIO* Input = (FAudioFileIO*)InUserData;

		if (Input && Input->SrcBuffer)
		{
			*IOData = *Input->SrcBuffer;

			int32 Num = *IONumberDataPackets;
			if ((Input->SrcReadFrames * (Input->SrcBytesPerFrame)) < Input->SrcBufferSize)
			{
				if ((Num * Input->SrcBytesPerFrame) < IOData->mBuffers[0].mDataByteSize)
				{
					IOData->mBuffers[0].mDataByteSize = Num * Input->SrcBytesPerFrame;
				}
				else
				{
					*IONumberDataPackets = IOData->mBuffers[0].mDataByteSize / Input->SrcBytesPerFrame;
				}
				Input->SrcReadFrames += *IONumberDataPackets;
			}
			else
			{
				*IONumberDataPackets = 0;
				IOData->mBuffers[0].mDataByteSize = 0;
				return eofErr;
			}
		}

		return noErr;
	}

	EVoiceCaptureState::Type CopyBuffer(uint8 InReadBuffer, uint8* OutVoiceBuffer, uint32 InReadOffset, uint32* ReadLen, uint32* WriteLen)
	{
		EVoiceCaptureState::Type State = VoiceCaptureState;
		if (*ReadLen)
		{
			uint8* Data = (uint8*)(BufferList[InReadBuffer].mData) + InReadOffset;

			if (StreamDesc.mSampleRate == OutputDesc.mSampleRate)
			{
				FMemory::Memcpy(OutVoiceBuffer, Data, *ReadLen);
				*WriteLen = *ReadLen;
				State = EVoiceCaptureState::Ok;
			}
			else
			{
				UInt32 FramesToCopy = (*ReadLen / OutputDesc.mBytesPerFrame);
				AudioBufferList OutputBuffer;
				OutputBuffer.mNumberBuffers = 1;
				OutputBuffer.mBuffers[0].mNumberChannels = StreamDesc.mChannelsPerFrame;
				OutputBuffer.mBuffers[0].mDataByteSize = *ReadLen;
				OutputBuffer.mBuffers[0].mData = OutVoiceBuffer;

				AudioBufferList InputBuffer;
				InputBuffer.mNumberBuffers = 1;
				InputBuffer.mBuffers[0].mNumberChannels = StreamDesc.mChannelsPerFrame;
				InputBuffer.mBuffers[0].mDataByteSize = BufferList[InReadBuffer].mDataByteSize;
				InputBuffer.mBuffers[0].mData = Data;

				FAudioFileIO inUserData;
				inUserData.SrcBuffer = &InputBuffer;
				inUserData.SrcBytesPerFrame = StreamDesc.mBytesPerFrame;
				inUserData.SrcBufferSize = *ReadLen;
				inUserData.SrcReadFrames = 0;

				OSStatus result = AudioConverterFillComplexBuffer(StreamConverter, &FVoiceCaptureCoreAudio::ConvertInputFormat, &inUserData, &FramesToCopy, &OutputBuffer, nullptr);
				if (result == 0 || result == eofErr)
				{
					State = EVoiceCaptureState::Ok;
					*ReadLen = inUserData.SrcReadFrames * StreamDesc.mBytesPerFrame;
					*WriteLen = FramesToCopy * OutputDesc.mBytesPerFrame;
				}
				else
				{
					State = EVoiceCaptureState::Error;
					VoiceCaptureState = EVoiceCaptureState::Error;
				}
			}
		}
		return State;
	}
	
private:
	/** Descriptor for stream format */
	AudioStreamBasicDescription StreamDesc;
	/** Descriptor for native HW format */
	AudioStreamBasicDescription NativeDesc;
	/** Descriptor for HW output format */
	AudioStreamBasicDescription OutputDesc;
	/** Input AudioUnit component instance */
	AudioComponentInstance StreamComponent;
	/** Sample rate converter instance */
	AudioConverterRef StreamConverter;
	/** State of capture device */
	EVoiceCaptureState::Type VoiceCaptureState;
	/** Outstanding CoreAudio buffer lists */
	TArray<AudioBuffer> BufferList;
	/** Input device buffer size */
	uint32 BufferSize;
	/** Input device latency */
	uint32 InputLatency;
	/** Current readable bytes */
	int32 ReadableBytes;
	/** Buffer CoreAudio should write to */
	uint8 WriteBuffer;
	/** Buffer game should read from */
	uint8 ReadBuffer;
	/** Current offset into ReadBuffer */
	uint32 ReadOffset;
};

bool InitVoiceCapture()
{
	return true;
}

void ShutdownVoiceCapture()
{
}

IVoiceCapture* CreateVoiceCaptureObject(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	IVoiceCapture* Capture = new FVoiceCaptureCoreAudio;
	if ( !Capture || !Capture->Init(DeviceName, SampleRate, NumChannels) )
	{
		delete Capture;
		Capture = nullptr;
	}
	return Capture;
}