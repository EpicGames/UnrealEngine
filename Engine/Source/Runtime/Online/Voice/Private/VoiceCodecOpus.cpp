// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoiceCodecOpus.h"
#include "VoiceModule.h"
#if PLATFORM_SUPPORTS_OPUS_CODEC

THIRD_PARTY_INCLUDES_START
#include "opus.h"
THIRD_PARTY_INCLUDES_END

/** Turn on extra logging for codec */
#define DEBUG_OPUS 0
/** Turn on entropy data in packets */
#define ADD_ENTROPY_TO_PACKET 0
/** Use UE memory allocation or Opus */
#define USE_UE_MEM_ALLOC 1
/** Maximum number of frames in a single Opus packet */
#define MAX_OPUS_FRAMES_PER_PACKET 48
/** Number of max frames for buffer calculation purposes */
#define MAX_OPUS_FRAMES 6

/**
 * Number of samples per channel of available space in PCM during decompression.  
 * If this is less than the maximum packet duration (120ms; 5760 for 48kHz), opus will not be capable of decoding some packets.
 */
#define MAX_OPUS_FRAME_SIZE MAX_OPUS_FRAMES * 320
/** Hypothetical maximum for buffer validation */
#define MAX_OPUS_UNCOMPRESSED_BUFFER_SIZE 48 * 1024
/** 20ms frame sizes are a good choice for most applications (1000ms / 20ms = 50) */
#define NUM_OPUS_FRAMES_PER_SEC 50

#define OPUS_CHECK_CTL(Category, CTL) \
	if (ErrCode != OPUS_OK) \
	{ \
		UE_LOGF(Category, Warning, "Failure to get CTL %s", #CTL); \
	}

/**
 * Output debug information regarding the state of the Opus encoder
 */
void DebugEncoderInfo(OpusEncoder* Encoder)
{
	int32 ErrCode = 0;

	int32 BitRate = 0;
	ErrCode = opus_encoder_ctl(Encoder, OPUS_GET_BITRATE(&BitRate));
	OPUS_CHECK_CTL(LogVoiceEncode, BitRate);

	int32 Vbr = 0;
	ErrCode = opus_encoder_ctl(Encoder, OPUS_GET_VBR(&Vbr));
	OPUS_CHECK_CTL(LogVoiceEncode, Vbr);

	int32 SampleRate = 0;
	ErrCode = opus_encoder_ctl(Encoder, OPUS_GET_SAMPLE_RATE(&SampleRate));
	OPUS_CHECK_CTL(LogVoiceEncode, SampleRate);

	int32 Application = 0;
	ErrCode = opus_encoder_ctl(Encoder, OPUS_GET_APPLICATION(&Application));
	OPUS_CHECK_CTL(LogVoiceEncode, Application);

	int32 Signal = 0;
	ErrCode = opus_encoder_ctl(Encoder, OPUS_GET_SIGNAL(&Signal));
	OPUS_CHECK_CTL(LogVoiceEncode, Signal);

	int32 Complexity = 0;
	ErrCode = opus_encoder_ctl(Encoder, OPUS_GET_COMPLEXITY(&Complexity));
	OPUS_CHECK_CTL(LogVoiceEncode, Complexity);

	UE_LOGF(LogVoiceEncode, Display, "Opus Encoder Details");
	UE_LOGF(LogVoiceEncode, Display, "- Application: %d", Application);
	UE_LOGF(LogVoiceEncode, Display, "- Signal: %d", Signal);
	UE_LOGF(LogVoiceEncode, Display, "- BitRate: %d", BitRate);
	UE_LOGF(LogVoiceEncode, Display, "- SampleRate: %d", SampleRate);
	UE_LOGF(LogVoiceEncode, Display, "- Vbr: %d", Vbr);
	UE_LOGF(LogVoiceEncode, Display, "- Complexity: %d", Complexity);
}

/**
 * Output debug information regarding the state of the Opus decoder
 */
void DebugDecoderInfo(OpusDecoder* Decoder)
{
	int32 ErrCode = 0;

	int32 Gain = 0;
	ErrCode = opus_decoder_ctl(Decoder, OPUS_GET_GAIN(&Gain));
	OPUS_CHECK_CTL(LogVoiceDecode, Gain);

	int32 Pitch = 0;
	ErrCode = opus_decoder_ctl(Decoder, OPUS_GET_PITCH(&Pitch));
	OPUS_CHECK_CTL(LogVoiceDecode, Pitch);

	UE_LOGF(LogVoiceDecode, Display, "Opus Decoder Details");
	UE_LOGF(LogVoiceDecode, Display, "- Gain: %d", Gain);
	UE_LOGF(LogVoiceDecode, Display, "- Pitch: %d", Pitch);
}

/**
 * Output debug information regarding the state of a single Opus packet
 */
void DebugFrameInfoInternal(const uint8* PacketData, uint32 PacketLength, uint32 SampleRate, bool bEncode)
{
	int32 NumFrames = opus_packet_get_nb_frames(PacketData, PacketLength);
	if (NumFrames == OPUS_BAD_ARG || NumFrames == OPUS_INVALID_PACKET)
	{
		UE_LOGF(LogVoice, Warning, "opus_packet_get_nb_frames: Invalid voice packet data!");
	}

	int32 NumSamples = opus_packet_get_nb_samples(PacketData, PacketLength, SampleRate);
	if (NumSamples == OPUS_BAD_ARG || NumSamples == OPUS_INVALID_PACKET)
	{
		UE_LOGF(LogVoice, Warning, "opus_packet_get_nb_samples: Invalid voice packet data!");
	}

	int32 NumSamplesPerFrame = opus_packet_get_samples_per_frame(PacketData, SampleRate);
	int32 Bandwidth = opus_packet_get_bandwidth(PacketData);

	const TCHAR* BandwidthStr = nullptr;
	switch (Bandwidth)
	{
	case OPUS_BANDWIDTH_NARROWBAND: // Narrowband (4kHz bandpass)
		BandwidthStr = TEXT("NB");
		break;
	case OPUS_BANDWIDTH_MEDIUMBAND: // Mediumband (6kHz bandpass)
		BandwidthStr = TEXT("MB");
		break;
	case OPUS_BANDWIDTH_WIDEBAND: // Wideband (8kHz bandpass)
		BandwidthStr = TEXT("WB");
		break;
	case OPUS_BANDWIDTH_SUPERWIDEBAND: // Superwideband (12kHz bandpass)
		BandwidthStr = TEXT("SWB");
		break;
	case OPUS_BANDWIDTH_FULLBAND: // Fullband (20kHz bandpass)
		BandwidthStr = TEXT("FB");
		break;
	case OPUS_INVALID_PACKET: 
	default:
		BandwidthStr = TEXT("Invalid");
		break;
	}

	/*
	 *	0
	 *	0 1 2 3 4 5 6 7
	 *	+-+-+-+-+-+-+-+-+
	 *	| config  |s| c |
	 *	+-+-+-+-+-+-+-+-+
	 */
	uint8 TOC = 0;
	// (max 48 x 2.5ms frames in a packet = 120ms)
	const uint8* frames[48];
	int16 size[48];
	int32 payload_offset = 0;
	int32 NumFramesParsed = opus_packet_parse(PacketData, PacketLength, &TOC, frames, size, &payload_offset);

	// Frame Encoding see http://tools.ietf.org/html/rfc6716#section-3.1
	int32 TOCEncoding = (TOC & 0xf8) >> 3;

	// Number of channels
	bool TOCStereo = (TOC & 0x4) != 0 ? true : false;

	// Number of frames and their configuration
	// 0: 1 frame in the packet
	// 1: 2 frames in the packet, each with equal compressed size
	// 2: 2 frames in the packet, with different compressed sizes
	// 3: an arbitrary number of frames in the packet
	int32 TOCMode = TOC & 0x3;

	if (bEncode)
	{
		UE_LOGF(LogVoiceEncode, Verbose, "PacketLength: %d NumFrames: %d NumSamples: %d Bandwidth: %ls Encoding: %d Stereo: %d FrameDesc: %d",
			PacketLength, NumFrames, NumSamples, BandwidthStr, TOCEncoding, TOCStereo, TOCMode);
	}
	else
	{
		UE_LOGF(LogVoiceDecode, Verbose, "PacketLength: %d NumFrames: %d NumSamples: %d Bandwidth: %ls Encoding: %d Stereo: %d FrameDesc: %d",
			PacketLength, NumFrames, NumSamples, BandwidthStr, TOCEncoding, TOCStereo, TOCMode);
	}
}

inline void DebugFrameEncodeInfo(const uint8* PacketData, uint32 PacketLength, uint32 SampleRate)
{
	DebugFrameInfoInternal(PacketData, PacketLength, SampleRate, true);
}

inline void DebugFrameDecodeInfo(const uint8* PacketData, uint32 PacketLength, uint32 SampleRate)
{
	DebugFrameInfoInternal(PacketData, PacketLength, SampleRate, false);
}

FVoiceEncoderOpus::FVoiceEncoderOpus() :
	SampleRate(0),
	NumChannels(0),
	FrameSize(0),
	Encoder(nullptr),
	LastEntropyIdx(0),
	Generation(0)
{
	FMemory::Memzero(Entropy, NUM_ENTROPY_VALUES * sizeof(uint32));
}

FVoiceEncoderOpus::~FVoiceEncoderOpus()
{
	Destroy();
}

bool FVoiceEncoderOpus::Init(int32 InSampleRate, int32 InNumChannels, EAudioEncodeHint EncodeHint)
{
	UE_LOGF(LogVoiceEncode, Display, "EncoderVersion: %ls", ANSI_TO_TCHAR(opus_get_version_string()));

	if (InSampleRate != 8000 &&
		InSampleRate != 12000 &&
		InSampleRate != 16000 &&
		InSampleRate != 24000 &&
		InSampleRate != 48000)
	{
		UE_LOGF(LogVoiceEncode, Warning, "Voice encoder doesn't support %d hz", InSampleRate);
		return false;
	}

	if (InNumChannels < 1 || InNumChannels > 2)
	{
		UE_LOGF(LogVoiceEncode, Warning, "Voice encoder only supports 1 or 2 channels");
		return false;
	}

	SampleRate = InSampleRate;
	NumChannels = InNumChannels;
	
	// 20ms frame sizes are a good choice for most applications (1000ms / 20ms = 50)
	FrameSize = SampleRate / NUM_OPUS_FRAMES_PER_SEC;
	//MaxFrameSize = FrameSize * MAX_OPUS_FRAMES;

	int32 EncError = 0;

	const int32 Application = (EncodeHint == EAudioEncodeHint::VoiceEncode_Audio) ? OPUS_APPLICATION_AUDIO : OPUS_APPLICATION_VOIP;

#if USE_UE_MEM_ALLOC
	const int32 EncSize = opus_encoder_get_size(NumChannels);
	Encoder = (OpusEncoder*)FMemory::Malloc(EncSize);
	EncError = opus_encoder_init(Encoder, SampleRate, NumChannels, Application);
#else
	Encoder = opus_encoder_create(SampleRate, NumChannels, Application, &EncError);
#endif

	if (EncError == OPUS_OK)
	{
		// Turn on variable bit rate encoding
		const int32 UseVbr = 1;
		opus_encoder_ctl(Encoder, OPUS_SET_VBR(UseVbr));

		// Turn off constrained VBR
		const int32 UseCVbr = 0;
		opus_encoder_ctl(Encoder, OPUS_SET_VBR_CONSTRAINT(UseCVbr));

		// Complexity (1-10)
		const int32 Complexity = 1;
		opus_encoder_ctl(Encoder, OPUS_SET_COMPLEXITY(Complexity));

		// Forward error correction
		const int32 InbandFEC = 0;
		opus_encoder_ctl(Encoder, OPUS_SET_INBAND_FEC(InbandFEC));

#if DEBUG_OPUS
		DebugEncoderInfo(Encoder);
#endif // DEBUG_OPUS
	}
	else
	{
		UE_LOGF(LogVoiceEncode, Warning, "Failed to init Opus Encoder: %ls", ANSI_TO_TCHAR(opus_strerror(EncError)));
		Destroy();
	}

	return EncError == OPUS_OK;
}

int32 FVoiceEncoderOpus::Encode(const uint8* RawPCMData, uint32 RawDataSize, uint8* OutCompressedData, uint32& OutCompressedDataSize)
{
	check(Encoder);
	SCOPE_CYCLE_COUNTER(STAT_Voice_Encoding);
	SET_DWORD_STAT(STAT_Encode_SampleRate, SampleRate);
	SET_DWORD_STAT(STAT_Encode_NumChannels, NumChannels);

	int32 HeaderSize = 0;
	const int32 BytesPerFrame = FrameSize * NumChannels * sizeof(opus_int16);
	const int32 MaxFramesEncoded = MAX_OPUS_UNCOMPRESSED_BUFFER_SIZE / BytesPerFrame;

	// total bytes / bytes per frame
	const int32 NumFramesToEncode = FMath::Min((int32)RawDataSize / BytesPerFrame, MaxFramesEncoded);
	const int32 DataRemainder = RawDataSize % BytesPerFrame;
	const int32 RawDataStride = BytesPerFrame;

	if (NumFramesToEncode == 0)
	{
		// We can avoid saving out an empty header if we know we're not going to send anything
		check(DataRemainder == RawDataSize);
		OutCompressedDataSize = 0;
		return DataRemainder;
	}

	// Store the number of frames to be encoded
	check(NumFramesToEncode < MAX_uint8);
	OutCompressedData[0] = (uint8)NumFramesToEncode;
	OutCompressedData[1] = Generation;
	HeaderSize += 2 * sizeof(uint8);
	
	// Store the offset to each encoded frame
	uint16* CompressedOffsets = (uint16*)(OutCompressedData + HeaderSize);
	const uint32 LengthOfCompressedOffsets = NumFramesToEncode * sizeof(uint16);
	HeaderSize += LengthOfCompressedOffsets;

	// Store the entropy to each encoded frame
#if ADD_ENTROPY_TO_PACKET
	uint32* EntropyOffsets = (uint32*)(OutCompressedData + HeaderSize);
	uint32 LengthOfEntropyOffsets = NumFramesToEncode * sizeof(uint32);
	HeaderSize += LengthOfEntropyOffsets;
#endif

	// Space available after overhead
	int32 AvailableBufferSize = OutCompressedDataSize - HeaderSize;
	
	// Start of the actual compressed data
	uint8* CompressedDataStart = OutCompressedData + HeaderSize;
	int32 CompressedBufferOffset = 0;
	for (int32 i = 0; i < NumFramesToEncode; i++)
	{
		int32 CompressedLength = opus_encode(Encoder, (const opus_int16*)(RawPCMData + (i * RawDataStride)), FrameSize, CompressedDataStart + CompressedBufferOffset, AvailableBufferSize);
		if (CompressedLength < 0)
		{
			const char* ErrorStr = opus_strerror(CompressedLength);
			UE_LOGF(LogVoiceEncode, Warning, "Failed to encode: [%d] %ls", CompressedLength, ANSI_TO_TCHAR(ErrorStr));

			// Mark header as nothing encoded
			OutCompressedData[0] = 0;
			OutCompressedDataSize = 0;
			return 0;
		}
		else if (CompressedLength != 1)
		{
#if ADD_ENTROPY_TO_PACKET
			opus_encoder_ctl(Encoder, OPUS_GET_FINAL_RANGE(&Entropy[LastEntropyIdx]));

			UE_LOGF(LogVoiceEncode, VeryVerbose, "Entropy[%d]=%d", i, Entropy[LastEntropyIdx]);
			EntropyOffsets[i] = Entropy[LastEntropyIdx];

			LastEntropyIdx = (LastEntropyIdx + 1) % NUM_ENTROPY_VALUES;
#endif

#if DEBUG_OPUS
			DebugFrameEncodeInfo(CompressedDataStart + CompressedBufferOffset, CompressedLength, SampleRate);
#endif // DEBUG_OPUS

			AvailableBufferSize -= CompressedLength;
			CompressedBufferOffset += CompressedLength;

			check(CompressedBufferOffset < MAX_uint16); 
			CompressedOffsets[i] = (uint16)CompressedBufferOffset;
		}
		else
		{
			UE_LOGF(LogVoiceEncode, Warning, "Nothing to encode!");
			CompressedOffsets[i] = 0;
#if ADD_ENTROPY_TO_PACKET
			EntropyOffsets[i] = 0;
#endif
		}
	}

	// End of buffer
	OutCompressedDataSize = HeaderSize + CompressedBufferOffset;

	UE_LOGF(LogVoiceEncode, VeryVerbose, "OpusEncode[%d]: RawSize: %d HeaderSize: %d CompressedSize: %d NumFramesEncoded: %d Remains: %d", Generation, RawDataSize, HeaderSize, OutCompressedDataSize, NumFramesToEncode, DataRemainder);

#if STATS
	int32 BitRate = 0;
	int32 ErrCode = opus_encoder_ctl(Encoder, OPUS_GET_BITRATE(&BitRate));
	SET_DWORD_STAT(STAT_Encode_Bitrate, BitRate);
	SET_FLOAT_STAT(STAT_Encode_CompressionRatio, (float)OutCompressedDataSize / (float)RawDataSize);
	SET_DWORD_STAT(STAT_Encode_OutSize, OutCompressedDataSize);
#endif // STATS

	Generation = (Generation + 1) % MAX_uint8;
	return DataRemainder;
}

bool FVoiceEncoderOpus::SetBitrate(int32 InBitRate)
{
	if (InBitRate >= 500 &&
		InBitRate <= 512000)
	{
		if (Encoder)
		{
			int32 EncError = opus_encoder_ctl(Encoder, OPUS_SET_BITRATE(InBitRate));

#if DEBUG_OPUS
			DebugEncoderInfo(Encoder);
#endif // DEBUG_OPUS

			return EncError == OPUS_OK;
		}
	}
	
	return false;
}

bool FVoiceEncoderOpus::SetVBR(bool bEnableVBR)
{
	if (Encoder)
	{
		const int32 UseVbr = bEnableVBR ? 1 : 0;
		int32 EncError = opus_encoder_ctl(Encoder, OPUS_SET_VBR(UseVbr));

#if DEBUG_OPUS
		DebugEncoderInfo(Encoder);
#endif // DEBUG_OPUS

		return EncError == OPUS_OK;
	}

	return false;
}

bool FVoiceEncoderOpus::SetComplexity(int32 InComplexity)
{
	if (InComplexity >= 0 &&
		InComplexity <= 10)
	{
		if (Encoder)
		{
			int32 EncError = opus_encoder_ctl(Encoder, OPUS_SET_COMPLEXITY(InComplexity));

#if DEBUG_OPUS
			DebugEncoderInfo(Encoder);
#endif // DEBUG_OPUS

			return EncError == OPUS_OK;
		}
	}

	return false;
}

void FVoiceEncoderOpus::Reset()
{
	int32 EncError = 0;
	if (Encoder)
	{
		EncError = opus_encoder_ctl(Encoder, OPUS_RESET_STATE);
		if (EncError != OPUS_OK)
		{
			UE_LOGF(LogVoiceEncode, Warning, "Failure to reset Opus encoder");
		}

#if DEBUG_OPUS
		DebugEncoderInfo(Encoder);
#endif // DEBUG_OPUS
	}
}

void FVoiceEncoderOpus::Destroy()
{
#if USE_UE_MEM_ALLOC
	FMemory::Free(Encoder);
#else
	opus_encoder_destroy(Encoder);
#endif
	Encoder = nullptr;
}

void FVoiceEncoderOpus::DumpState() const
{
	if (Encoder)
	{
		DebugEncoderInfo(Encoder);
	}
	else
	{
		UE_LOGF(LogVoiceEncode, Display, "No encoder to dump state");
	}
}

FVoiceDecoderOpus::FVoiceDecoderOpus() :
	SampleRate(0),
	NumChannels(0),
	FrameSize(0),
	Decoder(nullptr),
	LastEntropyIdx(0),
	LastGeneration(0)
{
	FMemory::Memzero(Entropy, NUM_ENTROPY_VALUES * sizeof(uint32));
}

FVoiceDecoderOpus::~FVoiceDecoderOpus()
{
	Destroy();
}

bool FVoiceDecoderOpus::Init(int32 InSampleRate, int32 InNumChannels)
{
	UE_LOGF(LogVoiceDecode, Display, "DecoderVersion: %ls", ANSI_TO_TCHAR(opus_get_version_string()));

	if (InSampleRate != 8000 &&
		InSampleRate != 12000 &&
		InSampleRate != 16000 &&
		InSampleRate != 24000 &&
		InSampleRate != 48000)
	{
		UE_LOGF(LogVoiceDecode, Warning, "Voice decoder doesn't support %d hz", InSampleRate);
		return false;
	}

	if (InNumChannels < 1 || InNumChannels > 2)
	{
		UE_LOGF(LogVoiceDecode, Warning, "Voice decoder only supports 1 or 2 channels");
		return false;
	}

	SampleRate = InSampleRate;
	NumChannels = InNumChannels;

	// 20ms frame sizes are a good choice for most applications (1000ms / 20ms = 50)
	FrameSize = SampleRate / NUM_OPUS_FRAMES_PER_SEC;

	int32 DecError = 0;

#if USE_UE_MEM_ALLOC
	const int32 DecSize = opus_decoder_get_size(NumChannels);
	Decoder = (OpusDecoder*)FMemory::Malloc(DecSize);
	DecError = opus_decoder_init(Decoder, SampleRate, NumChannels);
#else
	Decoder = opus_decoder_create(SampleRate, NumChannels, &DecError);
#endif
	if (DecError == OPUS_OK)
	{
#if DEBUG_OPUS
		DebugDecoderInfo(Decoder);
#endif // DEBUG_OPUS
	}
	else
	{
		UE_LOGF(LogVoiceDecode, Warning, "Failed to init Opus Decoder: %ls", ANSI_TO_TCHAR(opus_strerror(DecError)));
		Destroy();
	}

	return DecError == OPUS_OK;
}

void FVoiceDecoderOpus::Reset()
{
	int32 DecError = 0;
	if (Decoder)
	{
		DecError = opus_decoder_ctl(Decoder, OPUS_RESET_STATE);
		if (DecError != OPUS_OK)
		{
			UE_LOGF(LogVoiceDecode, Warning, "Failure to reset Opus decoder");
		}

#if DEBUG_OPUS
		DebugDecoderInfo(Decoder);
#endif // DEBUG_OPUS
	}
}

void FVoiceDecoderOpus::Destroy()
{
#if USE_UE_MEM_ALLOC
	FMemory::Free(Decoder);
#else
	opus_decoder_destroy(Decoder);
#endif 
	Decoder = nullptr;
}

inline bool SanityCheckHeader(uint32 HeaderSize, uint32 CompressedDataSize, int32 NumFramesToDecode, const uint16* CompressedOffsets)
{
	bool bHeaderDataOk = (HeaderSize <= CompressedDataSize);
	if (bHeaderDataOk)
	{
		// Validate that the sum of the encoded data sizes fit under the given amount of compressed data
		uint16 LastCompressedOffset = 0;
		int32 TotalCompressedBufferSize = 0;
		for (int32 Idx = 0; Idx < NumFramesToDecode; Idx++)
		{
			// Offsets should be monotonically increasing (prevent later values intentionally reducing bad previous values)
			if (CompressedOffsets[Idx] >= LastCompressedOffset)
			{
				TotalCompressedBufferSize += (CompressedOffsets[Idx] - LastCompressedOffset);
				LastCompressedOffset = CompressedOffsets[Idx];
			}
			else
			{
				bHeaderDataOk = false;
				break;
			}
		}

		bHeaderDataOk = bHeaderDataOk && ((HeaderSize + TotalCompressedBufferSize) <= CompressedDataSize);
	}

	return bHeaderDataOk;
}

void FVoiceDecoderOpus::Decode(const uint8* InCompressedData, uint32 CompressedDataSize, uint8* OutRawPCMData, uint32& OutRawDataSize)
{
	SCOPE_CYCLE_COUNTER(STAT_Voice_Decoding);
	SET_DWORD_STAT(STAT_Decode_SampleRate, SampleRate);
	SET_DWORD_STAT(STAT_Decode_NumChannels, NumChannels);

	uint32 HeaderSize = (2 * sizeof(uint8));
	if (!InCompressedData || (CompressedDataSize < HeaderSize))
	{
		OutRawDataSize = 0;
		return;
	}

	const int32 BytesPerFrame = FrameSize * NumChannels * sizeof(opus_int16);
	const int32 MaxFramesEncoded = MAX_OPUS_UNCOMPRESSED_BUFFER_SIZE / BytesPerFrame;

	const int32 NumFramesToDecode = InCompressedData[0];
	const int32 PacketGeneration = InCompressedData[1];

	if (PacketGeneration != LastGeneration + 1)
	{
		UE_LOGF(LogVoiceDecode, Verbose, "Packet generation skipped from %d to %d", LastGeneration, PacketGeneration);
	}

	if ((NumFramesToDecode > 0) && (NumFramesToDecode <= MaxFramesEncoded))
	{
		// Start of compressed data offsets
		const uint16* CompressedOffsets = (const uint16*)(InCompressedData + HeaderSize);
		uint32 LengthOfCompressedOffsets = NumFramesToDecode * sizeof(uint16);
		HeaderSize += LengthOfCompressedOffsets;

#if ADD_ENTROPY_TO_PACKET
		// Start of the entropy to each encoded frame
		const uint32* EntropyOffsets = (uint32*)(InCompressedData + HeaderSize);
		uint32 LengthOfEntropyOffsets = NumFramesToDecode * sizeof(uint32);
		HeaderSize += LengthOfEntropyOffsets;
#endif

		// At this point we have all our pointer fix up complete, but the data it references may be invalid in corrupt/spoofed packets
		// Sanity check the numbers to make sure everything works out
		if (SanityCheckHeader(HeaderSize, CompressedDataSize, NumFramesToDecode, CompressedOffsets))
		{
			// Start of compressed data
			const uint8* CompressedDataStart = (InCompressedData + HeaderSize);

			int32 CompressedBufferOffset = 0;
			int32 DecompressedBufferOffset = 0;
			uint16 LastCompressedOffset = 0;

			for (int32 i = 0; i < NumFramesToDecode; i++)
			{
				const int32 UncompressedBufferAvail = (OutRawDataSize - DecompressedBufferOffset);

				if (UncompressedBufferAvail >= (MAX_OPUS_FRAMES * BytesPerFrame))
				{
					if (CompressedOffsets[i] > 0)
					{
						const int32 CompressedBufferSize = (CompressedOffsets[i] - LastCompressedOffset);

						check(Decoder);
						const int32 NumDecompressedSamples = opus_decode(Decoder,
							CompressedDataStart + CompressedBufferOffset, CompressedBufferSize,
							(opus_int16*)(OutRawPCMData + DecompressedBufferOffset), MAX_OPUS_FRAME_SIZE, 0);

#if DEBUG_OPUS
						DebugFrameDecodeInfo(CompressedDataStart + CompressedBufferOffset, CompressedBufferSize, SampleRate);
#endif // DEBUG_OPUS

						if (NumDecompressedSamples < 0)
						{
							const char* ErrorStr = opus_strerror(NumDecompressedSamples);
							UE_LOGF(LogVoiceDecode, Warning, "Failed to decode: [%d] %ls", NumDecompressedSamples, ANSI_TO_TCHAR(ErrorStr));
						}
						else
						{
							if (NumDecompressedSamples != FrameSize)
							{
								UE_LOGF(LogVoiceDecode, Warning, "Unexpected decode result NumSamplesDecoded %d != FrameSize %d", NumDecompressedSamples, FrameSize);
							}

#if ADD_ENTROPY_TO_PACKET
							opus_decoder_ctl(Decoder, OPUS_GET_FINAL_RANGE(&Entropy[LastEntropyIdx]));

							if (Entropy[LastEntropyIdx] != EntropyOffsets[i])
							{
								UE_LOGF(LogVoiceDecode, Verbose, "Decoder Entropy[%d/%d] = %d expected %d", i, NumFramesToDecode - 1, Entropy[LastEntropyIdx], EntropyOffsets[i]);
							}


							LastEntropyIdx = (LastEntropyIdx + 1) % NUM_ENTROPY_VALUES;
#endif

							// Advance within the decompressed output stream
							DecompressedBufferOffset += (NumDecompressedSamples * NumChannels * sizeof(opus_int16));
						}

						// Advance within the compressed input stream
						CompressedBufferOffset += CompressedBufferSize;
						LastCompressedOffset = CompressedOffsets[i];
					}
					else
					{
						UE_LOGF(LogVoiceDecode, Verbose, "Decompression buffer skipped a frame");
						// Nothing to advance within the compressed input stream
					}
				}
				else
				{
					UE_LOGF(LogVoiceDecode, Warning, "Decompression buffer too small to decode voice");
					break;
				}
			}

			OutRawDataSize = DecompressedBufferOffset;
		}
		else
		{
			UE_LOGF(LogVoiceDecode, Warning, "Failed to decode: header corrupted");
			OutRawDataSize = 0;
		}
	}
	else
	{
		UE_LOGF(LogVoiceDecode, Warning, "Failed to decode: buffer corrupted");
		OutRawDataSize = 0;
	}

	UE_LOGF(LogVoiceDecode, VeryVerbose, "OpusDecode[%d]: RawSize: %d HeaderSize: %d CompressedSize: %d NumFramesDecoded: %d ", PacketGeneration, OutRawDataSize, HeaderSize, CompressedDataSize, NumFramesToDecode);
	SET_FLOAT_STAT(STAT_Decode_CompressionRatio, (float)CompressedDataSize / (float)OutRawDataSize);
	SET_DWORD_STAT(STAT_Decode_OutSize, OutRawDataSize);

	LastGeneration = PacketGeneration;
}

void FVoiceDecoderOpus::DumpState() const
{
	if (Decoder)
	{
		DebugDecoderInfo(Decoder);
	}
	else
	{
		UE_LOGF(LogVoiceDecode, Display, "No decoder to dump state");
	}
}

#endif
