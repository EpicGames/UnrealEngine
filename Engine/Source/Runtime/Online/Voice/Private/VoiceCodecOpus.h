// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/VoiceCodec.h"
#include "VoicePrivate.h"
#include "VoicePackage.h"

#if PLATFORM_SUPPORTS_OPUS_CODEC

/** Number of entropy values to store in the encoder/decoder (similar to a CRC) */
#define NUM_ENTROPY_VALUES 5

/**
 * Opus voice compression
 */
class FVoiceEncoderOpus : public IVoiceEncoder
{
public:

	FVoiceEncoderOpus();
	~FVoiceEncoderOpus();

	// IVoiceEncoder
	virtual bool Init(int32 SampleRate, int32 NumChannels, EAudioEncodeHint EncodeHint) override;
	virtual int32 Encode(const uint8* RawPCMData, uint32 RawDataSize, uint8* OutCompressedData, uint32& OutCompressedDataSize) override;
	virtual bool SetBitrate(int32 InBitRate) override;
	virtual bool SetVBR(bool bEnableVBR) override;
	virtual bool SetComplexity(int32 InComplexity) override;
	virtual void Reset() override;
	virtual void Destroy() override;
	virtual void DumpState() const override;

private:

	/** Sample rate encoding (supports 8000, 12000, 16000, 24000, 480000) */
	int32 SampleRate;
	/** Encoded channel count (supports 1,2) */
	int32 NumChannels;
	/**
	 * Number of samples encoded in a time slice "frame" (must match decoder)
	 * One frame defined as (2.5, 5, 10, 20, 40 or 60 ms) of audio data
	 * Voice encoding lower bound is 10ms (audio goes to 2.5ms). 
	 * Voice encoding upper bound is 60ms (audio goes to 20ms). 
	 * at 48 kHz the permitted values are 120 (2.5ms), 240 (5ms), 480 (10ms), 960 (20ms), 1920 (40ms), and 2880 (60ms)
	 */
	int32 FrameSize;
	/** Opus encoder stateful data */
	struct OpusEncoder* Encoder;
	/** Last values for error checking with the decoder */
	uint32 Entropy[NUM_ENTROPY_VALUES];
	/** Last recorded entropy index */
	uint32 LastEntropyIdx;
	/** Last value set in the call to Encode() */
	uint8 Generation;
};

/**
 * Opus voice decompression
 */
class FVoiceDecoderOpus : public IVoiceDecoder
{
public:

	FVoiceDecoderOpus();
	~FVoiceDecoderOpus();

	//IVoiceDecoder
	virtual bool Init(int32 SampleRate, int32 NumChannels) override;
	virtual void Decode(const uint8* CompressedData, uint32 CompressedDataSize, uint8* OutRawPCMData, uint32& OutRawDataSize) override;
	virtual void Reset() override;
	virtual void Destroy() override;
	virtual void DumpState() const override;

private:

	/** Sample rate to decode into, regardless of encoding (supports 8000, 12000, 16000, 24000, 480000) */
	int32 SampleRate;
	/** Decoded channel count (supports 1,2) */
	int32 NumChannels;
	/**
	 * Number of samples encoded in a time slice (must match encoder)
	 * at 48 kHz the permitted values are 120, 240, 480, 960, 1920, and 2880
	 */
	int32 FrameSize;
	/** Opus decoder stateful data */
	struct OpusDecoder* Decoder;
	/** Last values for error checking with the encoder */
	uint32 Entropy[NUM_ENTROPY_VALUES];
	/** Last recorded entropy index */
	uint32 LastEntropyIdx;
	/** Generation value received from the last incoming packet */
	uint8 LastGeneration;
};

#endif
