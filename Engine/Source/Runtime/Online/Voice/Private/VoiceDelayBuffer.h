// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/************************************************************************/
/* FVoiceDelayBuffer                                                    */
/* This class provides a simple template circular buffer.               */
/************************************************************************/
template<class SampleType>
class FVoiceDelayBuffer
{
private:
	TArray<SampleType> InternalBuffer;

	int32 WriteIndex;
	int32 ReadIndex;

public:
	/** Allocates resources for a maximum buffer of MaxLength samples. */
	void Init(int32 MaxLength)
	{
		check(MaxLength > 1);
		InternalBuffer.Init(0, MaxLength);
		WriteIndex = 0;
		ReadIndex = 0;
	}

	/** Sets delay between read and write in samples. */
	void SetDelay(int32 NumSamples)
	{
		const int32 Delta = WriteIndex - NumSamples;
		if (Delta < 0)
		{
			ReadIndex = InternalBuffer.Num() + Delta;
		}
		else
		{
			ReadIndex = Delta;
		}
	}

	/* Gets the amount of samples on the buffer. */
	int32 GetBufferCount()
	{
		if (ReadIndex > WriteIndex)
		{
			return WriteIndex + InternalBuffer.Num() - ReadIndex;
		}
		else
		{
			return WriteIndex - ReadIndex;
		}
	}

	/** Pushes sample onto the buffer. Decrements delay between read and write by one sample. */
	void PushSample(const SampleType& InSample)
	{
		WriteIndex = WriteIndex % InternalBuffer.Num();
		InternalBuffer[WriteIndex++] = InSample;
	}

	void PushFrame(const SampleType* InFrame, int32 NumChannels)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
		{
			PushSample(InFrame[ChannelIndex]);
		}
	}

	/** pop sample off of the buffer. Increments delay between read and write by one sample. */
	SampleType PopSample()
	{
		ReadIndex = ReadIndex % InternalBuffer.Num();
		return InternalBuffer[ReadIndex++];
	}

	/** Pushes InSample onto the buffer and pops OutSample. */
	void ProcessSample(const SampleType& InSample, SampleType& OutSample)
	{
		PushSample(InSample);
		OutSample = PopSample();
	}

	/* pop as much audio as possible onto OutBuffer. returns the number of samples copied. */
	int32 PopBufferedAudio(SampleType* OutBuffer, int32 MaxNumSamples)
	{
		ReadIndex = ReadIndex % InternalBuffer.Num();
		int32 SamplesToCopy = FMath::Min(MaxNumSamples, GetBufferCount());

		if (SamplesToCopy == 0)
		{
			return 0;
		}

		if (ReadIndex <= WriteIndex)
		{
			FMemory::Memcpy(OutBuffer, InternalBuffer.GetData() + ReadIndex, SamplesToCopy * sizeof(SampleType));
		}
		else
		{
			const int32 SamplesUntilEndOfBuffer = InternalBuffer.Num() - ReadIndex;
			//Copy the rest of the buffer to the end, then the remainder in the beginning.
			FMemory::Memcpy(OutBuffer, InternalBuffer.GetData() + ReadIndex, SamplesUntilEndOfBuffer * sizeof(SampleType));
			FMemory::Memcpy(OutBuffer + SamplesUntilEndOfBuffer, InternalBuffer.GetData(), (SamplesToCopy - SamplesUntilEndOfBuffer) * sizeof(SampleType));
		}
		ReadIndex = ReadIndex + SamplesToCopy;
		return SamplesToCopy;
	}
};



