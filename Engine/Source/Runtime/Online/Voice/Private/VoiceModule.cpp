// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoiceModule.h"
#include "Misc/ConfigCacheIni.h"
#include "VoicePrivate.h"
#include "VoiceCodecOpus.h"

IMPLEMENT_MODULE(FVoiceModule, Voice);

DEFINE_LOG_CATEGORY(LogVoice);
DEFINE_LOG_CATEGORY(LogVoiceEncode);
DEFINE_LOG_CATEGORY(LogVoiceDecode);
DEFINE_LOG_CATEGORY(LogVoiceCapture);

DEFINE_STAT(STAT_Voice_Encoding);
DEFINE_STAT(STAT_Voice_Decoding);
DEFINE_STAT(STAT_Encode_SampleRate);
DEFINE_STAT(STAT_Encode_NumChannels);
DEFINE_STAT(STAT_Encode_Bitrate);
DEFINE_STAT(STAT_Encode_CompressionRatio);
DEFINE_STAT(STAT_Encode_OutSize);
DEFINE_STAT(STAT_Decode_SampleRate);
DEFINE_STAT(STAT_Decode_NumChannels);
DEFINE_STAT(STAT_Decode_CompressionRatio);
DEFINE_STAT(STAT_Decode_OutSize);

#if PLATFORM_SUPPORTS_VOICE_CAPTURE
/** Implement these functions per platform to create the voice objects */
extern bool InitVoiceCapture();
extern void ShutdownVoiceCapture();
extern IVoiceCapture* CreateVoiceCaptureObject(const FString& DeviceName, int32 SampleRate, int32 NumChannels);
#endif

void FVoiceModule::StartupModule()
{	
	bEnabled = false;
	if (!GConfig->GetBool(TEXT("Voice"), TEXT("bEnabled"), bEnabled, GEngineIni))
	{
		UE_LOGF(LogVoice, Warning, "Missing bEnabled key in Voice of DefaultEngine.ini");
	}

#if PLATFORM_SUPPORTS_VOICE_CAPTURE
	if (bEnabled)
	{
		if (!InitVoiceCapture())
		{
			UE_LOGF(LogVoice, Warning, "Failed to initialize voice capture module");
			ShutdownVoiceCapture();
		}
	}
#endif
}

void FVoiceModule::ShutdownModule()
{
#if PLATFORM_SUPPORTS_VOICE_CAPTURE
	if (bEnabled)
	{
		ShutdownVoiceCapture();
	}
#endif
}

TSharedPtr<IVoiceCapture> FVoiceModule::CreateVoiceCapture(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
#if PLATFORM_SUPPORTS_VOICE_CAPTURE
	if (bEnabled)
	{
		// Create the platform specific instance
		return TSharedPtr<IVoiceCapture>(CreateVoiceCaptureObject(DeviceName, SampleRate, NumChannels));
	}
#endif
	return nullptr;
}

TSharedPtr<IVoiceEncoder> FVoiceModule::CreateVoiceEncoder(int32 SampleRate, int32 NumChannels, EAudioEncodeHint EncodeHint)
{
	if (bEnabled)
	{
		// Create the platform specific instance
		return TSharedPtr<IVoiceEncoder>(CreateVoiceEncoderObject(SampleRate, NumChannels, EncodeHint));
	}

	return nullptr;
}

IVoiceEncoder* FVoiceModule::CreateVoiceEncoderObject(int32 SampleRate, int32 NumChannels, EAudioEncodeHint EncodeHint)
{
#if PLATFORM_SUPPORTS_OPUS_CODEC
	FVoiceEncoderOpus* NewEncoder = new FVoiceEncoderOpus;
	if (!NewEncoder->Init(SampleRate, NumChannels, EncodeHint))
	{
		delete NewEncoder;
		NewEncoder = nullptr;
	}

	return NewEncoder;
#else
	return nullptr;
#endif
}

TSharedPtr<IVoiceDecoder> FVoiceModule::CreateVoiceDecoder(int32 SampleRate, int32 NumChannels)
{
	if (bEnabled)
	{
		// Create the platform specific instance
		return TSharedPtr<IVoiceDecoder>(CreateVoiceDecoderObject(SampleRate, NumChannels));
	}

	return nullptr;
}

IVoiceDecoder* FVoiceModule::CreateVoiceDecoderObject(int32 SampleRate, int32 NumChannels)
{
#if PLATFORM_SUPPORTS_OPUS_CODEC
	FVoiceDecoderOpus* NewDecoder = new FVoiceDecoderOpus;
	if (!NewDecoder->Init(SampleRate, NumChannels))
	{
		delete NewDecoder;
		NewDecoder = nullptr;
	}

	return NewDecoder;
#else
	return nullptr;
#endif
}

bool FVoiceModule::DoesPlatformSupportVoiceCapture()
{
	return PLATFORM_SUPPORTS_VOICE_CAPTURE;
}


