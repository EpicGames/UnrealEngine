// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_WINDOWS

#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
#include <Audiopolicy.h>
#include <Mmdeviceapi.h>
#include <Functiondiscoverykeys_devpkey.h>
#include <dsound.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

#endif // PLATFORM_WINDOWS

#define ANDROIDVOICE_SUPPORTED_PLATFORMS (PLATFORM_ANDROID)
#define PLATFORM_SUPPORTS_VOICE_CAPTURE (PLATFORM_WINDOWS || PLATFORM_MAC || ANDROIDVOICE_SUPPORTED_PLATFORMS || (PLATFORM_UNIX && VOICE_MODULE_WITH_CAPTURE))

// Module includes
