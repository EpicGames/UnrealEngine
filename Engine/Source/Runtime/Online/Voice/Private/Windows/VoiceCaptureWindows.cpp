// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoiceCaptureWindows.h"
#include "VoicePrivate.h"
#include "VoiceModule.h"
#include "DSP/Dsp.h"

#if PLATFORM_SUPPORTS_VOICE_CAPTURE

#include "Windows/AllowWindowsPlatformTypes.h"

static int32 DisplayAmplitudeCvar = 0;
FAutoConsoleVariableRef CVarDisplayAmplitude(
	TEXT("voice.debug.PrintAmplitude"),
	DisplayAmplitudeCvar,
	TEXT("when set to 1, the current incoming amplitude of the VOIP engine will be displayed on screen.\n")
	TEXT("0: disabled, 1: enabled."),
	ECVF_Default);

struct FVoiceCaptureWindowsVars
{
	/** GUID of current voice capture device */
	GUID VoiceCaptureDeviceGuid;
	/** Voice capture device */
	LPDIRECTSOUNDCAPTURE8 VoiceCaptureDev;
	/** Voice capture device caps */
	DSCCAPS VoiceCaptureDevCaps;
	/** Voice capture buffer */
	LPDIRECTSOUNDCAPTUREBUFFER8 VoiceCaptureBuffer8;
	/** Wave format of buffer */
	WAVEFORMATEX WavFormat;
	/** Buffer description */
	DSCBUFFERDESC VoiceCaptureBufferDesc;
	/** Buffer caps */
	DSCBCAPS VoiceCaptureBufferCaps8;
	/** Notification events */
	HANDLE StopEvent;
	/** Current audio position of valid data in capture buffer */
	DWORD NextCaptureOffset;

	FVoiceCaptureWindowsVars() :
		VoiceCaptureDev(nullptr),
		VoiceCaptureBuffer8(nullptr),
		NextCaptureOffset(0)
	{
		StopEvent = INVALID_HANDLE_VALUE;
		Reset();
	}

	void Reset()
	{
		if (StopEvent != INVALID_HANDLE_VALUE)
		{
			CloseHandle(StopEvent);
			StopEvent = INVALID_HANDLE_VALUE;
		}

		// Free up DirectSound resources
		if (VoiceCaptureBuffer8)
		{
			VoiceCaptureBuffer8->Release();
			VoiceCaptureBuffer8 = nullptr;
		}

		if (VoiceCaptureDev)
		{
			VoiceCaptureDev->Release();
			VoiceCaptureDev = nullptr;
		}

		NextCaptureOffset = 0;

		FMemory::Memzero(&VoiceCaptureDeviceGuid, sizeof(GUID));
		FMemory::Memzero(&VoiceCaptureDevCaps, sizeof(VoiceCaptureDevCaps));

		FMemory::Memzero(&WavFormat, sizeof(WavFormat));
		FMemory::Memzero(&VoiceCaptureBufferDesc, sizeof(VoiceCaptureBufferDesc));
		FMemory::Memzero(&VoiceCaptureBufferCaps8, sizeof(VoiceCaptureBufferCaps8));
	}
};

FVoiceCaptureWindows::FVoiceCaptureWindows() :
	CV(nullptr),
	LastCaptureTime(0.0),
	VoiceCaptureState(EVoiceCaptureState::UnInitialized)
{
	CV = new FVoiceCaptureWindowsVars();
}

FVoiceCaptureWindows::~FVoiceCaptureWindows()
{
	Shutdown();

	FVoiceCaptureDeviceWindows* VoiceCaptureDev = FVoiceCaptureDeviceWindows::Get();
 	if (VoiceCaptureDev)	
	{
 		VoiceCaptureDev->FreeVoiceCaptureObject(this);
 	}
	
	if (CV)
	{
		delete CV;
		CV = nullptr;
	}	
}

bool FVoiceCaptureWindows::Init(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	FVoiceCaptureDeviceWindows* VoiceDev = FVoiceCaptureDeviceWindows::Get();
	if (!VoiceDev)
	{
		UE_LOGF(LogVoiceCapture, Warning, "No voice capture interface.");
		return false;
	}
	
   	// init the sample counter to 0 on init
	SampleCounter = 0; 
	CachedSampleStart = 0;

	// set up level detector
	static IConsoleVariable* SilenceDetectionAttackCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.SilenceDetectionAttackTime"));
	check(SilenceDetectionAttackCVar);			
	static IConsoleVariable* SilenceDetectionReleaseCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.SilenceDetectionReleaseTime"));
	check(SilenceDetectionReleaseCVar);				

	Audio::FInlineEnvelopeFollowerInitParams EnvelopeFollowerInitParams;
	EnvelopeFollowerInitParams.SampleRate = SampleRate;
	EnvelopeFollowerInitParams.AttackTimeMsec = SilenceDetectionAttackCVar->GetFloat();
	EnvelopeFollowerInitParams.ReleaseTimeMsec = SilenceDetectionReleaseCVar->GetFloat();
	EnvelopeFollowerInitParams.Mode = Audio::EPeakMode::Peak;
	EnvelopeFollowerInitParams.bIsAnalog = MicSilenceDetectionConfig::IsAnalog;

	MicLevelDetector.Init(EnvelopeFollowerInitParams);
	
	const int32 AttackInSamples = SampleRate * SilenceDetectionAttackCVar->GetFloat() * 0.001f;
	LookaheadBuffer.Init(AttackInSamples + 1);
	LookaheadBuffer.SetDelay(AttackInSamples);

	NoiseGateAttenuator.Init(SampleRate);

	bIsMicActive = false;
	bWasMicAboveNoiseGateThreshold = false;

	return CreateCaptureBuffer(DeviceName.IsEmpty() ? VoiceDev->DefaultVoiceCaptureDevice.DeviceName : DeviceName, SampleRate, NumChannels);
}

bool FVoiceCaptureWindows::CreateCaptureBuffer(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	// Free the previous buffer
	FreeCaptureBuffer();
	VoiceCaptureState = EVoiceCaptureState::NotCapturing;

	if (SampleRate < 8000 || SampleRate > 48000)
	{
		UE_LOGF(LogVoiceCapture, Warning, "Voice capture doesn't support %d hz", SampleRate);
		return false;
	}

	if (NumChannels < 0 || NumChannels > 2)
	{
		UE_LOGF(LogVoiceCapture, Warning, "Voice capture only supports 1 or 2 channels");
		return false;
	}

	FVoiceCaptureDeviceWindows* VoiceDev = FVoiceCaptureDeviceWindows::Get();
	if (!VoiceDev)
	{
		UE_LOGF(LogVoiceCapture, Warning, "No voice capture interface.");
		return false;
	}

	FVoiceCaptureDeviceWindows::FCaptureDeviceInfo* DeviceInfo = nullptr;
	if (DeviceName.IsEmpty())
	{
		DeviceInfo = &VoiceDev->DefaultVoiceCaptureDevice;
	}
	else
	{
		DeviceInfo = VoiceDev->Devices.Find(DeviceName);
	}

	if (DeviceInfo)
	{
		UE_LOGF(LogVoiceCapture, Display, "Creating capture %ls [%d:%d]", *DeviceInfo->DeviceName, SampleRate, NumChannels);
		CV->VoiceCaptureDeviceGuid = DeviceInfo->DeviceId;

		// DSDEVID_DefaultCapture WAVEINCAPS 
		HRESULT hr = DirectSoundCaptureCreate8(&DeviceInfo->DeviceId, &CV->VoiceCaptureDev, nullptr);
		if (FAILED(hr))
		{
			//DSERR_ALLOCATED, DSERR_INVALIDPARAM, DSERR_NOAGGREGATION, DSERR_OUTOFMEMORY
			UE_LOGF(LogVoiceCapture, Warning, "Failed to create capture device 0x%08x", hr);
			return false;
		}

		// Device capabilities
		CV->VoiceCaptureDevCaps.dwSize = sizeof(DSCCAPS);
		hr = CV->VoiceCaptureDev->GetCaps(&CV->VoiceCaptureDevCaps);
		if (FAILED(hr))
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to get mic device caps 0x%08x", hr);
			return false;
		}

		// Wave format setup
		CV->WavFormat.wFormatTag = WAVE_FORMAT_PCM;
		CV->WavFormat.nChannels = NumChannels;
		CV->WavFormat.wBitsPerSample = 16;
		CV->WavFormat.nSamplesPerSec = SampleRate;
		CV->WavFormat.nBlockAlign = (CV->WavFormat.nChannels * CV->WavFormat.wBitsPerSample) / 8;
		CV->WavFormat.nAvgBytesPerSec = CV->WavFormat.nBlockAlign * CV->WavFormat.nSamplesPerSec;
		CV->WavFormat.cbSize = 0;

		// Buffer setup
		CV->VoiceCaptureBufferDesc.dwSize = sizeof(DSCBUFFERDESC);
		CV->VoiceCaptureBufferDesc.dwFlags = 0;
		CV->VoiceCaptureBufferDesc.dwBufferBytes = CV->WavFormat.nAvgBytesPerSec / 2; // 0.5 sec buffer
		CV->VoiceCaptureBufferDesc.dwReserved = 0;
		CV->VoiceCaptureBufferDesc.lpwfxFormat = &CV->WavFormat;
		CV->VoiceCaptureBufferDesc.dwFXCount = 0;
		CV->VoiceCaptureBufferDesc.lpDSCFXDesc = nullptr;

		LPDIRECTSOUNDCAPTUREBUFFER VoiceBuffer = nullptr;

		hr = CV->VoiceCaptureDev->CreateCaptureBuffer(&CV->VoiceCaptureBufferDesc, &VoiceBuffer, nullptr);
		if (FAILED(hr))
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to create voice capture buffer 0x%08x", hr);
			return false;
		}

		hr = VoiceBuffer->QueryInterface(IID_IDirectSoundCaptureBuffer8, (LPVOID*)&CV->VoiceCaptureBuffer8);
		VoiceBuffer->Release();
		VoiceBuffer = nullptr;
		if (FAILED(hr))
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to create voice capture buffer 0x%08x", hr);
			return false;
		}

		CV->VoiceCaptureBufferCaps8.dwSize = sizeof(DSCBCAPS);
		hr = CV->VoiceCaptureBuffer8->GetCaps(&CV->VoiceCaptureBufferCaps8);
		if (FAILED(hr))
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to get voice buffer caps 0x%08x", hr);
			return false;
		}

		// TEST ------------------------
		#if (0)
		{
			DWORD SizeWritten8 = 0;
			CV->VoiceCaptureBuffer8->GetFormat(nullptr, sizeof(WAVEFORMATEX), &SizeWritten8);

			LPWAVEFORMATEX BufferFormat8 = (WAVEFORMATEX*)FMemory::Malloc(SizeWritten8);
			CV->VoiceCaptureBuffer8->GetFormat(BufferFormat8, SizeWritten8, &SizeWritten8);
			FMemory::Free(BufferFormat8);
		}
		#endif
		// TEST ------------------------

		if (CreateNotifications(CV->VoiceCaptureBufferCaps8.dwBufferBytes))
		{
			// Reset notification related values
			LastCaptureTime = FPlatformTime::Seconds();
		}
		else
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to create voice buffer notifications");
			return false;
		}
	}
	else
	{
		UE_LOGF(LogVoiceCapture, Warning, "No voice capture device %ls found.", *DeviceName);
		return false;
	}

	UncompressedAudioBuffer.Init(0, CV->VoiceCaptureBufferDesc.dwBufferBytes);
	check(UncompressedAudioBuffer.Max() >= (int32)CV->VoiceCaptureBufferCaps8.dwBufferBytes);

	NumInputChannels = CV->WavFormat.nChannels;

	ReleaseBuffer.Init((int32)CV->VoiceCaptureBufferCaps8.dwBufferBytes);
	ReleaseBuffer.SetDelay(1);
	return true;
}

void FVoiceCaptureWindows::FreeCaptureBuffer()
{
	// Stop playback
	Stop();

	// Release all D3D8 resources
	CV->Reset();

	VoiceCaptureState = EVoiceCaptureState::UnInitialized;
}

void FVoiceCaptureWindows::Shutdown()
{
	FreeCaptureBuffer();
}

bool FVoiceCaptureWindows::Start()
{
	check(VoiceCaptureState != EVoiceCaptureState::UnInitialized);

	if (CV->VoiceCaptureBuffer8 == nullptr)
	{
		UE_LOGF(LogVoiceCapture, Warning, "CV->VoiceCaptureBuffer8 == nullptr");
		return false;
	}

	HRESULT hr = CV->VoiceCaptureBuffer8->Start(DSCBSTART_LOOPING);
	if (FAILED(hr))
	{
		UE_LOGF(LogVoiceCapture, Warning, "Failed to start capture 0x%08x", hr);
		return false;
	}

	VoiceCaptureState = EVoiceCaptureState::NoData;
	return true;
}

void FVoiceCaptureWindows::Stop()
{
	if (CV->VoiceCaptureBuffer8 &&
		VoiceCaptureState != EVoiceCaptureState::Stopping && 
		VoiceCaptureState != EVoiceCaptureState::NotCapturing)
	{
		CV->VoiceCaptureBuffer8->Stop();
		VoiceCaptureState = EVoiceCaptureState::Stopping;
	}
}

bool FVoiceCaptureWindows::ChangeDevice(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	if (VoiceCaptureState != EVoiceCaptureState::UnInitialized)
	{
		return CreateCaptureBuffer(DeviceName, SampleRate, NumChannels);
	}
	else
	{
		UE_LOGF(LogVoiceCapture, Warning, "Unable to change device, not initialized");
		return false;
	}
}

bool FVoiceCaptureWindows::IsCapturing()
{		
	if (CV->VoiceCaptureBuffer8)
	{
		DWORD Status = 0;

		HRESULT hr = CV->VoiceCaptureBuffer8->GetStatus(&Status);
		if (FAILED(hr))
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to get voice buffer status 0x%08x", hr);
		}

		// Status & DSCBSTATUS_LOOPING
		return Status & DSCBSTATUS_CAPTURING ? true : false;
	}

	return false;
}

EVoiceCaptureState::Type FVoiceCaptureWindows::GetCaptureState(uint32& OutAvailableVoiceData) const
{
	if (VoiceCaptureState != EVoiceCaptureState::UnInitialized &&
		VoiceCaptureState != EVoiceCaptureState::Error)
	{
		OutAvailableVoiceData = UncompressedAudioBuffer.Num();
	}
	else
	{
		OutAvailableVoiceData = 0;
	}

	return VoiceCaptureState;
}

void FVoiceCaptureWindows::ProcessData()
{
	DWORD CurrentCapturePos = 0;
	DWORD CurrentReadPos = 0;

	HRESULT hr = CV->VoiceCaptureBuffer8 ? CV->VoiceCaptureBuffer8->GetCurrentPosition(&CurrentCapturePos, &CurrentReadPos) : E_FAIL;
	if (FAILED(hr))
	{
		UE_LOGF(LogVoiceCapture, Warning, "Failed to get voice buffer cursor position 0x%08x", hr);
		VoiceCaptureState = EVoiceCaptureState::Error;
		return;
	}

	DWORD LockSize = ((CurrentReadPos - CV->NextCaptureOffset) + CV->VoiceCaptureBufferCaps8.dwBufferBytes) % CV->VoiceCaptureBufferCaps8.dwBufferBytes;
	if(LockSize != 0) 
	{ 
		//UE_LOGF( LogVoiceCapture, Log, "LockSize: %i, CurrentCapturePos: %i, CurrentReadPos: %i, NextCaptureOffset: %i", LockSize, CurrentCapturePos, CurrentReadPos, CV->NextCaptureOffset );

		DWORD CaptureFlags = 0;
		DWORD CaptureLength = 0;
		void* CaptureData = nullptr;
		DWORD CaptureLength2 = 0;
		void* CaptureData2 = nullptr;
		hr = CV->VoiceCaptureBuffer8->Lock(CV->NextCaptureOffset, LockSize,
			&CaptureData, &CaptureLength, 
			&CaptureData2, &CaptureLength2, CaptureFlags);
		if (SUCCEEDED(hr))
		{
			const DWORD OriginalCaptureLength = CaptureLength;
			const DWORD OriginalCaptureLength2 = CaptureLength2;

			if (UncompressedAudioBuffer.Num() + CaptureLength + CaptureLength2 > (DWORD)UncompressedAudioBuffer.Max())
			{ 
				UE_LOGF(LogVoiceCapture, Warning, "Resetting UncompressedAudioBuffer.");
				UncompressedAudioBuffer.Empty(UncompressedAudioBuffer.Max());
				VoiceCaptureState = EVoiceCaptureState::NoData;
			}

			const int32 Offset = UncompressedAudioBuffer.Num();

			CaptureLength = FMath::Min(CaptureLength, (DWORD)UncompressedAudioBuffer.Max());
			CaptureLength2 = FMath::Min(CaptureLength2, (DWORD)UncompressedAudioBuffer.Max() - CaptureLength);
			

			UncompressedAudioBuffer.AddUninitialized(CaptureLength + CaptureLength2 + (ReleaseBuffer.GetBufferCount() * sizeof(int16)));
			

			int16* AudioBuffer = (int16*)(UncompressedAudioBuffer.GetData() + Offset);
			int16* InputBuffer = (int16*)CaptureData;
			
			//First, if we have any cached audio from an onset mid-buffer, copy it in:
			int32 SamplesPushedToUncompressedAudioBuffer = ReleaseBuffer.PopBufferedAudio(AudioBuffer, ReleaseBuffer.GetBufferCount());
			AudioBuffer += SamplesPushedToUncompressedAudioBuffer;

			static IConsoleVariable* SilenceDetectionThresholdCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.SilenceDetectionThreshold"));
			check(SilenceDetectionThresholdCVar);			
			const float MicSilenceThreshold = SilenceDetectionThresholdCVar->GetFloat();

			static IConsoleVariable* NoiseGateThresholdCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.MicNoiseGateThreshold"));
			check(NoiseGateThresholdCVar);
			const float MicNoiseGateThreshold = NoiseGateThresholdCVar->GetFloat();
			
			static IConsoleVariable* NoiseGateAttackTimeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.MicNoiseAttackTime"));
			check(NoiseGateAttackTimeCVar);
			const float MicNoiseGateAttackTime = NoiseGateAttackTimeCVar->GetFloat();

			static IConsoleVariable* NoiseGateReleaseTimeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.MicNoiseReleaseTime"));
			check(NoiseGateReleaseTimeCVar);
			const float MicNoiseGateReleaseTime = NoiseGateReleaseTimeCVar->GetFloat();

			static IConsoleVariable* MicInputGainCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.MicInputGain"));
			check(MicInputGainCVar);
			const float MicInputGain = MicInputGainCVar->GetFloat();

			static IConsoleVariable* MicStereoBiasCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("voice.MicStereoBias"));
			check(MicStereoBiasCVar);
			const float MicStereoBias = FMath::Clamp(MicStereoBiasCVar->GetFloat(), -1.0f, 1.0f);

			float LeftGain = 1.0f;
			float RightGain = 1.0f;

			Audio::GetStereoPan(MicStereoBias, LeftGain, RightGain);

			// Since we don't interpolate the pan here, we normalize stereo gains.
			const float StereoGainMax = FMath::Max(LeftGain, RightGain);
			LeftGain /= StereoGainMax;
			RightGain /= StereoGainMax;

			bool bMicReleased = false;

			CurrentSampleStart = CachedSampleStart;
			const int32 TotalNumFrames = CaptureLength / sizeof(int16);

			//Begin looping through the first buffer:
			for (int32 FrameIndex = 0; FrameIndex < TotalNumFrames; FrameIndex += NumInputChannels)
			{
				int16 Temp = 0;

				for (int32 ChannelIndex = 0; ChannelIndex < NumInputChannels; ChannelIndex++)
				{
					Temp += InputBuffer[FrameIndex + ChannelIndex];
				}

				
				float Envelope = MicLevelDetector.ProcessSample(Temp / 32768.f);
				LookaheadBuffer.ProcessSample(Temp, Temp);

				bIsMicActive = Envelope > MicSilenceThreshold;

				// If we have just crossed the noise gate threshold, begin interpoloating to 1.0 or 0.0
				const bool bIsMicAboveNoiseGateThreshold = Envelope > MicNoiseGateThreshold;

				if (bIsMicAboveNoiseGateThreshold && !bWasMicAboveNoiseGateThreshold)
				{
					NoiseGateAttenuator.SetValue(1.0f, MicNoiseGateAttackTime);
				}
				else if (!bIsMicAboveNoiseGateThreshold && bWasMicAboveNoiseGateThreshold)
				{
					NoiseGateAttenuator.SetValue(0.0f, MicNoiseGateReleaseTime);
				}

				bWasMicAboveNoiseGateThreshold = bIsMicAboveNoiseGateThreshold;

				if (bIsMicActive)
				{
					if (bMicReleased)
					{
						// Apply noise gate attenuation.
						const float TotalMicGain = MicInputGain * NoiseGateAttenuator.GetNextValue();
						for (int32 ChannelIndex = 0; ChannelIndex < NumInputChannels; ChannelIndex++)
						{
							const float BiasedMicGain = (ChannelIndex % 2 == 0) ? (TotalMicGain * LeftGain) : (TotalMicGain * RightGain);

							Audio::TSampleRef<int16> SampleRef(InputBuffer[FrameIndex + ChannelIndex]);
							SampleRef = SampleRef * BiasedMicGain;
						}

						ReleaseBuffer.PushFrame(&InputBuffer[FrameIndex], NumInputChannels);


						if (!bSampleStartCached)
						{
							CachedSampleStart = SampleCounter;
							bSampleStartCached = true;
						}
					}
					else
					{
						// Apply noise gate attenuation.
						const float TotalMicGain = MicInputGain * NoiseGateAttenuator.GetNextValue();
						for (int32 ChannelIndex = 0; ChannelIndex < NumInputChannels; ChannelIndex++)
						{
							const float BiasedMicGain = (ChannelIndex % 2 == 0) ? (TotalMicGain * LeftGain) : (TotalMicGain * RightGain);

							Audio::TSampleRef<int16> SampleRef(InputBuffer[FrameIndex + ChannelIndex]);
							SampleRef = SampleRef * BiasedMicGain;
						}

						FMemory::Memcpy(&AudioBuffer[FrameIndex], &InputBuffer[FrameIndex], sizeof(int16) * NumInputChannels);
						SamplesPushedToUncompressedAudioBuffer += NumInputChannels;
					}
				}
				else
				{
					bMicReleased = true;
				}
				SampleCounter++;
			}

			//Set up second buffer and loop through that:
			AudioBuffer += TotalNumFrames;
			InputBuffer = (int16*)CaptureData2;
			const int32 TotalNumFrames2 = CaptureLength2 / sizeof(int16);
			for (int32 FrameIndex = 0; FrameIndex < TotalNumFrames2; FrameIndex += NumInputChannels)
			{
				int16 Temp = 0;
				
				for (int32 ChannelIndex = 0; ChannelIndex < NumInputChannels; ChannelIndex++)
				{
					CA_SUPPRESS(6385);
					Temp += InputBuffer[FrameIndex + ChannelIndex];
				}
				
				
				float Envelope = MicLevelDetector.ProcessSample(static_cast<float>(Temp) / 32768.f);
				LookaheadBuffer.ProcessSample(Temp, Temp);

				bIsMicActive = Envelope > MicSilenceThreshold;

				// If we have just crossed the noise gate threshold, begin interpoloating to 1.0 or 0.0
				const bool bIsMicAboveNoiseGateThreshold = Envelope > MicNoiseGateThreshold;

				if (bIsMicAboveNoiseGateThreshold && !bWasMicAboveNoiseGateThreshold)
				{
					NoiseGateAttenuator.SetValue(1.0f, MicNoiseGateAttackTime);
				}
				else if (!bIsMicAboveNoiseGateThreshold && bWasMicAboveNoiseGateThreshold)
				{
					NoiseGateAttenuator.SetValue(0.0f, MicNoiseGateReleaseTime);
				}

				bWasMicAboveNoiseGateThreshold = bIsMicAboveNoiseGateThreshold;

				if (bIsMicActive)
				{
					if (bMicReleased)
					{
						// Apply noise gate attenuation.
						const float TotalMicGain = MicInputGain * NoiseGateAttenuator.GetNextValue();
						for (int32 ChannelIndex = 0; ChannelIndex < NumInputChannels; ChannelIndex++)
						{
							const float BiasedMicGain = (ChannelIndex % 2 == 0) ? (TotalMicGain * LeftGain) : (TotalMicGain * RightGain);

							Audio::TSampleRef<int16> SampleRef(InputBuffer[FrameIndex + ChannelIndex]);
							SampleRef = SampleRef * BiasedMicGain;
						}

						ReleaseBuffer.PushFrame(&InputBuffer[FrameIndex], NumInputChannels);

						if (!bSampleStartCached)
						{
							CachedSampleStart = SampleCounter;
							bSampleStartCached = true;
						}
					}
					else
					{
						// Apply noise gate attenuation.
						const float TotalMicGain = MicInputGain * NoiseGateAttenuator.GetNextValue();
						for (int32 ChannelIndex = 0; ChannelIndex < NumInputChannels; ChannelIndex++)
						{
							const float BiasedMicGain = (ChannelIndex % 2 == 0) ? (TotalMicGain * LeftGain) : (TotalMicGain * RightGain);

							Audio::TSampleRef<int16> SampleRef(InputBuffer[FrameIndex + ChannelIndex]);
							SampleRef = SampleRef * BiasedMicGain;
						}

						FMemory::Memcpy(&AudioBuffer[FrameIndex], &InputBuffer[FrameIndex], sizeof(int16) * NumInputChannels);
						SamplesPushedToUncompressedAudioBuffer += NumInputChannels;
					}
				}
				else
				{
					bMicReleased = true;
				}
				SampleCounter++;
			}

			if (!bSampleStartCached)
			{
				CachedSampleStart = SampleCounter;
			}

			bSampleStartCached = false;

			UncompressedAudioBuffer.SetNum(Offset + (SamplesPushedToUncompressedAudioBuffer * sizeof(int16)), EAllowShrinking::No);

			CA_SUPPRESS(6385);
			CV->VoiceCaptureBuffer8->Unlock(CaptureData, OriginalCaptureLength, CaptureData2, OriginalCaptureLength2);

			// Move the capture offset forward.
			CV->NextCaptureOffset = (CV->NextCaptureOffset + CaptureLength) % CV->VoiceCaptureBufferCaps8.dwBufferBytes;
			CV->NextCaptureOffset = (CV->NextCaptureOffset + CaptureLength2) % CV->VoiceCaptureBufferCaps8.dwBufferBytes;

			
			if (SamplesPushedToUncompressedAudioBuffer > 0)
			{
				VoiceCaptureState = EVoiceCaptureState::Ok;
			}
			else
			{
				VoiceCaptureState = EVoiceCaptureState::NoData;
			}

#if !UE_BUILD_SHIPPING
			// TODO: look at actually using something like this for time stamping
			const double NewTime = FPlatformTime::Seconds();
			UE_LOGF(LogVoiceCapture, VeryVerbose, "LastCapture: %f %ls", (NewTime - LastCaptureTime) * 1000.0, EVoiceCaptureState::ToString(VoiceCaptureState));
			LastCaptureTime = NewTime;
#endif
		}
		else
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to lock voice buffer 0x%08x", hr);
			VoiceCaptureState = EVoiceCaptureState::Error;
		}
	}
}

EVoiceCaptureState::Type FVoiceCaptureWindows::GetVoiceData(uint8* OutVoiceBuffer, const uint32 InVoiceBufferSize, uint32& OutBytesWritten, uint64& OutSampleClockCounter)
{
	EVoiceCaptureState::Type NewMicState = VoiceCaptureState;
	OutBytesWritten = 0;

	if (VoiceCaptureState == EVoiceCaptureState::Ok ||
		VoiceCaptureState == EVoiceCaptureState::Stopping)
	{

		if (InVoiceBufferSize >= (uint32) UncompressedAudioBuffer.Num())
		{
			OutBytesWritten = UncompressedAudioBuffer.Num();
			FMemory::Memcpy(OutVoiceBuffer, UncompressedAudioBuffer.GetData(), OutBytesWritten);
			VoiceCaptureState = EVoiceCaptureState::NoData;
			UncompressedAudioBuffer.Reset();

			OutSampleClockCounter = CurrentSampleStart;
		}
		else
		{
			NewMicState = EVoiceCaptureState::BufferTooSmall;
		}
	}

	// If we have any sends for this microphones output, push to them here.
	if (MicrophoneOutput.Num() > 0)
	{
		// Convert our buffer from int16 to float:
		int16* OutputData = reinterpret_cast<int16*>(OutVoiceBuffer);
		uint32 NumSamples = OutBytesWritten / sizeof(int16);
		ConversionBuffer.Reset();
		// Note: Sample rate is unused for this operation.
		ConversionBuffer.Append(OutputData, NumSamples, NumInputChannels, 16000);
		
		if (NumInputChannels > 1)
		{
			// For consistency, mixdown to mono.
			ConversionBuffer.MixBufferToChannels(1);
		}

		MicrophoneOutput.PushAudio(ConversionBuffer.GetData(), ConversionBuffer.GetNumSamples());
	}

	// print debug string with current amplitude:
	if (DisplayAmplitudeCvar && GEngine)
	{
		static double TimeLastPrinted = FPlatformTime::Seconds();

		static const double AmplitudeStringDisplayRate = 0.05;
		static const int32 TotalNumTicks = 32;

		if (FPlatformTime::Seconds() - TimeLastPrinted > AmplitudeStringDisplayRate)
		{
			const float MicLevel = MicLevelDetector.GetValue();
			FString PrintString = FString::Printf(TEXT("Mic Amp: %.2f"), MicLevel);

			int32 NumTicks = FMath::FloorToInt(MicLevel * TotalNumTicks);

			for (int32 Iteration = 0; Iteration < NumTicks; Iteration++)
			{
				PrintString.AppendChar(TCHAR('|'));
			}

			FColor TextColor = FLinearColor::LerpUsingHSV(FLinearColor::Green, FLinearColor::Red, MicLevel).ToFColor(true);

			GEngine->AddOnScreenDebugMessage(30, static_cast<float>(AmplitudeStringDisplayRate), TextColor, PrintString, false);
			TimeLastPrinted = FPlatformTime::Seconds();
		}
	}

	return NewMicState;
}

EVoiceCaptureState::Type FVoiceCaptureWindows::GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData)
{
	uint64 UnusedSampleCounter = 0;
	return GetVoiceData(OutVoiceBuffer, InVoiceBufferSize, OutAvailableVoiceData, UnusedSampleCounter);
}

int32 FVoiceCaptureWindows::GetBufferSize() const
{
	if (VoiceCaptureState != EVoiceCaptureState::UnInitialized)
	{
		return CV->VoiceCaptureBufferCaps8.dwBufferBytes;
	}

	return 0;
}

bool FVoiceCaptureWindows::CreateNotifications(uint32 BufferSize)
{
	bool bSuccess = false;

	LPDIRECTSOUNDNOTIFY8 NotifyInt = nullptr;

	HRESULT hr = CV->VoiceCaptureBuffer8->QueryInterface(IID_IDirectSoundNotify, (LPVOID*)&NotifyInt);
	if (SUCCEEDED(hr))
	{
		// Create stop event
		CV->StopEvent = CreateEvent(nullptr, true, false, nullptr);
		if (CV->StopEvent == NULL || CV->StopEvent == INVALID_HANDLE_VALUE)
		{
			UE_LOGF(LogVoiceCapture, Warning, "Error creating stop event");
		}
		else
		{
			DSBPOSITIONNOTIFY StopEvent;

			// when buffer stops
			StopEvent.dwOffset = DSBPN_OFFSETSTOP;
			StopEvent.hEventNotify = CV->StopEvent;

			hr = NotifyInt->SetNotificationPositions(1, &StopEvent);
			if (SUCCEEDED(hr))
			{
				bSuccess = true;
			}
			else
			{
				UE_LOGF(LogVoiceCapture, Warning, "Failed to set stop notifications 0x%08x", hr);

				CloseHandle(CV->StopEvent);
				CV->StopEvent = INVALID_HANDLE_VALUE;
			}
		}

		NotifyInt->Release();
	}
	else
	{
		UE_LOGF(LogVoiceCapture, Warning, "Failed to create voice notification interface 0x%08x", hr);
	}

	return bSuccess;
}

bool FVoiceCaptureWindows::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FVoiceCaptureWindows_Tick);

	if (VoiceCaptureState != EVoiceCaptureState::UnInitialized &&
		VoiceCaptureState != EVoiceCaptureState::NotCapturing)
	{
		ProcessData();

		if (CV->StopEvent != INVALID_HANDLE_VALUE && WaitForSingleObject(CV->StopEvent, 0) == WAIT_OBJECT_0)
		{
			UE_LOGF(LogVoiceCapture, Verbose, "Voice capture stopped");
			ResetEvent(CV->StopEvent);
			VoiceCaptureState = EVoiceCaptureState::NotCapturing;
			UncompressedAudioBuffer.Empty(UncompressedAudioBuffer.Max());
		}
	}

	return true;
}

float FVoiceCaptureWindows::GetCurrentAmplitude() const
{
	return MicLevelDetector.GetValue();
}

void FVoiceCaptureWindows::DumpState() const
{
#if !NO_LOGGING
	if (CV)
	{
		extern FString PrintMSGUID(LPGUID Guid);
		UE_LOGF(LogVoiceCapture, Display, "Device %ls", *PrintMSGUID(&CV->VoiceCaptureDeviceGuid));

		UE_LOG(LogVoiceCapture, Display, TEXT("CaptureDev: 0x%08" UPTRINT_x_FMT), CV->VoiceCaptureDev);
		UE_LOG(LogVoiceCapture, Display, TEXT("CaptureBuffer: 0x%08" UPTRINT_x_FMT), CV->VoiceCaptureBuffer8);

		UE_LOGF(LogVoiceCapture, Display, "Capture Format");
		UE_LOGF(LogVoiceCapture, Display, "- Tag: %d", CV->WavFormat.wFormatTag);
		UE_LOGF(LogVoiceCapture, Display, "- Channels: %d", CV->WavFormat.nChannels);
		UE_LOGF(LogVoiceCapture, Display, "- BitsPerSample: %d", CV->WavFormat.wBitsPerSample);
		UE_LOGF(LogVoiceCapture, Display, "- SamplesPerSec: %d", CV->WavFormat.nSamplesPerSec);
		UE_LOGF(LogVoiceCapture, Display, "- BlockAlign: %d", CV->WavFormat.nBlockAlign);
		UE_LOGF(LogVoiceCapture, Display, "- AvgBytesPerSec: %d", CV->WavFormat.nAvgBytesPerSec);

		UE_LOGF(LogVoiceCapture, Display, "Capture Buffer");
		UE_LOGF(LogVoiceCapture, Display, "- Flags: 0x%08x", CV->VoiceCaptureBufferDesc.dwFlags);
		UE_LOGF(LogVoiceCapture, Display, "- BufferBytes: %d", CV->VoiceCaptureBufferDesc.dwBufferBytes);
		UE_LOG(LogVoiceCapture, Display, TEXT("- Format: 0x%08" UPTRINT_x_FMT), CV->VoiceCaptureBufferDesc.lpwfxFormat);

		UE_LOGF(LogVoiceCapture, Display, "Device Caps");
		UE_LOGF(LogVoiceCapture, Display, "- Size: %d", CV->VoiceCaptureDevCaps.dwSize);
		UE_LOGF(LogVoiceCapture, Display, "- Flags: 0x%08x", CV->VoiceCaptureDevCaps.dwFlags);
		UE_LOGF(LogVoiceCapture, Display, "- Formats: %d", CV->VoiceCaptureDevCaps.dwFormats);
		UE_LOGF(LogVoiceCapture, Display, "- Channels: %d", CV->VoiceCaptureDevCaps.dwChannels);

		UE_LOGF(LogVoiceCapture, Display, "D3D8 Caps");
		UE_LOGF(LogVoiceCapture, Display, "- Size: %d", CV->VoiceCaptureBufferCaps8.dwSize);
		UE_LOGF(LogVoiceCapture, Display, "- Flags: 0x%08x", CV->VoiceCaptureBufferCaps8.dwFlags);
		UE_LOGF(LogVoiceCapture, Display, "- BufferBytes: %d", CV->VoiceCaptureBufferCaps8.dwBufferBytes);
	}
	else
	{
		UE_LOGF(LogVoiceCapture, Display, "No capture device to dump state");
	}
#endif // !NO_LOGGING
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // PLATFORM_SUPPORTS_VOICE_CAPTURE
