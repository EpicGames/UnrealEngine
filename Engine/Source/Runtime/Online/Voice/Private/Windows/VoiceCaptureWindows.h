// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/VoiceCapture.h"
#include "VoicePackage.h"
#include "VoiceDelayBuffer.h"
#include "DSP/EnvelopeFollower.h"
#include "Containers/Ticker.h"
#include "Windows/WindowsHWrapper.h"
#include "SampleBuffer.h"

typedef struct IDirectSound8 *LPDIRECTSOUND8;

class IVoiceCapture;
class FVoiceCaptureWindows;
struct FVoiceCaptureWindowsVars;

/**
 * DirectSound wrapper for initialization / shutdown
 */ 
class FVoiceCaptureDeviceWindows
{
private:

    /** One instance of the direct sound API */
	static FVoiceCaptureDeviceWindows* Singleton;
	/** All outstanding voice capture objects */
	TArray<IVoiceCapture*> ActiveVoiceCaptures;
	/** Is DirectSound setup correctly */
	bool bInitialized;

    /** Initialize DirectSound and the voice capture device */
	bool Init();
    /** Shutdown DirectSound and the voice capture device */
	void Shutdown();

PACKAGE_SCOPE:

	/**
	 * Create the device singleton 
	 *
	 * @return singleton capable of providing voice capture features
	 */
	static FVoiceCaptureDeviceWindows* Create();
	/**
	 * Destroy the device singleton
	 */
	static void Destroy();
	/**
	 * Free a voice capture object created by CreateVoiceCaptureObject()
	 *
	 * @param VoiceCaptureObj voice capture object to free
	 */
	void FreeVoiceCaptureObject(IVoiceCapture* VoiceCaptureObj);

public:

	struct FCaptureDeviceInfo
	{
		/** Enumerated capture device name */
		FString DeviceName;
		/** Enumerated capture device guid */
		GUID DeviceId;
		/** If this is the default device. */
		bool bIsDefault;
	};

	/** DirectSound8 Interface */
	LPDIRECTSOUND8 DirectSound;

	/** HMD audio input device to use */
	FString HMDAudioInputDevice;
	/** GUID of selected voice capture device */
	FCaptureDeviceInfo DefaultVoiceCaptureDevice;

	/** All available capture devices */
	TMap<FString, FCaptureDeviceInfo> Devices;

	/** Constructors */
	FVoiceCaptureDeviceWindows();
	~FVoiceCaptureDeviceWindows();

	/**
	 * Create a single voice capture buffer
	 *
	 * @param DeviceName name of device to capture with, empty for default device
	 * @param SampleRate valid sample rate to capture audio data (8khz - 48khz)
	 * @param NumChannel number of audio channels (1 - mono, 2 - stereo)
	 *
	 * @return class capable of recording voice on a given device
	 */
	FVoiceCaptureWindows* CreateVoiceCaptureObject(const FString& DeviceName, int32 SampleRate, int32 NumChannels);

	/**
	 * Singleton accessor
	 * 
	 * @return access to the voice capture device
	 */
	static FVoiceCaptureDeviceWindows* Get();
};

/**
 * Windows implementation of voice capture using DirectSound
 */
class FVoiceCaptureWindows : public IVoiceCapture, public FTSTickerObjectBase
{
public:

	FVoiceCaptureWindows();
	~FVoiceCaptureWindows();

	// IVoiceCapture
	virtual bool Init(const FString& DeviceName, int32 SampleRate, int32 NumChannels) override;
	virtual void Shutdown() override;
	virtual bool Start() override;
	virtual void Stop() override;
	virtual bool ChangeDevice(const FString& DeviceName, int32 SampleRate, int32 NumChannels) override;
	virtual bool IsCapturing() override;
	virtual EVoiceCaptureState::Type GetCaptureState(uint32& OutAvailableVoiceData) const override;
	virtual EVoiceCaptureState::Type GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData) override;
	virtual EVoiceCaptureState::Type GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData, uint64& OutSampleCounter) override;
	virtual int32 GetBufferSize() const override;
	virtual void DumpState() const override;
	virtual float GetCurrentAmplitude() const override;

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;
	
private:

	/** All windows related variables to hide windows includes */
	FVoiceCaptureWindowsVars* CV;
	/** Last time data was captured */
	double LastCaptureTime;
	/** State of capture device */
	EVoiceCaptureState::Type VoiceCaptureState;
	/** Uncompressed audio buffer */
	TArray<uint8> UncompressedAudioBuffer;
	/** Number of samples of audio we've returned from the beginning of the capture */
	uint64 SampleCounter; 
	/** This value is used when we've detected a vocal onset in the middle of a buffer we are not sending audio for. */
	uint64 CachedSampleStart;
	/** This value is used to denote the index of the first sample of the current packet of audio data we are sending. */
	uint64 CurrentSampleStart;

	/** Boolean flag that is used to denote whether the starting index of a packet has been cached mid-buffer do to a vocal onset. */
	bool bSampleStartCached;

	/** Number of channels. */
	int32 NumInputChannels;

	/** This is used to convert audio from int16 to float to push to any potential connected patch outputs on IVoiceCapture::MicrophoneOutput */
	Audio::TSampleBuffer<float> ConversionBuffer;

	/**
	* This buffer is used so that we can detect when the player is speaking
	* without cutting off what they are saying.
	*/	
	FVoiceDelayBuffer<int16> LookaheadBuffer;
	/*
	* This buffer is used in case someone starts speaking again in the middle of a buffer
	* they were otherwise silent in.
	*/
	FVoiceDelayBuffer<int16> ReleaseBuffer;

	/*
	* Envelope following DSP object.
	* Configured using the MicSilenceDetectionConfig namespace in VoiceConfig.h and the CVars defined in VoiceConfig.cpp.
	*/
	Audio::FInlineEnvelopeFollower MicLevelDetector;
	/*
	* Whether the microphone level is above the threshold set 
	*/
	bool bIsMicActive;

	/**
	 * This linear ease is used to add attack fade in and release fadeout to the noise gate.
	 */
	Audio::FLinearEase NoiseGateAttenuator;

	/**
	 * This bool is used to track when we engage 
	 */
	bool bWasMicAboveNoiseGateThreshold;

	/**
	 * Create the D3D8 capture device
	 *
	 * @param DeviceName name of device to capture with, empty for default device
	 * @param SampleRate valid sample rate to capture audio data (8khz - 48khz)
	 * @param NumChannel number of audio channels (1 - mono, 2 - stereo)
	 * 
	 * @return true if successful, false otherwise
	 */
	bool CreateCaptureBuffer(const FString& DeviceName, int32 SampleRate, int32 NumChannels);

	/** Clear the capture buffer and release all resources */
	void FreeCaptureBuffer();

	/**
	 * Lock the DirectSound audio buffer and copy out the available data based on a notification
	 * and the current audio buffer capture cursor
	 */
	void ProcessData();
	/**
	 * Create notifications on the voice capture buffer, evenly distributed
	 *
	 * @param BufferSize size of buffer created
	 */
	bool CreateNotifications(uint32 BufferSize);
};
