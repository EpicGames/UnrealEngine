// Copyright Epic Games, Inc. All Rights Reserved.

#include "VoiceModule.h"
#include "VoicePrivate.h"

#include "VoiceCaptureWindows.h"
#include "VoiceCodecOpus.h"
#include "Voice.h"
#include "IHeadMountedDisplayModule.h"
#include "HAL/IConsoleManager.h"
#if PLATFORM_SUPPORTS_VOICE_CAPTURE

FVoiceCaptureDeviceWindows* FVoiceCaptureDeviceWindows::Singleton = nullptr;

/** Helper for printing MS guids */
FString PrintMSGUID(LPGUID Guid)
{
	FString Result;
	if (Guid)
	{
		Result = FString::Printf(TEXT("{%08lX-%04hX-%04hX-%02hhX%02hhX-%02hhX%02hhX%02hhX%02hhX%02hhX%02hhX}"),
			Guid->Data1, Guid->Data2, Guid->Data3,
			Guid->Data4[0], Guid->Data4[1], Guid->Data4[2], Guid->Data4[3],
			Guid->Data4[4], Guid->Data4[5], Guid->Data4[6], Guid->Data4[7]);
	}

	return Result;
}

/** Callback to access all the voice capture devices on the platform */
BOOL CALLBACK CaptureDeviceCallback(
	LPGUID lpGuid,
	LPCSTR lpcstrDescription,
	LPCSTR lpcstrModule,
	LPVOID lpContext
	)
{
	// @todo identify the proper device
	FVoiceCaptureDeviceWindows* VCPtr = (FVoiceCaptureDeviceWindows*)(lpContext);
	UE_LOG(LogVoiceCapture, Display, TEXT("Device: %hs Desc: %hs GUID: %s Context:0x%08" UPTRINT_x_FMT), lpcstrDescription, lpcstrModule, *PrintMSGUID(lpGuid), lpContext);

	if (lpGuid != nullptr)
	{
		// Save the enumerated device information for later use
		FString DeviceDescription((LPCWSTR)lpcstrDescription);
		FVoiceCaptureDeviceWindows::FCaptureDeviceInfo DeviceDesc;
		DeviceDesc.DeviceName = DeviceDescription;
		DeviceDesc.DeviceId = *lpGuid;
		// Default devices are list first in this callback
		DeviceDesc.bIsDefault = !VCPtr->Devices.Num();

		// Set the default voice capture device info here
		if (DeviceDesc.bIsDefault)
		{
			VCPtr->DefaultVoiceCaptureDevice = DeviceDesc;
		}
		VCPtr->Devices.Emplace(DeviceDescription, DeviceDesc);

		// Allow HMD to override the default voice capture device
		if (!VCPtr->HMDAudioInputDevice.IsEmpty() && !VCPtr->HMDAudioInputDevice.Compare((LPCWSTR)lpcstrModule))
		{
			UE_LOGF(LogVoice, Display, "VoiceCapture device overridden by HMD to use '%s' %ls", lpcstrDescription, *PrintMSGUID(lpGuid));
			VCPtr->DefaultVoiceCaptureDevice = DeviceDesc;
			VCPtr->Devices.Add(DeviceDescription, VCPtr->DefaultVoiceCaptureDevice);
		}
	}
	
	return true;
}

FVoiceCaptureDeviceWindows::FVoiceCaptureDeviceWindows() :
	bInitialized(false),
	DirectSound(nullptr)
{
}

FVoiceCaptureDeviceWindows::~FVoiceCaptureDeviceWindows()
{
	Shutdown();
}

FVoiceCaptureDeviceWindows* FVoiceCaptureDeviceWindows::Create()
{
	if (Singleton == nullptr)
	{
		Singleton = new FVoiceCaptureDeviceWindows();
		if (!Singleton->Init())
		{
			Singleton->Destroy();
		}
	}

	return Singleton;
}

void FVoiceCaptureDeviceWindows::Destroy()
{
	if (Singleton)
	{
		// calls Shutdown() above
		delete Singleton;
		Singleton = nullptr;
	}
}

FVoiceCaptureDeviceWindows* FVoiceCaptureDeviceWindows::Get()
{
	return Singleton;
}

class FAudioDuckingWindows
{
private:
	
	FAudioDuckingWindows() {}
	FAudioDuckingWindows(const FAudioDuckingWindows& Copy) {}
	~FAudioDuckingWindows() {}
	void operator=(const FAudioDuckingWindows&) {}

public:

	static HRESULT EnableDuckingOptOut(IMMDevice* pEndpoint, bool bDuckingOptOutChecked)
	{
		HRESULT hr = S_OK;

		// Activate session manager.
		IAudioSessionManager2* pSessionManager2 = nullptr;
		if (SUCCEEDED(hr))
		{
			hr = pEndpoint->Activate(__uuidof(IAudioSessionManager2),
				CLSCTX_INPROC_SERVER,
				nullptr,
				reinterpret_cast<void **>(&pSessionManager2));
		}

		IAudioSessionControl* pSessionControl = nullptr;
		if (SUCCEEDED(hr))
		{
			hr = pSessionManager2->GetAudioSessionControl(nullptr, 0, &pSessionControl);

			pSessionManager2->Release();
			pSessionManager2 = nullptr;
		}

		IAudioSessionControl2* pSessionControl2 = nullptr;
		if (SUCCEEDED(hr))
		{
			hr = pSessionControl->QueryInterface(
				__uuidof(IAudioSessionControl2),
				(void**)&pSessionControl2);

			pSessionControl->Release();
			pSessionControl = nullptr;
		}

		//  Sync the ducking state with the specified preference.
		if (SUCCEEDED(hr))
		{
			hr = pSessionControl2->SetDuckingPreference((::BOOL)bDuckingOptOutChecked);
			if (FAILED(hr))
			{
				UE_LOGF(LogVoiceCapture, Display, "Failed to duck audio endpoint. Error: 0x%08x", hr);
			}

			pSessionControl2->Release();
			pSessionControl2 = nullptr;
		}
		return hr;
	}

	static bool UpdateAudioDucking(bool bDuckingOptOutChecked)
	{
		HRESULT hr = S_OK;

		//  Start with the default endpoint.
		IMMDeviceEnumerator* pDeviceEnumerator = nullptr;
		hr = CoCreateInstance(__uuidof(MMDeviceEnumerator),
			nullptr,
			CLSCTX_INPROC_SERVER,
			IID_PPV_ARGS(&pDeviceEnumerator));

		if (SUCCEEDED(hr))
		{
			{
				IMMDevice* pEndpoint = nullptr;
				hr = pDeviceEnumerator->GetDefaultAudioEndpoint(eRender, eConsole, &pEndpoint);
				if (SUCCEEDED(hr))
				{
					LPWSTR Desc;
					pEndpoint->GetId(&Desc);
					UE_LOGF(LogVoiceCapture, Display, "%ls ducking on audio device. Desc: %ls", bDuckingOptOutChecked ? TEXT("Disabling") : TEXT("Enabling"), Desc);
					CoTaskMemFree(Desc);

					FAudioDuckingWindows::EnableDuckingOptOut(pEndpoint, bDuckingOptOutChecked);
					pEndpoint->Release();
					pEndpoint = nullptr;
				}
			}

			#if (0) // reference for enumerating all endpoints in case its necessary
			{
				IMMDeviceCollection* pDeviceCollection = nullptr;
				IMMDevice* pCollEndpoint = nullptr;
				hr = pDeviceEnumerator->EnumAudioEndpoints(eRender, DEVICE_STATE_ACTIVE, &pDeviceCollection);
				if (SUCCEEDED(hr))
				{
					IPropertyStore *pProps = nullptr;

					::UINT DeviceCount = 0;
					pDeviceCollection->GetCount(&DeviceCount);
					for (::UINT i = 0; i < DeviceCount; ++i)
					{
						hr = pDeviceCollection->Item(i, &pCollEndpoint);
						if (SUCCEEDED(hr) && pCollEndpoint)
						{
							LPWSTR Desc;
							pCollEndpoint->GetId(&Desc);

							hr = pCollEndpoint->OpenPropertyStore(STGM_READ, &pProps);
							if (SUCCEEDED(hr))
							{
								PROPVARIANT varName;
								// Initialize container for property value.
								PropVariantInit(&varName);

								// Get the endpoint's friendly-name property.
								hr = pProps->GetValue(
									PKEY_Device_FriendlyName, &varName);
								if (SUCCEEDED(hr))
								{
									// Print endpoint friendly name and endpoint ID.
									UE_LOGF(LogVoiceCapture, Display, "%ls ducking on audio device [%d]: \"%ls\" (%ls)",
										bDuckingOptOutChecked ? TEXT("Disabling") : TEXT("Enabling"),
										i, varName.pwszVal, Desc);

									CoTaskMemFree(Desc);

									Desc = nullptr;
									PropVariantClear(&varName);

									pProps->Release();
									pProps = nullptr;
								}
							}

							FAudioDuckingWindows::EnableDuckingOptOut(pCollEndpoint, bDuckingOptOutChecked);

							pCollEndpoint->Release();
							pCollEndpoint = nullptr;
						}
					}

					pDeviceCollection->Release();
					pDeviceCollection = nullptr;
				}
			}
			#endif

			pDeviceEnumerator->Release();
			pDeviceEnumerator = nullptr;
		}

		if (FAILED(hr))
		{
			UE_LOGF(LogVoiceCapture, Warning, "Failed to duck audio endpoint. Error: 0x%08x", hr);
		}

		return SUCCEEDED(hr);
	}
};

bool FVoiceCaptureDeviceWindows::Init()
{
	HRESULT hr = DirectSoundCreate8(nullptr, &DirectSound, nullptr);
	if (FAILED(hr))
	{
		UE_LOGF(LogVoiceCapture, Warning, "Failed to init DirectSound %d", hr);
		return false;
	}

	if (IHeadMountedDisplayModule::IsAvailable())
	{
		HMDAudioInputDevice = IHeadMountedDisplayModule::Get().GetAudioInputDevice();
	}

	Devices.Reset();

	hr = DirectSoundCaptureEnumerate((LPDSENUMCALLBACK)CaptureDeviceCallback, this);
	if (FAILED(hr))
	{
		UE_LOGF(LogVoiceCapture, Warning, "Failed to enumerate capture devices %d", hr);
		return false;
	}

	bool bDuckingOptOut = false;
	if (GConfig)
	{
		if (!GConfig->GetBool(TEXT("Voice"), TEXT("bDuckingOptOut"), bDuckingOptOut, GEngineIni))
		{
			bDuckingOptOut = false;
		}
	}
	FAudioDuckingWindows::UpdateAudioDucking(bDuckingOptOut);

	bInitialized = true;
	return true;
}

void FVoiceCaptureDeviceWindows::Shutdown()
{
	// Close any active captures
	for (int32 CaptureIdx = 0; CaptureIdx < ActiveVoiceCaptures.Num(); CaptureIdx++)
	{
		IVoiceCapture* VoiceCapture = ActiveVoiceCaptures[CaptureIdx];
		VoiceCapture->Shutdown();
	}

	ActiveVoiceCaptures.Empty();

	// Free up DirectSound
	if (DirectSound)
	{
		DirectSound->Release();
		DirectSound = nullptr;
	}

	bInitialized = false;
}

FVoiceCaptureWindows* FVoiceCaptureDeviceWindows::CreateVoiceCaptureObject(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	FVoiceCaptureWindows* NewVoiceCapture = nullptr;

	if (bInitialized)
	{
		NewVoiceCapture = new FVoiceCaptureWindows();
		if (NewVoiceCapture->Init(DeviceName, SampleRate, NumChannels))
		{
			ActiveVoiceCaptures.Add(NewVoiceCapture);
		}
		else
		{
			NewVoiceCapture = nullptr;
		}
	}

	return NewVoiceCapture;
}

void FVoiceCaptureDeviceWindows::FreeVoiceCaptureObject(IVoiceCapture* VoiceCaptureObj)
{
	if (VoiceCaptureObj != nullptr)
	{
		int32 RemoveIdx = ActiveVoiceCaptures.Find(VoiceCaptureObj);
		if (RemoveIdx != INDEX_NONE)
		{
			ActiveVoiceCaptures.RemoveAtSwap(RemoveIdx);
		}
		else
		{
			UE_LOGF(LogVoiceCapture, Warning, "Trying to free unknown voice object");
		}
	}
}

bool InitVoiceCapture()
{
	return FVoiceCaptureDeviceWindows::Create() != nullptr;
}

void ShutdownVoiceCapture()
{
	FVoiceCaptureDeviceWindows::Destroy();
}

IVoiceCapture* CreateVoiceCaptureObject(const FString& DeviceName, int32 SampleRate, int32 NumChannels)
{
	FVoiceCaptureDeviceWindows* VoiceCaptureDev = FVoiceCaptureDeviceWindows::Get();
	return VoiceCaptureDev ? VoiceCaptureDev->CreateVoiceCaptureObject(DeviceName, SampleRate, NumChannels) : nullptr;
}

#endif // PLATFORM_SUPPORTS_VOICE_CAPTURE
