// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VoicePackage.h"
#include "DSP/MultithreadedPatching.h"

class Error;

/**
 * Possible states related to voice capture
 */
namespace EVoiceCaptureState
{
	enum Type
	{
		UnInitialized,
		NotCapturing,
		Ok,
		NoData,
		Stopping,
		BufferTooSmall,
		Error
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(EVoiceCaptureState::Type VoiceCaptureState)
	{
		switch (VoiceCaptureState)
		{
		case UnInitialized:
			{
				return TEXT("Uninitialized");
			}
		case NotCapturing:
			{
				return TEXT("Not Capturing");
			}
		case Ok:
			{
				return TEXT("Ok");
			}
		case NoData:
			{
				return TEXT("No Data");
			}
		case Stopping:
			{
				return TEXT("Stopping");
			}
		case BufferTooSmall:
			{
				return TEXT("BufferTooSmall");
			}
		case Error:
			{
				return TEXT("Error");
			}
		}
		return TEXT("");
	}
}

/**
 * Interface for capturing voice data on any platform
 */
class IVoiceCapture : public TSharedFromThis<IVoiceCapture>
{

protected:

	IVoiceCapture() {};

	/**
	 * Optional patch point for MicrophoneOutput.
	 */
	Audio::FPatchSplitter MicrophoneOutput;

public:
	
	virtual ~IVoiceCapture() = default;

	/**
	 * Initialize the voice capture object
	 *
	 * @param DeviceName name of device to capture audio data with, empty for default device
	 * @param SampleRate sampling rate of voice capture
	 * @param NumChannels number of channels to capture
	 *
	 * @return true if successful, false otherwise
	 */
	virtual bool Init(const FString& DeviceName, int32 SampleRate, int32 NumChannels) = 0;

	/**
	 * Shutdown the voice capture object
	 */
	virtual void Shutdown() = 0;

	/**
	 * Start capturing voice
	 *
	 * @return true if successfully started, false otherwise
	 */
	virtual bool Start() = 0;

	/**
	 * Stop capturing voice
	 */
	virtual void Stop() = 0;

	/**
	 * Change the associated capture device
	 *
	 * @param DeviceName name of device to capture audio data with, empty for default device
	 * @param SampleRate sampling rate of voice capture
	 * @param NumChannels number of channels to capture
	 *
	 * @return true if change was successful, false otherwise
	 */
	virtual bool ChangeDevice(const FString& DeviceName, int32 SampleRate, int32 NumChannels) = 0;

	/**
	 * Is the voice capture object actively capturing
	 *
	 * @return true if voice is being captured, false otherwise
	 */
	virtual bool IsCapturing() = 0;

	/**
	 * Return the state of the voice data and its availability
	 *
	 * @param OutAvailableVoiceData size, in bytes, of available voice data
	 *
	 * @return state of the voice capture buffer
	 */
	virtual EVoiceCaptureState::Type GetCaptureState(uint32& OutAvailableVoiceData) const = 0;

	/**
	 * Fill a buffer with all available voice data
	 *
	 * @param OutVoiceBuffer allocated buffer to fill with voice data
	 * @param InVoiceBufferSize size, in bytes, of allocated buffer
	 * @param OutAvailableVoiceData size, in bytes, of data placed in the OutVoiceBuffer
	 * @param OutSampleCounter optional parameter for beginning sample of this buffer.
	 *
	 * @return state of the voice capture buffer
	 */
	virtual EVoiceCaptureState::Type GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData) { return EVoiceCaptureState::UnInitialized; };
	virtual EVoiceCaptureState::Type GetVoiceData(uint8* OutVoiceBuffer, uint32 InVoiceBufferSize, uint32& OutAvailableVoiceData, uint64& OutSampleCounter) { return GetVoiceData(OutVoiceBuffer, InVoiceBufferSize, OutAvailableVoiceData); };

	/**
	 * @return number of bytes currently allocated in the capture buffer
	 */
	virtual int32 GetBufferSize() const = 0;

	/** Dump the state of the voice capture device */
	virtual void DumpState() const = 0;

	/**
	 * @returns a new output for this mic's audio.
	 */
	Audio::FPatchOutputStrongPtr GetMicrophoneAudio(int32 MaxExpectedLatencyInSamples, float Gain)
	{
		return MicrophoneOutput.AddNewPatch(MaxExpectedLatencyInSamples, Gain);
	}

	/**
	 * @return the current loudness of the microphone. Returns -1.0 if this function is not supported by the current platform.
	 */
	virtual float GetCurrentAmplitude() const
	{
		return -1.0f;
	}
};
