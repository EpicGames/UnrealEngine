// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "VoicePackage.h"
#include "Net/VoiceConfig.h"

/** Stats for voice codec */
DECLARE_STATS_GROUP(TEXT("Voice"), STATGROUP_Voice, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("VoiceEncode"), STAT_Voice_Encoding, STATGROUP_Voice, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("VoiceDecode"), STAT_Voice_Decoding, STATGROUP_Voice, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Enc_SampleRate"), STAT_Encode_SampleRate, STATGROUP_Voice, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Enc_NumChannels"), STAT_Encode_NumChannels, STATGROUP_Voice, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Enc_Bitrate"), STAT_Encode_Bitrate, STATGROUP_Voice, );
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Enc_CompRatio"), STAT_Encode_CompressionRatio, STATGROUP_Voice, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Enc_OutSize"), STAT_Encode_OutSize, STATGROUP_Voice, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Dec_SampleRate"), STAT_Decode_SampleRate, STATGROUP_Voice, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Dec_NumChannels"), STAT_Decode_NumChannels, STATGROUP_Voice, );
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Dec_CompRatio"), STAT_Decode_CompressionRatio, STATGROUP_Voice, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Dec_OutSize"), STAT_Decode_OutSize, STATGROUP_Voice, );



/**
 * Interface for encoding raw voice for transmission over the wire
 */
class IVoiceEncoder : public TSharedFromThis<IVoiceEncoder>
{

protected:

	IVoiceEncoder() {}

public:

	virtual ~IVoiceEncoder() = default;

	/**
	 * Initialize the encoder
	 *
	 * @param SampleRate requested sample rate of the encoding
	 * @param NumChannel number of channels in the raw audio stream
	 * @param EncodeHint type of audio that will be encoded
	 * 
	 * @return true if initialization was successful, false otherwise
	 */
	virtual bool Init(int32 SampleRate, int32 NumChannels, EAudioEncodeHint EncodeHint) = 0;

	/**
	 * Encode a raw audio stream (expects 16bit PCM audio)
	 *
	 * @param RawPCMData array of raw PCM data to encode
	 * @param RawDataSize amount of raw PCM data in the buffer
	 * @param OutCompressedData buffer to contain encoded/compressed audio stream
	 * @param OutCompressedDataSize [in/out] amount of buffer used to encode the audio stream
	 *
	 * @return number of bytes at the end of the stream that wasn't encoded (some interfaces can only encode at a certain frame slice)
	 */
	virtual int32 Encode(const uint8* RawPCMData, uint32 RawDataSize, uint8* OutCompressedData, uint32& OutCompressedDataSize) = 0;

	/**
	 * Adjust the encoding bitrate
	 *
	 * @param InBitRate new bitrate value (bits/sec)
	 *
	 * @return true if successfully changed, false otherwise
	 */
	virtual bool SetBitrate(int32 InBitRate) = 0;

	/**
	 * Set the encoding to variable bitrate
	 *
	 * @param bEnableVBR true to use variable bit rate, false to turn it off
	 *
	 * @return true if successfully changed, false otherwise
	 */
	virtual bool SetVBR(bool bEnableVBR) = 0;

	/**
	 * Adjust the encoding complexity (platform specific)
	 *
	 * @param InComplexity new complexity
	 *
	 * @return true if successfully changed, false otherwise
	 */
	virtual bool SetComplexity(int32 InComplexity) = 0;

	/**
	 * Reset the encoder back to its initial state
	 */
	virtual void Reset() = 0;
	
	/**
	 * Cleanup the encoder
	 */
	virtual void Destroy() = 0;

	/**
	 * Output the state of the encoder
	 */
	virtual void DumpState() const = 0;
};

/**
 * Interface for decoding voice passed over the wire 
 */
class IVoiceDecoder : public TSharedFromThis<IVoiceDecoder>
{

protected:

	IVoiceDecoder() {}

public:

	virtual ~IVoiceDecoder() = default;

	/**
	 * Initialize the decoder
	 *
	 * @param SampleRate requested sample rate of the decoding (may be up sampled depending on the source data)
	 * @param NumChannel number of channels in the output decoded stream (mono streams can encode to stereo)
	 * 
	 * @return true if initialization was successful, false otherwise
	 */
	virtual bool Init(int32 SampleRate, int32 NumChannels) = 0;

	/**
	 * Decode an encoded audio stream (outputs 16bit PCM audio)
	 *
	 * @param CompressedData encoded/compressed audio stream
	 * @param CompressedDataSize amount of data in the buffer
	 * @param OutRawPCMData buffer to contain the decoded raw PCM data
	 * @param OutRawDataSize [in/out] amount of buffer used for the decoded raw PCM data
	 */
	virtual void Decode(const uint8* CompressedData, uint32 CompressedDataSize, uint8* OutRawPCMData, uint32& OutRawDataSize) = 0;

	/**
	 * Reset the decoder back to its initial state
	 */
	virtual void Reset() = 0;

	/**
	 * Cleanup the decoder
	 */
	virtual void Destroy() = 0;

	/**
	 * Output the state of the decoder
	 */
	virtual void DumpState() const = 0;
};
