// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/VoiceCapture.h"
#include "Interfaces/VoiceCodec.h"
#include "Net/VoiceConfig.h"

/** Default voice chat sample rate */
#define DEFAULT_VOICE_SAMPLE_RATE 16000
/** Deprecated value, use DEFAULT_VOICE_SAMPLE_RATE */
#define VOICE_SAMPLE_RATE DEFAULT_VOICE_SAMPLE_RATE

class IVoiceCapture;
class IVoiceEncoder;
class IVoiceDecoder;

/** Logging related to general voice chat flow (muting/registration/etc) */
VOICE_API DECLARE_LOG_CATEGORY_EXTERN(LogVoice, Display, All);
/** Logging related to encoding of local voice packets */
VOICE_API DECLARE_LOG_CATEGORY_EXTERN(LogVoiceEncode, Display, All);
/** Logging related to decoding of remote voice packets */
VOICE_API DECLARE_LOG_CATEGORY_EXTERN(LogVoiceDecode, Display, All);

/** Internal voice capture logging */
DECLARE_LOG_CATEGORY_EXTERN(LogVoiceCapture, Warning, All);

/**
 * Module for Voice capture/compression/decompression implementations
 */
class FVoiceModule : 
	public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FVoiceModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FVoiceModule>("Voice");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("Voice");
	}

	/**
	 * Instantiates a new voice capture object
	 *
	 * @param DeviceName name of device to capture audio data with, empty for default device
	 * @param SampleRate sampling rate of voice capture
	 * @param NumChannels number of channels to capture
	 *
	 * @return new voice capture object, possibly NULL
	 */
	virtual TSharedPtr<IVoiceCapture> CreateVoiceCapture(const FString& DeviceName, int32 SampleRate = UVOIPStatics::GetVoiceSampleRate(), int32 NumChannels = UVOIPStatics::GetVoiceNumChannels());

	/**
	 * Instantiates a new voice encoder object
	 *
	 * @param SampleRate sampling rate of voice capture
	 * @param NumChannels number of channels to capture
	 * @param EncodeHint hint to describe type of audio quality desired
	 *
	 * @return new voice encoder object, possibly NULL
	 */
	virtual TSharedPtr<IVoiceEncoder> CreateVoiceEncoder(int32 SampleRate = UVOIPStatics::GetVoiceSampleRate(), int32 NumChannels = UVOIPStatics::GetVoiceNumChannels(), EAudioEncodeHint EncodeHint = UVOIPStatics::GetAudioEncodingHint());

	/**
	 * Instantiates a new voice decoder object
	 *
	 * @param SampleRate sampling rate of voice capture
	 * @param NumChannels number of channels to capture
	 *
	 * @return new voice decoder object, possibly NULL
	 */
	virtual TSharedPtr<IVoiceDecoder> CreateVoiceDecoder(int32 SampleRate = UVOIPStatics::GetVoiceSampleRate(), int32 NumChannels = UVOIPStatics::GetVoiceNumChannels());

	/**
	* Checks to see if the current platform supports voice capture.
	*
	* @return True if the current platform support voice capture
	*/
	virtual bool DoesPlatformSupportVoiceCapture();

	/**
	 * @return true if voice is enabled
	 */
	inline bool IsVoiceEnabled() const
	{
		return bEnabled;
	}

private:

	IVoiceEncoder* CreateVoiceEncoderObject(int32 SampleRate, int32 NumChannels, EAudioEncodeHint EncodeHint);
	IVoiceDecoder* CreateVoiceDecoderObject(int32 SampleRate, int32 NumChannels);

	// IModuleInterface

	/**
	 * Called when voice module is loaded
	 * Initialize platform specific parts of voice handling
	 */
	virtual void StartupModule() override;
	
	/**
	 * Called when voice module is unloaded
	 * Shutdown platform specific parts of voice handling
	 */
	virtual void ShutdownModule() override;

	/** Is voice interface enabled */
	bool bEnabled;
};

