// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class Voice : ModuleRules
{
	public Voice(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDefinitions.Add("VOICE_PACKAGE=1");

		bool bDontNeedCapture = (Target.Type == TargetType.Server);

		if (bDontNeedCapture)
		{
			PublicDefinitions.Add("VOICE_MODULE_WITH_CAPTURE=0");
		}
		else
		{
			PublicDefinitions.Add("VOICE_MODULE_WITH_CAPTURE=1");
		}

		PublicIncludePathModuleNames.AddRange(
			new string[] {
					"HeadMountedDisplay"
			}
			);

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"AndroidPermission"
				}
				);
		}

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
                "AudioMixer",
				"SignalProcessing"
            }
			);

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DirectSound");
		}
		else if(Target.Platform == UnrealTargetPlatform.Mac)
		{
			bRequiresPlatformSDK = true;
			PublicFrameworks.AddRange(new string[] { "CoreAudio", "AudioUnit", "AudioToolbox" });
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux) && !bDontNeedCapture)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SDL3");
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "libOpus");

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			string ModulePath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(ModulePath, "AndroidVoiceImpl_UPL.xml"));
		}
	}
}
