// Copyright Epic Games, Inc. All Rights Reserved.

#include "LwsWebSocketClientConnection.h"

#if WITH_WEBSOCKETSERVER

#include "Misc/ScopeLock.h"
#include "WebSocketServerModule.h"

FLwsWebSocketClientConnection::FLwsWebSocketClientConnection(lws* InWsi, const FString& InPath, FLwsWebSocketServer* InServer)
	: Wsi(InWsi)
	, Server(InServer)
	, Path(InPath)
{
}

void FLwsWebSocketClientConnection::SendText(const FString& Message)
{
	FScopeLock Lock(&WsiLock);
	if (!Wsi)
	{
		return;
	}

	// Build lws frame: LWS_PRE bytes of header space + UTF-8 payload
	FTCHARToUTF8 Utf8(*Message);
	const int32 PayloadLen = Utf8.Length();
	TArray<uint8> Frame;
	Frame.AddZeroed(LWS_PRE);
	Frame.Append(reinterpret_cast<const uint8*>(Utf8.Get()), PayloadLen);

	SendQueue.Enqueue(MoveTemp(Frame));
	lws_callback_on_writable(Wsi);
}

void FLwsWebSocketClientConnection::Close(int32 Code, const FString& Reason)
{
	FScopeLock Lock(&WsiLock);
	if (!Wsi)
	{
		return;
	}

	// Store close params — lws_close_reason() must be called on the lws thread
	UE_LOGF(LogWebSocketServer, Display, "WS Close(): storing close request code=%d, calling lws_callback_on_writable", Code);
	CloseReason = Reason;
	CloseCode.Store(Code, EMemoryOrder::Relaxed);
	bCloseRequested.Store(true, EMemoryOrder::SequentiallyConsistent);

	// Schedule SERVER_WRITEABLE on the lws thread so LwsThreadFlushAndCheckClose() is called
	lws_callback_on_writable(Wsi);
}

bool FLwsWebSocketClientConnection::LwsThreadFlushAndCheckClose()
{
	// Flush outgoing send queue
	TArray<uint8> Frame;
	while (SendQueue.Dequeue(Frame))
	{
		const int32 PayloadLen = Frame.Num() - LWS_PRE;
		if (PayloadLen > 0)
		{
			UE_LOGF(LogWebSocketServer, Display, "WS lws_write: %d bytes", PayloadLen);
			int32 Written = lws_write(Wsi, Frame.GetData() + LWS_PRE, static_cast<size_t>(PayloadLen), LWS_WRITE_TEXT);
			UE_LOGF(LogWebSocketServer, Display, "WS lws_write returned: %d", Written);
		}
	}

	// If a close was requested, initiate it now on the lws thread
	if (bCloseRequested.Load(EMemoryOrder::SequentiallyConsistent))
	{
		UE_LOGF(LogWebSocketServer, Display, "WS lws_close_reason: code=%d reason='%ls'", CloseCode.Load(EMemoryOrder::Relaxed), *CloseReason);
		FTCHARToUTF8 ReasonUtf8(*CloseReason);
		lws_close_reason(Wsi,
			static_cast<lws_close_status>(CloseCode.Load(EMemoryOrder::Relaxed)),
			reinterpret_cast<unsigned char*>(const_cast<ANSICHAR*>(ReasonUtf8.Get())),
			static_cast<size_t>(ReasonUtf8.Length()));
		return true; // caller must return -1 from callback to initiate close
	}
	return false;
}

void FLwsWebSocketClientConnection::LwsThreadOnClosed()
{
	FScopeLock Lock(&WsiLock);
	Wsi = nullptr;
}

#endif // WITH_WEBSOCKETSERVER
