// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if WITH_WEBSOCKETSERVER

#include "IWebSocketClientConnection.h"
#include "Containers/Queue.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"
#include "Templates/Atomic.h"

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#endif
THIRD_PARTY_INCLUDES_START
#include "libwebsockets.h"
THIRD_PARTY_INCLUDES_END
#if PLATFORM_WINDOWS
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

class FLwsWebSocketServer;

// ─────────────────────────────────────────────────────────────────────────────
// Per-connection object — created on ESTABLISHED, destroyed after CLOSED
// ─────────────────────────────────────────────────────────────────────────────
class FLwsWebSocketClientConnection final
	: public IWebSocketClientConnection
	, public TSharedFromThis<FLwsWebSocketClientConnection>
{
public:
	FLwsWebSocketClientConnection(lws* InWsi, const FString& InPath, FLwsWebSocketServer* InServer);

	// IWebSocketClientConnection
	virtual void    SendText(const FString& Message) override;
	virtual void    Close(int32 Code = 1000, const FString& Reason = FString()) override;
	virtual FString GetPath() const override { return Path; }

	// Called on the lws thread from SERVER_WRITEABLE: flush send queue and handle pending close.
	// Returns true if the connection should be closed (caller must return -1 from callback).
	bool LwsThreadFlushAndCheckClose();

	// Called on the lws thread when LWS_CALLBACK_CLOSED / WSI_DESTROY fires
	void LwsThreadOnClosed();

private:
	FCriticalSection           WsiLock;
	lws*                       Wsi;          // Nulled on close
	FLwsWebSocketServer*       Server;       // Non-owning
	FString                    Path;

	// SPSC: game/any thread pushes, lws thread pops
	TQueue<TArray<uint8>, EQueueMode::Spsc> SendQueue;

	// Close request — set from game thread, read on lws thread
	TAtomic<bool>  bCloseRequested { false };
	TAtomic<int32> CloseCode       { 1000 };
	// CloseReason is written once before bCloseRequested is set true; safe to read after
	FString        CloseReason;
};

#endif // WITH_WEBSOCKETSERVER
