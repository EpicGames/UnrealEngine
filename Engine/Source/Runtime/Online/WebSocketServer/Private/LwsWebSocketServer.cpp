// Copyright Epic Games, Inc. All Rights Reserved.

#include "LwsWebSocketServer.h"

#if WITH_WEBSOCKETSERVER

#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Misc/ScopeLock.h"
#include "WebSocketServerModule.h"

// ─────────────────────────────────────────────────────────────────────────────
// FLwsWebSocketServer
// ─────────────────────────────────────────────────────────────────────────────

FLwsWebSocketServer::FLwsWebSocketServer(uint32 InPort)
	: Port(InPort)
{
	FMemory::Memzero(LwsProtocol, sizeof(LwsProtocol));
}

FLwsWebSocketServer::~FLwsWebSocketServer()
{
	StopListening();
}

bool FLwsWebSocketServer::StartListening()
{
	if (bListening)
	{
		return true;
	}

	// Register two protocols:
	//   [0] "" — catch-all: accepts clients that send no Sec-WebSocket-Protocol header (e.g. WinHttp)
	//   [1] "websocket" — accepts clients that explicitly request the "websocket" sub-protocol
	//   [2] zeroed — required lws terminator
	static const char ProtocolNameEmpty[]     = "";
	static const char ProtocolNameWebSocket[] = "websocket";
	LwsProtocol[0].name                 = ProtocolNameEmpty;
	LwsProtocol[0].callback              = &FLwsWebSocketServer::StaticLwsCallback;
	LwsProtocol[0].per_session_data_size = sizeof(FLwsWebSocketClientConnection*);
	LwsProtocol[0].rx_buffer_size        = 65536;
	LwsProtocol[1].name                 = ProtocolNameWebSocket;
	LwsProtocol[1].callback              = &FLwsWebSocketServer::StaticLwsCallback;
	LwsProtocol[1].per_session_data_size = sizeof(FLwsWebSocketClientConnection*);
	LwsProtocol[1].rx_buffer_size        = 65536;
	// LwsProtocol[2] stays zeroed — required terminator

	struct lws_context_creation_info Info;
	FMemory::Memzero(&Info, sizeof(Info));
	Info.port      = static_cast<int>(Port);
	Info.protocols = LwsProtocol;
	Info.user      = this;    // retrieve via lws_context_user()
	Info.options  |= LWS_SERVER_OPTION_PEER_CERT_NOT_REQUIRED
	              |  LWS_SERVER_OPTION_DISABLE_OS_CA_CERTS
	              |  LWS_SERVER_OPTION_DO_SSL_GLOBAL_INIT
	              |  LWS_SERVER_OPTION_VALIDATE_UTF8
	              |  LWS_SERVER_OPTION_DISABLE_IPV6;  // Bind IPv4 only (0.0.0.0), not IPv6

	// Silence most lws noise; only show errors/warnings
	lws_set_log_level(LLL_ERR | LLL_WARN, nullptr);

	LwsContext = lws_create_context(&Info);
	if (!LwsContext)
	{
		UE_LOGF(LogWebSocketServer, Error, "FLwsWebSocketServer: Failed to create lws context on port %u", Port);
		return false;
	}

	ExitRequest.Set(0);
	Thread = FRunnableThread::Create(this, TEXT("WebSocketServerThread"), 128 * 1024, TPri_BelowNormal);
	if (!Thread)
	{
		UE_LOGF(LogWebSocketServer, Error, "FLwsWebSocketServer: Failed to create service thread");
		lws_context_destroy(LwsContext);
		LwsContext = nullptr;
		return false;
	}

	bListening = true;
	return true;
}

void FLwsWebSocketServer::StopListening()
{
	if (!bListening)
	{
		return;
	}
	bListening = false;

	if (Thread)
	{
		ExitRequest.Set(1);
		WakeService();
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}

	if (LwsContext)
	{
		lws_context_destroy(LwsContext);
		LwsContext = nullptr;
	}

	{
		FScopeLock Lock(&ActiveConnectionsLock);
		ActiveConnections.Empty();
	}
	Connections.Empty();
}

uint32 FLwsWebSocketServer::Run()
{
	// ~30Hz polling loop (poll mode — lws_service returns immediately)
	const double TargetFrameTime = 1.0 / 30.0;

	while (!ExitRequest.GetValue())
	{
		double BeginTime = FPlatformTime::Seconds();
		if (LwsContext)
		{
			lws_service(LwsContext, 0);
		}
		double Elapsed = FPlatformTime::Seconds() - BeginTime;
		double SleepTime = TargetFrameTime - Elapsed;
		if (SleepTime > 0.0)
		{
			FPlatformProcess::SleepNoStats(static_cast<float>(SleepTime));
		}
	}
	return 0;
}

void FLwsWebSocketServer::Stop()
{
	ExitRequest.Set(1);
	WakeService();
}

void FLwsWebSocketServer::WakeService()
{
	if (LwsContext)
	{
		lws_cancel_service(LwsContext);
	}
}

// ── Game-thread event enqueuers (called from lws thread) ────────────────────

void FLwsWebSocketServer::LwsThread_OnEstablished(lws* Wsi, const FString& Path)
{
	TSharedPtr<FLwsWebSocketClientConnection> Conn = MakeShared<FLwsWebSocketClientConnection>(Wsi, Path, this);

	// Store pointer in lws per-session data so callback can retrieve it on RECEIVE/WRITEABLE
	FLwsWebSocketClientConnection** SessionData =
		static_cast<FLwsWebSocketClientConnection**>(lws_wsi_user(Wsi));
	if (SessionData)
	{
		*SessionData = Conn.Get();
	}

	// Register in thread-safe map so RECEIVE and CLOSED can look up the shared ptr
	{
		FScopeLock Lock(&ActiveConnectionsLock);
		ActiveConnections.Add(Wsi, Conn);
	}

	ConnectedQueue.Enqueue({ Conn });
}

void FLwsWebSocketServer::LwsThread_OnReceive(lws* Wsi, const FString& Message)
{
	TSharedPtr<FLwsWebSocketClientConnection> Conn;
	{
		FScopeLock Lock(&ActiveConnectionsLock);
		TSharedPtr<FLwsWebSocketClientConnection>* Found = ActiveConnections.Find(Wsi);
		if (Found)
		{
			Conn = *Found;
		}
	}
	if (Conn.IsValid())
	{
		MessageQueue.Enqueue({ Conn, Message });
	}
}

void FLwsWebSocketServer::LwsThread_OnClosed(lws* Wsi)
{
	TSharedPtr<FLwsWebSocketClientConnection> Conn;
	{
		FScopeLock Lock(&ActiveConnectionsLock);
		TSharedPtr<FLwsWebSocketClientConnection>* Found = ActiveConnections.Find(Wsi);
		if (Found)
		{
			Conn = *Found;
			ActiveConnections.Remove(Wsi);
		}
	}
	if (Conn.IsValid())
	{
		DisconnectedQueue.Enqueue({ Conn });
	}
}

// ── Game-thread Tick ─────────────────────────────────────────────────────────

void FLwsWebSocketServer::Tick(float DeltaTime)
{
	// 1. Drain connected events first (so Connections is populated before message events)
	{
		FConnectedEvent Event;
		while (ConnectedQueue.Dequeue(Event))
		{
			Connections.Add(Event.Conn);
			if (ConnectedHandler)
			{
				ConnectedHandler(Event.Conn.ToSharedRef());
			}
		}
	}

	// 2. Drain message events
	{
		FMessageEvent Event;
		while (MessageQueue.Dequeue(Event))
		{
			if (Event.Conn.IsValid() && MessageHandler)
			{
				MessageHandler(Event.Conn.ToSharedRef(), Event.Message);
			}
		}
	}

	// 3. Drain disconnected events
	{
		FDisconnectedEvent Event;
		while (DisconnectedQueue.Dequeue(Event))
		{
			if (Event.Conn.IsValid())
			{
				if (DisconnectedHandler)
				{
					DisconnectedHandler(Event.Conn.ToSharedRef());
				}
				Connections.RemoveSwap(Event.Conn);
			}
		}
	}
}

// ── Static lws callback ──────────────────────────────────────────────────────

int FLwsWebSocketServer::StaticLwsCallback(lws* Wsi, lws_callback_reasons Reason,
                                            void* UserData, void* Data, size_t Length)
{
	// Retrieve server from context user data
	lws_context* Context = lws_get_context(Wsi);
	FLwsWebSocketServer* Self = static_cast<FLwsWebSocketServer*>(lws_context_user(Context));
	if (!Self)
	{
		return 0;
	}

	// Per-session data holds FLwsWebSocketClientConnection* (after ESTABLISHED)
	FLwsWebSocketClientConnection* Conn = nullptr;
	if (UserData)
	{
		Conn = *static_cast<FLwsWebSocketClientConnection**>(UserData);
	}

	switch (Reason)
	{
	case LWS_CALLBACK_HTTP:
		// Accept HTTP requests so lws doesn't 404 before upgrading to WebSocket
		return 0;

	case LWS_CALLBACK_FILTER_PROTOCOL_CONNECTION:
		// Accept all incoming WebSocket connections regardless of Sec-WebSocket-Protocol
		return 0;

	case LWS_CALLBACK_ESTABLISHED:
	{
		// Read the request URI
		char UriBuf[512];
		FMemory::Memzero(UriBuf, sizeof(UriBuf));
		lws_hdr_copy(Wsi, UriBuf, sizeof(UriBuf) - 1, WSI_TOKEN_GET_URI);
		FString Path = UTF8_TO_TCHAR(UriBuf);

		Self->LwsThread_OnEstablished(Wsi, Path);
		break;
	}

	case LWS_CALLBACK_RECEIVE:
	{
		if (Data && Length > 0)
		{
			FUTF8ToTCHAR Convert(reinterpret_cast<const ANSICHAR*>(Data), static_cast<int32>(Length));
			FString Message(Convert.Length(), Convert.Get());
			Self->LwsThread_OnReceive(Wsi, Message);
		}
		break;
	}

	case LWS_CALLBACK_EVENT_WAIT_CANCELLED:
	{
		// WakeService() was called from another thread (send or close enqueued).
		// Schedule SERVER_WRITEABLE for all live connections so they can flush/close.
		FScopeLock Lock(&Self->ActiveConnectionsLock);
		for (auto& Pair : Self->ActiveConnections)
		{
			lws_callback_on_writable(Pair.Key);
		}
		break;
	}

	case LWS_CALLBACK_SERVER_WRITEABLE:
	{
		if (Conn)
		{
			if (Conn->LwsThreadFlushAndCheckClose())
			{
				return -1; // initiates lws close handshake
			}
		}
		break;
	}

	case LWS_CALLBACK_CLOSED:
	case LWS_CALLBACK_WSI_DESTROY:
	{
		UE_LOGF(LogWebSocketServer, Display, "WS CLOSED/WSI_DESTROY: Reason=%d Conn=%p", (int)Reason, Conn);
		if (Conn)
		{
			Conn->LwsThreadOnClosed();
			Self->LwsThread_OnClosed(Wsi);
			// Clear per-session pointer so we don't double-fire
			if (UserData)
			{
				*static_cast<FLwsWebSocketClientConnection**>(UserData) = nullptr;
			}
		}
		break;
	}

	default:
		break;
	}

	return 0;
}

#endif // WITH_WEBSOCKETSERVER
