// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if WITH_WEBSOCKETSERVER

#include "IWebSocketServer.h"
#include "LwsWebSocketClientConnection.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/Queue.h"

// ─────────────────────────────────────────────────────────────────────────────
// Server — owns the lws_context and service thread
// ─────────────────────────────────────────────────────────────────────────────
class FLwsWebSocketServer final
	: public IWebSocketServer
	, public FRunnable
{
public:
	explicit FLwsWebSocketServer(uint32 InPort);
	virtual ~FLwsWebSocketServer();

	// IWebSocketServer
	virtual bool StartListening() override;
	virtual void StopListening() override;
	virtual bool IsListening() const override { return bListening; }
	virtual void Tick(float DeltaTime) override;

	virtual void OnConnected(FWsConnectedHandler Handler) override    { ConnectedHandler    = MoveTemp(Handler); }
	virtual void OnMessage(FWsMessageHandler Handler) override        { MessageHandler      = MoveTemp(Handler); }
	virtual void OnDisconnected(FWsDisconnectedHandler Handler) override { DisconnectedHandler = MoveTemp(Handler); }

	// Called from lws thread (via StaticLwsCallback) to enqueue game-thread events
	void LwsThread_OnEstablished(lws* Wsi, const FString& Path);
	void LwsThread_OnReceive(lws* Wsi, const FString& Message);
	void LwsThread_OnClosed(lws* Wsi);

	// Wakes the lws service loop (safe to call from any thread)
	void WakeService();

private:
	// FRunnable — runs on dedicated lws thread
	virtual bool   Init() override  { return true; }
	virtual uint32 Run() override;
	virtual void   Stop() override;
	virtual void   Exit() override  {}

	static int StaticLwsCallback(lws* Wsi, lws_callback_reasons Reason,
	                              void* UserData, void* Data, size_t Length);

	uint32              Port;
	lws_context*        LwsContext  = nullptr;
	FRunnableThread*    Thread      = nullptr;
	FThreadSafeCounter  ExitRequest;
	bool                bListening  = false;

	lws_protocols       LwsProtocol[3]; // catch-all ("") + "websocket" + zero terminator

	// Game-thread event queues (MPSC: many lws connections → one game thread)
	struct FConnectedEvent    { TSharedPtr<FLwsWebSocketClientConnection> Conn; };
	struct FMessageEvent      { TSharedPtr<FLwsWebSocketClientConnection> Conn; FString Message; };
	struct FDisconnectedEvent { TSharedPtr<FLwsWebSocketClientConnection> Conn; };

	TQueue<FConnectedEvent,    EQueueMode::Mpsc> ConnectedQueue;
	TQueue<FMessageEvent,      EQueueMode::Mpsc> MessageQueue;
	TQueue<FDisconnectedEvent, EQueueMode::Mpsc> DisconnectedQueue;

	// User-supplied handlers (set from game thread before StartListening)
	FWsConnectedHandler    ConnectedHandler;
	FWsMessageHandler      MessageHandler;
	FWsDisconnectedHandler DisconnectedHandler;

	// All live connections (game thread only — drained from ConnectedQueue/DisconnectedQueue in Tick)
	TArray<TSharedPtr<FLwsWebSocketClientConnection>> Connections;

	// Map from lws* → shared connection ptr, protected by a lock.
	// Allows lws thread to enqueue message/disconnect events with a valid shared ptr.
	FCriticalSection                                             ActiveConnectionsLock;
	TMap<lws*, TSharedPtr<FLwsWebSocketClientConnection>>       ActiveConnections;
};

#endif // WITH_WEBSOCKETSERVER
