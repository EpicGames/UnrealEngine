// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebSocketServerModule.h"
#include "Modules/ModuleManager.h"

#if WITH_WEBSOCKETSERVER
#include "LwsWebSocketServer.h"
#endif

DEFINE_LOG_CATEGORY(LogWebSocketServer);

IMPLEMENT_MODULE(FWebSocketServerModule, WebSocketServer);

FWebSocketServerModule* FWebSocketServerModule::Singleton = nullptr;

FWebSocketServerModule::FWebSocketServerModule()
{
}

FWebSocketServerModule::~FWebSocketServerModule()
{
}

void FWebSocketServerModule::StartupModule()
{
	Singleton = this;
}

void FWebSocketServerModule::ShutdownModule()
{
	StopAllServers();
	Servers.Empty();
	Singleton = nullptr;
}

bool FWebSocketServerModule::IsAvailable()
{
	return Singleton != nullptr;
}

FWebSocketServerModule& FWebSocketServerModule::Get()
{
	if (!Singleton)
	{
		check(IsInGameThread());
		FModuleManager::LoadModuleChecked<FWebSocketServerModule>("WebSocketServer");
	}
	check(Singleton);
	return *Singleton;
}

TSharedPtr<IWebSocketServer> FWebSocketServerModule::GetWebSocketServer(uint32 Port)
{
	TSharedPtr<IWebSocketServer>* Existing = Servers.Find(Port);
	if (Existing)
	{
		return *Existing;
	}

#if WITH_WEBSOCKETSERVER
	TSharedPtr<IWebSocketServer> NewServer = MakeShared<FLwsWebSocketServer>(Port);

	if (bServersEnabled)
	{
		NewServer->StartListening();
	}

	Servers.Add(Port, NewServer);
	return NewServer;
#else
	UE_LOGF(LogWebSocketServer, Warning, "WebSocketServer is not supported on this platform.");
	return nullptr;
#endif
}

void FWebSocketServerModule::StartAllServers()
{
	bServersEnabled = true;
	for (auto& Pair : Servers)
	{
		if (!Pair.Value->IsListening())
		{
			Pair.Value->StartListening();
		}
	}
}

void FWebSocketServerModule::StopAllServers()
{
	bServersEnabled = false;
	for (auto& Pair : Servers)
	{
		if (Pair.Value->IsListening())
		{
			Pair.Value->StopListening();
		}
	}
}

bool FWebSocketServerModule::Tick(float DeltaTime)
{
	if (bServersEnabled)
	{
		for (auto& Pair : Servers)
		{
			Pair.Value->Tick(DeltaTime);
		}
	}
	return true;
}
