// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

/**
 * Handle to a single connected WebSocket client.
 * May be held past disconnection; SendText/Close are no-ops after disconnect.
 * All methods are thread-safe.
 */
class IWebSocketClientConnection
{
public:
	virtual ~IWebSocketClientConnection() {}

	/** Send a UTF-8 text frame to this client. */
	virtual void SendText(const FString& Message) = 0;

	/** Initiate a graceful close with the given status code. */
	virtual void Close(int32 Code = 1000, const FString& Reason = FString()) = 0;

	/** The URL path from the HTTP Upgrade request (e.g. "/webtests/websocketstests/echo/"). */
	virtual FString GetPath() const = 0;
};
