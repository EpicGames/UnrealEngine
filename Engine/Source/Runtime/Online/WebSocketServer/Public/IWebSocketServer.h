// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IWebSocketClientConnection.h"

/** Called when a new client connects. */
using FWsConnectedHandler    = TFunction<void(TSharedRef<IWebSocketClientConnection>)>;
/** Called when a client sends a text message. */
using FWsMessageHandler      = TFunction<void(TSharedRef<IWebSocketClientConnection>, const FString&)>;
/** Called when a client disconnects (clean or error). */
using FWsDisconnectedHandler = TFunction<void(TSharedRef<IWebSocketClientConnection>)>;

/**
 * A WebSocket server listening on a single port.
 * Handlers are invoked on the game thread.
 */
class IWebSocketServer
{
public:
	virtual ~IWebSocketServer() {}

	/** Begin accepting connections. Returns false if binding failed. */
	virtual bool StartListening() = 0;

	/** Stop accepting new connections and close all existing ones. */
	virtual void StopListening() = 0;

	/** Returns true if currently listening. */
	virtual bool IsListening() const = 0;

	/** Called every frame on the game thread to dispatch pending events. */
	virtual void Tick(float DeltaTime) = 0;

	/** Register a handler for new connections. Replaces any previous handler. */
	virtual void OnConnected(FWsConnectedHandler Handler) = 0;

	/** Register a handler for incoming text messages. Replaces any previous handler. */
	virtual void OnMessage(FWsMessageHandler Handler) = 0;

	/** Register a handler for disconnections. Replaces any previous handler. */
	virtual void OnDisconnected(FWsDisconnectedHandler Handler) = 0;
};
