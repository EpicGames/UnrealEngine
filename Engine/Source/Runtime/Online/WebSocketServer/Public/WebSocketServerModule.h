// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "IWebSocketServer.h"

DECLARE_LOG_CATEGORY_EXTERN(LogWebSocketServer, Log, All);

/**
 * Module that manages WebSocket servers.
 * Usage:
 *   TSharedPtr<IWebSocketServer> Server = FWebSocketServerModule::Get().GetWebSocketServer(8001);
 *   Server->OnMessage([](TSharedRef<IWebSocketClientConnection> C, const FString& M){ C->SendText(M); });
 *   FWebSocketServerModule::Get().StartAllServers();
 */
class FWebSocketServerModule
	: public IModuleInterface
	, public FTSTickerObjectBase
{
public:
	FWebSocketServerModule();
	virtual ~FWebSocketServerModule();

	/** Returns true if the module is loaded and ready. */
	static bool IsAvailable();

	/** Singleton access. Loads the module if needed. */
	static FWebSocketServerModule& Get();

	/**
	 * Returns (or creates) an IWebSocketServer for the given port.
	 * Bind handlers before calling StartAllServers().
	 */
	TSharedPtr<IWebSocketServer> GetWebSocketServer(uint32 Port);

	/** Start all servers (idempotent). */
	void StartAllServers();

	/** Stop all servers. */
	void StopAllServers();

	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// FTSTickerObjectBase — drives game-thread event dispatch
	virtual bool Tick(float DeltaTime) override;

private:
	static FWebSocketServerModule* Singleton;

	TMap<uint32, TSharedPtr<IWebSocketServer>> Servers;
	bool bServersEnabled = false;
};
