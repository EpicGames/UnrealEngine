// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebSocketServer : ModuleRules
{
	protected virtual bool bPlatformSupportsWebSocketServer
	{
		get
		{
			return
				Target.Platform == UnrealTargetPlatform.Win64 ||
				Target.Platform == UnrealTargetPlatform.Mac ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
		}
	}

	public WebSocketServer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
		});

		if (bPlatformSupportsWebSocketServer)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL", "libWebSockets", "zlib");
			PrivateDependencyModuleNames.Add("SSL");
		}

		PublicDefinitions.Add("WEBSOCKETSERVER_PACKAGE=1");
		PublicDefinitions.Add("WITH_WEBSOCKETSERVER=" + (bPlatformSupportsWebSocketServer ? "1" : "0"));
	}
}
