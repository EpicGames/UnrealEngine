// Copyright Epic Games, Inc. All Rights Reserved.

#include "LwsWebSocket.h"

#if WITH_WEBSOCKETS && WITH_LIBWEBSOCKETS

#include "LwsWebSocketsManager.h"
#include "WebSocketsModule.h"
#include "WebSocketsLog.h"
#if WITH_SSL
#include "Ssl.h"
#endif
#include "Misc/EmbeddedCommunication.h"
#include "Misc/ScopeLock.h"
#include "Misc/ConfigCacheIni.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "Misc/URLRequestFilter.h"

// FLwsSendBuffer 
FLwsSendBuffer::FLwsSendBuffer(const uint8* Data, SIZE_T Size, bool bInIsBinary)
	: bIsBinary(bInIsBinary)
	, BytesWritten(0)
	, bHasError(false)
{
	check((LWS_PRE + Size) < INT32_MAX);
	check(Data);
	Payload.Reserve(LWS_PRE + Size);
	Payload.AddDefaulted(LWS_PRE); // Reserve space for WS header data
	Payload.Append(Data, Size);
}

int32 FLwsSendBuffer::GetPayloadSize() const
{
	return Payload.Num() - (int32)(LWS_PRE);
}

// FLwsReceiveBufferBinary
FLwsReceiveBufferBinary::FLwsReceiveBufferBinary(const uint8* Data, const int32 Size, const int32 InBytesRemaining)
	: BytesRemaining(InBytesRemaining)
{
	check(Data);
	check(Size >= 0);
	check(InBytesRemaining >= 0);
	Payload.Append(Data, Size);
}

// FLwsReceiveBufferBinaryFragment
FLwsReceiveBufferBinaryFragment::FLwsReceiveBufferBinaryFragment(const uint8* Data, const int32 Size, const bool bInIsLastFragment)
	: bIsLastFragment(bInIsLastFragment)
{
	check(Data);
	check(Size >= 0);
	Payload.Append(Data, Size);
}

// FLwsReceiveBufferText
FLwsReceiveBufferText::FLwsReceiveBufferText(FString&& InText)
	: Text(MoveTemp(InText))
{
}

// FLwsWebSocket

int32 FLwsWebSocket::IncrementingIdentifier = 0;

const TCHAR* FLwsWebSocket::ToString(const EState InState)
{
#define LWSWEBSOCKET_ESTATE_TOSTRING(InName) case EState::InName: return TEXT(#InName);
	switch (InState)
	{
		LWSWEBSOCKET_ESTATE_TOSTRING(None);
		LWSWEBSOCKET_ESTATE_TOSTRING(StartConnecting);
		LWSWEBSOCKET_ESTATE_TOSTRING(Connecting);
		LWSWEBSOCKET_ESTATE_TOSTRING(Connected);
		LWSWEBSOCKET_ESTATE_TOSTRING(ClosingByRequest);
		LWSWEBSOCKET_ESTATE_TOSTRING(Closed);
		LWSWEBSOCKET_ESTATE_TOSTRING(Error);
	}
#undef LWSWEBSOCKET_ESTATE_TOSTRING
	return TEXT("Unknown");
}

FLwsWebSocket::FLwsWebSocket(FPrivateToken, const FString& InUrl, const TArray<FString>& InProtocols, const FString& InUpgradeHeader, uint64 TextMessageMemoryLimit)
	: State(EState::None)
	, LastGameThreadState(EState::None)
	, bWasSendQueueEmpty(true)
	, LwsConnection(nullptr)
	, Url(InUrl)
	, Protocols(InProtocols)
	, UpgradeHeader(InUpgradeHeader)
	, MaxTextMessageBufferSize(TextMessageMemoryLimit)
	, Identifier(++IncrementingIdentifier)
{
	UE_LOGF(LogWebSockets, VeryVerbose, "FLwsWebSocket[%d]: Constructed url=%ls protocols=%ls", Identifier, *InUrl, *FString::Join(Protocols, TEXT(",")));
}

FLwsWebSocket::~FLwsWebSocket()
{
	UE_LOGF(LogWebSockets, VeryVerbose, "FLwsWebSocket[%d]: Destroyed", Identifier);
	checkf(LwsConnection == nullptr, TEXT("FLwsWebSocket: Must have closed connection before destruction"));
	ClearData();
}

void FLwsWebSocket::Connect()
{
	if (LastGameThreadState != EState::None)
	{
		UE_LOGF(LogWebSockets, Warning, "FLwsWebSocket[%d]::Connect: State is not None (%ls), unable to start connecting!", Identifier, ToString(LastGameThreadState));
		return;
	}

	if (!FLwsWebSocketsManager::Get().bDisableDomainAllowlist)
	{
		UE::Core::FURLRequestFilter DomainFilter(TEXT("Online.HttpManager"), GEngineIni);
		if (!DomainFilter.IsRequestAllowed(Url))
		{
			State = EState::Error;
			UE_LOGF(LogWebSockets, Warning, "FLwsWebSocket[%d]::Connect: %ls is not allowlisted. Refusing to connect.", Identifier, *Url);
			OnConnectionError().Broadcast(TEXT("Invalid Domain"));
			return;
		}
	}
	else
	{
		UE_LOGF(LogWebSockets, Log, "FLwsWebSocket[%d]::Connect: Domain allowlisting has been disabled by config.", Identifier);
	}

	// No lock, we are not being processed on the websockets thread yet
	State = EState::StartConnecting;
	LastGameThreadState = State; // This is called on the game thread

	bWantsMessageEvents = OnMessage().IsBound();
	bWantsRawMessageEvents = OnRawMessage().IsBound();
	bWantsBinaryMessageEvents = OnBinaryMessage().IsBound();
	
	UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::Connect: setting State=%ls url=%ls bWantsMessageEvents=%d bWantsRawMessageEvents=%d", Identifier, ToString(State), *Url, (int32)bWantsMessageEvents, (int32)bWantsRawMessageEvents);

	// Clear up any data from previous runs
	ClearData();

	FLwsWebSocketsManager& WebSocketsManager = FLwsWebSocketsManager::Get();
	WebSocketsManager.StartProcessingWebSocket(this);
}

void FLwsWebSocket::Close(int32 Code, const FString& Reason)
{
	if (CloseRequest.Code != 0)
	{
		UE_LOGF(LogWebSockets, Warning, "FLwsWebSocket[%d]::Close: Already closing, ignoring subsequent attempt", Identifier);
		return;
	}

	// We are doing this conversion here so we don't have to do it on the ws thread
	FTCHARToUTF8 Convert(*Reason);
	ANSICHAR* ANSIReason = static_cast<ANSICHAR*>(FMemory::Malloc(Convert.Length() + 1));
	FCStringAnsi::Strncpy(ANSIReason, (const ANSICHAR*)Convert.Get(), Convert.Length() + 1);
	ANSIReason[Convert.Length()] = 0;

	UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::Close: Close queued with code=%d reason=%ls", Identifier, Code, *Reason);

	{
		UE::TScopeLock ScopeLock(StateLock);
		CloseRequest.Code = Code;
		CloseRequest.Reason = ANSIReason;
	}

	FLwsWebSocketsManager::Get().WakeService();
}

void FLwsWebSocket::Send(const void* Data, SIZE_T Size, bool bIsBinary)
{
	SendQueue.Enqueue(new FLwsSendBuffer(static_cast<const uint8*>(Data), Size, bIsBinary));
	FLwsWebSocketsManager& WebSocketsManager = FLwsWebSocketsManager::Get();
	WebSocketsManager.WakeService();
}

void FLwsWebSocket::Send(const FString& Data)
{
	FTCHARToUTF8 Converted(*Data);
	Send((uint8*)Converted.Get(), Converted.Length(), false);
	OnMessageSent().Broadcast(Data);
}


void FLwsWebSocket::SetTextMessageMemoryLimit(uint64 TextMessageMemoryLimit)
{
	MaxTextMessageBufferSize = TextMessageMemoryLimit;
}

void FLwsWebSocket::SendFromQueue()
{
	check(LwsConnection);

	FLwsSendBuffer* CurrentBuffer;
	// libwebsockets-3.0 only allows us to send once per LWS_CALLBACK_*_WRITABLE event
	if (SendQueue.Peek(CurrentBuffer))
	{
		int32 LastBytesWritten = CurrentBuffer->BytesWritten;
		const bool bWriteSuccessful = WriteBuffer(*CurrentBuffer);
		const bool bIsDone = CurrentBuffer->IsDone();
		if (!bWriteSuccessful)
		{
			// @TODO: bubble up error
			UE_LOGF(LogWebSockets, Warning, "FLwsWebSocket[%d]::SendFromQueue: Error writing buffer Size=%d BytesWritten=%d bIsBinary=%d", 
				Identifier, CurrentBuffer->GetPayloadSize(), CurrentBuffer->BytesWritten, (int32)CurrentBuffer->bIsBinary);
		}
		else if (LastBytesWritten != CurrentBuffer->BytesWritten)
		{
			UE_LOGF(LogWebSockets, VeryVerbose, "FLwsWebSocket[%d]::SendFromQueue: Wrote %d bytes, %d bytes remaining in this packet", 
				Identifier, (CurrentBuffer->BytesWritten - LastBytesWritten), (CurrentBuffer->GetPayloadSize() - CurrentBuffer->BytesWritten));
		}

		const bool bFinishedSending = bIsDone || !bWriteSuccessful;
		if (bFinishedSending)
		{
			SendQueue.Dequeue(CurrentBuffer);
			delete CurrentBuffer;
		}
	}

	// If we still have data to send, ask for a notification when ready to send more
	bWasSendQueueEmpty = SendQueue.IsEmpty();
	if (!bWasSendQueueEmpty)
	{
		lws_callback_on_writable(LwsConnection);
	}
}

void FLwsWebSocket::ClearData()
{
	check(State != EState::Connected);
	ReceiveBinaryQueue.Empty();
	ReceiveBinaryFragmentQueue.Empty();
	ReceiveTextQueue.Empty();
	FLwsSendBuffer* SendBuffer;
	while (SendQueue.Dequeue(SendBuffer))
	{
		delete SendBuffer;
	}
	ReceiveBuffer.Empty(0); // Also clear temporary receive buffer
	if (CloseRequest.Reason)
	{
		FMemory::Free(CloseRequest.Reason);
	}
	CloseRequest.Code = 0;
	CloseRequest.Reason = nullptr;
}

bool FLwsWebSocket::WriteBuffer(FLwsSendBuffer& Buffer)
{
	enum lws_write_protocol WriteProtocol;
	if (Buffer.BytesWritten > 0)
	{
		WriteProtocol = LWS_WRITE_CONTINUATION;
	}
	else if (Buffer.bIsBinary)
	{
		WriteProtocol = LWS_WRITE_BINARY;
	}
	else
	{
		WriteProtocol = LWS_WRITE_TEXT;
	}

	// Payload is modified in the call to lws_write, for libwebsockets to be able to use already allocated memory instead of needing to allocate more
	uint8* Payload = Buffer.Payload.GetData();
	const int32 PayloadSize = Buffer.Payload.Num();
	const int32 Offset = LWS_PRE + Buffer.BytesWritten;
	const int32 CurrentBytesWritten = lws_write(LwsConnection, Payload + Offset, PayloadSize - Offset, WriteProtocol);

	if (CurrentBytesWritten > 0)
	{
		Buffer.BytesWritten += CurrentBytesWritten;
	}

	// lws_write returns -1 on error
	return CurrentBytesWritten >= 0;
}

int FLwsWebSocket::LwsCallback(lws* Instance, lws_callback_reasons Reason, void* Data, size_t Length)
{
	switch (Reason)
	{
	case LWS_CALLBACK_CLIENT_ESTABLISHED:
	{
		// No lock because we are the only thread that writes to State
		EState PreviousState = State;
		{
			UE::TScopeLock ScopeLock(StateLock);
			State = EState::Connected;
		}
		UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_CLIENT_ESTABLISHED, setting State=%ls PreviousState=%ls", Identifier, ToString(State), ToString(PreviousState));

		LwsConnection = Instance;
		bWasSendQueueEmpty = SendQueue.IsEmpty();
		if (!bWasSendQueueEmpty)
		{
			lws_callback_on_writable(LwsConnection);
		}
		break;
	}
	case LWS_CALLBACK_CLIENT_RECEIVE:
	{
		const SIZE_T BytesLeft = lws_remaining_packet_payload(Instance);
		bool bIsFinalFragment = (bool)lws_is_final_fragment(Instance);
		bool bIsBinary = (bool)lws_frame_is_binary(Instance);
		UE_LOG(LogWebSockets, VeryVerbose, TEXT("FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_CLIENT_RECEIVE Length=%" SIZE_T_FMT " BytesLeftInFragment=%" SIZE_T_FMT " IsFinalFragment=%s"), Identifier, Length, BytesLeft, bIsFinalFragment ? TEXT("true") : TEXT("false"));
		bool bWakeGameThread = false;
		if (bWantsMessageEvents && !bIsBinary)
		{
			FUTF8ToTCHAR Convert((const ANSICHAR*)Data, Length);

			if (ReceiveBuffer.Len() + Length > MaxTextMessageBufferSize)
			{
				FString ReasonString = FString::Printf(TEXT("Received text message exceeded memory limit of %" UINT64_FMT " bytes"), MaxTextMessageBufferSize);
				Close(LWS_CLOSE_STATUS_MESSAGE_TOO_LARGE, ReasonString);

				UE_LOGF(LogWebSockets, Verbose, "Received text message too large - use SetTextMessageMemoryLimit() to increase buffer size. Current Size=%llu", MaxTextMessageBufferSize);
			}
			else
			{
				ReceiveBuffer.Append(Convert.Get(), Convert.Length());
				if (BytesLeft == 0 && bIsFinalFragment)
				{
					bWakeGameThread = true;
					ReceiveTextQueue.Enqueue(MakeUnique<FLwsReceiveBufferText>(MoveTemp(ReceiveBuffer)));
					ReceiveBuffer.Empty();
				}
			}
		}
		if (bWantsBinaryMessageEvents && bIsBinary)
		{
			bWakeGameThread = true;
			bool bIsLast = (bool)bIsFinalFragment && BytesLeft == 0;
			ReceiveBinaryFragmentQueue.Enqueue(MakeUnique<FLwsReceiveBufferBinaryFragment>(static_cast<const uint8*>(Data), Length, bIsLast));
		}
		if (bWantsRawMessageEvents)
		{
			bWakeGameThread = true;
			ReceiveBinaryQueue.Enqueue(MakeUnique<FLwsReceiveBufferBinary>(static_cast<const uint8*>(Data), Length, BytesLeft));
		}

		if (bWakeGameThread)
		{
			FEmbeddedCommunication::WakeGameThread();
		}
		break;
	}
	case LWS_CALLBACK_WS_PEER_INITIATED_CLOSE:
	{
		uint16 CloseStatus = *(static_cast<const uint16*>(Data));
#if PLATFORM_LITTLE_ENDIAN
		// The status is the first two bytes of the message in network byte order
		CloseStatus = BYTESWAP_ORDER16(CloseStatus);
#endif
		FString CloseReasonString;
		if (Length > sizeof(uint16))
		{
			auto Convert = StringCast<TCHAR>((const ANSICHAR*)Data + sizeof(uint16), Length - sizeof(uint16));
			CloseReasonString.AppendChars(Convert.Get(), Convert.Length());
		}

		// We only modify our state if we are Connected or ClosingByRequest (effectively connected)
		if (State == EState::Connected ||
			State == EState::ClosingByRequest)
		{
			LwsConnection = nullptr;
			const bool bPeerSpecifiedReason = !CloseReasonString.IsEmpty();
			if (!bPeerSpecifiedReason)
			{
				CloseReasonString = TEXT("Peer did not specify a reason for initiating the closing");
			}
			UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_WS_PEER_INITIATED_CLOSE, setting State=%ls CloseStatus=%d Reason=%ls bPeerSpecifiedReason=%d PreviousState=%ls",
				Identifier, ToString(EState::Closed), (int32)CloseStatus, *CloseReasonString, (int32)bPeerSpecifiedReason, ToString(State));
			{
				UE::TScopeLock ScopeLock(StateLock);
				State = EState::Closed;
				ClosedReason.Reason = MoveTemp(CloseReasonString);
				ClosedReason.CloseStatus = CloseStatus;
				ClosedReason.bWasClean = true;
			}
			FEmbeddedCommunication::WakeGameThread();
		}
		else
		{
			UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_WS_PEER_INITIATED_CLOSE, but ignoring because our State=%ls CloseStatus=%d Reason=%ls",
				Identifier, ToString(EState::Closed), (int32)CloseStatus, *CloseReasonString);
		}
		// TODO:  Confirm what we want to return here.  non-zero will close the socket immediately, zero will cause libwebsockets to mirror the close packet back to the server and wait for a response
		return 1; // Close the connection without waiting for a response
	}
	case LWS_CALLBACK_WSI_DESTROY:
		// Getting a WSI_DESTROY before a connection has been established and no errors reported usually means there was a timeout establishing a connection
		if (State == EState::Connecting)
		{
			FString CloseReasonString("Connection timed out");
			UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_WSI_DESTROY, setting State=%ls PreviousState=%ls", Identifier, ToString(EState::Error), ToString(State));
		
			{
				UE::TScopeLock ScopeLock(StateLock);
				State = EState::Error;
				ClosedReason.Reason = MoveTemp(CloseReasonString);
			}
			FEmbeddedCommunication::WakeGameThread();
		}
		else
		{
			if (State == EState::Connected)
			{
				{
					UE::TScopeLock ScopeLock(StateLock);
					State = EState::Closed;
					ClosedReason.bWasClean = false;
					ClosedReason.CloseStatus = LWS_CLOSE_STATUS_ABNORMAL_CLOSE;
				}
				FEmbeddedCommunication::WakeGameThread();
			}
			else if (State == EState::ClosingByRequest)
			{		
				FString ClosedReasonString;
				int32 ClosedStatus;

				if (CloseRequest.Code == LWS_CLOSE_STATUS_MESSAGE_TOO_LARGE)
				{
					ClosedReasonString = TEXT("Received text message exceeded memory limit");
					ClosedStatus = LWS_CLOSE_STATUS_MESSAGE_TOO_LARGE;
				}
				else
				{
					ClosedReasonString = TEXT("Successfully closed connection to our peer");
					ClosedStatus = LWS_CLOSE_STATUS_NORMAL;
				}

				{
					UE::TScopeLock ScopeLock(StateLock);
					State = EState::Closed;
					ClosedReason.Reason = MoveTemp(ClosedReasonString);
					ClosedReason.CloseStatus = ClosedStatus;
					ClosedReason.bWasClean = true;
				}
				FEmbeddedCommunication::WakeGameThread();
			}
			UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_WSI_DESTROY, State=%ls", Identifier, ToString(State));
		}
		LwsConnection = nullptr;
		break;
	case LWS_CALLBACK_CLOSED:
	{
		// We only modify our state if we are Connected or ClosingByRequest (effectively connected)
		if (State == EState::Connected ||
			State == EState::ClosingByRequest)
		{
			const bool bClosingByRequest = (State == EState::ClosingByRequest);
			LwsConnection = nullptr;

			static const FString LocalInitiatedReason = "Successfully closed connection to our peer";
			static const FString PeerInitiatedReason = "Connection closed by peer";
			FString CloseReasonString = bClosingByRequest ? LocalInitiatedReason : PeerInitiatedReason;
			UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_CLOSED, setting State=%ls CloseReason=%ls PreviousState=%ls",
				Identifier, ToString(EState::Closed), *CloseReasonString, ToString(State));
			{
				UE::TScopeLock ScopeLock(StateLock);
				State = EState::Closed;
				ClosedReason.Reason = MoveTemp(CloseReasonString);
				ClosedReason.CloseStatus = LWS_CLOSE_STATUS_NORMAL;
				ClosedReason.bWasClean = bClosingByRequest;
			}
			FEmbeddedCommunication::WakeGameThread();
		}
		else
		{
			UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_CLOSED, ignoring and waiting for LWS_CALLBACK_WSI_DESTROY, State=%ls",
				Identifier, ToString(EState::Closed));
		}
		break;
	}
	case LWS_CALLBACK_CLIENT_CONNECTION_ERROR:
	{
		LwsConnection = nullptr;

		FUTF8ToTCHAR Convert((const ANSICHAR*)Data, Length);
		FString CloseReasonString = FString::ConstructFromPtrSize(Convert.Get(), Convert.Length());
		bool bWakeGameThread = false;
		UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsCallback: Received LWS_CALLBACK_CLIENT_CONNECTION_ERROR, setting State=%ls CloseReason=%ls PreviousState=%ls",
			Identifier, ToString(EState::Error), *CloseReasonString, ToString(State));
		{
			UE::TScopeLock ScopeLock(StateLock);
			if (State == EState::Connected ||
				State == EState::ClosingByRequest)
			{
				State = EState::Closed;
				ClosedReason.bWasClean = false;
				ClosedReason.CloseStatus = LWS_CLOSE_STATUS_ABNORMAL_CLOSE;
				bWakeGameThread = true;
			}
			else if (State != EState::Closed)
			{
				State = EState::Error;
				bWakeGameThread = true;
			}
			ClosedReason.Reason = MoveTemp(CloseReasonString);
		}
		if (bWakeGameThread)
		{
			FEmbeddedCommunication::WakeGameThread();
		}

		return -1;
	}
	case LWS_CALLBACK_RECEIVE_PONG:
		break;
	case LWS_CALLBACK_CLIENT_WRITEABLE:
	case LWS_CALLBACK_SERVER_WRITEABLE:
	{
		if (State == EState::ClosingByRequest)
		{
			LwsConnection = nullptr;
			
			int32 CloseCodeCopy;
			ANSICHAR* CloseReasonCopy;
			{
				// TODO:  Is this scope lock necessary?  We only got here because we recognized CloseRequest.Code was valid, and we do not allow it to be modified afer being set on the game thread, so shouldn't this thread have the correct value?
				UE::TScopeLock ScopeLock(StateLock);
				CloseCodeCopy = CloseRequest.Code;
				CloseReasonCopy = CloseRequest.Reason;
			}
			// This only sets the reason for closing the connection
			lws_close_reason(Instance, (enum lws_close_status)CloseCodeCopy, (unsigned char *)CloseReasonCopy, (SIZE_T)FCStringAnsi::Strlen(CloseReasonCopy));
			return -1; // Returning non-zero will close the current connection
		}
		else if (State == EState::Connected)
		{
			SendFromQueue();
		}
		break;
	}
	case LWS_CALLBACK_CLIENT_APPEND_HANDSHAKE_HEADER:
	{
		if (!UpgradeHeader.IsEmpty())
		{
			char** WriteableString = reinterpret_cast<char**>(Data);
			auto UpgradeHeaderAnsi = StringCast<ANSICHAR>(*UpgradeHeader);
			FCStringAnsi::Strncpy(*WriteableString, UpgradeHeaderAnsi.Get(), Length);
			*WriteableString += UpgradeHeaderAnsi.Length();
		}
		break;
	}
#if WITH_SSL
	case LWS_CALLBACK_OPENSSL_PERFORM_SERVER_CERT_VERIFICATION:
	{
		// in FLwsWebSocketsManager::CallbackWrapper, we copied UserData to Data, so Data is the X509_STORE_CTX* instead of the SSL*
		X509_STORE_CTX* Context = static_cast<X509_STORE_CTX*>(Data);
		int PreverifyOk = 1;
		if (!FLwsWebSocketsManager::Get().bDisableCertValidation)
		{
			PreverifyOk = Length;
			if (PreverifyOk == 1)
			{
				const FString Domain = FGenericPlatformHttp::GetUrlDomain(Url);
				if (!FSslModule::Get().GetCertificateManager().VerifySslCertificates(Context, Domain))
				{
					PreverifyOk = 0;
				}
			}
		}
		else
		{
			X509_STORE_CTX_set_error(Context, X509_V_OK);
		}

		return PreverifyOk == 1 ? 0 : 1;
	}
#endif
	default:
		break;
	}

	return 0;
}

void FLwsWebSocket::GameThreadTick()
{
	EState CurrentState;
	{
		UE::TScopeLock ScopeLock(StateLock);
		CurrentState = State;
	}
	if (CurrentState != LastGameThreadState)
	{
		LastGameThreadState = CurrentState;

		// State changed, broadcast events
		if (CurrentState == EState::Connected)
		{
			OnConnected().Broadcast();
		}
	}

	// If we requested a close then we don't care about any messages we receive
	// No lock, only this thread will modify CloseRequest
	if (CloseRequest.Code == 0)
	{
		FLwsReceiveBufferTextPtr BufferText;
		while (ReceiveTextQueue.Dequeue(BufferText))
		{
			OnMessage().Broadcast(BufferText->Text);
		}

		FLwsReceiveBufferBinaryPtr BufferBinary;
		while (ReceiveBinaryQueue.Dequeue(BufferBinary))
		{
			OnRawMessage().Broadcast(BufferBinary->Payload.GetData(), BufferBinary->Payload.Num(), BufferBinary->BytesRemaining);
		}

		FLwsReceiveBufferBinaryFragmentPtr BufferBinaryFragment;
		while (ReceiveBinaryFragmentQueue.Dequeue(BufferBinaryFragment))
		{
			OnBinaryMessage().Broadcast(BufferBinaryFragment->Payload.GetData(), BufferBinaryFragment->Payload.Num(), BufferBinaryFragment->bIsLastFragment);
		}
	}
}

void FLwsWebSocket::GameThreadFinalize()
{
	EState PreviousState;
	FClosedReason LastClosedReason;

	{
		// TODO:  The contract requires the libwebsockets be done with this object prior to this being called.  Is there a better way to ensure we get the last set value by the libwebsockets thread?  FPlatformMisc::MemoryBarrier maybe?
		UE::TScopeLock ScopeLock(StateLock);
		PreviousState = State;
		State = EState::None; // Will be re-usable on final delegate triggering
		LastGameThreadState = State;
		LastClosedReason = MoveTemp(ClosedReason);
	}

	UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::GameThreadFinalize: setting State=%ls PreviousState=%ls",
		Identifier, ToString(EState::None), ToString(PreviousState));

	const bool bWasError = (PreviousState == EState::Error);
	if (bWasError)
	{
		OnConnectionError().Broadcast(LastClosedReason.Reason);
	}
	else
	{
		// TODO:  Do we need to ensure that an OnConnected() was triggered before OnClosed()?
		OnClosed().Broadcast(LastClosedReason.CloseStatus, LastClosedReason.Reason, LastClosedReason.bWasClean);
	}
}

bool FLwsWebSocket::LwsThreadInitialize(struct lws_context &LwsContext)
{
	check(State == EState::StartConnecting);
	ConnectInternal(LwsContext);
	return State == EState::Connecting;
}

void FLwsWebSocket::LwsThreadTick()
{
	if (State == EState::Connected)
	{
		check(LwsConnection);
		{
			// Check if we want to close
			UE::TScopeLock ScopeLock(StateLock);
			if (CloseRequest.Code != 0)
			{
				State = EState::ClosingByRequest;
			}
		}

		if (State == EState::ClosingByRequest)
		{
			UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::LwsThreadTick: Close requested while connected, setting State=%ls PreviousState=%ls",
				Identifier, ToString(EState::ClosingByRequest), ToString(EState::Connected));
			if (bWasSendQueueEmpty)
			{
				lws_callback_on_writable(LwsConnection);
			}
		}
		else
		{
			// When connected, request callback on writeable when we go from not having data to having data
			bool bQueueIsEmpty = SendQueue.IsEmpty();
			if (bWasSendQueueEmpty && !bQueueIsEmpty)
			{
				lws_callback_on_writable(LwsConnection);
			}
			bWasSendQueueEmpty = bQueueIsEmpty;
		}
	}
}

void FLwsWebSocket::ConnectInternal(struct lws_context &LwsContext)
{
	check(!LwsConnection);
	checkf(State == EState::StartConnecting, TEXT("FLwsWebSocket::ConnectInternal: State must be %s, but is %s instead"), ToString(EState::StartConnecting), ToString(State));

	UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::ConnectInternal: setting State=%ls PreviousState=%ls",
		Identifier, ToString(EState::Connecting), ToString(EState::StartConnecting));
	{
		UE::TScopeLock ScopeLock(StateLock);
		State = EState::Connecting;
	}

	FTCHARToUTF8 UrlUtf8(*Url);
	const char *UrlProtocol;
	const char *TmpUrlPath;
	const char* ParsedAddress;
	int ParsedPort;
	if (lws_parse_uri((char*)UrlUtf8.Get(), &UrlProtocol, &ParsedAddress, &ParsedPort, &TmpUrlPath))
	{
		FString Reason("Bad URL");

		UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::ConnectInternal: setting State=%ls PreviousState=%ls Reason=%ls",
			Identifier, ToString(EState::Error), ToString(EState::Connecting), *Reason);
		{
			UE::TScopeLock ScopeLock(StateLock);
			State = EState::Error;
			ClosedReason.Reason = MoveTemp(Reason);
		}
		return;
	}

	int32 UrlPathLength = FCStringAnsi::Strlen(TmpUrlPath) + 2;
	TArray<ANSICHAR> UrlPath;
	UrlPath.Reserve(UrlPathLength);
	UrlPath.Add('/');
	UrlPath.Append(TmpUrlPath, UrlPathLength - 2);
	UrlPath.Add('\0');

	int SslConnection = 0;

	// Use SSL and require a valid cerver cert
	if (FCStringAnsi::Stricmp(UrlProtocol, "wss") == 0)
	{
		SslConnection = 1; // LCCSCF_USE_SSL

		// Use ssl, and allow self-signed certs with bDisableCertValidation.
		if (FLwsWebSocketsManager::Get().bDisableCertValidation)
		{
			SslConnection = 3; // LCCSCF_USE_SSL | LCCSCF_ALLOW_SELFSIGNED;
		}
	}
	// Use SSL, and allow self-signed certs
	else if (FCStringAnsi::Stricmp(UrlProtocol, "wss+insecure") == 0)
	{
		SslConnection = 3; // LCCSCF_USE_SSL | LCCSCF_ALLOW_SELFSIGNED;
	}
	// No encryption
	else if (FCStringAnsi::Stricmp(UrlProtocol, "ws") == 0)
	{
		SslConnection = 0;
	}
	// Else return an error
	else
	{
		FString Reason(FString::Printf(TEXT("Bad protocol '%s'. Use either 'ws', 'wss', or 'wss+insecure'"), UTF8_TO_TCHAR(UrlProtocol)));

		UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::ConnectInternal: setting State=%ls PreviousState=%ls Reason=%ls",
			Identifier, ToString(EState::Error), ToString(EState::Connecting), *Reason);
		{
			UE::TScopeLock ScopeLock(StateLock);
			State = EState::Error;
			ClosedReason.Reason = MoveTemp(Reason);
		}
		return;
	}

	FString ParsedPortString = FString::FromInt(ParsedPort);
	FTCHARToUTF8 ParsedPortStringUTF8(*ParsedPortString);
	// Length is based on format [addr]:[port]\0
	int32 ParsedAddressLength = FCStringAnsi::Strlen(ParsedAddress);
	bool bUsingDefaultPort = (SslConnection == 0 && ParsedPort == 80) || (SslConnection != 0 && ParsedPort == 443);
	int32 HostLength = bUsingDefaultPort ? (ParsedAddressLength + 1) : (ParsedAddressLength + 1 + ParsedPortStringUTF8.Length() + 1); 
	TArray<ANSICHAR> Host;
	Host.Reserve(HostLength);
	Host.Append(ParsedAddress, ParsedAddressLength);
	if (!bUsingDefaultPort)
	{
		Host.Add(':');
		Host.Append(ParsedPortStringUTF8.Get(), ParsedPortStringUTF8.Length());
	}
	Host.Add('\0');

	// If we specify protocols, convert the comma separated list into UTF8
	TOptional<FTCHARToUTF8> OptionalCombinedProtocolsUTF8;
	if (Protocols.Num() > 0)
	{
		const FString CombinedProtocols(FString::Join(Protocols, TEXT(",")));
		OptionalCombinedProtocolsUTF8.Emplace(*CombinedProtocols);
	}

	const char* const CombinedProtocolsUTF8 = OptionalCombinedProtocolsUTF8.IsSet() ? (const char*)OptionalCombinedProtocolsUTF8.GetValue().Get() : nullptr;

	struct lws_client_connect_info ConnectInfo = {};
	ConnectInfo.context = &LwsContext;
	ConnectInfo.address = ParsedAddress;
	ConnectInfo.port = ParsedPort;
	ConnectInfo.ssl_connection = SslConnection;
	ConnectInfo.path = UrlPath.GetData();
	ConnectInfo.host = Host.GetData();
	ConnectInfo.origin = ConnectInfo.address;
	ConnectInfo.protocol = CombinedProtocolsUTF8;
	ConnectInfo.ietf_version_or_minus_one = -1;
	ConnectInfo.userdata = this;

	if (lws_client_connect_via_info(&ConnectInfo))
	{
		UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::ConnectInternal: lws_client_connect_via_info succeeded", Identifier);
	}
	else
	{
		FString Reason("Could not initialize connection");
		
		UE_LOGF(LogWebSockets, Verbose, "FLwsWebSocket[%d]::ConnectInternal: setting State=%ls PreviousState=%ls Reason=%ls",
			Identifier, ToString(EState::Error), ToString(EState::Connecting), *Reason);
		{
			UE::TScopeLock ScopeLock(StateLock);
			State = EState::Error;
			ClosedReason.Reason = MoveTemp(Reason);
		}
	}
}

#endif // #if WITH_WEBSOCKETS
