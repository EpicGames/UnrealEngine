// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "Containers/Queue.h"
#include "Misc/TransactionallySafeCriticalSection.h"

#if WITH_WEBSOCKETS && WITH_LIBWEBSOCKETS

#include "IWebSocket.h"

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include "libwebsockets.h"
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

/** Buffer for one outgoing packet */
struct FLwsSendBuffer
{
	/**
	 * Constructor
	 * @param Data pointer to data to fill our buffer with
	 * @param Size size of Data
	 * @param bInIsBinary Whether or not this should be treated as a binary packet
	 */
	FLwsSendBuffer(const uint8* Data, const SIZE_T Size, const bool bInIsBinary);

	/** 
	 * Get the actual payload size
	 * Payload includes additional room for libwebsockets to use
	 * @return payload size
	 */
	int32 GetPayloadSize() const;

	/**
	 * Whether we have written our entire payload yet or not
	 * @return whether we have written our entire payload yet or not
	 */
	bool IsDone() const { return !HasError() && BytesWritten >= GetPayloadSize(); }

	/**
	 * Whether we have encountered an error or not
	 * @return whether we have encountered an error or not
	 */
	bool HasError() const { return bHasError; }

	/** Whether or not the packet is a binary packet, if not it is treated as a string */
	const bool bIsBinary;
	/** Number of bytes from Payload already written */
	int32 BytesWritten;
	/** Payload of the packet */
	TArray<uint8> Payload;
	/** Has an error occurred while writing? */
	bool bHasError;
};

/** Buffer for one incoming binary packet */
struct FLwsReceiveBufferBinary
{
	/**
	 * Constructor
	 * @param Data pointer to data to fill our buffer with
	 * @param Size size of Data
	 * @param InBytesRemaining Number of bytes remaining in the packet
	 */
	FLwsReceiveBufferBinary(const uint8* Data, const int32 Size, const int32 InBytesRemaining);

	/** Payload received */
	TArray<uint8> Payload;
	/** Number of bytes remaining in the packet */
	const int32 BytesRemaining;
};

typedef TUniquePtr<FLwsReceiveBufferBinary> FLwsReceiveBufferBinaryPtr;

/** Buffer for one incoming binary packet fragment */
struct FLwsReceiveBufferBinaryFragment
{
	/**
	 * Constructor
	 * @param Data pointer to data to fill our buffer with
	 * @param Size size of Data
	 * @param bInIsLastFragment Whether or not this is the last fragment
	 */
	FLwsReceiveBufferBinaryFragment(const uint8* Data, const int32 Size, const bool bInIsLastFragment);

	/** Payload received */
	TArray<uint8> Payload;
	/** Whether or not this is the last fragment */
	const bool bIsLastFragment;
};

typedef TUniquePtr<FLwsReceiveBufferBinaryFragment> FLwsReceiveBufferBinaryFragmentPtr;

/** Buffer for one incoming text packet, fully received */
struct FLwsReceiveBufferText
{
	/**
	 * Constructor
	 * @param InText The packet contents
	 */
	FLwsReceiveBufferText(FString&& InText);

	/** Text packet received */
	const FString Text;
};

typedef TUniquePtr<FLwsReceiveBufferText> FLwsReceiveBufferTextPtr;

class FLwsWebSocketsManager;

class FLwsWebSocket
	: public IWebSocket
	, public TSharedFromThis<FLwsWebSocket>
{

public:
	/** Destructor */
	virtual ~FLwsWebSocket();

	// IWebSocket
	virtual void Connect() override;
	virtual void Close(const int32 Code = 1000, const FString& Reason = FString()) override;

	virtual bool IsConnected() override
	{
		return LastGameThreadState == EState::Connected || LastGameThreadState == EState::ClosingByRequest;
	}

	virtual void Send(const FString& Data);
	virtual void Send(const void* Data, SIZE_T Size, bool bIsBinary) override;
	virtual void SetTextMessageMemoryLimit(uint64 TextMessageMemoryLimit) override;

	/** Delegate called when a web socket connection has been established */
	DECLARE_DERIVED_EVENT(FLwsWebSocket, IWebSocket::FWebSocketConnectedEvent, FWebSocketConnectedEvent);
	virtual FWebSocketConnectedEvent& OnConnected() override
	{
		return ConnectedEvent;
	}

	/** Delegate called when a web socket connection has errored */
	DECLARE_DERIVED_EVENT(FLwsWebSocket, IWebSocket::FWebSocketConnectionErrorEvent, FWebSocketConnectionErrorEvent);
	virtual FWebSocketConnectionErrorEvent& OnConnectionError() override
	{
		return ConnectionErrorEvent;
	}

	/** Delegate called when a web socket connection has been closed */
	DECLARE_DERIVED_EVENT(FLwsWebSocket, IWebSocket::FWebSocketClosedEvent, FWebSocketClosedEvent);
	virtual FWebSocketClosedEvent& OnClosed() override
	{
		return ClosedEvent;
	}

	/** Delegate called when a web socket text message has been received */
	DECLARE_DERIVED_EVENT(FLwsWebSocket, IWebSocket::FWebSocketMessageEvent, FWebSocketMessageEvent);
	virtual FWebSocketMessageEvent& OnMessage() override
	{
		return MessageEvent;
	}

	/** Delegate called when a web socket binary message has been received */
	DECLARE_DERIVED_EVENT(FLwsWebSocket, IWebSocket::FWebSocketBinaryMessageEvent, FWebSocketBinaryMessageEvent);
	virtual FWebSocketBinaryMessageEvent& OnBinaryMessage() override
	{
		return BinaryMessageEvent;
	}

	/** Delegate called when a any web socket message has been received */
	DECLARE_DERIVED_EVENT(FLwsWebSocket, IWebSocket::FWebSocketRawMessageEvent, FWebSocketRawMessageEvent);
	virtual FWebSocketRawMessageEvent& OnRawMessage() override
	{
		return RawMessageEvent;
	}


	DECLARE_DERIVED_EVENT(FLwsWebSocket, IWebSocket::FWebSocketMessageSentEvent, FWebSocketMessageSentEvent);
	virtual FWebSocketMessageSentEvent& OnMessageSent() override
	{
		return OnMessageSentEvent;
	}

	/** Callback on events for our libwebsockets connection */
	int LwsCallback(lws* Instance, lws_callback_reasons Reason, void* Data, size_t Length);

	/**
	 * Tick on the game thread
	 */
	void GameThreadTick();

	/** 
	 * Handle being removed the game thread - trigger our OnClosed/OnConnectionError delegate
	 */
	void GameThreadFinalize();

	/** 
	 * Setup to be run on the libwebsockets thread
	 * @param LwsContext libwebsockets context, should only be used when creating a libwebsockets socket
	 * @return true if we successfully initialized, false if we are done (should not be ticked on the libwebsockets thread)
	 */
	bool LwsThreadInitialize(struct lws_context &LwsContext);

	/**
	 * Tick on the libwebsockets thread
	 */
	void LwsThreadTick();

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

	friend class FLwsWebSocketsManager;

public:
	/** Constructor */
	FLwsWebSocket(FPrivateToken, const FString& Url, const TArray<FString>& Protocols, const FString& UpgradeHeader, uint64 TextMessageMemoryLimit);

private:
	/**
	 * Start connecting
	 * @param LwsContext libwebsockets context
	 */
	void ConnectInternal(struct lws_context &LwsContext);

	/** Send data from our pending outgoing queue */
	void SendFromQueue();

	/** Clear our queues (does not send / process) */
	void ClearData();

	/**
	 * Write the specified send buffer
	 * @return true if the write was successful (not necessarily finished), false if an error occurred
	 */
	bool WriteBuffer(FLwsSendBuffer& Buffer);

private:

	/** Critical section to lock access to state and close request variables */
	FTransactionallySafeCriticalSection StateLock;

	/** Possible state values */
	enum class EState : uint8
	{
		/** Constructed, nothing to do */
		None,
		/** Awaiting connection start */
		StartConnecting,
		/** Connecting */
		Connecting,
		/** Connected */
		Connected,
		/** Closing (self initiated) */
		ClosingByRequest,
		/** Closed */
		Closed,
		/** Errored, nothing to do */
		Error,
	};
	
	/** Get string representation of EState */
	static const TCHAR* ToString(const EState InState);

	/** Structure containing reason for entering the close/error state */
	struct FClosedReason
	{
		/** Descriptive reason for the state change */
		FString Reason;
		/** Close status (for State=Closed) */
		uint16 CloseStatus;
		/** Was close clean? (for State=Closed) */
		bool bWasClean;
	};

	/** Our current state */
	EState State;
	/** Reason for changing state */
	FClosedReason ClosedReason;
	/** Last state seen on the game thread */
	EState LastGameThreadState;

	/** Parameters from a close request from the owner of this web socket */
	struct FCloseRequest
	{
		/** Constructor */
		FCloseRequest() : Code(0), Reason(nullptr) {}

		/** Code specified when calling Close() */
		int32 Code;
		/** Reason specified when calling Close(), converted to ANSICHAR for libwebsockets */
		ANSICHAR *Reason;
	};

	/** Close request from the owner of this web socket, to be processed on the libwebsockets thread */
	FCloseRequest CloseRequest;

	/** Was the send queue empty last time we checked it on our thread? */
	bool bWasSendQueueEmpty;

	// Events
	FWebSocketConnectedEvent ConnectedEvent;
	FWebSocketConnectionErrorEvent ConnectionErrorEvent;
	FWebSocketClosedEvent ClosedEvent;
	FWebSocketMessageEvent MessageEvent;
	FWebSocketBinaryMessageEvent BinaryMessageEvent;
	FWebSocketRawMessageEvent RawMessageEvent;
	FWebSocketMessageSentEvent OnMessageSentEvent;

	/** libwebsockets connection */
	struct lws *LwsConnection;
	/** Url we are connecting to */
	FString Url;
	/** Protocols to use with this connection */
	TArray<FString> Protocols;
	/** The upgrade header(s) to send with the upgrade request */
	FString UpgradeHeader;

	/**
	 * Whether or not OnMessage was bound to when Connect() was called.
	 * For performance reasons if nothing was bound at Connect() time, we will never trigger OnMessage
	 */
	bool bWantsMessageEvents;
	/**
	 * Whether or not OnRawMessage was bound to when Connect() was called.
	 * For performance reasons if nothing was bound at Connect() time, we will never trigger OnRawMessage
	 */
	bool bWantsRawMessageEvents;
	/**
	 * Whether or not OnBinaryMessage was bound to when Connect() was called.
	 * For performance reasons if nothing was bound at Connect() time, we will never trigger OnBinaryMessage
	 */
	bool bWantsBinaryMessageEvents;


	/** Buffer of an incomplete packet received */
	FString ReceiveBuffer;
	/** Received binary packets, waiting for delegates to be triggered on the game thread */
	TQueue<FLwsReceiveBufferBinaryPtr, EQueueMode::Spsc> ReceiveBinaryQueue;
	/** Received fragments of binary packets, waiting for delegates to be triggered on the game thread */
	TQueue<FLwsReceiveBufferBinaryFragmentPtr, EQueueMode::Spsc> ReceiveBinaryFragmentQueue;
	/** Received text packets, waiting for delegates to be triggered on the game thread */
	TQueue<FLwsReceiveBufferTextPtr, EQueueMode::Spsc> ReceiveTextQueue;
	/** Pending outgoing packets, populated by the game thread and processed on the libwebsockets thread */
	TQueue<FLwsSendBuffer*, EQueueMode::Spsc> SendQueue;

	uint64 MaxTextMessageBufferSize;

	// Unique identifier for logging
	/** Incrementing identifier to give each web socket a unique identifier */
	static int32 IncrementingIdentifier;
	/** Our unique identifier */
	const int32 Identifier;
};

typedef TSharedRef<FLwsWebSocket> FLwsWebSocketRef;

#endif
