// Copyright Epic Games, Inc. All Rights Reserved.

#include "LwsWebSocketsManager.h"

#if WITH_WEBSOCKETS && WITH_LIBWEBSOCKETS

#include "WebSocketsModule.h"
#include "HAL/RunnableThread.h"
#if WITH_SSL
#include "Ssl.h"
#endif
#include "PlatformHttp.h"
#include "WebSocketsLog.h"
#include "Containers/BackgroundableTicker.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Fork.h"
#include "Stats/Stats.h"

LLM_DEFINE_TAG(WebSockets);

namespace {
	static const struct lws_extension LwsExtensions[] = {
		{
			"permessage-deflate",
			lws_extension_callback_pm_deflate,
			"permessage-deflate; client_max_window_bits"
		},
		{
			"deflate-frame",
			lws_extension_callback_pm_deflate,
			"deflate_frame"
		},
		// zero terminated:
		{ nullptr, nullptr, nullptr }
	};
}

namespace UE::LwsWebSocketsManager
{
	// We declare these explicitly rather than just casting in case the enum changes in future
	const int32 ThreadPriorities[] =
	{
		EThreadPriority::TPri_Lowest,
		EThreadPriority::TPri_BelowNormal,
		EThreadPriority::TPri_SlightlyBelowNormal,
		EThreadPriority::TPri_Normal,
		EThreadPriority::TPri_AboveNormal
	};
}

static void LwsLog(int Level, const char* LogLine);

// FLwsWebSocketsManager
FLwsWebSocketsManager::FLwsWebSocketsManager()
	: 
#if WITH_SSL
	SslContext(nullptr),
#endif
	LwsContext(nullptr)
	, Thread(nullptr)
{
}

FLwsWebSocketsManager& FLwsWebSocketsManager::Get()
{
	FLwsWebSocketsManager* Manager = static_cast<FLwsWebSocketsManager*>(FWebSocketsModule::Get().WebSocketsManager);
	check(Manager);
	return *Manager;
}

void FLwsWebSocketsManager::UpdateConfigs()
{
	const bool bWasPollingService = bPollService;
	// For forked dedicated servers, the parent process may be running in single threaded mode, which will force bPollService to true below
	// Restore bPollService to its code defined default to avoid the code specified default not applying to a post-fork multithreaded server
#if UE_WEBSOCKETS_PLATFORM_SUPPORT_EVENT_LOOP
	bPollService = false;
#else
	bPollService = true;
#endif
	GConfig->GetBool(TEXT("WebSockets.LibWebSockets"), TEXT("bPollService"), bPollService, GEngineIni);
	GConfig->GetInt(TEXT("WebSockets.LibWebSockets"), TEXT("ServiceTimeoutMs"), ServiceTimeoutMs, GEngineIni);
	GConfig->GetDouble(TEXT("WebSockets.LibWebSockets"), TEXT("ThreadTargetFrameTimeInSeconds"), ThreadTargetFrameTimeInSeconds, GEngineIni);
	GConfig->GetDouble(TEXT("WebSockets.LibWebSockets"), TEXT("ThreadMinimumSleepTimeInSeconds"), ThreadMinimumSleepTimeInSeconds, GEngineIni);
	
	GConfig->GetBool(TEXT("LwsWebSocket"), TEXT("bDisableDomainAllowlist"), bDisableDomainAllowlist, GEngineIni);
	
	GConfig->GetBool(TEXT("LwsWebSocket"), TEXT("bDisableCertValidation"), bDisableCertValidation, GEngineIni);

#if !UE_BUILD_SHIPPING
	// Allow -lws-insecure= to override ini flags
	FParse::Bool(FCommandLine::Get(), TEXT("lws-insecure="), bDisableCertValidation);
#endif

	// Check if it's supported on platform
#if !UE_WEBSOCKETS_PLATFORM_SUPPORT_EVENT_LOOP
	if (!bPollService)
	{
		UE_LOGF(LogWebSockets, Warning, "FLwsWebSocketsManager::UpdateConfigs: Service polling disabled by config while it's not supported on platform. Forcing service polling to avoid blocking service call");
		bPollService = true;
	}
#endif

	// Check blocking service calls are supported if requested
	if (!bPollService)
	{
		// Ensure multi-threading is supported otherwise 
		const bool bSupportsMultithreading = FPlatformProcess::SupportsMultithreading() || FForkProcessHelper::IsForkedMultithreadInstance();
		if (!bSupportsMultithreading)
		{
			UE_LOGF(LogWebSockets, Warning, "FLwsWebSocketsManager::UpdateConfigs: Service polling disabled by config in single threaded environment. Forcing service polling to avoid blocking service call");
			bPollService = true;
		}
	}
	
	if (bWasPollingService != bPollService)
	{
		if (bPollService)
		{
			UE_LOGF(LogWebSockets, Log, "FLwsWebSocketsManager::UpdateConfigs: Service polling enabled by config");

			if (LwsContext)
			{
				// Cancel blocking service call in FLwsWebSocketsManager::Tick to allow sleep, service, sleep polling loop to start in
				// FLwsWebSocketsManager::Run   
				lws_cancel_service(LwsContext);
			}
		}
		else
		{
			UE_LOGF(LogWebSockets, Log, "FLwsWebSocketsManager::UpdateConfigs: Service polling disabled by config");
		}
	}
}

void FLwsWebSocketsManager::InitWebSockets(TArrayView<const FString> Protocols)
{
	UE_LOGF(LogWebSockets, Log, "FLwsWebSocketsManager::InitWebSockets");
	
	// Pull initial config values. UpdateConfigs may be called again later 
	UpdateConfigs();
	
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FLwsWebSocketsManager_InitWebSockets);
	check(!Thread && LwsProtocols.Num() == 0);

	LwsProtocols.Reserve(Protocols.Num() + 1);
	for (const FString& Protocol : Protocols)
	{
		FTCHARToUTF8 ConvertName(*Protocol);

		// We need to hold on to the converted strings
		ANSICHAR* Converted = static_cast<ANSICHAR*>(FMemory::Malloc(ConvertName.Length() + 1));
		FCStringAnsi::Strncpy(Converted, (const ANSICHAR*)ConvertName.Get(), ConvertName.Length() + 1);
		lws_protocols LwsProtocol;
		FMemory::Memset(&LwsProtocol, 0, sizeof(LwsProtocol));
		LwsProtocol.name = Converted;
		LwsProtocol.callback = &FLwsWebSocketsManager::StaticCallbackWrapper;
		LwsProtocol.per_session_data_size = 0;	// libwebsockets has two methods of specifying userdata that is used in callbacks
												// we can set it ourselves (during lws_client_connect_via_info - we do this, or via lws_set_wsi_user)
												// or libwebsockets can allocate memory for us using this parameter.  We want to set it ourself, so set this to 0.
		LwsProtocol.rx_buffer_size = 65536; // Largest frame size we support
		
		LwsProtocols.Emplace(MoveTemp(LwsProtocol));
	}

	// LWS requires a zero terminator (we don't pass the length)
	LwsProtocols.Add({ nullptr, nullptr, 0, 0 });

	// Subscribe to log events.  Everything except LLL_PARSER
	static_assert(LLL_COUNT == 11, "If LLL_COUNT increases, libwebsockets has added new log categories, analyze if we should be listening to them");
	lws_set_log_level(LLL_ERR | LLL_WARN | LLL_NOTICE | LLL_INFO | LLL_DEBUG | LLL_HEADER | LLL_EXT | LLL_CLIENT | LLL_LATENCY | LLL_USER, &LwsLog);

	struct lws_context_creation_info ContextInfo = {};

	ContextInfo.port = CONTEXT_PORT_NO_LISTEN;
	ContextInfo.protocols = LwsProtocols.GetData();
	ContextInfo.uid = -1;
	ContextInfo.gid = -1;
	ContextInfo.options |= LWS_SERVER_OPTION_PEER_CERT_NOT_REQUIRED | LWS_SERVER_OPTION_DISABLE_OS_CA_CERTS | LWS_SERVER_OPTION_DO_SSL_GLOBAL_INIT;
	ContextInfo.max_http_header_data = 0;

	int32 MaxHttpHeaderData = 1024 * 32;
	GConfig->GetInt(TEXT("WebSockets.LibWebSockets"), TEXT("MaxHttpHeaderData"), MaxHttpHeaderData, GEngineIni);
	ContextInfo.max_http_header_data2 = MaxHttpHeaderData;
	ContextInfo.pt_serv_buf_size = MaxHttpHeaderData;
	
	int32 PingPongInterval = 0;
	GConfig->GetInt(TEXT("WebSockets.LibWebSockets"), TEXT("PingPongInterval"), PingPongInterval, GEngineIni);
	ContextInfo.ws_ping_pong_interval = PingPongInterval;

	// HTTP proxy — read using same precedence as FHttpModule: command-line, INI, OS proxy
	FString ProxyAddress = FPlatformHttp::GetConfiguredProxyAddress();
	TOptional<FTCHARToUTF8> Converter;
	if (!ProxyAddress.IsEmpty())
	{
		Converter.Emplace(*ProxyAddress);
		ContextInfo.http_proxy_address = (const char*)Converter->Get();
	}

#if WITH_SSL
	// SSL client options
	// Create a context for SSL
	FSslModule& SslModule = FModuleManager::LoadModuleChecked<FSslModule>("SSL");
	ISslManager& SslManager = SslModule.GetSslManager();
	if (SslManager.InitializeSsl())
	{
		SslContext = SslManager.CreateSslContext(FSslContextCreateOptions());
		ContextInfo.provided_client_ssl_ctx = SslContext;
		ContextInfo.options &= ~(LWS_SERVER_OPTION_DO_SSL_GLOBAL_INIT); // Do not need to globally init
	}
	else
#endif
	{
		ContextInfo.options |= LWS_SERVER_OPTION_DO_SSL_GLOBAL_INIT;
	}

	if (!ContextInfo.provided_client_ssl_ctx)
	{
		UE_LOGF(LogWebSockets, Verbose, "Failed to create our SSL context, this will result in libwebsockets managing its own SSL context, which calls SSLs global cleanup functions, impacting other uses of SSL");
	}
	
	// Extensions
	// TODO:  Investigate why enabling LwsExtensions prevents us from receiving packets larger than 1023 bytes, and also why lws_remaining_packet_payload returns 0 in that case
	ContextInfo.extensions = nullptr;// LwsExtensions;

	LwsContext = lws_create_context(&ContextInfo);
	if (LwsContext == nullptr)
	{
		UE_LOGF(LogWebSockets, Error, "Failed to initialize libwebsockets");
		return;
	}
	
	ExitRequest.Set(false);
	int32 ThreadStackSize = 128 * 1024;
	GConfig->GetInt(TEXT("WebSockets.LibWebSockets"), TEXT("ThreadStackSize"), ThreadStackSize, GEngineIni);

	int32 ThreadPriorityIndex = 1; // TPri_BelowNormal
	GConfig->GetInt(TEXT("WebSockets.LibWebSockets"), TEXT("ThreadPriority"), ThreadPriorityIndex, GEngineIni);

	const EThreadPriority ThreadPriority = static_cast<EThreadPriority>(UE::LwsWebSocketsManager::ThreadPriorities[FMath::Clamp(ThreadPriorityIndex, 0, UE_ARRAY_COUNT(UE::LwsWebSocketsManager::ThreadPriorities) - 1)]);

	Thread = FForkProcessHelper::CreateForkableThread(this, TEXT("LibwebsocketsThread"), 128 * 1024, ThreadPriority, FPlatformAffinity::GetNoAffinityMask());
	if (!Thread)
	{
		UE_LOGF(LogWebSockets, Error, "FLwsWebSocketsManager failed to initialize thread!");
		lws_context_destroy(LwsContext);
		LwsContext = nullptr;
		return;
	}

	// Setup our game thread tick
	FTickerDelegate TickDelegate = FTickerDelegate::CreateRaw(this, &FLwsWebSocketsManager::GameThreadTick);
	TickHandle = FTSBackgroundableTicker::GetCoreTicker().AddTicker(TickDelegate, 0.0f);

	// If we get forked, we may want to enable / disable service polling
	if (FForkProcessHelper::IsForkRequested())
	{
		if (!OnChildEndFramePostForkHandle.IsValid())
		{
			OnChildEndFramePostForkHandle = FCoreDelegates::OnChildEndFramePostFork.AddLambda([this, ProtocolsCopy = TArray<FString>(Protocols)]()
				{
					// We save a copy of the delegate handle and then reset the original (which stops this lambda from being invalidated)
					FDelegateHandle SavedOnChildEndFramePostForkHandle = OnChildEndFramePostForkHandle;
					OnChildEndFramePostForkHandle.Reset();

					ShutdownWebSockets();

					// Now we restore the original delegate handle before we reinitialize to prevent it from duplicating this lambda
					OnChildEndFramePostForkHandle = SavedOnChildEndFramePostForkHandle;

					InitWebSockets(ProtocolsCopy);
				});
		}
	}
}

void FLwsWebSocketsManager::ShutdownWebSockets()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FLwsWebSocketsManager_ShutdownWebSockets);
	
	if (TickHandle.IsValid())
	{
		FTSBackgroundableTicker::GetCoreTicker().RemoveTicker(TickHandle);
		TickHandle.Reset();
	}

	if (OnChildEndFramePostForkHandle.IsValid())
	{
		FCoreDelegates::OnChildEndFramePostFork.Remove(OnChildEndFramePostForkHandle);
		OnChildEndFramePostForkHandle.Reset();
	}

	if (Thread)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}

	if (LwsContext)
	{
		lws_context_destroy(LwsContext);
		LwsContext = nullptr;
	}

	// Cleanup our allocated protocols
	for (const lws_protocols& Protocol : LwsProtocols)
	{
		FMemory::Free(const_cast<ANSICHAR*>(Protocol.name));
	}
	LwsProtocols.Reset();

	SocketsToStart.Empty();
	SocketsToStop.Empty(); // TODO:  Should we trigger the OnClosed/OnConnectionError delegates?
	Sockets.Empty();

#if WITH_SSL
	FSslModule& SslModule = FModuleManager::LoadModuleChecked<FSslModule>("SSL");
	ISslManager& SslManager = SslModule.GetSslManager();

	if (SslContext)
	{
		SslManager.DestroySslContext(SslContext);
		SslContext = nullptr;
	}

	SslManager.ShutdownSsl();
#endif
}

bool FLwsWebSocketsManager::Init()
{
	return true;
}

uint32 FLwsWebSocketsManager::Run()
{
	while (!ExitRequest.GetValue())
	{
		double BeginTime = FPlatformTime::Seconds();
		Tick();
		double EndTime = FPlatformTime::Seconds();
		double TotalTime = EndTime - BeginTime;

		if (bPollService)
		{
			double SleepTime = FMath::Max(ThreadTargetFrameTimeInSeconds - TotalTime, ThreadMinimumSleepTimeInSeconds);
			FPlatformProcess::SleepNoStats(SleepTime);
		}
	}

	return 0;
}

void FLwsWebSocketsManager::Stop()
{
	ExitRequest.Set(true);

	if (LwsContext)
	{
		WakeService();
	}
}

void FLwsWebSocketsManager::Exit()
{
	for (FLwsWebSocket* Socket : SocketsTickingOnThread)
	{
		SocketsToStop.Enqueue(Socket);
	}
	SocketsTickingOnThread.Empty();
}

static inline bool LwsLogLevelIsWarning(int Level)
{
	return Level == LLL_ERR ||
		Level == LLL_WARN;
}

static inline const TCHAR* LwsLogLevelToString(int Level)
{
	switch (Level)
	{
	case LLL_ERR: return TEXT("Error");
	case LLL_WARN: return TEXT("Warning");
	case LLL_NOTICE: return TEXT("Notice");
	case LLL_INFO: return TEXT("Info");
	case LLL_DEBUG: return TEXT("Debug");
	case LLL_PARSER: return TEXT("Parser");
	case LLL_HEADER: return TEXT("Header");
	case LLL_EXT: return TEXT("Ext");
	case LLL_CLIENT: return TEXT("Client");
	case LLL_LATENCY: return TEXT("Latency");
	}
	return TEXT("Invalid");
}

static void LwsLog(int Level, const char* LogLine)
{
	bool bIsWarning = LwsLogLevelIsWarning(Level);
	if (bIsWarning || UE_LOG_ACTIVE(LogWebSockets, Verbose))
	{
		FUTF8ToTCHAR Converter(LogLine);
		// Trim trailing newline
		FString ConvertedLogLine(Converter.Get());
		if (ConvertedLogLine.EndsWith(TEXT("\n")))
		{
			ConvertedLogLine[ConvertedLogLine.Len() - 1] = TEXT(' ');
		}
		if (bIsWarning)
		{
			UE_LOGF(LogWebSockets, Warning, "Lws(%ls): %ls", LwsLogLevelToString(Level), *ConvertedLogLine);
		}
		else
		{
			UE_LOGF(LogWebSockets, Verbose, "Lws(%ls): %ls", LwsLogLevelToString(Level), *ConvertedLogLine);
		}
	}
}

int FLwsWebSocketsManager::StaticCallbackWrapper(lws* Connection, lws_callback_reasons Reason, void* UserData, void* Data, size_t Length)
{
	FLwsWebSocketsManager& This = FLwsWebSocketsManager::Get();
	return This.CallbackWrapper(Connection, Reason, UserData, Data, Length);
}

int FLwsWebSocketsManager::CallbackWrapper(lws* Connection, lws_callback_reasons Reason, void* UserData, void* Data, size_t Length)
{
	FLwsWebSocket* Socket = static_cast<FLwsWebSocket*>(UserData);

	switch (Reason)
	{
	case LWS_CALLBACK_RECEIVE_PONG:
		return 0;
#if WITH_SSL
	case LWS_CALLBACK_OPENSSL_LOAD_EXTRA_CLIENT_VERIFY_CERTS:
	case LWS_CALLBACK_OPENSSL_LOAD_EXTRA_SERVER_VERIFY_CERTS:
	{
		FSslModule::Get().GetCertificateManager().AddCertificatesToSslContext(static_cast<SSL_CTX*>(UserData));
		return 0;
	}
	case LWS_CALLBACK_OPENSSL_PERFORM_SERVER_CERT_VERIFICATION:
	{
		// LWS reuses the UserData param for the X509_STORE_CTX, so need to grab the socket from the lws connection user data
		Socket = static_cast<FLwsWebSocket*>(lws_wsi_user(Connection));
		// We only care about the X509_STORE_CTX* (UserData), and not the SSL* (Data)
		Data = UserData;

		// Call the socket's LwsCallback below
		break;
	}
#endif
	case LWS_CALLBACK_WSI_DESTROY:
	{
		SocketsDestroyedDuringService.Add(Socket);
		break;
	}
	case LWS_CALLBACK_CLIENT_ESTABLISHED:
	case LWS_CALLBACK_CLIENT_RECEIVE:
	case LWS_CALLBACK_WS_PEER_INITIATED_CLOSE:
	case LWS_CALLBACK_CLOSED:
	case LWS_CALLBACK_CLIENT_CONNECTION_ERROR:
	case LWS_CALLBACK_CLIENT_WRITEABLE:
	case LWS_CALLBACK_SERVER_WRITEABLE:
	case LWS_CALLBACK_CLIENT_APPEND_HANDSHAKE_HEADER:
		break;
	default:
		// Only process the callback reasons defined below this block
		return 0;
	}

	return Socket->LwsCallback(Connection, Reason, Data, Length);
}

void FLwsWebSocketsManager::Tick()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FLwsWebSocketsManager_Tick);
	LLM_SCOPE_BYTAG(WebSockets);

	{
		FLwsWebSocket* SocketToStart;
		while (SocketsToStart.Dequeue(SocketToStart))
		{
			if (LwsContext && SocketToStart->LwsThreadInitialize(*LwsContext))
			{
				SocketsTickingOnThread.Add(SocketToStart);
			}
			else
			{
				// avoid destroying it twice
				if (!SocketsDestroyedDuringService.Contains(SocketToStart))
				{
					SocketsToStop.Enqueue(SocketToStart);
				}
			}
		}
	}
	for (FLwsWebSocket* Socket : SocketsTickingOnThread)
	{
		Socket->LwsThreadTick();
	}
	if (LwsContext)
	{
		lws_service(LwsContext, bPollService ? 0 : ServiceTimeoutMs);
	}
	for (FLwsWebSocket* Socket : SocketsDestroyedDuringService)
	{
		SocketsTickingOnThread.RemoveSwap(Socket);
		SocketsToStop.Enqueue(Socket);
	}
	SocketsDestroyedDuringService.Empty();
}

TSharedRef<IWebSocket> FLwsWebSocketsManager::CreateWebSocket(const FString& Url, const TArray<FString>& Protocols, const TMap<FString, FString>& UpgradeHeaders)
{
	FString UpgradeHeaderString;
	for (const auto& OneHeader : UpgradeHeaders)
	{
		UpgradeHeaderString += FString::Printf(TEXT("%s: %s\r\n"), *OneHeader.Key, *OneHeader.Value);
	}

	// default memory limit for IWebSocket text messages
	int TextMessageMemoryLimit = 1024 * 1024;
	GConfig->GetInt(TEXT("WebSockets"), TEXT("TextMessageMemoryLimit"), TextMessageMemoryLimit, GEngineIni);

	FLwsWebSocketRef Socket = MakeShared<FLwsWebSocket>(FLwsWebSocket::FPrivateToken{}, Url, Protocols, UpgradeHeaderString, TextMessageMemoryLimit);
	return Socket;
}

void FLwsWebSocketsManager::StartProcessingWebSocket(FLwsWebSocket* Socket)
{
	Sockets.Emplace(Socket->AsShared());
	SocketsToStart.Enqueue(Socket);
	WakeService();
}

void FLwsWebSocketsManager::WakeService()
{
	if (bPollService)
	{
		return;
	}
	
	// Safe to call from other threads
	lws_cancel_service(LwsContext);
}

bool FLwsWebSocketsManager::GameThreadTick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FLwsWebSocketsManager_GameThreadTick);

	for (const FLwsWebSocketRef& Socket : Sockets)
	{
		Socket->GameThreadTick();
	}
	{
		FLwsWebSocket* Socket;
		while (SocketsToStop.Dequeue(Socket))
		{
			// Add reference then remove from Sockets, so the final callback delegates can let the owner immediately re-use the socket
			TSharedRef<FLwsWebSocket> SocketRef(Socket->AsShared());
			Sockets.RemoveSwap(SocketRef);
			// Trigger final delegates.
			Socket->GameThreadFinalize();
		}
	}
	return true;
}

#endif // #if WITH_WEBSOCKETS
