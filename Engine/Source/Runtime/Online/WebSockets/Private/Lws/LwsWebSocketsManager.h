// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBSOCKETS && WITH_LIBWEBSOCKETS

#include "IWebSocketsManager.h"
#include "LwsWebSocket.h"
#include "HAL/Runnable.h"
#include "Misc/SingleThreadRunnable.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/Ticker.h"

#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include "libwebsockets.h"
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#	include "Windows/HideWindowsPlatformTypes.h"
#endif

class FRunnableThread;

class FLwsWebSocketsManager
	: public IWebSocketsManager
	, public FRunnable
	, public FSingleThreadRunnable

{
public:
	/** Default constructor */
	FLwsWebSocketsManager();

	static FLwsWebSocketsManager& Get();

	/**
	 * Start processing a websocket on our thread.  Called by FLwsWebSocket on the game thread.
	 * @param Socket websocket to process on our thread
	 */
	void StartProcessingWebSocket(FLwsWebSocket* Socket);

	/**
	 * Trigger/Wake up the websocket thread to process new requests.
	 * Note: This is a no-op if bPollService = true
	 */
	void WakeService();

	// IWebSocketsManager
	virtual void InitWebSockets(TArrayView<const FString> Protocols) override;
	virtual void ShutdownWebSockets() override;
	virtual TSharedRef<IWebSocket> CreateWebSocket(const FString& Url, const TArray<FString>& Protocols, const TMap<FString, FString>& UpgradeHeaders) override;
	virtual void UpdateConfigs() override;

private:

	//~ Begin FRunnable Interface
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;
	//~ End FRunnable Interface
	
	// FSingleThreadRunnable
	/**
	 * FSingleThreadRunnable accessor for ticking this FRunnable when multi-threading is disabled.
 	 * @return FSingleThreadRunnable Interface for this FRunnable object.
	 */
	virtual class FSingleThreadRunnable* GetSingleThreadInterface() override { return this; }

	virtual void Tick() override;

	// FLwsWebSocketsManager
	/** Game thread tick to flush events etc */
	bool GameThreadTick(float DeltaTime);
	/** Static callback on events for a libwebsockets connection */
	static int StaticCallbackWrapper(lws* Connection, lws_callback_reasons Reason, void* UserData, void* Data, size_t Length);
	/** Callback on events for a libwebsockets connection */
	int CallbackWrapper(lws* Connection, lws_callback_reasons Reason, void* UserData, void* Data, size_t Length);

#if WITH_SSL
	/** OpenSSL context */
	SSL_CTX* SslContext;
#endif
	/** libwebsockets context */
	lws_context* LwsContext;
	/** array of protocols that we have registered with libwebsockets */
	TArray<lws_protocols> LwsProtocols;

	/** Array of all WebSockets we know about.  Shared ref count modifications only occurs on the game thread */
	TArray<TSharedRef<FLwsWebSocket>> Sockets;
	/** Array of all WebSockets we are ticking on our thread */
	TArray<FLwsWebSocket*> SocketsTickingOnThread;
	/** Queue of WebSockets to start processing on our thread */
	TQueue<FLwsWebSocket*> SocketsToStart;
	/** Queue of WebSockets that we are done processing on our thread and want to be removed from our Sockets array */
	TQueue<FLwsWebSocket*> SocketsToStop;
	/** Array of WebSockets destroyed during our call to lws_service, to be added to SocketsToStop after lws_service completes */
	TArray<FLwsWebSocket*> SocketsDestroyedDuringService;

	/** Delegate for callbacks to GameThreadTick */
	FTSTicker::FDelegateHandle TickHandle;

	/** Delegate handle for FCoreDelegates::OnChildEndFramePostFork */
	FDelegateHandle OnChildEndFramePostForkHandle;

	// Thread variables
	/** Pointer to Runnable Thread */
	FRunnableThread* Thread;

	/** signal request to stop and exit thread */
	FThreadSafeCounter ExitRequest;

	/**
	 * If true (default), our thread will explicitly sleep (according to ThreadTargetFrameTimeInSeconds & ThreadMinimumSleepTimeInSeconds) in
	 * between calls to lws_service with 0 timeout.
	 * 
	 * If false, lws_service will use a ServiceTimoutMs timeout, allowing lws to block and handle wake / sleep on activity internally itself.
	 */
#if UE_WEBSOCKETS_PLATFORM_SUPPORT_EVENT_LOOP
	bool bPollService = false;
#else
	bool bPollService = true;
#endif
	
	/** Timeout value in milliseconds to be used by lws_service when bPollService is false */
	int32 ServiceTimeoutMs = MAX_int32;

	/** Target frame time for our thread's tick */
	double ThreadTargetFrameTimeInSeconds = 1.0f / 30.0f; // 30Hz;
	/** Minimum time to sleep in our thread's tick, even if the sleep makes us exceed our target frame time */
	double ThreadMinimumSleepTimeInSeconds = 0.0f;

	bool bDisableDomainAllowlist = false;
	bool bDisableCertValidation = false;

	friend class FLwsWebSocket;
};

#endif // WITH_WEBSOCKETS && WITH_LIBWEBSOCKETS
