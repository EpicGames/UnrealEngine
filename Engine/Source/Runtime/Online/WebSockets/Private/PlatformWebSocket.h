// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if defined(WEBSOCKETS_MANAGER_PLATFORM_INCLUDE)
	#include WEBSOCKETS_MANAGER_PLATFORM_INCLUDE
	typedef WEBSOCKETS_MANAGER_PLATFORM_CLASS FPlatformWebSocketsManager;
#else
	#error "Web Sockets not implemented on this platform yet"
#endif