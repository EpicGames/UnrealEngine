// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebSocketsModule.h"
#include "WebSocketsLog.h"

#if WITH_WEBSOCKETS
#include "PlatformWebSocket.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#endif // #if WITH_WEBSOCKETS

DEFINE_LOG_CATEGORY(LogWebSockets);

// FWebSocketsModule
IMPLEMENT_MODULE(FWebSocketsModule, WebSockets);

FWebSocketsModule* FWebSocketsModule::Singleton = nullptr;

/*static*/ FString FWebSocketsModule::BuildUpgradeHeader(const TMap<FString, FString>& Headers)
{
	FString HeaderString;
	for (const auto& OneHeader : Headers)
	{
		HeaderString += FString::Printf(TEXT("%s: %s\r\n"), *OneHeader.Key, *OneHeader.Value);
	}
	return HeaderString;
}

void FWebSocketsModule::StartupModule()
{
	Singleton = this;

#if WITH_WEBSOCKETS
	// Default configuration values can be found in BaseEngine.ini
	TArray<FString> Protocols;
	GConfig->GetArray(TEXT("WebSockets"), TEXT("WebSocketsProtocols"), Protocols, GEngineIni);
	if (Protocols.IsEmpty())
	{
		Protocols.Add(TEXT("ws"));
		Protocols.Add(TEXT("wss"));
	}

	WebSocketsManager = new FPlatformWebSocketsManager;
	WebSocketsManager->InitWebSockets(Protocols);

	// Subscrive to config changes so we can forward to WebSocketsManager  
	FCoreDelegates::TSOnConfigSectionsChanged().AddRaw(this, &FWebSocketsModule::OnConfigSectionsChanged);
#endif
}

void FWebSocketsModule::ShutdownModule()
{
#if WITH_WEBSOCKETS
	FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);
	
	if (WebSocketsManager)
	{
		WebSocketsManager->ShutdownWebSockets();
		delete WebSocketsManager;
		WebSocketsManager = nullptr;
	}
#endif

	Singleton = nullptr;
}

#if WITH_WEBSOCKETS
void FWebSocketsModule::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (WebSocketsManager)
	{
		WebSocketsManager->UpdateConfigs();
	}
}
#endif

FWebSocketsModule& FWebSocketsModule::Get()
{
	if (nullptr == Singleton)
	{
		check(IsInGameThread());
		FModuleManager::LoadModuleChecked<FWebSocketsModule>("WebSockets");
	}
	check(Singleton);
	return *Singleton;
}

#if WITH_WEBSOCKETS
TSharedRef<IWebSocket> FWebSocketsModule::CreateWebSocket(const FString& Url, const TArray<FString>& Protocols, const TMap<FString, FString>& UpgradeHeaders)
{
	check(WebSocketsManager);

	TArray<FString> ProtocolsCopy = Protocols;
	ProtocolsCopy.RemoveAll([](const FString& Protocol){ return Protocol.IsEmpty(); });	
	TSharedRef<IWebSocket> WebSocket = WebSocketsManager->CreateWebSocket(Url, ProtocolsCopy, UpgradeHeaders);
	OnWebSocketCreated.Broadcast(WebSocket, Protocols, Url);
	
	return WebSocket;
}

TSharedRef<IWebSocket> FWebSocketsModule::CreateWebSocket(const FString& Url, const FString& Protocol, const TMap<FString, FString>& UpgradeHeaders)
{
	TArray<FString> Protocols;
	if (!Protocol.IsEmpty())
	{
		Protocols.Add(Protocol);
	}
	
	return CreateWebSocket(Url, Protocols, UpgradeHeaders);
}
#endif // #if WITH_WEBSOCKETS
