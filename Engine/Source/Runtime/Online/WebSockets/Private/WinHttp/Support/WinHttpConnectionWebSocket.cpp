// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "WinHttp/Support/WinHttpConnectionWebSocket.h"
#include "WinHttp/Support/WinHttpSession.h"
#include "WinHttp/Support/WinHttpErrorHelper.h"
#include "WinHttp/Support/WinHttpWebSocketErrorHelper.h"
#include "WinHttp/Support/WinHttpTypes.h"
#include "Misc/ScopeLock.h"
#include "GenericPlatform/GenericPlatformHttp.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <errhandlingapi.h>
#include <WinHttp.h>

// MS recommends reading in 8kB chunks to match one of their buffer sizes
#define UE_WINHTTP_READ_BUFFER_BYTES (8*1024)

#define UE_WEBSOCKET_CLOSE_NORMAL_CLOSURE 1000
#define UE_WEBSOCKET_CLOSE_APP_FAILURE 1006

void CALLBACK UE_WinHttpWebSocketStatusCallback(HINTERNET hInternet, DWORD_PTR dwContext, DWORD dwInternetStatus, LPVOID lpvStatusInformation, DWORD dwStatusInformationLength)
{
	const EWinHttpCallbackStatus Status = static_cast<EWinHttpCallbackStatus>(dwInternetStatus);
	if (!IsValidStatus(Status))
	{
		UE_LOGF(LogWinHttp, Warning, "WebSocket[%p]: Received unknown WinHttp Status %lu", reinterpret_cast<void*>(dwContext), dwInternetStatus);
		return;
	}

	if (dwContext)
	{
		FWinHttpConnectionWebSocket* const RequestContext = reinterpret_cast<FWinHttpConnectionWebSocket*>(dwContext);
		RequestContext->HandleWebSocketStatusCallback(hInternet, Status, lpvStatusInformation, dwStatusInformationLength);
	}
}

/*static*/
TSharedPtr<FWinHttpConnectionWebSocket, ESPMode::ThreadSafe> FWinHttpConnectionWebSocket::CreateWebSocketConnection(
	FWinHttpSession& InSession,
	const FString& InUrl,
	const TArray<FString>& InProtocols,
	const TMap<FString, FString>& InUpgradeHeaders)
{
	if (!InSession.IsValid())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp WebSocket without an active session");
		return nullptr;
	}
	if (InUrl.IsEmpty())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp WebSocket with an empty url");
		return nullptr;
	}

	const bool bIsSecure = FGenericPlatformHttp::IsSecureProtocol(InUrl).Get(false);
	if (!bIsSecure && InSession.AreOnlySecureConnectionsAllowed())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create an insecure WinHttp WebSocket which is disabled on this platform");
		return nullptr;
	}

	const FString Domain = FGenericPlatformHttp::GetUrlDomain(InUrl);
	if (Domain.IsEmpty())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp WebSocket with an unset domain");
		return nullptr;
	}
	const FString PathAndQuery = FGenericPlatformHttp::GetUrlPath(InUrl, true, false);
	if (PathAndQuery.IsEmpty())
	{
		UE_LOGF(LogWinHttp, Warning, "Attempted to create a WinHttp WebSocket with an unset path");
		return nullptr;
	}

	TOptional<uint16> Port = FGenericPlatformHttp::GetUrlPort(InUrl);
	TSharedPtr<FWinHttpConnectionWebSocket, ESPMode::ThreadSafe> WebSocket = MakeShareable(new FWinHttpConnectionWebSocket(InSession, InUrl, bIsSecure, Domain, Port, PathAndQuery, InProtocols, InUpgradeHeaders));
	if (!WebSocket->IsValid())
	{
		return nullptr;
	}

	return WebSocket;
}

bool FWinHttpConnectionWebSocket::IsValid() const
{
	return ConnectionHandle.IsValid() && (RequestHandle.IsValid() || WebSocketHandle.IsValid());
}

void FWinHttpConnectionWebSocket::PumpMessages()
{
	check(IsInGameThread());

	// Don't lock our object if we don't have any events ready (this is an optimization to skip locking on game thread every tick if there's no events waiting)
	if (!bHasPendingDelegate)
	{
		return;
	}

	TSharedRef<IWinHttpConnection, ESPMode::ThreadSafe> LocalKeepAlive(AsShared());

	const FScopeLock ScopeLock(&SyncObject);
	if (bRequestCancelled)
	{
		return;
	}

	// Pump HTTP's message loop until our websocket is valid
	if (!WebSocketHandle.IsValid())
	{
		FWinHttpConnectionHttp::PumpMessages();
	}

	// Now that we're done potentially pumping the HTTP connection, ensure the pending delegate flag is reset
	bHasPendingDelegate = false;

	// Process connection opened success messages
	if (bHasPendingConnectedEvent)
	{
		bHasPendingConnectedEvent = false;

		OnConnectedHandler.ExecuteIfBound();
		if (bRequestCancelled)
		{
			return;
		}
	}

	// Process received web socket messages
	if (!ReceieveMessageQueue.IsEmpty())
	{
		const int32 MaxMessagesPerPump = 10;

		int32 MessageCount = 0;
		FPendingWebSocketMessage Message;
		while (ReceieveMessageQueue.Dequeue(Message) && MessageCount < MaxMessagesPerPump)
		{
			// Call our delegate to let the owning socket know there's data
			OnMessageHandler.ExecuteIfBound(Message.MessageType, Message.Data);
			if (bRequestCancelled)
			{
				return;
			}

			++MessageCount;
		}
	}

	// Process connection closed message
	if (OnClosedHandler.IsBound())
	{
		if (ReceivedCloseInfo.IsSet())
		{
			// Call our delegate
			OnClosedHandler.ExecuteIfBound(ReceivedCloseInfo.GetValue().Code, ReceivedCloseInfo.GetValue().Reason, SentCloseInfo.IsSet());

			// Unbind so we don't call the completion delegate again
			OnClosedHandler.Unbind();
			if (bRequestCancelled)
			{
				return;
			}
		}
		else if (FinalState.IsSet())
		{
			// Call our delegate
			OnClosedHandler.ExecuteIfBound(UE_WEBSOCKET_CLOSE_APP_FAILURE, FString(), false);

			// Unbind so we don't call the completion delegate again
			OnClosedHandler.Unbind();
			if (bRequestCancelled)
			{
				return;
			}
		}
	}
}

void FWinHttpConnectionWebSocket::PumpStates()
{
	TSharedRef<IWinHttpConnection, ESPMode::ThreadSafe> LocalKeepAlive(AsShared());

	const FScopeLock ScopeLock(&SyncObject);
	if (bRequestCancelled)
	{
		return;
	}

	// Pump HTTP's state loop until our websocket is valid
	if (!WebSocketHandle.IsValid())
	{
		FWinHttpConnectionHttp::PumpStates();
	}

	// Pump WinHttp's Send and Receive WebSocket functions to make sure we receive data
	if (WebSocketHandle.IsValid())
	{
		const int32 MaxMessagesPerPump = 5;

		if (!IsReadInProgress())
		{
			// Starts reading data if we aren't already, and if there is data immediately available, reads up to 10 message (or message fragments) worth
			ReadData(MaxMessagesPerPump);
		}

		if (!IsWriteInProgress())
		{
			// If we have data to send, queue up to 10 messages to be written
			// How many actually get immediately copied to be written will depend on how larger each message is.
			WriteData(MaxMessagesPerPump);
		}
	}
}

void FWinHttpConnectionWebSocket::SetWebSocketConnectedHandler(FWinHttpConnectionWebSocketOnConnected&& Handler)
{
	FScopeLock StateLock(&SyncObject);
	OnConnectedHandler = MoveTemp(Handler);
}

void FWinHttpConnectionWebSocket::SetWebSocketMessageHandler(FWinHttpConnectionWebSocketOnMessage&& Handler)
{
	FScopeLock StateLock(&SyncObject);
	OnMessageHandler = MoveTemp(Handler);
}

void FWinHttpConnectionWebSocket::SetWebSocketClosedHandler(FWinHttpConnectionWebSocketOnClosed&& Handler)
{
	FScopeLock StateLock(&SyncObject);
	OnClosedHandler = MoveTemp(Handler);
}

namespace
{
	WINHTTP_WEB_SOCKET_BUFFER_TYPE GetWebSocketMessageBufferType(EWebSocketMessageType MessageType)
	{
		return MessageType == EWebSocketMessageType::Utf8
			? WINHTTP_WEB_SOCKET_UTF8_MESSAGE_BUFFER_TYPE
			: WINHTTP_WEB_SOCKET_BINARY_MESSAGE_BUFFER_TYPE;
	}
}

bool FWinHttpConnectionWebSocket::SendMessage(EWebSocketMessageType MessageType, TArray<uint8>&& Message)
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp WebSocket[%p]: SendMessage MessageType=[%ls] MessageSize=[%d]", this, LexToString(MessageType), Message.Num());

	FScopeLock StateLock(&SyncObject);

	// We need our socket handle to send messages
	if (!WebSocketHandle.IsValid())
	{
		return false;
	}

	// We can't send messages if we've started the close process
	if (SentCloseInfo.IsSet())
	{
		return false;
	}

	// Create our message
	TUniquePtr<FPendingWebSocketMessage> NewMessage = MakeUnique<FPendingWebSocketMessage>(MessageType, MoveTemp(Message));

	// If we have a send in progress, add this new message to our queue
	if (PendingMessage.IsValid() || bWebSocketWriteInProgress)
	{
		return SentMessagesQueue.Enqueue(MoveTemp(NewMessage));
	}
	
	// Pending message must be set before calls to SocketSend, as we can get immediate callbacks WRITE_COMPLETE events if buffer sizes allow for it
	PendingMessage = MoveTemp(NewMessage);

	WriteData(1);
	return true;
}

bool FWinHttpConnectionWebSocket::IsConnected() const
{
	FScopeLock StateLock(&SyncObject);
	return WebSocketHandle.IsValid() && !FinalState.IsSet() && !SentCloseInfo.IsSet();
}

bool FWinHttpConnectionWebSocket::CloseConnection(const uint16 Code, const FString& Reason)
{
	UE_LOGF(LogWinHttp, Verbose, "WinHttp WebSocket[%p]: sending Close message to server. Code=[%d] Reason=[%ls]", this, static_cast<int32>(Code), *Reason);

	FScopeLock StateLock(&SyncObject);

	if (!WebSocketHandle.IsValid())
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp WebSocket[%p]: could not send close message to server, our WebSocket has already shutdown. Code=[%d] Reason=[%ls]", this, static_cast<int32>(Code), *Reason);
		return false;
	}

	// Make sure we don't shutdown twice
	if (SentCloseInfo.IsSet())
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp WebSocket[%p]: could not send close message to server, we have already sent a close message. Code=[%d] Reason=[%ls]", this, static_cast<int32>(Code), *Reason);
		return false;
	}

	PVOID Utf8Reason = nullptr;
	DWORD Utf8ReasonLength = 0;

	// Only convert if we have a reason
	TOptional<FTCHARToUTF8> Converter;
	if (!Reason.IsEmpty())
	{
		Converter.Emplace(*Reason, Reason.Len());
		Utf8Reason = const_cast<ANSICHAR*>(Converter->Get());
		Utf8ReasonLength = Converter->Length();
	}

	const DWORD ErrorCode = WinHttpWebSocketShutdown(WebSocketHandle.Get(), Code, Utf8Reason, Utf8ReasonLength);
	if (ErrorCode != NO_ERROR && ErrorCode != ERROR_IO_PENDING)
	{
		FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketShutdownFailure(ErrorCode);
		return false;
	}

	SentCloseInfo.Emplace(Code, Reason);
	return true;
}

FWinHttpConnectionWebSocket::FWinHttpConnectionWebSocket(
	FWinHttpSession& InSession,
	const FString& InRequestUrl,
	const bool bInIsSecure,
	const FString& InDomain,
	const TOptional<uint16> InPort,
	const FString& InPathAndQuery,
	const TArray<FString>& InProtocols,
	const TMap<FString, FString>& InUpgradeHeaders)
	: FWinHttpConnectionHttp(InSession, FString("GET"), InRequestUrl, bInIsSecure, InDomain, InPort, InPathAndQuery, InUpgradeHeaders, nullptr)
{
	if (!FWinHttpConnectionHttp::IsValid())
	{
		return;
	}

#pragma warning(push)
#pragma warning(disable : 6387) // Disable StaticAnalysis warning about lpBuffer being nullptr, it is required to be nullptr per the documentation
	const bool bSetWebSocketUpgradeSuccess = WinHttpSetOption(RequestHandle.Get(), WINHTTP_OPTION_UPGRADE_TO_WEB_SOCKET, nullptr, 0);
#pragma warning(pop)
	if (!bSetWebSocketUpgradeSuccess)
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpSetOptionFailure(ErrorCode);

		RequestHandle.Reset();
		ConnectionHandle.Reset();
		return;
	}

	if (!InProtocols.IsEmpty())
	{
		SetHeader(TEXT("Sec-WebSocket-Protocol"), FString::Join(InProtocols, TEXT(", ")));
	}
}

bool FWinHttpConnectionWebSocket::IsReadInProgress() const
{
	const FScopeLock ScopeLock(&SyncObject);
	return bWebSocketReadInProgress;
}

void FWinHttpConnectionWebSocket::ReadData(const int32 MaxMessagesToRead)
{
	const FScopeLock ScopeLock(&SyncObject);

	if (bRequestCancelled)
	{
		return;
	}

	if (!WebSocketHandle.IsValid())
	{
		return;
	}

	check(!bWebSocketReadInProgress);

	// Make sure we have at least a full chunk of data to read
	ReceiveBuffer.SetNumUninitialized(ReceiveBufferBytesWritten + UE_WINHTTP_READ_BUFFER_BYTES, EAllowShrinking::No);

	// Keep reading until we don't get data (or until we read MaxMessagesToRead messages)
	int32 MessagesRead = 0;
	while (!bWebSocketReadInProgress && MessagesRead < MaxMessagesToRead)
	{
		bWebSocketReadInProgress = true;
		++MessagesRead;

		// Ask for more data
		PVOID BufferDestination = ReceiveBuffer.GetData() + ReceiveBufferBytesWritten;
		DWORD BufferDestinationLength = ReceiveBuffer.Num() - ReceiveBufferBytesWritten;

#pragma warning(push)
#pragma warning(disable : 6387) // Disable StaticAnalysis warning about pdwBytesRead and peBufferType being nullptr, they are required to be nullptr as we are in async mode
		const DWORD ErrorCode = WinHttpWebSocketReceive(WebSocketHandle.Get(), BufferDestination, BufferDestinationLength, nullptr, nullptr);
#pragma warning(pop)
		if (ErrorCode != NO_ERROR)
		{
			// We didn't actually start the read
			bWebSocketReadInProgress = false;

			FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketReceiveFailure(ErrorCode);
			
			// TODO: figure out if this is a serious enough error to be worth tearing down the websocket (we don't currently)
			break;
		}
	}
}

bool FWinHttpConnectionWebSocket::IsWriteInProgress() const
{
	const FScopeLock ScopeLock(&SyncObject);
	return bWebSocketWriteInProgress && PendingMessage.IsValid();
}

void FWinHttpConnectionWebSocket::WriteData(const int32 MaxMessagesToWrite)
{
	const FScopeLock ScopeLock(&SyncObject);

	if (bRequestCancelled)
	{
		return;
	}

	// Can't send messages if we don't have a socket handle
	if (!WebSocketHandle.IsValid())
	{
		return;
	}

	// Can't send messages if we've asked to close our connection
	if (SentCloseInfo.IsSet())
	{
		return;
	}
	
	check(!bWebSocketWriteInProgress);

	// Keep writing until we don't immediately finish writing data (or until we read MaxMessagesToRead messages)
	int32 MessagesWritten = 0;
	while (!bWebSocketWriteInProgress && MessagesWritten < MaxMessagesToWrite)
	{
		// Send pending message if there is one, or wait until there is one ready
		if (PendingMessage.IsValid() || SentMessagesQueue.Dequeue(PendingMessage))
		{
			check(PendingMessage.IsValid());
			++MessagesWritten;

			bWebSocketWriteInProgress = true;

			// Build message type
			WINHTTP_WEB_SOCKET_BUFFER_TYPE MessageBufferType = GetWebSocketMessageBufferType(PendingMessage->MessageType);
			const DWORD ErrorCode = WinHttpWebSocketSend(WebSocketHandle.Get(), MessageBufferType, PendingMessage->Data.GetData(), PendingMessage->Data.Num());
			if (ErrorCode != NO_ERROR)
			{
				FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketSendFailure(ErrorCode);
				// Handle ERROR_WINHTTP_TIMEOUT to show a proper message
				if(ErrorCode == ERROR_WINHTTP_TIMEOUT)
				{
					ReceivedCloseInfo.Emplace(ErrorCode, FString("errors.com.epicgames.winhttp.timeout"));

					bHasPendingDelegate = true;
				}
				bWebSocketWriteInProgress = false;
				PendingMessage.Reset();
				FinishRequest(EHttpRequestStatus::Failed);
				return;
			}
		}
		else
		{
			break;
		}
	}
}

void FWinHttpConnectionWebSocket::HandleHeadersAvailable()
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp WebSocket[%p]: Received Callback Status=[HEADERS_AVAILABLE]", this);

	const FScopeLock ScopeLock(&SyncObject);

	if (bRequestCancelled)
	{
		return;
	}
	
	// Read status code header
	EHttpResponseCodes::Type WebSocketUpgradeResponseCode = EHttpResponseCodes::Unknown;

	{
		DWORD StatusCode = 0;
		DWORD InfoLevel = WINHTTP_QUERY_STATUS_CODE | WINHTTP_QUERY_FLAG_NUMBER;
		LPCWSTR HeaderName = WINHTTP_HEADER_NAME_BY_INDEX;
		LPVOID BufferDestination = &StatusCode;
		DWORD OutHeaderByteSize = sizeof(StatusCode);
		LPDWORD OutHeaderIndex = WINHTTP_NO_HEADER_INDEX;
		if (!WinHttpQueryHeaders(RequestHandle.Get(), InfoLevel, HeaderName, BufferDestination, &OutHeaderByteSize, OutHeaderIndex))
		{
			// Unexpected failure
			const DWORD ErrorCode = GetLastError();
			FWinHttpErrorHelper::LogWinHttpQueryHeadersFailure(ErrorCode);

			FinishRequest(EHttpRequestStatus::Failed);
			return;
		}
		WebSocketUpgradeResponseCode = static_cast<EHttpResponseCodes::Type>(StatusCode);
	}

	// Ensure we received our HTTP status 101, confirming the server wants to upgrade to a protocol we offered
	if (WebSocketUpgradeResponseCode != EHttpResponseCodes::SwitchProtocol)
	{
		UE_LOGF(LogWinHttp, Warning, "WinHttp WebSocket[%p]: Did not receive expected Switch Protocol response code, instead received %d", this, WebSocketUpgradeResponseCode);
		FinishRequest(EHttpRequestStatus::Failed);
		return;
	}

	// Create our websocket handle
	DWORD_PTR Context = reinterpret_cast<DWORD_PTR>(this);
	WebSocketHandle = WinHttpWebSocketCompleteUpgrade(RequestHandle.Get(), Context);
	if (!WebSocketHandle.IsValid())
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketCompleteUpgradeFailure(ErrorCode);

		UE_LOGF(LogWinHttp, Warning, "WinHttp WebSocket[%p]: Failed to complete WebSocket upgrade", this);
		FinishRequest(EHttpRequestStatus::Failed);
		return;
	}

	// Bind listener on our WebSocket handle
	const DWORD WebSocketNotifications = WINHTTP_CALLBACK_FLAG_READ_COMPLETE | WINHTTP_CALLBACK_FLAG_WRITE_COMPLETE | WINHTTP_CALLBACK_STATUS_CLOSE_COMPLETE | WINHTTP_CALLBACK_FLAG_REQUEST_ERROR | WINHTTP_CALLBACK_STATUS_HANDLE_CLOSING;
	if (WinHttpSetStatusCallback(WebSocketHandle.Get(), UE_WinHttpWebSocketStatusCallback, WebSocketNotifications, 0) == WINHTTP_INVALID_STATUS_CALLBACK)
	{
		const DWORD ErrorCode = GetLastError();
		FWinHttpErrorHelper::LogWinHttpSetStatusCallbackFailure(ErrorCode);

		UE_LOGF(LogWinHttp, Warning, "WinHttp WebSocket[%p]: Failed to to set WebSocket callback", this);
		FinishRequest(EHttpRequestStatus::Failed);
		return;
	}

	// We do not need the request handle anymore after creating the WebSocket handle, so release it
	// It also happens to make all of the HTTP request behaviour stop doing things, which is nice!
	RequestHandle.Reset();

	bHasPendingConnectedEvent = true;

	bHasPendingDelegate = true;
}

void FWinHttpConnectionWebSocket::HandleHandleClosing()
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp WebSocket[%p]: Received Callback Status=[HANDLE_CLOSING]", this);

	// Only reset our keepalive if both of request and websocket handles are invalid
	bool bShouldResetKeepAlive = false;
	{
		const FScopeLock ScopeLock(&SyncObject);
		bShouldResetKeepAlive = !RequestHandle.IsValid() && !WebSocketHandle.IsValid();
	}

	if (bShouldResetKeepAlive)
	{
		KeepAlive.Reset();
		// We may not exist anymore here, don't do anything that uses a member!
	}
}

bool FWinHttpConnectionWebSocket::FinishRequest(const EHttpRequestStatus::Type NewFinalState)
{
	const FScopeLock ScopeLock(&SyncObject);

	// Attempt to Close the websocket if we're in a good state (repeated calls to FinishRequest will immediately abort though!)
	if (WebSocketHandle.IsValid())
	{
		if (!SentCloseInfo.IsSet())
		{
			const uint16 Code = NewFinalState == EHttpRequestStatus::Succeeded
				? UE_WEBSOCKET_CLOSE_NORMAL_CLOSURE
				: UE_WEBSOCKET_CLOSE_APP_FAILURE;

			if (CloseConnection(Code, FString()))
			{
				return true;
			}
		}
	}

	// Abort the connection right now
	if (!FWinHttpConnectionHttp::FinishRequest(NewFinalState))
	{
		return false;
	}

	// If the base HTTP request was finished above, reset our websocket handle too
	WebSocketHandle.Reset();
	return true;
}

void FWinHttpConnectionWebSocket::HandleWebSocketReadComplete(const uint32 BytesRead, const EWebSocketMessageType MessageType, const bool bIsFragment)
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp WebSocket[%p]: Received Callback Status=[READ_COMPLETE] BytesRead=[%u] MessageType=[%ls] bIsFragment=[%d]", this, BytesRead, LexToString(MessageType), bIsFragment);

	const FScopeLock ScopeLock(&SyncObject);

	if (bRequestCancelled)
	{
		return;
	}

	// Ensure that we exepected this read callback
	check(bWebSocketReadInProgress);

	// We are no longer reading a message
	bWebSocketReadInProgress = false;

	// If we received a message fragment last time, verify the messagetype of this message matches
	if (ReceiveFragmentMessageType.IsSet())
	{
		if (ReceiveFragmentMessageType.GetValue() != MessageType)
		{
			UE_LOGF(LogWinHttp, Warning, "WinHttp WebSocket[%p]: Received message with mismatched data type. ExpectedMessageType=[%ls] ActualMessageType=[%ls]", this, LexToString(ReceiveFragmentMessageType.GetValue()), LexToString(MessageType));

			FinishRequest(EHttpRequestStatus::Failed);
			return;
		}
	}

	// Ensure that the amounts of bytes written didn't exceed our buffer and then increment how much we've read
	check(BytesRead + ReceiveBufferBytesWritten <= static_cast<uint32>(ReceiveBuffer.Num()));
	ReceiveBufferBytesWritten += BytesRead;

	if (bIsFragment)
	{
		// Update our message type if it's not set to verify we receive the same data type across message fragments
		if (!ReceiveFragmentMessageType.IsSet())
		{
			ReceiveFragmentMessageType = MessageType;
		}
	}
	else
	{
		ReceiveFragmentMessageType.Reset();

		// Set our buffer to the actual size of written data before we copy it out to the receieve queue
		ReceiveBuffer.SetNumUninitialized(ReceiveBufferBytesWritten, EAllowShrinking::No);

		UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp WebSocket[%p]: Received complete message MessageType=[%ls]", this, LexToString(MessageType));
		ReceieveMessageQueue.Enqueue(FPendingWebSocketMessage(MessageType, ReceiveBuffer));

		bHasPendingDelegate = true;

		// Reset our read bytes now that we've copied our data out
		ReceiveBufferBytesWritten = 0;
	}

	// Set our buffer to hold at least one chunk
	ReceiveBuffer.SetNumUninitialized(ReceiveBufferBytesWritten + UE_WINHTTP_READ_BUFFER_BYTES, EAllowShrinking::No);
}

void FWinHttpConnectionWebSocket::HandleWebSocketWriteComplete(const uint32 BytesWritten, const EWebSocketMessageType MessageType)
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp WebSocket[%p]: Received Callback Status=[WRITE_COMPLETE] BytesWritten=[%u] MessageType=[%ls]", this, BytesWritten, LexToString(MessageType));

	const FScopeLock ScopeLock(&SyncObject);

	if (bRequestCancelled)
	{
		return;
	}

	// Verify last message was (likely) the right data
	check(PendingMessage.IsValid());
	check(BytesWritten == PendingMessage->Data.Num());
	check(MessageType == PendingMessage->MessageType);

	PendingMessage.Reset();
	bWebSocketWriteInProgress = false;
}

void FWinHttpConnectionWebSocket::HandleWebSocketCloseComplete()
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp WebSocket[%p]: Received Callback Status=[CLOSE_COMPLETE]", this);
	
	const FScopeLock ScopeLock(&SyncObject);

	if (!WebSocketHandle.IsValid())
	{
		return;
	}

	USHORT CloseCode = 0u;
	ANSICHAR CloseReasonBuffer[WINHTTP_WEB_SOCKET_MAX_CLOSE_REASON_LENGTH] = {0};
	DWORD CloseReasonActualLength = 0u;

	DWORD ErrorCode = WinHttpWebSocketQueryCloseStatus(WebSocketHandle.Get(), &CloseCode, &CloseReasonBuffer, sizeof(CloseReasonBuffer), &CloseReasonActualLength);
	if (ErrorCode != NO_ERROR)
	{
		FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketQueryCloseStatusFailure(ErrorCode);

		FinishRequest(EHttpRequestStatus::Failed);
		return;
	}

	FUTF8ToTCHAR Converter(CloseReasonBuffer, CloseReasonActualLength);
	FString CloseReason = FString::ConstructFromPtrSize(Converter.Get(), Converter.Length());

	UE_LOGF(LogWinHttp, Verbose, "WinHttp WebSocket[%p]: Received Close message from the server. Code=[%d] Reason=[%ls]", this, static_cast<int32>(CloseCode), *CloseReason);

	ReceivedCloseInfo.Emplace(CloseCode, CloseReason);
	FinalState = (CloseCode == UE_WEBSOCKET_CLOSE_NORMAL_CLOSURE) ? EHttpRequestStatus::Succeeded : EHttpRequestStatus::Failed;

	bHasPendingDelegate = true;

	WebSocketHandle.Reset();
}

void FWinHttpConnectionWebSocket::HandleWebSocketRequestError(const int32 WebSocketOperationId, const uint32 ErrorCode)
{
	UE_LOGF(LogWinHttp, VeryVerbose, "WinHttp WebSocket[%p]: Received Callback Status=[REQUEST_ERROR] OperationId=[%d] ErrorCode=[0x%0.8X]", this, WebSocketOperationId, ErrorCode);

	const FScopeLock ScopeLock(&SyncObject);

	if (!RequestHandle.IsValid())
	{
		// Request was probably cancelled, just stop here
		return;
	}

	WINHTTP_WEB_SOCKET_OPERATION OperationId = static_cast<WINHTTP_WEB_SOCKET_OPERATION>(WebSocketOperationId);
	switch (OperationId)
	{
	case WINHTTP_WEB_SOCKET_SEND_OPERATION:
		FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketSendFailure(ErrorCode);
		break;
	case WINHTTP_WEB_SOCKET_RECEIVE_OPERATION:
		FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketReceiveFailure(ErrorCode);
		break;
	case WINHTTP_WEB_SOCKET_CLOSE_OPERATION:
		FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketCloseFailure(ErrorCode);
		break;
	case WINHTTP_WEB_SOCKET_SHUTDOWN_OPERATION:
		FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketShutdownFailure(ErrorCode);
		break;
	default:
		UE_LOGF(LogWinHttp, Error, "Unknown WebSocket API (%d) failed with error code 0x%0.8X", WebSocketOperationId, ErrorCode);
		break;
	}

	FinishRequest(EHttpRequestStatus::Failed);
}

namespace
{
	void GetCompletionDataFromSocketStatus(const WINHTTP_WEB_SOCKET_STATUS& SocketStatus, uint32& BytesSent, EWebSocketMessageType& MessageType, bool& bIsFragment)
	{
		BytesSent = SocketStatus.dwBytesTransferred;

		const bool bIsUtf8 = (SocketStatus.eBufferType == WINHTTP_WEB_SOCKET_UTF8_FRAGMENT_BUFFER_TYPE || SocketStatus.eBufferType == WINHTTP_WEB_SOCKET_UTF8_MESSAGE_BUFFER_TYPE);
		MessageType = bIsUtf8 ? EWebSocketMessageType::Utf8 : EWebSocketMessageType::Binary;
		
		bIsFragment = (SocketStatus.eBufferType == WINHTTP_WEB_SOCKET_BINARY_FRAGMENT_BUFFER_TYPE || SocketStatus.eBufferType == WINHTTP_WEB_SOCKET_UTF8_FRAGMENT_BUFFER_TYPE);
	}
}

void FWinHttpConnectionWebSocket::HandleWebSocketStatusCallback(HINTERNET ResourceHandle, EWinHttpCallbackStatus Status, void* StatusInformation, uint32 StatusInformationLength)
{
	// Prevent the request from dying while we're any callback
	TSharedRef<IWinHttpConnection, ESPMode::ThreadSafe> LocalKeepAlive(AsShared());

	switch (Status)
	{
		/**
		 * We have received data
		 */
		case EWinHttpCallbackStatus::ReadComplete:
		{
			verify(StatusInformationLength == sizeof(WINHTTP_WEB_SOCKET_STATUS));
			verify(StatusInformation != nullptr);
			const WINHTTP_WEB_SOCKET_STATUS* SocketStatus = reinterpret_cast<WINHTTP_WEB_SOCKET_STATUS*>(StatusInformation);
			
			// Special case receiving a Closed message as the server gracefully responding to a Close message
			if (SocketStatus->eBufferType == WINHTTP_WEB_SOCKET_CLOSE_BUFFER_TYPE)
			{
				HandleWebSocketCloseComplete();
				return;
			}

			uint32 BytesSent = 0;
			EWebSocketMessageType MessageType = EWebSocketMessageType::Binary;
			bool bIsFragment = false;
			GetCompletionDataFromSocketStatus(*SocketStatus, BytesSent, MessageType, bIsFragment);

			HandleWebSocketReadComplete(BytesSent, MessageType, bIsFragment);
			return;
		}
		case EWinHttpCallbackStatus::WriteComplete:
		{
			verify(StatusInformationLength == sizeof(WINHTTP_WEB_SOCKET_STATUS));
			verify(StatusInformation != nullptr);
			const WINHTTP_WEB_SOCKET_STATUS* SocketStatus = reinterpret_cast<WINHTTP_WEB_SOCKET_STATUS*>(StatusInformation);

			// This shouldn't happen, but ignoring this case since we don't handle it below
			if (SocketStatus->eBufferType == WINHTTP_WEB_SOCKET_CLOSE_BUFFER_TYPE)
			{
				UE_LOGF(LogWinHttp, Error, "WinHttp WebSocket sent unexpected Close message");
				return;
			}

			const uint32 BytesSent = SocketStatus->dwBytesTransferred;
			const EWebSocketMessageType MessageType = SocketStatus->eBufferType == WINHTTP_WEB_SOCKET_UTF8_MESSAGE_BUFFER_TYPE ? EWebSocketMessageType::Utf8 : EWebSocketMessageType::Binary;

			HandleWebSocketWriteComplete(BytesSent, MessageType);
			return;
		}
		case EWinHttpCallbackStatus::CloseComplete:
		{
			HandleWebSocketCloseComplete();
			return;
		}
		case EWinHttpCallbackStatus::RequestError:
		{
			check(StatusInformationLength == sizeof(WINHTTP_WEB_SOCKET_ASYNC_RESULT));
			check(StatusInformation != nullptr);

			WINHTTP_WEB_SOCKET_ASYNC_RESULT* WebSocketAsyncResult = static_cast<WINHTTP_WEB_SOCKET_ASYNC_RESULT*>(StatusInformation);

			const int32 WebSocketOperationId = static_cast<int32>(WebSocketAsyncResult->Operation);
			const uint32 ErrorCode = WebSocketAsyncResult->AsyncResult.dwError;

			HandleWebSocketRequestError(WebSocketOperationId, ErrorCode);
			return;
		}
		case EWinHttpCallbackStatus::HandleClosing:
		{
			HandleHandleClosing();
			return;
		}
	}
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
