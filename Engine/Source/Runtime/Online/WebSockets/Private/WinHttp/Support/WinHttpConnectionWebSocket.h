// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "CoreMinimal.h"
#include "WinHttp/Support/WinHttpConnectionHttp.h"
#include "WinHttp/Support/WinHttpWebSocketTypes.h"
#include "Containers/Queue.h"

DECLARE_DELEGATE(FWinHttpConnectionWebSocketOnConnected);
DECLARE_DELEGATE_TwoParams(FWinHttpConnectionWebSocketOnMessage, EWebSocketMessageType /*MessageType*/, TArray<uint8>& /*MessagePayload*/);
DECLARE_DELEGATE_ThreeParams(FWinHttpConnectionWebSocketOnClosed, uint16 /*Code*/, const FString& /*Reason*/, bool /*bGracefulDisconnect*/);

class FWinHttpConnectionWebSocket
	: public FWinHttpConnectionHttp
{
public:
	static TSharedPtr<FWinHttpConnectionWebSocket, ESPMode::ThreadSafe> CreateWebSocketConnection(
		FWinHttpSession& Session,
		const FString& RequestUrl,
		const TArray<FString>& Protocols,
		const TMap<FString, FString>& UpgradeHeaders);

	virtual ~FWinHttpConnectionWebSocket() = default;
	FWinHttpConnectionWebSocket(const FWinHttpConnectionWebSocket& Other) = delete;
	FWinHttpConnectionWebSocket(FWinHttpConnectionWebSocket&& Other) = delete;
	FWinHttpConnectionWebSocket& operator=(const FWinHttpConnectionWebSocket& Other) = delete;
	FWinHttpConnectionWebSocket& operator=(FWinHttpConnectionWebSocket&& Other) = delete;

	//~ Begin FWinHttpConnectionHttp Public Interface
	virtual bool IsValid() const override;
	virtual void PumpMessages() override;
	virtual void PumpStates() override;
	//~ End FWinHttpConnectionHttp Public Interface

	/// WebSocket Events

	void SetWebSocketConnectedHandler(FWinHttpConnectionWebSocketOnConnected&& Handler);
	void SetWebSocketMessageHandler(FWinHttpConnectionWebSocketOnMessage&& Handler);
	void SetWebSocketClosedHandler(FWinHttpConnectionWebSocketOnClosed&& Handler);

	/// State checking

	/**
	 * Check if our connection has been fully established. If this returns true, it's safe to start sending messages.
	 *
	 * @return True if our connection is active and fully established, false otherwise
	 */
	bool IsConnected() const;

	///

	/**
	 * Sends a websocket message if we are in a connected state. Delivery is not gaurenteed even if this returns true.
	 *
	 * @param MessageType The type of message being sent
	 * @param Message The message to send to the peer
	 */
	bool SendMessage(EWebSocketMessageType MessageType, TArray<uint8>&& Message);

	/**
	 * Gracefully request the connection to close with the provided code and reason.
	 *
	 * @param Code The close code to send to the server
	 * @param Reason The reason we are closing
	 */
	bool CloseConnection(const uint16 Code, const FString& Reason);

protected:
	FWinHttpConnectionWebSocket(
		FWinHttpSession& Session,
		const FString& RequestUrl,
		const bool bIsSecure,
		const FString& Domain,
		const TOptional<uint16> Port,
		const FString& PathAndQuery,
		const TArray<FString>& Protocols,
		const TMap<FString, FString>& UpgradeHeaders);

	bool IsReadInProgress() const;
	void ReadData(const int32 MaxMessagesToRead);
	bool IsWriteInProgress() const;
	void WriteData(const int32 MaxMessagesToWrite);

	//~ Begin FWinHttpConnectionHttp Protected Interface
	virtual bool FinishRequest(const EHttpRequestStatus::Type FinalState) override;
	virtual void HandleHeadersAvailable() override;
	virtual void HandleHandleClosing() override;
	//~ End FWinHttpConnectionHttp Protected Interface

	/// Async Events
	void HandleWebSocketReadComplete(const uint32 BytesRead, const EWebSocketMessageType MessageType, const bool bIsFragment);
	void HandleWebSocketWriteComplete(const uint32 BytesWritten, const EWebSocketMessageType MessageType);
	void HandleWebSocketCloseComplete();
	void HandleWebSocketRequestError(const int32 WebSocketOperationId, const uint32 ErrorCode);

private:
	// Hide HTTP-events from this class's interface
	using FWinHttpConnectionHttp::CreateHttpConnection;
	using FWinHttpConnectionHttp::SetDataTransferredHandler;
	using FWinHttpConnectionHttp::SetHeaderReceivedHandler;
	using FWinHttpConnectionHttp::SetRequestCompletedHandler;
	
	/**
	 * Handle receiving WebSocket status callbacks from WinHttp
	 *
	 * @param ResourceHandle The handle of the resource which generated this Status
	 * @param InternetStatus The event that is being triggered
	 * @param StatusInformation Pointer to optional data for this event, if there is any
	 * @param StatusInformationLength The length of data available to read in StatusInformation or 0 if not used
	 */
	void HandleWebSocketStatusCallback(HINTERNET ResourceHandle, EWinHttpCallbackStatus Status, void* StatusInformation, uint32 StatusInformationLength);

	/** Mark our callback as a friend so they can call the above status callback function */
	friend void CALLBACK UE_WinHttpWebSocketStatusCallback(HINTERNET, DWORD_PTR, DWORD, LPVOID, DWORD);

protected:
	FWinHttpHandle WebSocketHandle;

	TOptional<EWebSocketMessageType> ReceiveFragmentMessageType;
	TArray<uint8> ReceiveBuffer;
	int32 ReceiveBufferBytesWritten = 0;
	bool bWebSocketReadInProgress = false;

	TQueue<TUniquePtr<FPendingWebSocketMessage>, EQueueMode::Spsc> SentMessagesQueue;
	TUniquePtr<FPendingWebSocketMessage> PendingMessage;
	bool bWebSocketWriteInProgress = false;

	FWinHttpConnectionWebSocketOnConnected OnConnectedHandler;
	bool bHasPendingConnectedEvent = false;

	FWinHttpConnectionWebSocketOnMessage OnMessageHandler;
	TQueue<FPendingWebSocketMessage, EQueueMode::Spsc> ReceieveMessageQueue;
	
	FWinHttpConnectionWebSocketOnClosed OnClosedHandler;
	TOptional<FWebSocketCloseInfo> SentCloseInfo;
	TOptional<FWebSocketCloseInfo> ReceivedCloseInfo;
};

#endif //WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
