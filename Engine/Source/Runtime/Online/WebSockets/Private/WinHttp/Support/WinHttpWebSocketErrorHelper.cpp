// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "WinHttp/Support/WinHttpWebSocketErrorHelper.h"
#include "WinHttp/Support/WinHttpTypes.h"

#include "Windows/AllowWindowsPlatformTypes.h"

void FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketCompleteUpgradeFailure(const uint32 ErrorCode)
{
	// No errors are predefined in the documentation for this, but it can still possibly fail!

	UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketCompleteUpgrade failed due to an unknown error (%u)", ErrorCode);
}

void FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketReceiveFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_INVALID_OPERATION:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketReceive failed due to receive data due to a pending close or send operation, or the receive channel has already been closed");
			break;
		case ERROR_INVALID_PARAMETER:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketReceive failed due to an invalid parameter");
			break;
		case ERROR_WINHTTP_INVALID_SERVER_RESPONSE:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketReceive failed due to an internal error");
			break;
		case ERROR_WINHTTP_OPERATION_CANCELLED:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketReceive failed due to an internal error");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketReceive failed due to an unknown error (%u)", ErrorCode);
			break;
	}
}

void FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketSendFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_INVALID_OPERATION:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketSend failed due to a pending send or close operation, or the send channel has already been closed");
			break;
		case ERROR_INVALID_PARAMETER:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketSend failed due to an invalid parameter");
			break;
		case ERROR_WINHTTP_TIMEOUT:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketSend failed due to a timeout");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketSend failed due to an unknown error (%u)", ErrorCode);
			break;
	}
}

void FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketCloseFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_INVALID_OPERATION:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketClose failed due to a pending send or close operation");
			break;
		case ERROR_INVALID_PARAMETER:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketClose failed due to an invalid parameter");
			break;
		case ERROR_WINHTTP_INVALID_SERVER_RESPONSE:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketClose failed due to an invalid message from the server");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketClose failed due to an unknown error (%u)", ErrorCode);
			break;
	}
}

void FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketShutdownFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_IO_PENDING:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketShutdown will complete asynchronously");
			break;
		case ERROR_WINHTTP_TIMEOUT:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketShutdown failed due to a timeout");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketShutdown failed due to an unknown error (%u)", ErrorCode);
			break;
	}
}

void FWinHttpWebSocketErrorHelper::LogWinHttpWebSocketQueryCloseStatusFailure(const uint32 ErrorCode)
{
	switch (ErrorCode)
	{
		case ERROR_INSUFFICIENT_BUFFER:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketQueryCloseStatus failed as the error response did not fit into the provided buffer");
			break;
		case ERROR_INVALID_OPERATION:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketQueryCloseStatus failed as a close frame has not been received yet");
			break;
		case ERROR_INVALID_PARAMETER:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketQueryCloseStatus failed due to an invalid parameter");
			break;
		default:
			UE_LOGF(LogWinHttp, Error, "WinHttpWebSocketQueryCloseStatus failed due an unknown error (%u)", ErrorCode);
			break;
	}
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
