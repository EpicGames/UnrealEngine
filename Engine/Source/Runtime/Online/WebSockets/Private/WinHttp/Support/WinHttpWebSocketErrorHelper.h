// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "CoreMinimal.h"

class FWinHttpWebSocketErrorHelper
{
public:
	/**
	 * Log that an WinHttpWebSocketCompleteUpgrade call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static void LogWinHttpWebSocketCompleteUpgradeFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpWebSocketReceive call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static void LogWinHttpWebSocketReceiveFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpWebSocketSend call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static void LogWinHttpWebSocketSendFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpWebSocketClose call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static void LogWinHttpWebSocketCloseFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpWebSocketShutdown call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static void LogWinHttpWebSocketShutdownFailure(const uint32 ErrorCode);

	/**
	 * Log that an WinHttpWebSocketQueryCloseStatus call failed with the specified ErrorCode
	 *
	 * @param ErrorCode The error that occured
	 */
	static void LogWinHttpWebSocketQueryCloseStatusFailure(const uint32 ErrorCode);

private:
	/** This class should not be instantiated */
	FWinHttpWebSocketErrorHelper() = delete;
};


#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
