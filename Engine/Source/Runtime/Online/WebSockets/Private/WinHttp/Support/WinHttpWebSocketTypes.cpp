// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "WinHttp/Support/WinHttpWebSocketTypes.h"

const TCHAR* LexToString(const EWebSocketMessageType MessageType)
{
	switch (MessageType)
	{
	case EWebSocketMessageType::Binary:
		return TEXT("Binary");
	case EWebSocketMessageType::Utf8:
		return TEXT("Utf8");
	}

	checkNoEntry();
	return TEXT("");
}

FPendingWebSocketMessage::FPendingWebSocketMessage(EWebSocketMessageType InMessageType, TArray<uint8>&& InData)
	: MessageType(InMessageType)
	, Data(MoveTemp(InData))
{
}
FPendingWebSocketMessage::FPendingWebSocketMessage(EWebSocketMessageType InMessageType, const TArray<uint8>& InData)
	: MessageType(InMessageType)
	, Data(InData)
{
}

FWebSocketCloseInfo::FWebSocketCloseInfo(uint16 InCode, FString&& InReason)
	: Code(InCode)
	, Reason(MoveTemp(InReason))
{
}

FWebSocketCloseInfo::FWebSocketCloseInfo(uint16 InCode, const FString& InReason)
	: Code(InCode)
	, Reason(InReason)
{
}

#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
