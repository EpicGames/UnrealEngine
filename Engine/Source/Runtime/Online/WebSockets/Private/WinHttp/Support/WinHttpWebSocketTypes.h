// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "CoreMinimal.h"

/** Status code for sucessful websocket closure */
#define UE_WEBSOCKET_CLOSE_NORMAL_CLOSURE 1000
/** Status code for application-failure websocket closure */
#define UE_WEBSOCKET_CLOSE_APP_FAILURE 1006

/**
 * Structure to represent if a WebSocket message is UTF8 or Binary
 */
enum class EWebSocketMessageType : uint8
{
	Binary,
	Utf8
};

/**
 * Convert a WebSocketMessageType into a string for logging
 */
const TCHAR* LexToString(const EWebSocketMessageType MessageType);

/**
 * Structure to hold a websocket message to be read or sent
 */
struct FPendingWebSocketMessage
{
	FPendingWebSocketMessage() = default;
	FPendingWebSocketMessage(EWebSocketMessageType InMessageType, TArray<uint8>&& InData);
	FPendingWebSocketMessage(EWebSocketMessageType InMessageType, const TArray<uint8>& InData);

public:
	/** The type of message Data is */
	EWebSocketMessageType MessageType = EWebSocketMessageType::Binary;
	/** The actual payload of the message */
	TArray<uint8> Data;
};

/**
 * Structure to hold information related to close messages from the server or from the client
 */
struct FWebSocketCloseInfo
{
	FWebSocketCloseInfo(uint16 InCode, FString&& InReason);
	FWebSocketCloseInfo(uint16 InCode, const FString& InReason);

public:
	/** Numeric code on the status of the closure */
	uint16 Code = 0u;
	/** String reason on why the connection was closed */
	FString Reason;
};

#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
