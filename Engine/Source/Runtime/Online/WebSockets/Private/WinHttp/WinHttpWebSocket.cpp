// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "WinHttp/WinHttpWebSocket.h"
#include "WinHttp/Support/WinHttpConnectionWebSocket.h"
#include "WinHttp/Support/WinHttpWebSocketTypes.h"

#include "WinHttp/WinHttpHttpManager.h"
#include "WinHttp/Support/WinHttpSession.h"

#include "WebSocketsLog.h"
#include "Misc/URLRequestFilter.h"
#include "Misc/ConfigCacheIni.h"

const TCHAR* LexToString(const EWebSocketConnectionState State)
{
	switch (State)
	{
	case EWebSocketConnectionState::NotStarted:
		return TEXT("NotStarted");
	case EWebSocketConnectionState::Connecting:
		return TEXT("Connecting");
	case EWebSocketConnectionState::Connected:
		return TEXT("Connected");
	case EWebSocketConnectionState::FailedToConnect:
		return TEXT("FailedToConnect");
	case EWebSocketConnectionState::Disconnected:
		return TEXT("Disconnected");
	case EWebSocketConnectionState::Closed:
		return TEXT("Closed");
	}

	checkNoEntry();
	return TEXT("");
}

FWinHttpWebSocket::FWinHttpWebSocket(const FString& InUrl, const TArray<FString>& InProtocols, const TMap<FString, FString>& InUpgradeHeaders)
	: Url(InUrl)
	, Protocols(InProtocols)
	, UpgradeHeaders(InUpgradeHeaders)
{
}

FWinHttpWebSocket::~FWinHttpWebSocket()
{
	if (WebSocket.IsValid())
	{
		if (WebSocket->IsValid())
		{
			Close(UE_WEBSOCKET_CLOSE_NORMAL_CLOSURE, FString());
		}

		WebSocket.Reset();
	}
}

void FWinHttpWebSocket::Connect()
{
	if (State == EWebSocketConnectionState::Connecting ||
		State == EWebSocketConnectionState::Connected)
	{
		// Already connecting/connected
		UE_LOGF(LogWebSockets, Verbose, "WinHttp WebSocket[%p]: Attempted to connect while in the %ls state.", this, LexToString(State));
		return;
	}

	State = EWebSocketConnectionState::Connecting;

	// Check Domain allowedlist if enabled
	bool bDisableDomainAllowlist = false;
	GConfig->GetBool(TEXT("WinHttpWebSocket"), TEXT("bDisableDomainAllowlist"), bDisableDomainAllowlist, GEngineIni);
	if (!bDisableDomainAllowlist)
	{
		UE::Core::FURLRequestFilter DomainFilter(TEXT("Online.HttpManager"), GEngineIni);
		if (!DomainFilter.IsRequestAllowed(Url))
		{
			UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: %ls is not in the allowed list, refusing to connect.", this, *Url);
			HandleCloseComplete(EWebSocketConnectionState::FailedToConnect, UE_WEBSOCKET_CLOSE_APP_FAILURE, FString("Invalid Domain"));
			return;
		}
	}
	else
	{
		UE_LOGF(LogWebSockets, Log, "WinHttp WebSocket[%p]: Domain allowed list has been disabled by config.", this);
	}

	FWinHttpHttpManager* Manager = FWinHttpHttpManager::GetManager();
	if (!Manager)
	{
		UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: WinHttp Manager shutdown", this);

		HandleCloseComplete(EWebSocketConnectionState::FailedToConnect, UE_WEBSOCKET_CLOSE_APP_FAILURE, FString("WinHttp Manager shutdown"));
		return;
	}

	bSessionCreationInProgress = true;
	Manager->QuerySessionForUrl(Url, FWinHttpQuerySessionComplete::CreateSP(AsShared(), &FWinHttpWebSocket::HandleSessionCreated));
}

void FWinHttpWebSocket::Close(const int32 Code, const FString& Reason)
{
	switch (State)
	{
		case EWebSocketConnectionState::NotStarted:
		case EWebSocketConnectionState::FailedToConnect:
		case EWebSocketConnectionState::Disconnected:
		case EWebSocketConnectionState::Closed:
		{
			// Not connected, ignore close request
			UE_LOGF(LogWebSockets, Verbose, "WinHttp WebSocket[%p]: Close socket while in %ls state, ignoring", this, LexToString(State));
			return;
		}
		case EWebSocketConnectionState::Connecting:
		case EWebSocketConnectionState::Connected:
		{
			if (bCloseRequested)
			{
				// Already closing
				UE_LOGF(LogWebSockets, Verbose, "WinHttp WebSocket[%p]: Call to close while another close was in progress.", this);
				return;
			}

			bCloseRequested = true;

			if (WebSocket.IsValid())
			{
				// We can gracefully close if we're still connected
				if (!WebSocket->CloseConnection(Code, Reason))
				{
					// If we can't gracefully close, just tear down the connection
					WebSocket->CancelRequest();
				}
			}
			else
			{
				// If we don't have a websocket, we're in the middle of creating a session
				QueuedCloseCode = Code;
				QueuedCloseReason = Reason;
			}
			return;
		}
	}

	// Ensure we had a handled case above
	checkNoEntry();
}

bool FWinHttpWebSocket::IsConnected()
{
	return WebSocket.IsValid() && WebSocket->IsConnected() && State == EWebSocketConnectionState::Connected;
}

void FWinHttpWebSocket::Send(const FString& Data)
{
	if (!IsConnected())
	{
		UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: Failed to send message, we are not connected", this);
		return;
	}

	// Convert data to UTF-8
	FTCHARToUTF8 Utf8Data(*Data, Data.Len());

	// Send Message
	TArray<uint8> Message(reinterpret_cast<const uint8*>(Utf8Data.Get()), Utf8Data.Length());
	const EWebSocketMessageType MessageType = EWebSocketMessageType::Utf8;
	WebSocket->SendMessage(MessageType, MoveTemp(Message));

	TSharedRef<FWinHttpWebSocket> KeepAlive = AsShared();
	OnMessageSent().Broadcast(Data);
}

void FWinHttpWebSocket::Send(const void* Data, SIZE_T Size, bool bIsBinary)
{
	if (!IsConnected())
	{
		UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: Failed to send message, we are not connected", this);
		return;
	}

	TArray<uint8> Message(static_cast<const uint8*>(Data), Size);
	EWebSocketMessageType MessageType = bIsBinary ? EWebSocketMessageType::Binary : EWebSocketMessageType::Utf8;

	WebSocket->SendMessage(MessageType, MoveTemp(Message));
}

void FWinHttpWebSocket::SetTextMessageMemoryLimit(uint64 TextMessageMemoryLimit)
{
	UE_LOGF(LogWebSockets, Verbose, "SetTextMessageMemoryLimit not implemented for WinHttpWebSocket.");
}

FWinHttpWebSocket::FWebSocketConnectedEvent& FWinHttpWebSocket::OnConnected()
{
	return OnConnectedHandler;
}

FWinHttpWebSocket::FWebSocketConnectionErrorEvent& FWinHttpWebSocket::OnConnectionError()
{
	return OnErrorHandler;
}

FWinHttpWebSocket::FWebSocketClosedEvent& FWinHttpWebSocket::OnClosed()
{
	return OnClosedHandler;
}

FWinHttpWebSocket::FWebSocketMessageEvent& FWinHttpWebSocket::OnMessage()
{
	return OnMessageHandler;
}

FWinHttpWebSocket::FWebSocketBinaryMessageEvent& FWinHttpWebSocket::OnBinaryMessage()
{ 
	return BinaryMessageHandler; 
}

FWinHttpWebSocket::FWebSocketRawMessageEvent& FWinHttpWebSocket::OnRawMessage()
{
	return OnRawMessageHandler;
}

FWinHttpWebSocket::FWebSocketMessageSentEvent& FWinHttpWebSocket::OnMessageSent()
{
	return OnMessageSentHandler;
}

void FWinHttpWebSocket::HandleSessionCreated(FWinHttpSession* SessionPtr)
{
	bSessionCreationInProgress = false;

	// If we have a close requested, that means there was a call to close before we even started our connection.
	// We should just stop this connection and call the appropriate delegates
	if (bCloseRequested)
	{
		bCloseRequested = false;

		UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: connection closed before it could start.", this);

		FString Reason;
		if (QueuedCloseReason.IsSet())
		{
			Reason = MoveTemp(QueuedCloseReason.GetValue());
			QueuedCloseReason.Reset();
		}

		uint16 Code = QueuedCloseCode.Get(UE_WEBSOCKET_CLOSE_APP_FAILURE);

		HandleCloseComplete(EWebSocketConnectionState::FailedToConnect, Code, Reason);
		return;
	}

	if (!SessionPtr)
	{
		// Could not create session
		UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: Unable to create WinHttp Session, failing request", this);
		HandleCloseComplete(EWebSocketConnectionState::FailedToConnect, UE_WEBSOCKET_CLOSE_APP_FAILURE, FString("Unable to create WinHttp Session"));
		return;
	}

	// Create connection object
	TSharedPtr<FWinHttpConnectionWebSocket, ESPMode::ThreadSafe> LocalWebsocket = FWinHttpConnectionWebSocket::CreateWebSocketConnection(*SessionPtr, Url, Protocols, UpgradeHeaders);
	if (!LocalWebsocket.IsValid())
	{
		UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: Failed to create connection", this);
		HandleCloseComplete(EWebSocketConnectionState::FailedToConnect, UE_WEBSOCKET_CLOSE_APP_FAILURE, FString("Failed to create connection"));
		return;
	}

	// Bind listeners
	TSharedRef<FWinHttpWebSocket> StrongThisRef = AsShared();
	LocalWebsocket->SetWebSocketConnectedHandler(FWinHttpConnectionWebSocketOnConnected::CreateSP(StrongThisRef, &FWinHttpWebSocket::HandleWebSocketConnected));
	LocalWebsocket->SetWebSocketMessageHandler(FWinHttpConnectionWebSocketOnMessage::CreateSP(StrongThisRef, &FWinHttpWebSocket::HandleWebSocketMessage));
	LocalWebsocket->SetWebSocketClosedHandler(FWinHttpConnectionWebSocketOnClosed::CreateSP(StrongThisRef, &FWinHttpWebSocket::HandleWebSocketClosed));

	// Start request!
	if (!LocalWebsocket->StartRequest())
	{
		UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: Unable to start Connection", this);
		HandleCloseComplete(EWebSocketConnectionState::FailedToConnect, UE_WEBSOCKET_CLOSE_APP_FAILURE, FString("Failed to create connection"));
		return;
	}

	// Save object
	WebSocket = MoveTemp(LocalWebsocket);
}

void FWinHttpWebSocket::HandleCloseComplete(const EWebSocketConnectionState NewState, const uint16 Code, const FString& Reason)
{
	checkf(NewState == EWebSocketConnectionState::FailedToConnect
		|| NewState == EWebSocketConnectionState::Disconnected
		|| NewState == EWebSocketConnectionState::Closed,
		TEXT("NewState was unexpected value %d"), static_cast<int32>(NewState));

	// If we have an async request (session creation) already pending for this WebSocket, wait until that finishes instead of closing now
	if (bSessionCreationInProgress)
	{
		UE_LOGF(LogWebSockets, Warning, "WinHttp WebSocket[%p]: connection closed before while session creation in progress. Code=[%u] Reason=[%ls]", this, Code, *Reason);
		return;
	}

	// Reset state now that we're closed
	QueuedCloseCode.Reset();
	QueuedCloseReason.Reset();
	bCloseRequested = false;

	// Shutdown our websocket if it's still around
	if (WebSocket.IsValid())
	{
		if (!WebSocket->IsComplete())
		{
			WebSocket->CancelRequest();
		}
		WebSocket.Reset();
	}

	// Store our current state before we update it
	const EWebSocketConnectionState PreviousState = State;

	// Update our state
	State = NewState;

	// Determine what delegate (if any) to call
	switch (PreviousState)
	{
		case EWebSocketConnectionState::NotStarted:
		case EWebSocketConnectionState::FailedToConnect:
		case EWebSocketConnectionState::Disconnected:
		case EWebSocketConnectionState::Closed:
		{
			// We didn't actually have an active connection, so no need to do anything
			UE_LOGF(LogWebSockets, Verbose, "WinHttp WebSocket[%p]: Connection close occurred while in %ls state, ignoring. Code=[%u] Reason=[%ls]", this, LexToString(PreviousState), Code, *Reason);
			return;
		}
		case EWebSocketConnectionState::Connecting:
		{
			UE_LOGF(LogWebSockets, Log, "WinHttp WebSocket[%p]: Connection error occurred while connecting. Code=[%u] Reason=[%ls]", this, Code, *Reason);

			TSharedRef<FWinHttpWebSocket> KeepAlive = AsShared();
			OnConnectionError().Broadcast(Reason);
			return;
		}
		case EWebSocketConnectionState::Connected:
		{
			UE_LOGF(LogWebSockets, Log, "WinHttp WebSocket[%p]: Connection closed. Code=[%u] Reason=[%ls]", this, Code, *Reason);

			TSharedRef<FWinHttpWebSocket> KeepAlive = AsShared();
			OnClosed().Broadcast(Code, Reason, NewState == EWebSocketConnectionState::Closed);
			return;
		}
	}

	checkNoEntry();
}

void FWinHttpWebSocket::GameThreadTick()
{
	if (WebSocket.IsValid())
	{
		WebSocket->PumpStates();
		WebSocket->PumpMessages();
		// WebSocket may be invalid here
	}
}

void FWinHttpWebSocket::HandleWebSocketConnected()
{
	if (State == EWebSocketConnectionState::Connecting)
	{
		State = EWebSocketConnectionState::Connected;

		TSharedRef<FWinHttpWebSocket> KeepAlive = AsShared();
		OnConnected().Broadcast();
	}
}

void FWinHttpWebSocket::HandleWebSocketMessage(EWebSocketMessageType MessageType, TArray<uint8>& MessagePayload)
{
	TSharedRef<FWinHttpWebSocket> KeepAlive = AsShared();

	if (MessageType == EWebSocketMessageType::Utf8 && OnMessage().IsBound())
	{
		const FUTF8ToTCHAR TCHARConverter(reinterpret_cast<const ANSICHAR*>(MessagePayload.GetData()), MessagePayload.Num());
		const FString Message = FString::ConstructFromPtrSize(TCHARConverter.Get(), TCHARConverter.Length());

		OnMessage().Broadcast(Message);
	}

	const SIZE_T BytesLeft = 0;
	OnRawMessage().Broadcast(MessagePayload.GetData(), MessagePayload.Num(), BytesLeft);
}

void FWinHttpWebSocket::HandleWebSocketClosed(uint16 Code, const FString& Reason, bool bGracefulDisconnect)
{
	UE_LOGF(LogWebSockets, Verbose, "WinHttp WebSocket[%p]: Received connection close event. Code=[%u] Reason=[%ls] bWasGraceful=[%d]", this, Code, *Reason, bGracefulDisconnect);

	EWebSocketConnectionState NewState;
	if (State == EWebSocketConnectionState::Connecting)
	{
		NewState = EWebSocketConnectionState::FailedToConnect;
	}
	else
	{
		NewState = bGracefulDisconnect ? EWebSocketConnectionState::Closed : EWebSocketConnectionState::Disconnected;
	}

	HandleCloseComplete(NewState, Code, Reason);
}

#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
