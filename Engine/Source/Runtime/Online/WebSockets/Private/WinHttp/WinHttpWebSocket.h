// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "CoreMinimal.h"
#include "IWebSocket.h"

class FWinHttpConnectionWebSocket;
class FWinHttpSession;
enum class EWebSocketMessageType : uint8;

enum class EWebSocketConnectionState : uint8
{
	/** We have not started a connection yet */
	NotStarted,
	/** We are in the process of connecting */
	Connecting,
	/** We have an active valid connection */
	Connected,
	/** We failed to connect (or couldn't connect) last time we tried */
	FailedToConnect,
	/** We were forcefully disconnected from a valid connection */
	Disconnected,
	/** We gracefully disconnected from a valid connection*/
	Closed
};

const TCHAR* LexToString(const EWebSocketConnectionState);

class FWinHttpWebSocket
	: public IWebSocket
	, public TSharedFromThis<FWinHttpWebSocket>
{
public:
	FWinHttpWebSocket(const FString& Url, const TArray<FString>& Protocols, const TMap<FString, FString>& UpgradeHeaders);
	virtual ~FWinHttpWebSocket();

	//~ Begin IWebSocket Interface
	virtual void Connect() override final;
	virtual void Close(const int32 Code = 1000, const FString& Reason = FString()) override final;
	virtual bool IsConnected() override final;
	virtual void Send(const FString& Data) override final;
	virtual void Send(const void* Data, SIZE_T Size, bool bIsBinary) override final;
	virtual void SetTextMessageMemoryLimit(uint64 TextMessageMemoryLimit) override final;
	DECLARE_DERIVED_EVENT(FWinHttpWebSocket, IWebSocket::FWebSocketConnectedEvent, FWebSocketConnectedEvent);
	virtual FWebSocketConnectedEvent& OnConnected() override final;
	DECLARE_DERIVED_EVENT(FWinHttpWebSocket, IWebSocket::FWebSocketConnectionErrorEvent, FWebSocketConnectionErrorEvent);
	virtual FWebSocketConnectionErrorEvent& OnConnectionError() override final;
	DECLARE_DERIVED_EVENT(FWinHttpWebSocket, IWebSocket::FWebSocketClosedEvent, FWebSocketClosedEvent);
	virtual FWebSocketClosedEvent& OnClosed() override final;
	DECLARE_DERIVED_EVENT(FWinHttpWebSocket, IWebSocket::FWebSocketMessageEvent, FWebSocketMessageEvent);
	virtual FWebSocketMessageEvent& OnMessage() override final;
	DECLARE_DERIVED_EVENT(FWinHttpWebSocket, IWebSocket::FWebSocketRawMessageEvent, FWebSocketRawMessageEvent);
	DECLARE_DERIVED_EVENT(FWinHttpWebSocket, IWebSocket::FWebSocketBinaryMessageEvent, FWebSocketBinaryMessageEvent);
	virtual FWebSocketBinaryMessageEvent& OnBinaryMessage() override final;
	virtual FWebSocketRawMessageEvent& OnRawMessage() override final;
	DECLARE_DERIVED_EVENT(FWinHttpWebSocket, IWebSocket::FWebSocketMessageSentEvent, FWebSocketMessageSentEvent);
	virtual FWebSocketMessageSentEvent& OnMessageSent() override final;
	//~ End IWebSocket Interface

protected:
	void HandleSessionCreated(FWinHttpSession* HttpSessionPtr);
	void HandleCloseComplete(const EWebSocketConnectionState NewState, const uint16 Code, const FString& Reason);

	/// FHttpConnectionWebSocket Callback Handling
	void GameThreadTick();

	//~ Begin FWinHttpConnectionWebSocket Callbacks
	void HandleWebSocketConnected();
	void HandleWebSocketMessage(EWebSocketMessageType MessageType, TArray<uint8>& MessagePayload);
	void HandleWebSocketClosed(uint16 Code, const FString& Reason, bool bGracefulDisconnect);
	//~ End FWinHttpConnectionWebSocket Callbacks

protected:
	FString Url;
	TArray<FString> Protocols;
	TMap<FString, FString> UpgradeHeaders;

	EWebSocketConnectionState State = EWebSocketConnectionState::NotStarted;
	bool bSessionCreationInProgress = false;
	bool bCloseRequested = false;
	TOptional<int32> QueuedCloseCode;
	TOptional<FString> QueuedCloseReason;
	TSharedPtr<FWinHttpConnectionWebSocket, ESPMode::ThreadSafe> WebSocket;

	FWebSocketConnectedEvent OnConnectedHandler;
	FWebSocketConnectionErrorEvent OnErrorHandler;
	FWebSocketClosedEvent OnClosedHandler;
	FWebSocketMessageEvent OnMessageHandler;
	FWebSocketBinaryMessageEvent BinaryMessageHandler;
	FWebSocketRawMessageEvent OnRawMessageHandler;
	FWebSocketMessageSentEvent OnMessageSentHandler;

	friend class FWinHttpWebSocketsManager;
};

#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
