// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "WinHttp/WinHttpWebSocketsManager.h"
#include "WinHttp/WinHttpWebSocket.h"
#include "WinHttp/WinHttpHttpManager.h"
#include "PlatformHttp.h"
#include "Modules/ModuleManager.h"
#include "Containers/BackgroundableTicker.h"
#include "Stats/Stats.h"

void FWinHttpWebSocketsManager::InitWebSockets(TArrayView<const FString> Protocols)
{
	if (FWinHttpHttpManager::GetManager() == nullptr)
	{
		InitHttpManager();
	}

	if (ensure(!TickHandle.IsValid()))
	{
		TickHandle = FTSBackgroundableTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FWinHttpWebSocketsManager::GameThreadTick), 0.0f);
	}
}

void FWinHttpWebSocketsManager::InitHttpManager()
{
	WinHttpHttpManager = FPlatformHttp::CreateWinHttpHttpManager();
	check(WinHttpHttpManager);
}

void FWinHttpWebSocketsManager::ShutdownWebSockets()
{
	if (WinHttpHttpManager)
	{
		delete WinHttpHttpManager;
		WinHttpHttpManager = nullptr;
	}

	for (TWeakPtr<FWinHttpWebSocket>& WeakWebSocket : ActiveWebSockets)
	{
		if (TSharedPtr<FWinHttpWebSocket> StrongWebSocket = WeakWebSocket.Pin())
		{
			StrongWebSocket->Close();
		}
	}
	ActiveWebSockets.Empty();

	if (ensure(TickHandle.IsValid()))
	{
		FTSBackgroundableTicker::GetCoreTicker().RemoveTicker(TickHandle);
		TickHandle.Reset();
	}
}

TSharedRef<IWebSocket> FWinHttpWebSocketsManager::CreateWebSocket(const FString& Url, const TArray<FString>& Protocols, const TMap<FString, FString>& UpgradeHeaders)
{
	TSharedRef<FWinHttpWebSocket> WebSocket = MakeShared<FWinHttpWebSocket>(Url, Protocols, UpgradeHeaders);
	ActiveWebSockets.Emplace(WebSocket);
	return WebSocket;
}

bool FWinHttpWebSocketsManager::GameThreadTick(float /*DeltaTime*/)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FWinHttpWebSocketsManager_GameThreadTick);
	for (TArray<TWeakPtr<FWinHttpWebSocket>>::TIterator It = ActiveWebSockets.CreateIterator(); It; ++It)
	{
		if (TSharedPtr<FWinHttpWebSocket> StrongWebSocket = It->Pin())
		{
			StrongWebSocket->GameThreadTick();
		}
		else
		{
			It.RemoveCurrent();
		}
	}

	return true;
}

#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
