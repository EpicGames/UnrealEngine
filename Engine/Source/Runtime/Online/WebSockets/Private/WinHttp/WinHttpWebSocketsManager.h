// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS

#include "CoreMinimal.h"
#include "IWebSocketsManager.h"
#include "Containers/Ticker.h"

class FWinHttpHttpManager;
class FWinHttpWebSocket;

class FWinHttpWebSocketsManager
	: public IWebSocketsManager
{
public:
	virtual ~FWinHttpWebSocketsManager() = default;

	//~ Begin IWebSocketsManager Interface
	virtual void InitWebSockets(TArrayView<const FString> Protocols) override;
	virtual void ShutdownWebSockets() override;
	virtual TSharedRef<IWebSocket> CreateWebSocket(const FString& Url, const TArray<FString>& Protocols, const TMap<FString, FString>& UpgradeHeaders) override;
	//~ End IWebSocketsManager Interface

protected:
	bool GameThreadTick(float DeltaTime);

	void InitHttpManager();

protected:
	FTSTicker::FDelegateHandle TickHandle;

	FWinHttpHttpManager* WinHttpHttpManager = nullptr;

	TArray<TWeakPtr<FWinHttpWebSocket>> ActiveWebSockets;
};

#endif // WITH_WEBSOCKETS && WITH_WINHTTPWEBSOCKETS
