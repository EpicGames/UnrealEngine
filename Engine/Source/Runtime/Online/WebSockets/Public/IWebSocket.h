// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class IWebSocket
{
public:

	virtual ~IWebSocket() = default;

	/**
	 * Initiate a client connection to the server.
	 * Use this after setting up event handlers or to reconnect after connection errors.
	 */
	virtual void Connect() = 0;

	/**
	 * Close the current connection.
	 * @param Code Numeric status code explaining why the connection is being closed. Default is 1000. See WebSockets spec for valid codes.
	 * @param Reason Human readable string explaining why the connection is closing.
	 */
	virtual void Close(int32 Code = 1000, const FString& Reason = FString()) = 0;

	/**
	 * Inquire if this web socket instance is connected to a server.
	 */
	virtual bool IsConnected() = 0;
	
	/**
	 * Transmit data over the connection.
	 * @param Data data to be sent as a UTF-8 encoded string.
	 */
	virtual void Send(const FString& Data) = 0;

	/**
	 * Transmit data over the connection.
	 * @param Data raw binary data to be sent.
	 * @param Size number of bytes to send.
	 * @param bIsBinary set to true to send binary frame to the peer instead of text.
	 */
	virtual void Send(const void* Data, SIZE_T Size, bool bIsBinary = false) = 0;

	/**
	 * Optionally change memory limit for receiving UTF-8 text on this socket. 
	 * Default from config TextMessageMemoryLimit under [WebSockets] or 1MB.
	 * @param TextMessageMemoryLimit new buffer size upper limit in bytes.
	 */
	virtual void SetTextMessageMemoryLimit(uint64 TextMessageMemoryLimit) = 0;

	/**
	 * Delegate called when a web socket connection has been established successfully.
	 *
	 */
	DECLARE_EVENT(IWebSocket, FWebSocketConnectedEvent);
	virtual FWebSocketConnectedEvent& OnConnected() = 0;

	/**
	 * Delegate called when a web socket connection could not be established.
	 *
	 */
	DECLARE_EVENT_OneParam(IWebSocket, FWebSocketConnectionErrorEvent, const FString& /* Error */);
	virtual FWebSocketConnectionErrorEvent& OnConnectionError() = 0;

	/**
	 * Delegate called when a web socket connection has been closed.
	 *
	 */
	DECLARE_EVENT_ThreeParams(IWebSocket, FWebSocketClosedEvent, int32 /* StatusCode */, const FString& /* Reason */, bool /* bWasClean */);
	virtual FWebSocketClosedEvent& OnClosed() = 0;

	/**
	 * Delegate called when a web socket text message has been received.
	 * Assumes the payload is encoded as UTF8. For binary data, bind to OnRawMessage instead.
	 *
	 */
	DECLARE_EVENT_OneParam(IWebSocket, FWebSocketMessageEvent, const FString& /* MessageString */);
	virtual FWebSocketMessageEvent& OnMessage() = 0;
	
      /**
	 * Delegate called when web socket binary data has been received.
	 * bIsLastFragment will be true if it is the last fragment of the current packet.
	 */
	DECLARE_EVENT_ThreeParams(IWebSocket, FWebSocketBinaryMessageEvent, const void* /* Data */, SIZE_T /* Size */, bool /* bIsLastFragment */);
	virtual FWebSocketBinaryMessageEvent& OnBinaryMessage() = 0;
	
      /**
	 * Delegate called when any web socket data has been received.
	 * May be called multiple times for a message if the message was split into multiple frames. 
	 * This is called for every web socket fragment.
	 * BytesRemaining is the number of bytes left in the current fragment.
	 */
	DECLARE_EVENT_ThreeParams(IWebSocket, FWebSocketRawMessageEvent, const void* /* Data */, SIZE_T /* Size */, SIZE_T /* BytesRemaining */);
	virtual FWebSocketRawMessageEvent& OnRawMessage() = 0;

	/**
	* Delegate called when a web socket text message has been sent.
	* Assume UTF-8 encoding.
	*/
	DECLARE_EVENT_OneParam(IWebSocket, FWebSocketMessageSentEvent, const FString& /* MessageString */);
	virtual FWebSocketMessageSentEvent& OnMessageSent() = 0;
};
