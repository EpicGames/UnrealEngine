// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Containers/ArrayView.h"

class IWebSocket;

class IWebSocketsManager
{
public:

	/**
	 * Web sockets start-up: call before creating any web sockets
	 */ 
	virtual void InitWebSockets(TArrayView<const FString> Protocols) = 0;


	/**
	 * Web sockets teardown: call at shutdown, in particular after all use of SSL has finished
	 */ 
	virtual void ShutdownWebSockets() = 0;

	/**
	 * Instantiates a new web socket for the current platform
	 *
	 * @param Url The URL to which to connect; this should be the URL to which the WebSocket server will respond.
	 * @param Protocols a list of protocols the client will handle.
	 * @return new IWebSocket instance
	 */
	virtual TSharedRef<IWebSocket> CreateWebSocket(const FString& Url, const TArray<FString>& Protocols, const TMap<FString, FString>& UpgradeHeaders) = 0;

	/**
	 * Virtual destructor
	 */
	virtual ~IWebSocketsManager() = default;

	/**
	 * Update configuration. Called when config has been updated and we need to apply any changes.
	 */
	virtual void UpdateConfigs() {}
};
