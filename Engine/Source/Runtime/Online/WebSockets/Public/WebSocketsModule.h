// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

#if WITH_WEBSOCKETS
class IWebSocket;
class IWebSocketsManager;
#endif // #if WITH_WEBSOCKETS

/**
 * Module for web socket implementations
 */
class FWebSocketsModule :
	public IModuleInterface
{

public:

	// FWebSocketModule
	FWebSocketsModule()
#if WITH_WEBSOCKETS
		: WebSocketsManager(nullptr)
#endif // #if WITH_WEBSOCKETS
	{
	}
#if WITH_WEBSOCKETS
	/**
	 * Delegate fired when a web socket is created.
	 *
	 * @param WebSocket Reference to newly created web socket
	 * @param Protocols Array of protocols on web socket
	 * @param Url Url the web socket is connected to
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnWebSocketCreated, const TSharedPtr<class IWebSocket>& /* WebSocket */, const TArray<FString>& /* Protocols */, const FString& /* Url */);
	FOnWebSocketCreated OnWebSocketCreated;
#endif // #if WITH_WEBSOCKETS
	
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static WEBSOCKETS_API FWebSocketsModule& Get();

#if WITH_WEBSOCKETS
	/**
	 * Instantiates a new web socket for the current platform
	 *
	 * @param Url The URL to which to connect; this should be the URL to which the WebSocket server will respond.
	 * @param Protocols a list of protocols the client will handle.
	 * @return new IWebSocket instance
	 */
	WEBSOCKETS_API virtual TSharedRef<IWebSocket> CreateWebSocket(const FString& Url, const TArray<FString>& Protocols, const TMap<FString, FString>& UpgradeHeaders = TMap<FString, FString>());


	/**
	 * Instantiates a new web socket for the current platform
	 *
	 * @param Url The URL to which to connect; this should be the URL to which the WebSocket server will respond.
	 * @param Protocol an optional sub-protocol. If missing, an empty string is assumed.
	 * @return new IWebSocket instance
	 */
	WEBSOCKETS_API virtual TSharedRef<IWebSocket> CreateWebSocket(const FString& Url, const FString& Protocol = FString(), const TMap<FString, FString>& UpgradeHeaders = TMap<FString, FString>());
#endif // #if WITH_WEBSOCKETS

private:
	static WEBSOCKETS_API FString BuildUpgradeHeader(const TMap<FString, FString>& Headers);

	// IModuleInterface

	/**
	 * Called when WebSockets module is loaded
	 * Initialize implementation specific parts of WebSockets handling
	 */
	WEBSOCKETS_API virtual void StartupModule() override;
	
	/**
	 * Called when WebSockets module is unloaded
	 * Shutdown implementation specific parts of WebSockets handling
	 */
	WEBSOCKETS_API virtual void ShutdownModule() override;

#if WITH_WEBSOCKETS
	/** Manages active web sockets */
	IWebSocketsManager* WebSocketsManager;
	friend class FLwsWebSocketsManager;
	friend class FLwsWebSocket;

	/** Delegate for config file changes. */
	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);
#endif // #if WITH_WEBSOCKETS

	/** singleton for the module while loaded and available */
	static WEBSOCKETS_API FWebSocketsModule* Singleton;
};
