// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebSockets : ModuleRules
{
	protected virtual bool PlatformSupportsLibWebsockets
	{
		get
		{
			return
				Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) ||
				Target.Platform == UnrealTargetPlatform.Android ||
				Target.Platform == UnrealTargetPlatform.Mac ||
				Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) ||
				Target.Platform == UnrealTargetPlatform.IOS;
		}
	}

	protected virtual bool bPlatformSupportEventLoop
	{
		get => true;
	}

	protected virtual bool bPlatformSupportsWinHttpWebSockets
	{
		get
		{
			// Availability requires Windows 8.1 or greater, as this is the min version of WinHttp that supports WebSockets
			return Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.TargetWindowsVersion >= 0x0603;
		}
	}

	protected virtual bool bPlatformSupportsWinRTWebsockets
	{
		get => false;
	}

	protected virtual bool UsePlatformSSL
	{
		get => false;
	}

	protected virtual bool ShouldUseModule
	{
		get
		{
			return PlatformSupportsLibWebsockets || bPlatformSupportsWinRTWebsockets || bPlatformSupportsWinHttpWebSockets;
		}
	}

	protected virtual string WebSocketsManagerPlatformInclude
	{
		get
		{
			if (PlatformSupportsLibWebsockets)
			{
				return "Lws/LwsWebSocketsManager.h";
			}
			else if (bPlatformSupportsWinHttpWebSockets)
			{
				return "WinHttp/WinHttpWebSocketsManager.h";
			}
			else
			{
				return "";
			}
		}
	}

	protected virtual string WebSocketsManagerPlatformClass
	{
		get
		{
			if (PlatformSupportsLibWebsockets)
			{
				return "FLwsWebSocketsManager";
			}
			else if (bPlatformSupportsWinHttpWebSockets)
			{
				return "FWinHttpWebSocketsManager";
			}
			else
			{
				return "";
			}
		}
	}

	public WebSockets(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"HTTP"
			}
		);

		bool bWithWebSockets = false;
		bool bWithLibWebSockets = false;
		bool bWithWinHttpWebSockets = false;

		if (ShouldUseModule)
		{
			bWithWebSockets = true;

			if (PlatformSupportsLibWebsockets)
			{
				bWithLibWebSockets = true;

				if (UsePlatformSSL)
				{
					PrivateDefinitions.Add("WITH_SSL=0");
					AddEngineThirdPartyPrivateStaticDependencies(Target, "libWebSockets");
				}
				else
				{
					AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL", "libWebSockets", "zlib");
					PrivateDependencyModuleNames.Add("SSL");
				}
			}
			else if (bPlatformSupportsWinHttpWebSockets)
			{
				// Enable WinHttp Support
				bWithWinHttpWebSockets = true;

				AddEngineThirdPartyPrivateStaticDependencies(Target, "WinHttp");
			}
		}

		PrivateDefinitions.Add("UE_WEBSOCKETS_PLATFORM_SUPPORT_EVENT_LOOP=" + (bPlatformSupportEventLoop ? "1" : "0"));

		PublicDefinitions.Add("WEBSOCKETS_PACKAGE=1");
		PublicDefinitions.Add("WITH_WEBSOCKETS=" + (bWithWebSockets ? "1" : "0"));
		PublicDefinitions.Add("WITH_LIBWEBSOCKETS=" + (bWithLibWebSockets ? "1" : "0"));
		PublicDefinitions.Add("WITH_WINHTTPWEBSOCKETS=" + (bWithWinHttpWebSockets ? "1" : "0"));
		string PlatformInclude = WebSocketsManagerPlatformInclude;
		if (PlatformInclude.Length > 0)
		{
			PublicDefinitions.Add("WEBSOCKETS_MANAGER_PLATFORM_INCLUDE=\"" + WebSocketsManagerPlatformInclude + "\"");
			PublicDefinitions.Add("WEBSOCKETS_MANAGER_PLATFORM_CLASS=" + WebSocketsManagerPlatformClass);
		}
	}
}
