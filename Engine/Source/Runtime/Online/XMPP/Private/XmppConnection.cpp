// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppConnection.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"

enum class XMPP_RESOURCE_VERSION : uint8
{
	INITIAL = 1,
	ADDED_PLATFORMUSERID = 2,
	
	// -----<new versions can be added before this line>-------------------------------------------------
	// - this needs to be the last line (see note below)
	VERSION_PLUSONE,
	LATEST = VERSION_PLUSONE - 1
};

FXmppUserJid FXmppUserJid::FromFullJid(const FString& JidString)
{
	FString User;
	FString Domain;
	FString Resource;

	FString DomainAndResource;
	if (JidString.Split(TEXT("@"), &User, &DomainAndResource, ESearchCase::CaseSensitive, ESearchDir::FromStart))
	{
		if (!DomainAndResource.Split(TEXT("/"), &Domain, &Resource, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			// If we don't have a resource, the domain is all of DomainAndResource
			Domain = MoveTemp(DomainAndResource);
		}
	}
	else
	{
		if (!JidString.Split(TEXT("/"), &Domain, &Resource, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			// If we don't have a resource, we only have the Domain in the JidString
			Domain = JidString;
		}
	}

	return FXmppUserJid(MoveTemp(User), MoveTemp(Domain), MoveTemp(Resource));
}

bool FXmppUserJid::ParseResource(const FString& InResource, FString& OutAppId, FString& OutPlatform, FString& OutPlatformUserId)
{
	OutAppId.Empty();
	OutPlatform.Empty();

	TArray<FString> ParsedResource;
	if (InResource.ParseIntoArray(ParsedResource, TEXT(":"), false) > 1)
	{
		if (ParsedResource[0].StartsWith(TEXT("V")))
		{
			const int32 Version = FCString::Atoi((*ParsedResource[0]) + 1);
			if (Version >= static_cast<int32>(XMPP_RESOURCE_VERSION::INITIAL))
			{
				const int32 NumResources = ParsedResource.Num();
				if (NumResources >= 3)
				{
					OutAppId = ParsedResource[1];
					OutPlatform = ParsedResource[2];
					if (Version >= static_cast<int32>(XMPP_RESOURCE_VERSION::ADDED_PLATFORMUSERID))
					{
						if (NumResources >= 4)
						{
							OutPlatformUserId = ParsedResource[3];
						}
					}
					return true;
				}
			}
		}
	}
	else
	{
		FString ClientId, NotUsed;
		if (InResource.Split(TEXT("-"), &ClientId, &NotUsed) &&
			!ClientId.IsEmpty())
		{
			OutAppId = ClientId;
			return true;
		}
	}
	return false;
}

FString FXmppUserJid::CreateResource(const FString& AppId, const FString& Platform, const FString& PlatformUserId)
{
	return FString::Printf(TEXT("V%d:%s:%s:%s:%s"),
		static_cast<int32>(XMPP_RESOURCE_VERSION::LATEST), *AppId, *Platform, *PlatformUserId, *FGuid::NewGuid().ToString(EGuidFormats::Digits));
}

FString FXmppUserJid::ParseMucUserResource(const FString& InResource)
{
	FString UserResource;
	TArray<FString> Tokens;
	if (InResource.ParseIntoArray(Tokens, TEXT(":"), false) > 3)
	{
		const FString& VersionToken = Tokens[2];
		if (VersionToken.StartsWith(TEXT("V")) && (VersionToken.Len() > 1))
		{
			const TCHAR* PossibleNum = &((*VersionToken)[1]);
			if (FCString::IsNumeric(PossibleNum))
			{
				// Take the portion of the resource starting with :V<version>
				int32 NumChars = Tokens[0].Len() + Tokens[1].Len() + 2;
				if (NumChars < InResource.Len())
				{
					UserResource = InResource.Mid(NumChars);
				}
			}
		}
	}

	return UserResource;
}

FString FXmppUserJid::ToDebugString() const
{
	static const bool bUseShortDebugString = []()
	{
		bool bValue = false;
		GConfig->GetBool(TEXT("XMPP"), TEXT("bUseShortDebugString"), bValue, GEngineIni);
		return bValue;
	}();

	if (bUseShortDebugString)
	{
		return FString::Printf(TEXT("%s:%s"), *Id, *Resource);
	}
	else
	{
		return FString::Printf(TEXT("%s:%s:%s"), *Id, *Domain, *Resource);
	}
}
