// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppModule.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"
#include "XmppMessages.h"
#include "XmppPubSub.h"
#include "XmppLog.h"
#include "XmppTests.h"
#include "XmppNull.h"
#if WITH_XMPP_STROPHE
#include "XmppStrophe/XmppStrophe.h"
#include "WebSocketsModule.h"
#endif

DEFINE_LOG_CATEGORY(LogXmpp);

// FXmppModule

IMPLEMENT_MODULE(FXmppModule, XMPP);

FXmppModule* FXmppModule::Singleton = nullptr;

void FXmppModule::StartupModule()
{
	Singleton = this;

	bEnabled = true;
	GConfig->GetBool(TEXT("XMPP"), TEXT("bEnabled"), bEnabled, GEngineIni);

	if (bEnabled)
	{
#if WITH_XMPP_STROPHE
		FXmppStrophe::Init();
		FModuleManager::LoadModuleChecked<FWebSocketsModule>("WebSockets");
#endif
	}
}

void FXmppModule::ShutdownModule()
{
	for (auto It = ActiveConnections.CreateIterator(); It; ++It)
	{
		CleanupConnection(It.Value());
	}

	if (bEnabled)
	{
#if WITH_XMPP_STROPHE
		FXmppStrophe::Cleanup();
#endif
	}

	Singleton = nullptr;
}

bool FXmppModule::HandleXmppCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FXmppServer XmppServer;
	XmppServer.bUseSSL = true;
	XmppServer.AppId = TEXT("ue_xmpp_test");

	//@todo sz1 - for debug only
	FString ConfigOverride;
	FParse::Value(FCommandLine::Get(), TEXT("EPICAPP="), ConfigOverride);
	if (ConfigOverride.IsEmpty())
	{
		FParse::Value(FCommandLine::Get(), TEXT("EPICENV="), ConfigOverride);
	}
	if (ConfigOverride.IsEmpty())
	{
		FParse::Value(FCommandLine::Get(), TEXT("MCPCONFIG="), ConfigOverride);
	}

	XmppServer.ServerAddr = TEXT("127.0.0.1");
	XmppServer.Domain = TEXT("localhost.net");
	XmppServer.ServerPort = 5222;


	if (FParse::Command(&Cmd, TEXT("Test")))
	{
		FString UserName = FParse::Token(Cmd, false);
		FString Password = FParse::Token(Cmd, false);

		// deletes itself when done with test tasks
		(new FXmppTest())->Test(UserName, Password, XmppServer);

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("Login")))
	{
		FString UserName = FParse::Token(Cmd, false);
		FString Password = FParse::Token(Cmd, false);

		if (UserName.IsEmpty() || Password.IsEmpty())
		{
			UE_LOGF(LogXmpp, Warning, "Usage: XMPP Login <username> <password>");
		}
		else
		{
			TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
			if (Connection.IsValid() &&
				(Connection->GetLoginStatus() == EXmppLoginStatus::LoggedIn || Connection->GetLoginStatus() == EXmppLoginStatus::ProcessingLogin))
			{
				UE_LOGF(LogXmpp, Warning, "Already logged in as <%ls>", *UserName);
			}
			else
			{
				Connection = CreateConnection(UserName);
				Connection->SetServer(XmppServer);
				Connection->Login(UserName, Password);
			}
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("Logout")))
	{
		FString UserName = FParse::Token(Cmd, false);

		if (UserName.IsEmpty())
		{
			UE_LOGF(LogXmpp, Warning, "Usage: XMPP Logout <username>");
		}
		else
		{
			TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
			if (!Connection.IsValid())
			{
				UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
			}
			else
			{
				Connection->Logout();
				//RemoveConnection(Connection.ToSharedRef());
			}
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("Presence")))
	{
		FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
		FString OnlineStr = FParse::Token(Cmd, false);	// "ONLINE, AWAY, DND, OFFLINE, etc"
		FString StatusStr = FParse::Token(Cmd, false);	// "status message treated as raw string"

		if (UserName.IsEmpty() || OnlineStr.IsEmpty() || StatusStr.IsEmpty())
		{
			UE_LOGF(LogXmpp, Warning, "Usage: XMPP Presence <username> <ONLINE,AWAY,DND,OFFLINE,XA> <status text>");
		}
		else
		{
			TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
			if (!Connection.IsValid())
			{
				UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
			}
			else
			{
				if (Connection->Presence().IsValid())
				{
					FXmppUserPresence XmppPresence = Connection->Presence()->GetPresence();
					XmppPresence.bIsAvailable = true;
					if (OnlineStr.Equals(TEXT("ONLINE"), ESearchCase::IgnoreCase))
					{
						XmppPresence.Status = EXmppPresenceStatus::Online;
					}
					else if (OnlineStr.Equals(TEXT("AWAY"), ESearchCase::IgnoreCase))
					{
						XmppPresence.Status = EXmppPresenceStatus::Away;
					}
					else if (OnlineStr.Equals(TEXT("DND"), ESearchCase::IgnoreCase))
					{
						XmppPresence.Status = EXmppPresenceStatus::DoNotDisturb;
					}
					else if (OnlineStr.Equals(TEXT("OFFLINE"), ESearchCase::IgnoreCase))
					{
						XmppPresence.Status = EXmppPresenceStatus::Offline;
					}
					else if (OnlineStr.Equals(TEXT("XA"), ESearchCase::IgnoreCase))
					{
						XmppPresence.Status = EXmppPresenceStatus::ExtendedAway;
					}
					if (!StatusStr.IsEmpty())
					{
						XmppPresence.StatusStr = StatusStr;
					}
					Connection->Presence()->UpdatePresence(XmppPresence);
				}
			}
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("PresenceQuery")))
	{
		FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
		FString RosterUser = FParse::Token(Cmd, false);	// User must be in current roster

		if (UserName.IsEmpty() || RosterUser.IsEmpty())
		{
			UE_LOGF(LogXmpp, Warning, "Usage: XMPP PresenceQuery <username> <rosteruser>");
		}
		else
		{
			TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
			if (!Connection.IsValid())
			{
				UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
			}
			else
			{
				if (Connection->Presence().IsValid())
				{
					Connection->Presence()->QueryPresence(RosterUser);
				}
			}
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("Message")))
	{
		FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
		FString Recipient = FParse::Token(Cmd, false);	// User to send message to
		FString Payload = FParse::Token(Cmd, false);	// payload to send

		if (UserName.IsEmpty() || Recipient.IsEmpty())
		{
			UE_LOGF(LogXmpp, Warning, "Usage: XMPP Message <username> <recipient> <optional payload string>");
		}
		else
		{
			TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
			if (!Connection.IsValid())
			{
				UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
			}
			else
			{
				if (Connection->Messages().IsValid())
				{
					FXmppUserJid RecipientId(Recipient, Connection->GetServer().Domain);
					Connection->Messages()->SendMessage(RecipientId, TEXT("test"), Payload, false);
				}
			}
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("Chat")))
	{
		FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
		FString Recipient = FParse::Token(Cmd, false);	// User to send message to
		FString Body = FParse::Token(Cmd, false);		// payload to send

		if (UserName.IsEmpty() || Recipient.IsEmpty())
		{
			UE_LOGF(LogXmpp, Warning, "Usage: XMPP Chat <username> <recipient> <body>");
		}
		else
		{
			TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
			if (!Connection.IsValid())
			{
				UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
			}
			else
			{
				if (Connection->PrivateChat().IsValid())
				{
					FXmppUserJid RecipientId(Recipient, Connection->GetServer().Domain);
					Connection->PrivateChat()->SendChat(RecipientId, Body);
				}
			}
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("Muc")))
	{
		if (FParse::Command(&Cmd, TEXT("Create")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString RoomId = FParse::Token(Cmd, false);		// room to create
			FString IsPrivate = FParse::Token(Cmd, false);		// whether to make room private
			FString Password = FParse::Token(Cmd, false);		// room secret, if private

			if (UserName.IsEmpty() || RoomId.IsEmpty() || (IsPrivate == TEXT("1") && Password.IsEmpty()))
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP Muc Create <username> <room> <private 1/0> <password>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->MultiUserChat().IsValid())
					{
						FXmppRoomConfig RoomConfig;
						RoomConfig.RoomName = TEXT("Test") + RoomId;
						RoomConfig.bIsPrivate = false;
						RoomConfig.bIsPersistent = false;
						if (IsPrivate == TEXT("1"))
						{
							RoomConfig.bIsPrivate = true;
						}
						RoomConfig.Password = Password;
						Connection->MultiUserChat()->OnRoomCreated().AddRaw(this, &FXmppModule::OnXmppRoomCreated);
						Connection->MultiUserChat()->OnRoomConfigured().AddRaw(this, &FXmppModule::OnXmppRoomConfigured);
						Connection->MultiUserChat()->CreateRoom(RoomId, UserName, RoomConfig);
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Join")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString RoomId = FParse::Token(Cmd, false);		// room to join
			FString Nickname = FParse::Token(Cmd, false);	// nickname to join (must be unique or join will fail)
			FString Password = FParse::Token(Cmd, false);	// optional password if joining private

			if (UserName.IsEmpty() || RoomId.IsEmpty() || Nickname.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP Muc Join <username> <room> <nickname> <?password>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->MultiUserChat().IsValid())
					{
						if (Password.IsEmpty())
						{
							Connection->MultiUserChat()->JoinPublicRoom(RoomId, Nickname);
						}
						else
						{
							Connection->MultiUserChat()->JoinPrivateRoom(RoomId, Nickname, Password);
						}
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Exit")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString RoomId = FParse::Token(Cmd, false);		// room to join

			if (UserName.IsEmpty() || RoomId.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP Muc Exit <username> <room>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->MultiUserChat().IsValid())
					{
						Connection->MultiUserChat()->ExitRoom(RoomId);
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Config")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString RoomId = FParse::Token(Cmd, false);		// room to configure
			FString IsPrivate = FParse::Token(Cmd, false);		// whether to make room private
			FString Password = FParse::Token(Cmd, false);		// room secret, if private

			if (UserName.IsEmpty() || RoomId.IsEmpty() || (IsPrivate == TEXT("1") && Password.IsEmpty()))
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP Muc Config <username> <room> <subject> <private 1/0> <password>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->MultiUserChat().IsValid())
					{
						FXmppRoomConfig RoomConfig;
						RoomConfig.bIsPrivate = false;
						if (IsPrivate == TEXT("1"))
						{
							RoomConfig.bIsPrivate = true;
						}
						RoomConfig.Password = Password;
						Connection->MultiUserChat()->ConfigureRoom(RoomId, RoomConfig);
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Refresh")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString RoomId = FParse::Token(Cmd, false);		// roomid

			if (UserName.IsEmpty() || RoomId.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP Muc Refresh <username> <room>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->MultiUserChat().IsValid())
					{
						Connection->MultiUserChat()->RefreshRoomInfo(RoomId);
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Chat")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString RoomId = FParse::Token(Cmd, false);		// room to join
			FString Body = FParse::Token(Cmd, false);		// payload to send
			FString ExtraInfo = FParse::Token(Cmd, false);	// optional extra info to send (typically json payload)

			if (UserName.IsEmpty() || RoomId.IsEmpty() || Body.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP Muc Chat <username> <room> <body>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->MultiUserChat().IsValid())
					{
						Connection->MultiUserChat()->SendChat(RoomId, Body, ExtraInfo);
					}
				}
			}
		}

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("PubSub")))
	{
		if (FParse::Command(&Cmd, TEXT("Create")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString NodeId = FParse::Token(Cmd, false);		// node to create

			if (UserName.IsEmpty() || NodeId.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP PubSub Create <username> <node>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->PubSub().IsValid())
					{
						FXmppPubSubConfig PubSubConfig;
						Connection->PubSub()->CreateNode(NodeId, PubSubConfig);
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Destroy")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString NodeId = FParse::Token(Cmd, false);		// node to destroy

			if (UserName.IsEmpty() || NodeId.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP PubSub Destroy <username> <node>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->PubSub().IsValid())
					{
						Connection->PubSub()->DestroyNode(NodeId);
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Subscribe")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString NodeId = FParse::Token(Cmd, false);		// node to subscribe to

			if (UserName.IsEmpty() || NodeId.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP PubSub Subscribe <username> <node>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->PubSub().IsValid())
					{
						Connection->PubSub()->Subscribe(NodeId);
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Unsubscribe")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString NodeId = FParse::Token(Cmd, false);		// node to subscribe to

			if (UserName.IsEmpty() || NodeId.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP PubSub Unsubscribe <username> <node>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->PubSub().IsValid())
					{
						Connection->PubSub()->Unsubscribe(NodeId);
					}
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("Publish")))
		{
			FString UserName = FParse::Token(Cmd, false);	// Assumes user is already logged in
			FString NodeId = FParse::Token(Cmd, false);		// node to subscribe to
			FString PayloadStr = FParse::Token(Cmd, false);	// message payload string

			if (UserName.IsEmpty() || NodeId.IsEmpty() || PayloadStr.IsEmpty())
			{
				UE_LOGF(LogXmpp, Warning, "Usage: XMPP PubSub Publish <username> <node> <text>");
			}
			else
			{
				TSharedPtr<IXmppConnection> Connection = GetConnection(UserName);
				if (!Connection.IsValid())
				{
					UE_LOGF(LogXmpp, Warning, "No logged in user found for <%ls>", *UserName);
				}
				else
				{
					if (Connection->PubSub().IsValid())
					{
						FXmppPubSubMessage Message;
						Message.Payload = PayloadStr;
						Connection->PubSub()->PublishMessage(NodeId, Message);
					}
				}
			}
		}

		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("LogVerbosity")))
	{
		FName Verbosity = *FParse::Token(Cmd, false);

		const FName NAME_NoLogging("NoLogging");
		const FName NAME_Fatal("Fatal");
		const FName NAME_Display("Display");
		const FName NAME_Log("Log");
		const FName NAME_Verbose("Verbose");
		const FName NAME_VeryVerbose("VeryVerbose");

		if (Verbosity == NAME_NoLogging)
		{
			UE_SET_LOG_VERBOSITY(LogXmpp, NoLogging);
		}
		else if (Verbosity == NAME_Fatal)
		{
			UE_SET_LOG_VERBOSITY(LogXmpp, Fatal);
		}
		else if (Verbosity == NAME_Error)
		{
			UE_SET_LOG_VERBOSITY(LogXmpp, Error);
		}
		else if (Verbosity == NAME_Warning)
		{
			UE_SET_LOG_VERBOSITY(LogXmpp, Warning);
		}
		else if (Verbosity == NAME_Display)
		{
			UE_SET_LOG_VERBOSITY(LogXmpp, Display);
		}
		else if (Verbosity == NAME_Log)
		{
			UE_SET_LOG_VERBOSITY(LogXmpp, Log);
		}
		else if (Verbosity == NAME_Verbose)
		{
			UE_SET_LOG_VERBOSITY(LogXmpp, Verbose);
		}
		else if (Verbosity == NAME_VeryVerbose)
		{
			UE_SET_LOG_VERBOSITY(LogXmpp, VeryVerbose);
		}
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("Dump")))
	{
		UE_LOGF(LogXmpp, Warning, "Dumping %d XMPP connections: ", ActiveConnections.Num());
		for (const auto& Pair : ActiveConnections)
		{
			const TSharedRef<IXmppConnection>& Connection = Pair.Value;
			Connection->DumpState();
		}
		return true;
	}

	return false;
}

bool FXmppModule::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Ignore any execs that don't start with Xmpp
	if (FParse::Command(&Cmd, TEXT("XMPP")))
	{
		return HandleXmppCommand( Cmd, Ar );
	}
	return false;
}

FXmppModule& FXmppModule::Get()
{
	if (Singleton == nullptr)
	{
		check(IsInGameThread());
		FModuleManager::LoadModuleChecked<FXmppModule>("XMPP");
	}
	check(Singleton != nullptr);
	return *Singleton;
}

bool FXmppModule::IsAvailable()
{
	return Singleton != nullptr;
}

TSharedRef<IXmppConnection> FXmppModule::CreateConnection(const FString& UserId)
{
	TSharedPtr<IXmppConnection> Connection = GetConnection(UserId);
	if (Connection.IsValid())
	{
		return Connection.ToSharedRef();
	}
	else
	{
#if WITH_XMPP_STROPHE
		if (bEnabled)
		{
			Connection = FXmppStrophe::CreateConnection();
		}
		else
#endif
		{
			Connection = FXmppNull::CreateConnection();
		}

		TSharedRef<IXmppConnection> ConnectionRef = Connection.ToSharedRef();
		ActiveConnections.Add(UserId, ConnectionRef);
		OnXmppConnectionCreated.Broadcast(ConnectionRef);
		
		return ConnectionRef;
	}
}

TSharedPtr<IXmppConnection> FXmppModule::GetConnection(const FString& UserId) const
{
	TSharedPtr<IXmppConnection> Result;

	const TSharedRef<IXmppConnection>* Found = ActiveConnections.Find(UserId);
	if (Found != nullptr)
	{
		Result = *Found;
	}

	return Result;
}

void FXmppModule::RemoveConnection(const FString& UserId)
{
	TSharedPtr<IXmppConnection> Existing = GetConnection(UserId);
	if (Existing.IsValid())
	{
		CleanupConnection(Existing.ToSharedRef());
		// If we found a TSharedPtr, keep on a removal list until the next tick so it doesn't get destroyed while ticking other things that depend on it
		PendingRemovals.Add(Existing);
	}

	ActiveConnections.Remove(UserId);
}

bool FXmppModule::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FXmppModule_Tick);
	ProcessPendingRemovals();
	return true;
}

void FXmppModule::ProcessPendingRemovals()
{
	if (PendingRemovals.Num() > 0)
	{
		PendingRemovals.Empty();
	}
}

void FXmppModule::RemoveConnection(const TSharedRef<IXmppConnection>& Connection)
{
	for (auto It = ActiveConnections.CreateIterator(); It; ++It)
	{
		if (It.Value() == Connection)
		{
			CleanupConnection(Connection);
			It.RemoveCurrent();
			break;
		}
	}
}

void FXmppModule::CleanupConnection(const TSharedRef<class IXmppConnection>& Connection)
{

}

void FXmppModule::OnXmppRoomCreated(const TSharedRef<IXmppConnection>& Connection, bool bSuccess, const FXmppRoomId& RoomId, const FString& Error)
{
	Connection->MultiUserChat()->OnRoomCreated().RemoveAll(this);

	UE_LOGF(LogXmpp, Log, "FXmppModule::OnXmppRoomCreated - entered - user(%ls) room(%ls)", *Connection->GetUserJid().Id, *RoomId);
}

void FXmppModule::OnXmppRoomConfigured(const TSharedRef<IXmppConnection>& Connection, bool bSuccess, const FXmppRoomId& RoomId, const FString& Error)
{
	Connection->MultiUserChat()->OnRoomConfigured().RemoveAll(this);

	UE_LOGF(LogXmpp, Log, "FXmppModule::OnXmppRoomConfigured - entered - user(%ls) room(%ls)", *Connection->GetUserJid().Id, *RoomId);
}
