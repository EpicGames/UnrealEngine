// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppNull.h"
#include "XmppConnection.h"
#include "XmppLog.h"

class FXmppConnectionNull
	: public IXmppConnection
{
public:

	// IXmppConnection

	virtual void SetServer(const FXmppServer& Server) override
	{}
	virtual const FXmppServer& GetServer() const override
	{ 
		return ServerConfig; 
	}
	virtual void Login(const FString& UserId, const FString& Auth) override
	{
		UserJid.Id = UserId;
		OnLoginComplete().Broadcast(UserJid, false, TEXT("not implemented"));	
	}
	virtual void Logout() override
	{
		OnLogoutComplete().Broadcast(UserJid, false, TEXT("not implemented"));
	}
	virtual EXmppLoginStatus::Type GetLoginStatus() const override
	{ 
		return EXmppLoginStatus::LoggedOut; 
	}
	virtual const FXmppUserJid& GetUserJid() const override
	{
		return UserJid;
	}
	virtual FOnXmppLoginComplete& OnLoginComplete() override { return OnXmppLoginCompleteDelegate; }
	virtual FOnXmppLoginChanged& OnLoginChanged() override { return OnXmppLoginChangedDelegate; }
	virtual FOnXmppLogoutComplete& OnLogoutComplete() override { return OnXmppLogoutCompleteDelegate; }
	virtual FOnXmppStanzaSent& OnStanzaSent() override { return OnXmppStanzaSentDelegate; }
	virtual FOnXmppStanzaReceived& OnStanzaReceived() override { return OnXmppStanzaReceivedDelegate; }

	virtual IXmppPresencePtr Presence() override { return NULL; }
	virtual IXmppPubSubPtr PubSub() override { return NULL; }
	virtual IXmppMessagesPtr Messages() override { return NULL; }
	virtual IXmppMultiUserChatPtr MultiUserChat() override { return NULL; }
	virtual IXmppChatPtr PrivateChat() override { return NULL; }

	virtual void DumpState() const override {}

	// FXmppConnectionNull

	FXmppConnectionNull() {};
	virtual ~FXmppConnectionNull() {};

private:

	FXmppServer ServerConfig;
	FXmppUserJid UserJid;

	// completion delegates
	FOnXmppLoginComplete OnXmppLoginCompleteDelegate;
	FOnXmppLoginChanged OnXmppLoginChangedDelegate;
	FOnXmppLogoutComplete OnXmppLogoutCompleteDelegate;
	FOnXmppStanzaReceived OnXmppStanzaReceivedDelegate;
	FOnXmppStanzaSent OnXmppStanzaSentDelegate;
};


TSharedRef<IXmppConnection> FXmppNull::CreateConnection()
{
	UE_LOGF(LogXmpp, Warning, "Xmpp not implemented. Creating FXmppNull connection");

	return MakeShareable(new FXmppConnectionNull());
}
