// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Mock for when Xmpp implementation is not available on a platform */
class FXmppNull
{
public:

	// FXmppNull

	static TSharedRef<class IXmppConnection> CreateConnection();
};

