// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/StropheConnection.h"
#include "XmppStrophe/StropheContext.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/StropheError.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppLog.h"
#include "HAL/PlatformProcess.h"

#if WITH_XMPP_STROPHE

THIRD_PARTY_INCLUDES_START
#include "strophe.h"
THIRD_PARTY_INCLUDES_END

int StropheStanzaEventHandler(xmpp_conn_t* const UnusedPtr, xmpp_stanza_t* const EventStanza, void* const VoidConnectionPtr)
{
	check(VoidConnectionPtr != nullptr);

	const FStropheStanza IncomingStanza(EventStanza);

	// Ignore session stanza (bug in libstrophe that we see get this at all)
	static const FString LoginSessionStanza("_xmpp_session1");
	if (IncomingStanza.GetId() != LoginSessionStanza)
	{
		// We want to forward our stanza to our connection and out to our handlers
		FXmppConnectionStrophe* const ConnectionPtr = static_cast<FXmppConnectionStrophe* const>(VoidConnectionPtr);
		ConnectionPtr->ReceiveStanza(IncomingStanza);
	}

	return 1;
}

void StropheConnectionEventHandler(xmpp_conn_t* const UnusedPtr,
	const xmpp_conn_event_t ConnectionEvent,
	const int ErrorNo,
	xmpp_stream_error_t* const StreamError,
	void* const VoidConnectionPtr)
{
	check(VoidConnectionPtr != nullptr);
	FXmppConnectionStrophe* const ConnectionPtr = static_cast<FXmppConnectionStrophe* const>(VoidConnectionPtr);

	EStropheConnectionEvent Event = EStropheConnectionEvent::Fail;
	switch (ConnectionEvent)
	{
	case XMPP_CONN_CONNECT:
		Event = EStropheConnectionEvent::Connect;
		break;
	case XMPP_CONN_RAW_CONNECT:
		Event = EStropheConnectionEvent::RawConnect;
		break;
	case XMPP_CONN_DISCONNECT:
		Event = EStropheConnectionEvent::Disconnect;
		break;
	case XMPP_CONN_FAIL:
		Event = EStropheConnectionEvent::Fail;
		break;
	}

	ConnectionPtr->ReceiveConnectionStateChange(Event);

	if (StreamError != nullptr)
	{
		const FStropheError Error(*StreamError, ErrorNo);
		ConnectionPtr->ReceiveConnectionError(Error, Event);
	}
}

FStropheConnection::FStropheConnection(FStropheContext& InContext)
	: Context(InContext)
	, XmppConnectionPtr(nullptr)
	, ConnectionTimeoutSeconds(30)
	, ConnectionPingIntervalSeconds(60)
{
	XmppConnectionPtr = xmpp_conn_new(Context.GetContextPtr());
	check(XmppConnectionPtr);

	// Set our default timeout
	xmpp_conn_set_keepalive(XmppConnectionPtr, ConnectionTimeoutSeconds, ConnectionPingIntervalSeconds);
}

FStropheConnection::~FStropheConnection()
{
	check(XmppConnectionPtr != nullptr);

	xmpp_conn_release(XmppConnectionPtr);
	XmppConnectionPtr = nullptr;
}

int32 FStropheConnection::GetTimeout() const
{
	return ConnectionTimeoutSeconds;
}

int32 FStropheConnection::GetPingInterval() const
{
	return ConnectionPingIntervalSeconds;
}

void FStropheConnection::SetKeepAlive(int32 Timeout, int32 PingInterval)
{
	ConnectionTimeoutSeconds = Timeout;
	ConnectionPingIntervalSeconds = PingInterval;

	xmpp_conn_set_keepalive(XmppConnectionPtr, ConnectionTimeoutSeconds, ConnectionPingIntervalSeconds);
}

FString FStropheConnection::GetUserId() const
{
	return FString(UTF8_TO_TCHAR(xmpp_conn_get_jid(XmppConnectionPtr)));
}

void FStropheConnection::SetUserId(const FXmppUserJid& NewUserJid)
{
	SetUserId(NewUserJid.GetFullPath());
}

void FStropheConnection::SetUserId(const FString& NewUserId)
{
	xmpp_conn_set_jid(XmppConnectionPtr, TCHAR_TO_UTF8(*NewUserId));
}

FString FStropheConnection::GetPassword() const
{
	return FString(UTF8_TO_TCHAR(xmpp_conn_get_pass(XmppConnectionPtr)));
}

void FStropheConnection::SetPassword(const FString& NewPassword)
{
	xmpp_conn_set_pass(XmppConnectionPtr, TCHAR_TO_UTF8(*NewPassword));
}

bool FStropheConnection::Connect(const FString& Domain, uint16 Port, FXmppConnectionStrophe& ConnectionManager)
{
	bool bSuccess = true;

	if (xmpp_connect_client(XmppConnectionPtr, TCHAR_TO_UTF8(*Domain), Port, StropheConnectionEventHandler, &ConnectionManager) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to connect to host %ls:%d", *Domain, Port);
		bSuccess = false;
	}

	return bSuccess;
}

void FStropheConnection::Disconnect()
{
	xmpp_disconnect(XmppConnectionPtr);
}

bool FStropheConnection::SendStanza(const FStropheStanza& Stanza)
{
	if (GetConnectionState() != EStropheConnectionState::Connected)
	{
		return false;
	}

	xmpp_send(XmppConnectionPtr, Stanza.GetStanzaPtr());
	return true;
}

void FStropheConnection::XmppThreadTick()
{
	constexpr const int32 DefaultTimeoutMs = 5;
	constexpr const float DefaultTimeBetweenPollsSec = 0.005f;

	// Will process any data queued for send/receive and return. Will wait up to DefaultTimeoutMs if the socket is blocked.
	xmpp_run_once(Context.GetContextPtr(), DefaultTimeoutMs);

	// Since xmpp_run_once will not wait if the socket is not blocked, add a sleep between polls to avoid monopolizing the CPU
	FPlatformProcess::Sleep(DefaultTimeBetweenPollsSec);
}

void FStropheConnection::RegisterStropheHandler(FXmppConnectionStrophe& ConnectionManager)
{
	xmpp_handler_add(XmppConnectionPtr, StropheStanzaEventHandler, nullptr, nullptr, nullptr, &ConnectionManager);
}

void FStropheConnection::RemoveStropheHandler()
{
	xmpp_handler_delete(XmppConnectionPtr, StropheStanzaEventHandler);
}

EStropheConnectionState FStropheConnection::GetConnectionState() const
{
	if (xmpp_conn_is_connected(XmppConnectionPtr) != 0)
	{
		return EStropheConnectionState::Connected;
	}
	else if (xmpp_conn_is_disconnected(XmppConnectionPtr) != 0)
	{
		return EStropheConnectionState::Disconnected;
	}
	else if (xmpp_conn_is_connecting(XmppConnectionPtr) != 0)
	{
		return EStropheConnectionState::Connecting;
	}

	checkf(false, TEXT("Missing libstrophe xmpp connection state"));
	return EStropheConnectionState::Unknown;
}

#endif
