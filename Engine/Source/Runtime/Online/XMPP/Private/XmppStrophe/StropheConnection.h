// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppStrophe/StropheContext.h"

#if WITH_XMPP_STROPHE

class FStropheStanza;
class FXmppConnectionStrophe;
class FXmppUserJid;

typedef struct _xmpp_conn_t xmpp_conn_t;
typedef struct _xmpp_ctx_t xmpp_ctx_t;

enum class EStropheConnectionState : uint8
{
	Unknown,
	Disconnected,
	Connecting,
	Connected
};

enum class EStropheConnectionEvent : uint8
{
	Connect,
	RawConnect,
	Disconnect,
	Fail
};

class FStropheConnection
{
public:
	explicit FStropheConnection(FStropheContext& InContext);
	~FStropheConnection();

	// Do not allow copies/assigns
	FStropheConnection(const FStropheConnection& Other) = delete;
	FStropheConnection(FStropheConnection&& Other) = delete;
	FStropheConnection& operator=(const FStropheConnection& Other) = delete;
	FStropheConnection& operator=(FStropheConnection&& Other) = delete;

	/** Get server timeout in milliseconds */
	int32 GetTimeout() const;
	/** Get ping interval in milliseconds */
	int32 GetPingInterval() const;
	/** Set KeepAlive information in milliseconds */
	void SetKeepAlive(int32 Timeout, int32 PingInterval);

	/** Get our current user */
	FString GetUserId() const;
	/** Set our current user */
	void SetUserId(const FXmppUserJid& NewUserJid);
	void SetUserId(const FString& NewUserId);

	/** Get our current password/auth */
	FString GetPassword() const;
	/** Set our current password/auth */
	void SetPassword(const FString& NewPassword);

	/** Connect to specified domain/port; will use previously set user/pass information */
	bool Connect(const FString& Domain, uint16 Port, FXmppConnectionStrophe& ConnectionManager);
	/** Disconnect from the server*/
	void Disconnect();

	/** Queue a stanza to be sent */
	bool SendStanza(const FStropheStanza& Stanza);

	/** Tick to process events */
	void XmppThreadTick();

	/** Manage Strophe event handlers*/
	void RegisterStropheHandler(FXmppConnectionStrophe& ConnectionManager);
	void RemoveStropheHandler();

	/** Get our context pointer */
	xmpp_ctx_t* GetContextPtr() const { return Context.GetContextPtr(); }

	/** Get our current connection state */
	EStropheConnectionState GetConnectionState() const;

protected:
	FStropheContext& Context;

	xmpp_conn_t* XmppConnectionPtr;
	int32 ConnectionTimeoutSeconds;
	int32 ConnectionPingIntervalSeconds;
};

#endif
