// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/StropheContext.h"
#include "XmppLog.h"

#if WITH_XMPP_STROPHE

THIRD_PARTY_INCLUDES_START
#include "strophe.h"
THIRD_PARTY_INCLUDES_END

static
void StropheLogger(void* const UnusedContextPtr,
	const xmpp_log_level_t StropheLogLevel,
	const char* const Area,
	const char* const Message)
{
	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();

	switch (StropheLogLevel)
	{
	case XMPP_LEVEL_DEBUG:
		UE_LOGF(LogXmpp, VeryVerbose, "libstrophe[%u] %ls debug: %ls", ThreadId, UTF8_TO_TCHAR(Area), UTF8_TO_TCHAR(Message));
		break;
	case XMPP_LEVEL_INFO:
		UE_LOGF(LogXmpp, Log, "libstrophe[%u] %ls info: %ls", ThreadId, UTF8_TO_TCHAR(Area), UTF8_TO_TCHAR(Message));
		break;
	case XMPP_LEVEL_WARN:
		UE_LOGF(LogXmpp, Warning, "libstrophe[%u] %ls warning: %ls", ThreadId, UTF8_TO_TCHAR(Area), UTF8_TO_TCHAR(Message));
		break;
	case XMPP_LEVEL_ERROR:
		UE_LOGF(LogXmpp, Error, "libstrophe[%u] %ls error: %ls", ThreadId, UTF8_TO_TCHAR(Area), UTF8_TO_TCHAR(Message));
		break;
	}
}

static void* StropheAlloc(const size_t Size, void* const Userdata)
{
	return FMemory::Malloc(Size);
}

static void StropheFree(void* Ptr, void* const Userdata)
{
	FMemory::Free(Ptr);
}

static void* StropheRealloc(void* Ptr, const size_t Size, void* const Userdata)
{
	return FMemory::Realloc(Ptr, Size);
}

FStropheContext::FStropheContext()
	: XmppContextPtr(nullptr)
{
	static const xmpp_mem_t MemoryAllocatorOptions = { StropheAlloc, StropheFree, StropheRealloc, nullptr };
	static const xmpp_log_t LoggingOptions = {StropheLogger, nullptr};

	XmppContextPtr = xmpp_ctx_new(&MemoryAllocatorOptions, &LoggingOptions);
	check(XmppContextPtr != nullptr);
}

FStropheContext::~FStropheContext()
{
	if (XmppContextPtr != nullptr)
	{
		xmpp_ctx_free(XmppContextPtr);
		XmppContextPtr = nullptr;
	}
}

#endif
