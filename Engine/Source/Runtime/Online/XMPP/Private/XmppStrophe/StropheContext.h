// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_XMPP_STROPHE

typedef struct _xmpp_ctx_t xmpp_ctx_t;

class FStropheContext
{
public:
	explicit FStropheContext();
	~FStropheContext();

	// Do not allow copies/assigns
	FStropheContext(const FStropheContext& Other) = delete;
	FStropheContext(FStropheContext&& Other) = delete;
	FStropheContext& operator=(const FStropheContext& Other) = delete;
	FStropheContext& operator=(FStropheContext&& Other) = delete;

	xmpp_ctx_t* GetContextPtr() const { return XmppContextPtr; }

private:
	xmpp_ctx_t* XmppContextPtr;
};

#endif
