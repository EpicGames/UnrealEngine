// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/StropheError.h"

#if WITH_XMPP_STROPHE

THIRD_PARTY_INCLUDES_START
#include "strophe.h"
THIRD_PARTY_INCLUDES_END

FStropheError::FStropheError(const xmpp_stream_error_t& StreamError, int32 InErrorNumber)
	: ErrorType(ConvertStropheError(StreamError.type))
	, ErrorString(UTF8_TO_TCHAR(StreamError.text))
	, ErrorNumber(InErrorNumber)
	, ErrorStanza(StreamError.stanza)
{
}

EStropheErrorType FStropheError::GetErrorType() const
{
	return ErrorType;
}

const FString& FStropheError::GetErrorString() const
{
	return ErrorString;
}

int32 FStropheError::GetErrorNumber() const
{
	return ErrorNumber;
}

const FStropheStanza& FStropheError::GetStanza() const
{
	return ErrorStanza;
}

EStropheErrorType FStropheError::ConvertStropheError(xmpp_error_type_t InErrorType)
{
	switch (InErrorType)
	{
	case XMPP_SE_BAD_FORMAT:
		return EStropheErrorType::BadFormat;
	case XMPP_SE_BAD_NS_PREFIX:
		return EStropheErrorType::BadNSPrefix;
	case XMPP_SE_CONFLICT:
		return EStropheErrorType::Conflict;
	case XMPP_SE_CONN_TIMEOUT:
		return EStropheErrorType::ConnectionTimeout;
	case XMPP_SE_HOST_GONE:
		return EStropheErrorType::HostGone;
	case XMPP_SE_HOST_UNKNOWN:
		return EStropheErrorType::HostUnknown;
	case XMPP_SE_IMPROPER_ADDR:
		return EStropheErrorType::ImproperAddr;
	case XMPP_SE_INTERNAL_SERVER_ERROR:
		return EStropheErrorType::InternalServerError;
	case XMPP_SE_INVALID_FROM:
		return EStropheErrorType::InvalidFrom;
	case XMPP_SE_INVALID_ID:
		return EStropheErrorType::InvalidId;
	case XMPP_SE_INVALID_NS:
		return EStropheErrorType::InvalidNS;
	case XMPP_SE_INVALID_XML:
		return EStropheErrorType::InvalidXML;
	case XMPP_SE_NOT_AUTHORIZED:
		return EStropheErrorType::NotAuthorized;
	case XMPP_SE_POLICY_VIOLATION:
		return EStropheErrorType::PolicyViolation;
	case XMPP_SE_REMOTE_CONN_FAILED:
		return EStropheErrorType::RemoteConnectionFailed;
	case XMPP_SE_RESOURCE_CONSTRAINT:
		return EStropheErrorType::ResourceConstraint;
	case XMPP_SE_RESTRICTED_XML:
		return EStropheErrorType::RestrictedXML;
	case XMPP_SE_SEE_OTHER_HOST:
		return EStropheErrorType::SeeOtherHost;
	case XMPP_SE_SYSTEM_SHUTDOWN:
		return EStropheErrorType::SystemShutdown;
	case XMPP_SE_UNDEFINED_CONDITION:
		return EStropheErrorType::UndefinedCondition;
	case XMPP_SE_UNSUPPORTED_ENCODING:
		return EStropheErrorType::UnsupportedEncoding;
	case XMPP_SE_UNSUPPORTED_STANZA_TYPE:
		return EStropheErrorType::UnsupportedStanzaType;
	case XMPP_SE_UNSUPPORTED_VERSION:
		return EStropheErrorType::UnsupportedVersion;
	case XMPP_SE_XML_NOT_WELL_FORMED:
		return EStropheErrorType::XMLNotWellFormed;
	}

	return EStropheErrorType::Unknown;
}

#endif
