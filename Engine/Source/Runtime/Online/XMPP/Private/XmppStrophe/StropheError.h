// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppStrophe/StropheStanza.h"

#if WITH_XMPP_STROPHE

THIRD_PARTY_INCLUDES_START
#include "strophe.h"
THIRD_PARTY_INCLUDES_END

enum class EStropheErrorType : uint8
{
	Unknown,
	BadFormat,
	BadNSPrefix,
	Conflict,
	ConnectionTimeout,
	HostGone,
	HostUnknown,
	ImproperAddr,
	InternalServerError,
	InvalidFrom,
	InvalidId,
	InvalidNS,
	InvalidXML,
	NotAuthorized,
	PolicyViolation,
	RemoteConnectionFailed,
	ResourceConstraint,
	RestrictedXML,
	SeeOtherHost,
	SystemShutdown,
	UndefinedCondition,
	UnsupportedEncoding,
	UnsupportedStanzaType,
	UnsupportedVersion,
	XMLNotWellFormed
};

class FStropheError
{
public:
	explicit FStropheError(const xmpp_stream_error_t& StreamError, int32 InErrorNumber);

	EStropheErrorType GetErrorType() const;
	const FString& GetErrorString() const;
	int32 GetErrorNumber() const;
	const FStropheStanza& GetStanza() const;

protected:
	static EStropheErrorType ConvertStropheError(xmpp_error_type_t InErrorType);

protected:
	EStropheErrorType ErrorType;
	FString ErrorString;
	int32 ErrorNumber;
	const FStropheStanza ErrorStanza;
};

#endif
