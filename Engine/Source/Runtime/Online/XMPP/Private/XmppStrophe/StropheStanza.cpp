// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/StropheContext.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppStrophe/XmppStrophe.h"
#include "XmppLog.h"

#if WITH_XMPP_STROPHE

THIRD_PARTY_INCLUDES_START
#include "strophe.h"
THIRD_PARTY_INCLUDES_END

FStropheStanza::FStropheStanza(const FXmppConnectionStrophe& Connection, const FString& StanzaName /*= FString()*/)
	: XmppStanzaPtr(nullptr)
{
	// Create new stanza
	XmppStanzaPtr = xmpp_stanza_new(Connection.GetContext().GetContextPtr());
	check(XmppStanzaPtr != nullptr);

	if (!StanzaName.IsEmpty())
	{
		SetName(StanzaName);
	}
}

FStropheStanza::FStropheStanza(const FStropheStanza& Other)
	: XmppStanzaPtr(nullptr)
{
	// Copy their stanza
	XmppStanzaPtr = xmpp_stanza_copy(Other.XmppStanzaPtr);
	check(XmppStanzaPtr != nullptr);
}

FStropheStanza::FStropheStanza(FStropheStanza&& Other)
	: XmppStanzaPtr(nullptr)
{
	// Take their stanza
	XmppStanzaPtr = Other.XmppStanzaPtr;

	// Give Other a new stanza now that we stole theirs
	Other.XmppStanzaPtr = xmpp_stanza_new(xmpp_stanza_get_context(XmppStanzaPtr));
	check(Other.XmppStanzaPtr != nullptr);
}

FStropheStanza::FStropheStanza(xmpp_stanza_t* const OtherStanzaPtr)
	: XmppStanzaPtr(nullptr)
{
	check(OtherStanzaPtr);

	// Clone the other object (we'll point to the same stanza but with an increased ref count)
	XmppStanzaPtr = xmpp_stanza_clone(OtherStanzaPtr);
	check(XmppStanzaPtr != nullptr);
}

FStropheStanza::FStropheStanza(xmpp_ctx_t* const StropheContextPtr)
	: XmppStanzaPtr(nullptr)
{
	// Create new stanza
	XmppStanzaPtr = xmpp_stanza_new(StropheContextPtr);
	check(XmppStanzaPtr != nullptr);
}

FStropheStanza::~FStropheStanza()
{
	// Ensure we haven't destructed already
	check(XmppStanzaPtr != nullptr);

	xmpp_stanza_release(XmppStanzaPtr);
	XmppStanzaPtr = nullptr;
}

FStropheStanza& FStropheStanza::operator=(const FStropheStanza& Other)
{
	if (this != &Other)
	{
		// Release our current object
		xmpp_stanza_release(XmppStanzaPtr);

		// Make us a copy of Other
		XmppStanzaPtr = xmpp_stanza_copy(Other.XmppStanzaPtr);
		check(XmppStanzaPtr);
	}

	return *this;
}

FStropheStanza& FStropheStanza::operator=(FStropheStanza&& Other)
{
	if (this != &Other)
	{
		// Release our current object
		xmpp_stanza_release(XmppStanzaPtr);

		// Take Other's ptr
		XmppStanzaPtr = Other.XmppStanzaPtr;

		// Assign Other to have a new stanza instead
		Other.XmppStanzaPtr = xmpp_stanza_new(xmpp_stanza_get_context(XmppStanzaPtr));
		check(Other.XmppStanzaPtr);
	}

	return *this;
}

FStropheStanza FStropheStanza::Clone()
{
	return FStropheStanza(XmppStanzaPtr);
}

void FStropheStanza::AddChild(const FStropheStanza& Child)
{
	if (xmpp_stanza_add_child(XmppStanzaPtr, Child.XmppStanzaPtr) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to add child");
	}
}

TOptional<FStropheStanza> FStropheStanza::GetChildStropheStanza(const FString& ChildName)
{
	TOptional<FStropheStanza> OptionalStanza;

	xmpp_stanza_t* FoundStanzaPtr = xmpp_stanza_get_child_by_name(XmppStanzaPtr, TCHAR_TO_UTF8(*ChildName));
	if (FoundStanzaPtr != nullptr)
	{
		OptionalStanza = FStropheStanza(FoundStanzaPtr);
	}

	return OptionalStanza;
}

TUniquePtr<IXmppStanza> FStropheStanza::GetChild(const FString& ChildName) const
{
	TOptional<const FStropheStanza> Stanza = GetChildStropheStanza(ChildName);
	if (Stanza.IsSet())
	{
		return MakeUnique<FStropheStanza>(Stanza.GetValue());
	}
	return nullptr;
}

TOptional<const FStropheStanza> FStropheStanza::GetChildStropheStanza(const FString& ChildName) const
{
	TOptional<const FStropheStanza> OptionalStanza;

	xmpp_stanza_t* FoundStanzaPtr = xmpp_stanza_get_child_by_name(XmppStanzaPtr, TCHAR_TO_UTF8(*ChildName));
	if (FoundStanzaPtr != nullptr)
	{
		OptionalStanza = FStropheStanza(FoundStanzaPtr);
	}

	return OptionalStanza;
}

static
xmpp_stanza_t* FindStanzaByNameAndNamespace(xmpp_stanza_t* ParentStanzaPtr, const FString& ChildName, const FString& Namespace)
{
	check(ParentStanzaPtr != nullptr);

	for (xmpp_stanza_t* ChildStanza = xmpp_stanza_get_children(ParentStanzaPtr);
		 ChildStanza != nullptr;
		 ChildStanza = xmpp_stanza_get_next(ParentStanzaPtr))
	{
		const bool bIsNamedStanza = xmpp_stanza_is_tag(ChildStanza) != 0;
		if (bIsNamedStanza &&
			ChildName == UTF8_TO_TCHAR(xmpp_stanza_get_name(ChildStanza)) &&
			Namespace == UTF8_TO_TCHAR(xmpp_stanza_get_ns(ChildStanza)))
		{
			return ChildStanza;
		}
	}

	return nullptr;
}

TOptional<FStropheStanza> FStropheStanza::GetChildByNameAndNamespace(const FString& ChildName, const FString& Namespace)
{
	TOptional<FStropheStanza> OptionalStanza;

	xmpp_stanza_t* ChildStanza = FindStanzaByNameAndNamespace(XmppStanzaPtr, ChildName, Namespace);
	if (ChildStanza != nullptr)
	{
		OptionalStanza = FStropheStanza(ChildStanza);
	}

	return OptionalStanza;
}

TOptional<const FStropheStanza> FStropheStanza::GetChildByNameAndNamespace(const FString& ChildName, const FString& Namespace) const
{
	TOptional<const FStropheStanza> OptionalStanza;

	xmpp_stanza_t* ChildStanza = FindStanzaByNameAndNamespace(XmppStanzaPtr, ChildName, Namespace);
	if (ChildStanza != nullptr)
	{
		OptionalStanza = FStropheStanza(ChildStanza);
	}

	return OptionalStanza;
}

bool FStropheStanza::HasChild(const FString& ChildName) const
{
	return xmpp_stanza_get_child_by_name(XmppStanzaPtr, TCHAR_TO_UTF8(*ChildName)) != nullptr;
}

bool FStropheStanza::HasChildByNameAndNamespace(const FString& ChildName, const FString& Namespace) const
{
	return FindStanzaByNameAndNamespace(XmppStanzaPtr, ChildName, Namespace) != nullptr;
}

TArray<FStropheStanza> FStropheStanza::GetChildren()
{
	TArray<FStropheStanza> ChildrenArray;

	for (xmpp_stanza_t* ChildStanza = xmpp_stanza_get_children(XmppStanzaPtr);
		 ChildStanza != nullptr;
		 ChildStanza = xmpp_stanza_get_next(XmppStanzaPtr))
	{
		ChildrenArray.Emplace(ChildStanza);
	}

	return ChildrenArray;
}

const TArray<FStropheStanza> FStropheStanza::GetChildren() const
{
	TArray<FStropheStanza> ChildrenArray;

	for (xmpp_stanza_t* ChildStanza = xmpp_stanza_get_children(XmppStanzaPtr);
		 ChildStanza != nullptr;
		 ChildStanza = xmpp_stanza_get_next(XmppStanzaPtr))
	{
		ChildrenArray.Emplace(ChildStanza);
	}

	return ChildrenArray;
}

void FStropheStanza::SetNamespace(const FString& Namespace)
{
	if (xmpp_stanza_set_ns(XmppStanzaPtr, TCHAR_TO_UTF8(*Namespace)) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to set namespace to %ls", *Namespace);
	}
}

FString FStropheStanza::GetNamespace() const
{
	return FString(UTF8_TO_TCHAR(xmpp_stanza_get_ns(XmppStanzaPtr)));
}

void FStropheStanza::SetAttribute(const FString& Key, const FString& Value)
{
	if (xmpp_stanza_set_attribute(XmppStanzaPtr, TCHAR_TO_UTF8(*Key), TCHAR_TO_UTF8(*Value)) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to set attribute %ls to %ls", *Key, *Value);
	}
}

FString FStropheStanza::GetAttribute(const FString& Key) const
{
	return FString(UTF8_TO_TCHAR(xmpp_stanza_get_attribute(XmppStanzaPtr, TCHAR_TO_UTF8(*Key))));
}

bool FStropheStanza::HasAttribute(const FString& Key) const
{
	return xmpp_stanza_get_attribute(XmppStanzaPtr, TCHAR_TO_UTF8(*Key)) != nullptr;
}

void FStropheStanza::SetName(const FString& Name)
{
	if (xmpp_stanza_set_name(XmppStanzaPtr, TCHAR_TO_UTF8(*Name)) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to set name to %ls", *Name);
	}
}

FString FStropheStanza::GetName() const
{
	return FString(UTF8_TO_TCHAR(xmpp_stanza_get_name(XmppStanzaPtr)));
}

void FStropheStanza::SetText(const FString& Text)
{
	FStropheStanza TextStanza(xmpp_stanza_get_context(XmppStanzaPtr));

	if (xmpp_stanza_set_text(TextStanza.XmppStanzaPtr, TCHAR_TO_UTF8(*Text)) == XMPP_EOK)
	{
		AddChild(TextStanza);
	}
	else
	{
		UE_LOGF(LogXmpp, Error, "Failed to set text to %ls", *Text);
	}
}

FString FStropheStanza::GetText() const
{
	FString Result;

	char* StanzaTextCopy = xmpp_stanza_get_text(XmppStanzaPtr);
	if (StanzaTextCopy != nullptr)
	{
		Result = UTF8_TO_TCHAR(StanzaTextCopy);
		xmpp_free(xmpp_stanza_get_context(XmppStanzaPtr), StanzaTextCopy);
	}
	else
	{
		UE_LOGF(LogXmpp, Warning, "No text found in stanza");
	}

	return Result;
}

void FStropheStanza::SetType(const FString& Type)
{
	if (xmpp_stanza_set_type(XmppStanzaPtr, TCHAR_TO_UTF8(*Type)) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to set type to %ls", *Type);
	}
}

FString FStropheStanza::GetType() const
{
	return FString(UTF8_TO_TCHAR(xmpp_stanza_get_type(XmppStanzaPtr)));
}

void FStropheStanza::SetId(const FString& Id)
{
	if (xmpp_stanza_set_id(XmppStanzaPtr, TCHAR_TO_UTF8(*Id)) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to set ID to %ls", *Id);
	}
}

FString FStropheStanza::GetId() const
{
	return FString(UTF8_TO_TCHAR(xmpp_stanza_get_id(XmppStanzaPtr)));
}

void FStropheStanza::SetTo(const FXmppUserJid& To)
{
	SetTo(FXmppStrophe::JidToString(To));
}

void FStropheStanza::SetTo(const FString& To)
{
	if (xmpp_stanza_set_to(XmppStanzaPtr, TCHAR_TO_UTF8(*To)) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to set To value to %ls", *To);
	}
}

FXmppUserJid FStropheStanza::GetTo() const
{
	return FXmppStrophe::JidFromStropheString(xmpp_stanza_get_to(XmppStanzaPtr));
}

void FStropheStanza::SetFrom(const FXmppUserJid& From)
{
	SetFrom(FXmppStrophe::JidToString(From));
}

void FStropheStanza::SetFrom(const FString& From)
{
	if (xmpp_stanza_set_from(XmppStanzaPtr, TCHAR_TO_UTF8(*From)) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to set From value to %ls", *From);
	}
}

FXmppUserJid FStropheStanza::GetFrom() const
{
	return FXmppStrophe::JidFromStropheString(xmpp_stanza_get_from(XmppStanzaPtr));
}

bool FStropheStanza::AddBodyWithText(const FString& Text)
{
	if (xmpp_message_set_body(XmppStanzaPtr, TCHAR_TO_UTF8(*Text)) != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Failed to add body text: %ls", *Text);
		return false;
	}

	return true;
}

TOptional<FString> FStropheStanza::GetBodyText() const
{
	TOptional<FString> BodyText;

	char* StanzaTextCopy = xmpp_message_get_body(XmppStanzaPtr);
	if (StanzaTextCopy != nullptr)
	{
		BodyText = UTF8_TO_TCHAR(StanzaTextCopy);
		xmpp_free(xmpp_stanza_get_context(XmppStanzaPtr), StanzaTextCopy);
	}

	return BodyText;
}

#endif
