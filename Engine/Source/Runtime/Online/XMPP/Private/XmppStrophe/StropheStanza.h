// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "XmppStanza.h"

#if WITH_XMPP_STROPHE

class FXmppConnectionStrophe;
class FStropheContext;
class FXmppUserJid;

typedef struct _xmpp_ctx_t xmpp_ctx_t;
typedef struct _xmpp_conn_t xmpp_conn_t;
typedef struct _xmpp_stanza_t xmpp_stanza_t;

class FStropheStanza
	: public IXmppStanza
{
	// For TArray<FStropheStanza> Emplace clone from xmp_stanza_t*
	friend class TArray<FStropheStanza>;
	// For Clone constructor access
	friend class FStropheError;
	// For GetStanzaPtr access
	friend class FStropheConnection;
	friend class FStropheWebsocketConnection;

	// For cloning stanzas from our socket
	friend int StropheStanzaEventHandler(xmpp_conn_t* const Connection, xmpp_stanza_t* const IncomingStanza, void* const UserData);
	friend int StropheWebsocketStanzaEventHandler(xmpp_conn_t* const Connection, xmpp_stanza_t* const IncomingStanza, void* const UserData);

public:
	explicit FStropheStanza(const FXmppConnectionStrophe& Context, const FString& StanzaName = FString());
	FStropheStanza(const FStropheStanza& Other);
	FStropheStanza(FStropheStanza&& Other);
	virtual ~FStropheStanza();
	FStropheStanza& operator=(const FStropheStanza& Other);
	FStropheStanza& operator=(FStropheStanza&& Other);

	FStropheStanza Clone();

	void AddChild(const FStropheStanza& Child);
	virtual TUniquePtr<IXmppStanza> GetChild(const FString& ChildName) const override;
	TOptional<FStropheStanza> GetChildStropheStanza(const FString& ChildName);
	TOptional<const FStropheStanza> GetChildStropheStanza(const FString& ChildName) const;
	TOptional<FStropheStanza> GetChildByNameAndNamespace(const FString& ChildName, const FString& Namespace);
	TOptional<const FStropheStanza> GetChildByNameAndNamespace(const FString& ChildName, const FString& Namespace) const;
	virtual bool HasChild(const FString& ChildName) const override;
	bool HasChildByNameAndNamespace(const FString& ChildName, const FString& Namespace) const;

	TArray<FStropheStanza> GetChildren();
	const TArray<FStropheStanza> GetChildren() const;

	void SetNamespace(const FString& Namespace);
	FString GetNamespace() const;

	void SetAttribute(const FString& Key, const FString& Value);
	virtual FString GetAttribute(const FString& Key) const override;
	virtual bool HasAttribute(const FString& Key) const override;

	void SetName(const FString& Name);
	virtual FString GetName() const override;

	void SetText(const FString& Text);
	virtual FString GetText() const override;

	void SetType(const FString& Type);
	virtual FString GetType() const override;

	void SetId(const FString& Id);
	virtual FString GetId() const override;

	void SetTo(const FXmppUserJid& To);
	void SetTo(const FString& To);
	virtual FXmppUserJid GetTo() const override;

	void SetFrom(const FXmppUserJid& From);
	void SetFrom(const FString& From);
	virtual FXmppUserJid GetFrom() const override;

	// Helpers for Message stanzas */

	/** Add a child stanza of name Body with the requested text.  Fails if we already have a body stanza, or if we are a text stanza. */
	bool AddBodyWithText(const FString& Text);
	/** Get the text from a child Body stanza, if one exists */
	virtual TOptional<FString> GetBodyText() const override;

protected:
	/** Get the current stanza */
	xmpp_stanza_t* GetStanzaPtr() const { return XmppStanzaPtr; }

	/** Passed in stanzas will be cloned instead of copied here */
	explicit FStropheStanza(xmpp_stanza_t* const OtherStanzaPtr);

	/** Create a new stanza directly from a context */
	explicit FStropheStanza(xmpp_ctx_t* const StropheContextPtr);

protected:
	/** Ref-counted pointer to libstrophe stanza data*/
	xmpp_stanza_t* XmppStanzaPtr;
};

#endif
