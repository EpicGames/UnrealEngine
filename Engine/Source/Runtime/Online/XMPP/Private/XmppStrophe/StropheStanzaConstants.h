// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace Strophe
{
// SN means Stanza Name

const TCHAR* const SN_BODY = TEXT("body");
const TCHAR* const SN_MESSAGE = TEXT("message");
const TCHAR* const SN_DELAY = TEXT("delay");
const TCHAR* const SN_PRESENCE = TEXT("presence");
const TCHAR* const SN_SHOW = TEXT("show");
const TCHAR* const SN_STATUS = TEXT("status");
const TCHAR* const SN_X = TEXT("x");
const TCHAR* const SN_ITEM = TEXT("item");
const TCHAR* const SN_PASSWORD = TEXT("password");
const TCHAR* const SN_ERROR = TEXT("error");
const TCHAR* const SN_FORBIDDEN = TEXT("forbidden");
const TCHAR* const SN_BAD_REQUEST = TEXT("bad-request");
const TCHAR* const SN_NOT_AUTHORIZED = TEXT("not-authorized");
const TCHAR* const SN_ITEM_NOT_FOUND = TEXT("item-not-found");
const TCHAR* const SN_NOT_ALLOWED = TEXT("not-allowed");
const TCHAR* const SN_NOT_ACCEPTABLE = TEXT("not-acceptable");
const TCHAR* const SN_REGISTRATION_REQUIRED = TEXT("registration-required");
const TCHAR* const SN_CONFLICT = TEXT("conflict");
const TCHAR* const SN_SERVICE_UNAVAILABLE = TEXT("service-unavailable");
const TCHAR* const SN_HISTORY = TEXT("history");
const TCHAR* const SN_SUBJECT = TEXT("subject");
const TCHAR* const SN_IQ = TEXT("iq");
const TCHAR* const SN_QUERY = TEXT("query");
const TCHAR* const SN_FIELD = TEXT("field");
const TCHAR* const SN_VALUE = TEXT("value");
const TCHAR* const SN_RECIPIENT_UNAVAILABLE = TEXT("recipient-unavailable");
const TCHAR* const SN_PING = TEXT("ping");

// ST means Stanza Type

const TCHAR* const ST_CHAT = TEXT("chat");
const TCHAR* const ST_UNAVAILABLE = TEXT("unavailable");
const TCHAR* const ST_GROUPCHAT = TEXT("groupchat");
const TCHAR* const ST_ERROR = TEXT("error");
const TCHAR* const ST_SET = TEXT("set");
const TCHAR* const ST_GET = TEXT("get");
const TCHAR* const ST_SUBMIT = TEXT("submit");
const TCHAR* const ST_AUTH = TEXT("auth");
const TCHAR* const ST_RESULT = TEXT("result");

// SA means Stanza Attribute

const TCHAR* const SA_STAMP = TEXT("stamp");
const TCHAR* const SA_TYPE = TEXT("type");
const TCHAR* const SA_ROLE = TEXT("role");
const TCHAR* const SA_AFFILIATION = TEXT("affiliation");
const TCHAR* const SA_MAXSTANZAS = TEXT("maxstanzas");
const TCHAR* const SA_VAR = TEXT("var");
const TCHAR* const SA_CODE = TEXT("code");
const TCHAR* const SA_JID = TEXT("jid");

// SNS means Stanza NameSpace

const TCHAR* const SNS_DELAY = TEXT("urn:xmpp:delay");
const TCHAR* const SNS_MUC = TEXT("http://jabber.org/protocol/muc");
const TCHAR* const SNS_MUC_USER = TEXT("http://jabber.org/protocol/muc#user");
const TCHAR* const SNS_MUC_OWNER = TEXT("http://jabber.org/protocol/muc#owner");
const TCHAR* const SNS_X_DATA = TEXT("jabber:x:data");
const TCHAR* const SNS_PING = TEXT("urn:xmpp:ping");
const TCHAR* const SNS_DISCO_INFO = TEXT("http://jabber.org/protocol/disco#info");

// SC means Status Code

const TCHAR* const SC_104 = TEXT("104");

} // End Strophe Namespace
