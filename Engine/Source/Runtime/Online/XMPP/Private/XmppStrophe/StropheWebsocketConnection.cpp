// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/StropheWebsocketConnection.h"

#include "XmppConnection.h"
#include "XmppLog.h"
#include "XmppStrophe/StropheContext.h"
#include "XmppStrophe/StropheError.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/XmppConnectionStrophe.h"

#if WITH_XMPP_STROPHE

#include "WebSocketsModule.h"
#include "IWebSocket.h"

THIRD_PARTY_INCLUDES_START
#include "strophe.h"
THIRD_PARTY_INCLUDES_END

int StropheWebsocketStanzaEventHandler(xmpp_conn_t* const UnusedPtr, xmpp_stanza_t* const EventStanza, void* const VoidConnectionPtr)
{
	check(VoidConnectionPtr != nullptr);

	const FStropheStanza IncomingStanza(EventStanza);

	// Ignore session stanza (bug in libstrophe that we see get this at all)
	if (!IncomingStanza.GetId().Equals(TEXT("_xmpp_session1"), ESearchCase::CaseSensitive))
	{
		// We want to forward our stanza to our connection and out to our handlers
		FXmppConnectionStrophe* const ConnectionPtr = static_cast<FXmppConnectionStrophe* const>(VoidConnectionPtr);
		ConnectionPtr->ReceiveStanza(IncomingStanza);
	}

	return 1;
}

static void StropheWebsocketConnectionEventHandler(xmpp_conn_t* const UnusedPtr,
	const xmpp_conn_event_t ConnectionEvent,
	const int ErrorNo,
	xmpp_stream_error_t* const StreamError,
	void* const VoidConnectionPtr)
{
	check(VoidConnectionPtr != nullptr);
	FXmppConnectionStrophe* const ConnectionPtr = static_cast<FXmppConnectionStrophe* const>(VoidConnectionPtr);

	EStropheConnectionEvent Event = EStropheConnectionEvent::Fail;
	switch (ConnectionEvent)
	{
	case XMPP_CONN_CONNECT:
		Event = EStropheConnectionEvent::Connect;
		break;
	case XMPP_CONN_RAW_CONNECT:
		Event = EStropheConnectionEvent::RawConnect;
		break;
	case XMPP_CONN_DISCONNECT:
		Event = EStropheConnectionEvent::Disconnect;
		break;
	case XMPP_CONN_FAIL:
		Event = EStropheConnectionEvent::Fail;
		break;
	}

	ConnectionPtr->ReceiveConnectionStateChange(Event);

	if (StreamError != nullptr)
	{
		const FStropheError Error(*StreamError, ErrorNo);
		ConnectionPtr->ReceiveConnectionError(Error, Event);
	}
}

FStropheWebsocketConnection::FStropheWebsocketConnection(FStropheContext& InContext, const FString& Url)
	: Context(InContext)
{
	TArray<FString> Protocols;
	Protocols.Add(TEXT("xmpp"));
	Websocket = FWebSocketsModule::Get().CreateWebSocket(Url, Protocols);

	Websocket->OnConnected().AddRaw(this, &FStropheWebsocketConnection::OnWebsocketConnected);
	Websocket->OnConnectionError().AddRaw(this, &FStropheWebsocketConnection::OnWebsocketConnectionError);
	Websocket->OnClosed().AddRaw(this, &FStropheWebsocketConnection::OnWebsocketClosed);
	Websocket->OnRawMessage().AddRaw(this, &FStropheWebsocketConnection::OnRawMessage);

	checkf(Context.GetContextPtr() != nullptr, TEXT("xmpp_ctx_t is null"));
	XmppConnectionPtr = xmpp_conn_new(Context.GetContextPtr());
	const int32 IsWebsocket = 1;
	xmpp_conn_extsock_t ExternalSocket = 
	{
		&FStropheWebsocketConnection::WebsocketConnectHandler,
		&FStropheWebsocketConnection::WebsocketCloseHandler,
		&FStropheWebsocketConnection::WebsocketSendHandler,
		IsWebsocket,
		this
	};

	xmpp_conn_set_extsock_handlers(XmppConnectionPtr, &ExternalSocket);
}

FStropheWebsocketConnection::~FStropheWebsocketConnection()
{
	Websocket->OnConnected().RemoveAll(this);
	Websocket->OnConnectionError().RemoveAll(this);
	Websocket->OnClosed().RemoveAll(this);
	Websocket->OnRawMessage().RemoveAll(this);

	xmpp_conn_release(XmppConnectionPtr);
}

bool FStropheWebsocketConnection::Connect(const FXmppUserJid& UserJid, const FString& Password, FXmppConnectionStrophe& ConnectionManager)
{
	bool bSuccess = true;

	ConnectionManager.ProcessLoginStatusChange(EXmppLoginStatus::ProcessingLogin);

	const FString UserJidFullPath = UserJid.GetFullPath();
	xmpp_conn_set_jid(XmppConnectionPtr, TCHAR_TO_UTF8(*UserJidFullPath));
	xmpp_conn_set_pass(XmppConnectionPtr, TCHAR_TO_UTF8(*Password));

	const int Result = xmpp_extsock_connect_client(XmppConnectionPtr, StropheWebsocketConnectionEventHandler, &ConnectionManager);
	if (Result != XMPP_EOK)
	{
		UE_LOGF(LogXmpp, Error, "Websocket failed to connect. Error %i", Result);
		bSuccess = false;

		ConnectionManager.ProcessLoginStatusChange(EXmppLoginStatus::LoggedOut);
	}

	xmpp_handler_add(XmppConnectionPtr, StropheWebsocketStanzaEventHandler, nullptr, nullptr, nullptr, &ConnectionManager);

	return bSuccess;
}

bool FStropheWebsocketConnection::SendStanza(FStropheStanza&& Stanza)
{
	if (!Websocket->IsConnected())
	{
		return false;
	}

	xmpp_send(XmppConnectionPtr, Stanza.GetStanzaPtr());
	return true;
}

void FStropheWebsocketConnection::Tick()
{
	// Handles timeouts, processes timers
	if (ensureMsgf(Context.GetContextPtr() != nullptr, TEXT("xmpp_ctx_t is null")))
	{
		xmpp_run_once(Context.GetContextPtr(), 0);
	}
}

void FStropheWebsocketConnection::Disconnect()
{
	xmpp_disconnect(XmppConnectionPtr);

	xmpp_handler_delete(XmppConnectionPtr, StropheWebsocketStanzaEventHandler);
}

void FStropheWebsocketConnection::WebsocketConnectHandler(void* const Userdata)
{
	FStropheWebsocketConnection* const Websocket = static_cast<FStropheWebsocketConnection* const>(Userdata);
	Websocket->WebsocketConnect();
}

void FStropheWebsocketConnection::WebsocketCloseHandler(void* const Userdata)
{
	FStropheWebsocketConnection* const Websocket = static_cast<FStropheWebsocketConnection* const>(Userdata);
	Websocket->WebsocketClose();
}

void FStropheWebsocketConnection::WebsocketSendHandler(const char* const Data, const size_t Length, void* const Userdata)
{
	FStropheWebsocketConnection* const Websocket = static_cast<FStropheWebsocketConnection* const>(Userdata);
	Websocket->WebsocketSend(Data, Length);

}

void FStropheWebsocketConnection::WebsocketConnect()
{
	Websocket->Connect();
}

void FStropheWebsocketConnection::WebsocketClose()
{
	Websocket->Close();
}

void FStropheWebsocketConnection::WebsocketSend(const char* const Data, const size_t Length)
{
	Websocket->Send(Data, Length);
}

void FStropheWebsocketConnection::OnWebsocketConnected()
{
	xmpp_extsock_connected(XmppConnectionPtr);
}

void FStropheWebsocketConnection::OnWebsocketConnectionError(const FString& Error)
{
	xmpp_extsock_connection_error(XmppConnectionPtr, TCHAR_TO_UTF8(*Error));
}

void FStropheWebsocketConnection::OnWebsocketClosed(int32 StatusCode, const FString& Reason, bool bWasClean)
{
	Disconnect();
}

void FStropheWebsocketConnection::OnRawMessage(const void* Data, SIZE_T Size, SIZE_T BytesRemaining)
{
	xmpp_extsock_receive(XmppConnectionPtr, static_cast<const char*>(Data), Size);
	if (BytesRemaining == 0)
	{
		// need to reset the parser on message boundaries because xmpp websockets have each message
		//   as an xml document instead of the full stream being parsable as a single document
		xmpp_extsock_parser_reset(XmppConnectionPtr);
	}
}

#endif
