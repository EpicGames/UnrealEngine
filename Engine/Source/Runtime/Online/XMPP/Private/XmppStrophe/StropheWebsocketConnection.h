// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppStrophe/StropheContext.h"

#if WITH_XMPP_STROPHE

class FStropheStanza;
class FXmppConnectionStrophe;
class FXmppUserJid;
class IWebSocket;

typedef struct _xmpp_conn_t xmpp_conn_t;

class FStropheWebsocketConnection
{
public:
	explicit FStropheWebsocketConnection(FStropheContext& InContext, const FString& Url);
	~FStropheWebsocketConnection();

	// Do not allow copies/assigns
	FStropheWebsocketConnection(const FStropheWebsocketConnection& Other) = delete;
	FStropheWebsocketConnection(FStropheWebsocketConnection&& Other) = delete;
	FStropheWebsocketConnection& operator=(const FStropheWebsocketConnection& Other) = delete;
	FStropheWebsocketConnection& operator=(FStropheWebsocketConnection&& Other) = delete;

	/** Connect to previously set url
	 * @param UserJid username
	 * @param Password password
	 * @param ConnectionManager connection object that will receive the stanzas/errors
	 *
	 * @return true if no errors were detected
	 */
	bool Connect(const FXmppUserJid& UserJid, const FString& Password, FXmppConnectionStrophe& ConnectionManager);
	/** Disconnect from the server*/
	void Disconnect();

	/** Send a stanza */
	bool SendStanza(FStropheStanza&& Stanza);

	/** Tick to process events */
	void Tick();

private:
	FStropheContext& Context;

	TSharedPtr<IWebSocket> Websocket;

	xmpp_conn_t* XmppConnectionPtr;

	/* Handlers for strophe external socket events */
	static void WebsocketConnectHandler(void* const Userdata);
	static void WebsocketCloseHandler(void* const Userdata);
	static void WebsocketSendHandler(const char* const Data, const size_t Length, void* const Userdata);

	void WebsocketConnect();
	void WebsocketClose();
	void WebsocketSend(const char* const Data, const size_t Length);

	/* Handlers for websocket events */
	void OnWebsocketConnected();
	void OnWebsocketConnectionError(const FString& Error);
	void OnWebsocketClosed(int32 StatusCode, const FString& Reason, bool bWasClean);
	void OnRawMessage(const void* Data, SIZE_T Size, SIZE_T BytesRemaining);
};

#endif
