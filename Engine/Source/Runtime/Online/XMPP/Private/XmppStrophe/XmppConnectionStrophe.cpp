// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppStrophe/XmppMessagesStrophe.h"
#include "XmppStrophe/XmppMultiUserChatStrophe.h"
#include "XmppStrophe/XmppPingStrophe.h"
#include "XmppStrophe/XmppPresenceStrophe.h"
#include "XmppStrophe/XmppPrivateChatStrophe.h"
#include "XmppStrophe/XmppPubSubStrophe.h"
#include "XmppStrophe/StropheContext.h"
#include "XmppStrophe/StropheConnection.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/StropheStanzaConstants.h"
#include "XmppStrophe/StropheError.h"
#include "XmppLog.h"
#include "Containers/BackgroundableTicker.h"
#include "Stats/Stats.h"

#if WITH_XMPP_STROPHE

FXmppConnectionStrophe::FXmppConnectionStrophe()
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
	, LoginStatus(EXmppLoginStatus::NotStarted)
{
	MessagesStrophe = MakeShared<FXmppMessagesStrophe, ESPMode::ThreadSafe>(*this);
	MultiUserChatStrophe = MakeShared<FXmppMultiUserChatStrophe, ESPMode::ThreadSafe>(*this);
	PingStrophe = MakeShared<FXmppPingStrophe, ESPMode::ThreadSafe>(*this);
	PresenceStrophe = MakeShared<FXmppPresenceStrophe, ESPMode::ThreadSafe>(*this);
	PrivateChatStrophe = MakeShared<FXmppPrivateChatStrophe, ESPMode::ThreadSafe>(*this);
	PubSubStrophe = MakeShared<FXmppPubSubStrophe, ESPMode::ThreadSafe>(*this);
}

FXmppConnectionStrophe::~FXmppConnectionStrophe()
{
	// Shutdown our connections before we're fully destructed; strophe calls back into us on its disconnect event
	if (StropheThread.IsValid())
	{
		StopXmppThread();
	}

	if (WebsocketConnection.IsValid())
	{
		WebsocketConnection.Reset();
	}
}

void FXmppConnectionStrophe::SetServer(const FXmppServer& NewServerConfiguration)
{
	ServerConfiguration = NewServerConfiguration;
	ServerConfiguration.ClientResource = FXmppUserJid::CreateResource(ServerConfiguration.AppId, ServerConfiguration.Platform, ServerConfiguration.PlatformUserId);
}

const FXmppServer& FXmppConnectionStrophe::GetServer() const
{
	return ServerConfiguration;
}

void FXmppConnectionStrophe::Login(const FString& UserId, const FString& Auth)
{
	FString ErrorStr;

	FXmppUserJid NewJid(UserId, ServerConfiguration.Domain, ServerConfiguration.ClientResource);
	if (!NewJid.IsValid())
	{
		ErrorStr = FString::Printf(TEXT("Invalid Jid %s"), *UserJid.GetFullPath());
	}
	else
	{
		UE_LOGF(LogXmpp, Log, "Starting Login on connection");
		UE_LOGF(LogXmpp, Log, "  Server = %ls:%d", *ServerConfiguration.ServerAddr, ServerConfiguration.ServerPort);
		UE_LOGF(LogXmpp, Log, "  User = %ls", *NewJid.ToDebugString());

		if (LoginStatus == EXmppLoginStatus::ProcessingLogin)
		{
			ErrorStr = TEXT("Still processing last login");
		}
		else if (LoginStatus == EXmppLoginStatus::ProcessingLogout)
		{
			ErrorStr = TEXT("Still processing last logout");
		}
		else if (LoginStatus == EXmppLoginStatus::LoggedIn)
		{
			UE_LOGF(LogXmpp, Log, "Reusing existing connection for Jid=%ls", *GetUserJid().ToDebugString());

			OnLoginComplete().Broadcast(GetUserJid(), true, FString());
			OnLoginChanged().Broadcast(GetUserJid(), EXmppLoginStatus::LoggedIn);
			ReconnectLogin();
			return;
		}
		else
		{
			// Close down any existing thread
			if (StropheThread.IsValid() || WebsocketConnection.IsValid())
			{
				Logout();
			}

			UserJid = MoveTemp(NewJid);
			MucDomain = FString::Printf(TEXT("muc.%s"), *ServerConfiguration.Domain);

			if (ServerConfiguration.ServerAddr.StartsWith(TEXT("wss://")) || ServerConfiguration.ServerAddr.StartsWith(TEXT("ws://")))
			{
				WebsocketConnection = MakeUnique<FStropheWebsocketConnection>(StropheContext, FString::Printf(TEXT("%s:%i"), *ServerConfiguration.ServerAddr, ServerConfiguration.ServerPort));
				WebsocketConnection->Connect(UserJid, Auth, *this);
			}
			else
			{
				StartXmppThread(UserJid, Auth);
			}
		}
	}

	if (!ErrorStr.IsEmpty())
	{
		UE_LOGF(LogXmpp, Warning, "Login failed. %ls", *ErrorStr);
		OnLoginComplete().Broadcast(GetUserJid(), false, ErrorStr);
	}
}

void FXmppConnectionStrophe::ReconnectLogin()
{
	MessagesStrophe->OnReconnect();
	MultiUserChatStrophe->OnReconnect();
	PingStrophe->OnReconnect();
	PresenceStrophe->OnReconnect();
	PrivateChatStrophe->OnReconnect();
	PubSubStrophe->OnReconnect();
}

void FXmppConnectionStrophe::Logout()
{
	if (StropheThread.IsValid())
	{
		StopXmppThread();
	}

	if (WebsocketConnection.IsValid())
	{
		WebsocketConnection.Reset();
	}

	MessagesStrophe->OnDisconnect();
	MultiUserChatStrophe->OnDisconnect();
	PingStrophe->OnDisconnect();
	PresenceStrophe->OnDisconnect();
	PrivateChatStrophe->OnDisconnect();
	PubSubStrophe->OnDisconnect();
}

EXmppLoginStatus::Type FXmppConnectionStrophe::GetLoginStatus() const
{
	EXmppLoginStatus::Type CurrentLoginStatus = LoginStatus;
	if (CurrentLoginStatus == EXmppLoginStatus::LoggedIn || CurrentLoginStatus == EXmppLoginStatus::ProcessingLogin)
	{
		return CurrentLoginStatus;
	}
	else
	{
		return EXmppLoginStatus::LoggedOut;
	}
}

const FXmppUserJid& FXmppConnectionStrophe::GetUserJid() const
{
	return UserJid;
}

IXmppMessagesPtr FXmppConnectionStrophe::Messages()
{
	return MessagesStrophe;
}

IXmppMultiUserChatPtr FXmppConnectionStrophe::MultiUserChat()
{
	return MultiUserChatStrophe;
}

IXmppPresencePtr FXmppConnectionStrophe::Presence()
{
	return PresenceStrophe;
}

IXmppChatPtr FXmppConnectionStrophe::PrivateChat()
{
	return PrivateChatStrophe;
}

IXmppPubSubPtr FXmppConnectionStrophe::PubSub()
{
	return PubSubStrophe;
}

bool FXmppConnectionStrophe::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FXmppConnectionStrophe_Tick);

	// Logout if we've been requested to from the XMPP Thread
	if (RequestLogout)
	{
		RequestLogout = false;
		Logout();
	}

	while (!IncomingLoginStatusChanges.IsEmpty())
	{
		EXmppLoginStatus::Type NewLoginStatus;
		if (IncomingLoginStatusChanges.Dequeue(NewLoginStatus))
		{
			ProcessLoginStatusChange(NewLoginStatus);
		}
	}

	if (WebsocketConnection.IsValid())
	{
		WebsocketConnection->Tick();
	}

	return true;
}

bool FXmppConnectionStrophe::SendStanza(FStropheStanza&& Stanza)
{
	if (LoginStatus != EXmppLoginStatus::LoggedIn)
	{
		return false;
	}

	OnStanzaSent().Broadcast(Stanza);
	bool bQueuedStanzaToBeSent = true;
	if (StropheThread.IsValid())
	{
		bQueuedStanzaToBeSent = StropheThread->SendStanza(MoveTemp(Stanza));
	}
	else if (WebsocketConnection.IsValid())
	{
		bQueuedStanzaToBeSent = WebsocketConnection->SendStanza(MoveTemp(Stanza));
	}
	else
	{
		return false;
	}

	if (bQueuedStanzaToBeSent)
	{
		// Reset our ping timer now that we're queuing a different message to be sent
		if (PingStrophe.IsValid())
		{
			PingStrophe->ResetPingTimer();
		}
	}
	
	return bQueuedStanzaToBeSent;
}

void FXmppConnectionStrophe::StartXmppThread(const FXmppUserJid& ConnectionUser, const FString& ConnectionAuth)
{
	UE_LOGF(LogXmpp, Log, "Starting Strophe XMPP thread");

	StropheThread = MakeUnique<FXmppStropheThread>(*this, ConnectionUser, ConnectionAuth, ServerConfiguration);
}

void FXmppConnectionStrophe::StopXmppThread()
{
	UE_LOGF(LogXmpp, Log, "Stopping Strophe XMPP thread");

	StropheThread.Reset();
}

void FXmppConnectionStrophe::ReceiveConnectionStateChange(EStropheConnectionEvent StateChange, bool bQueue)
{
	EXmppLoginStatus::Type NewLoginStatus = EXmppLoginStatus::LoggedOut;
	switch (StateChange)
	{
	case EStropheConnectionEvent::Connect:
	case EStropheConnectionEvent::RawConnect:
		NewLoginStatus = EXmppLoginStatus::LoggedIn;
		break;
	case EStropheConnectionEvent::Disconnect:
	case EStropheConnectionEvent::Fail:
		NewLoginStatus = EXmppLoginStatus::LoggedOut;
		RequestLogout = true;
		break;
	}

	if (bQueue)
	{
		UE_LOGF(LogXmpp, Log, "Strophe XMPP thread received state change Was: %ls Now: %ls", EXmppLoginStatus::ToString(LoginStatus), EXmppLoginStatus::ToString(NewLoginStatus));

		QueueNewLoginStatus(NewLoginStatus);
	}
	else
	{
		ProcessLoginStatusChange(NewLoginStatus);
	}
}

void FXmppConnectionStrophe::ReceiveConnectionError(const FStropheError& Error, EStropheConnectionEvent Event)
{
	UE_LOGF(LogXmpp, Error, "Received Strophe XMPP Stanza %ls with error %ls", *Error.GetStanza().GetName(), *Error.GetErrorString());
}

void FXmppConnectionStrophe::ReceiveStanza(const FStropheStanza& Stanza)
{
	UE_LOGF(LogXmpp, Verbose, "Received Strophe XMPP Stanza %ls", *Stanza.GetName());
	OnStanzaReceived().Broadcast(Stanza);
	// Reset our ping timer now that we've received traffic
	if (PingStrophe.IsValid() && ServerConfiguration.bResetPingTimeoutOnReceiveStanza)
	{
		PingStrophe->ResetPingTimer();
	}

	// If ReceiveStanza returns true, the stanza has been consumed and we need to return
	if (MessagesStrophe.IsValid())
	{
		if (MessagesStrophe->ReceiveStanza(Stanza))
		{
			UE_LOGF(LogXmpp, VeryVerbose, "%ls Stanza handled by Messages", *Stanza.GetName());
			return;
		}
	}
	if (MultiUserChatStrophe.IsValid())
	{
		if (MultiUserChatStrophe->ReceiveStanza(Stanza))
		{
			UE_LOGF(LogXmpp, VeryVerbose, "%ls Stanza handled by MultiUserChat", *Stanza.GetName());
			return;
		}
	}
	if (PingStrophe.IsValid())
	{
		if (PingStrophe->ReceiveStanza(Stanza))
		{
			UE_LOGF(LogXmpp, VeryVerbose, "%ls Stanza handled by Ping", *Stanza.GetName());
			return;
		}
	}
	if (PresenceStrophe.IsValid())
	{
		if (PresenceStrophe->ReceiveStanza(Stanza))
		{
			UE_LOGF(LogXmpp, VeryVerbose, "%ls Stanza handled by Presence", *Stanza.GetName());
			return;
		}
	}
	if (PrivateChatStrophe.IsValid())
	{
		if (PrivateChatStrophe->ReceiveStanza(Stanza))
		{
			UE_LOGF(LogXmpp, VeryVerbose, "%ls Stanza handled by PrivateChat", *Stanza.GetName());
			return;
		}
	}
	if (PubSubStrophe.IsValid())
	{
		if (PubSubStrophe->ReceiveStanza(Stanza))
		{
			UE_LOGF(LogXmpp, VeryVerbose, "%ls Stanza handled by PubSub", *Stanza.GetName());
			return;
		}
	}

	UE_LOGF(LogXmpp, Warning, "%ls Stanza left unhandled", *Stanza.GetName());
}

void FXmppConnectionStrophe::QueueNewLoginStatus(EXmppLoginStatus::Type NewStatus)
{
	IncomingLoginStatusChanges.Enqueue(NewStatus);
}

void FXmppConnectionStrophe::ProcessLoginStatusChange(EXmppLoginStatus::Type NewLoginStatus)
{
	EXmppLoginStatus::Type OldLoginStatus = LoginStatus;
	if (OldLoginStatus != NewLoginStatus)
	{
		UE_LOGF(LogXmpp, Log, "Strophe processing LoginStatus change Was: %ls Now: %ls", EXmppLoginStatus::ToString(OldLoginStatus), EXmppLoginStatus::ToString(NewLoginStatus));

		// New login status needs to be set before calling below delegates
		LoginStatus = NewLoginStatus;

		if (NewLoginStatus == EXmppLoginStatus::LoggedIn)
		{
			UE_LOGF(LogXmpp, Log, "Logged IN JID=%ls", *GetUserJid().ToDebugString());
			if (OldLoginStatus == EXmppLoginStatus::ProcessingLogin)
			{
				OnLoginComplete().Broadcast(GetUserJid(), true, FString());
			}
			OnLoginChanged().Broadcast(GetUserJid(), EXmppLoginStatus::LoggedIn);
		}
		else if (NewLoginStatus == EXmppLoginStatus::LoggedOut)
		{
			UE_LOGF(LogXmpp, Log, "Logged OUT JID=%ls", *GetUserJid().ToDebugString());
			if (OldLoginStatus == EXmppLoginStatus::ProcessingLogin)
			{
				OnLoginComplete().Broadcast(GetUserJid(), false, FString());
			}
			else if (OldLoginStatus == EXmppLoginStatus::ProcessingLogout)
			{
				OnLogoutComplete().Broadcast(GetUserJid(), true, FString());
			}
			if (OldLoginStatus == EXmppLoginStatus::LoggedIn ||
				OldLoginStatus == EXmppLoginStatus::ProcessingLogout)
			{
				OnLoginChanged().Broadcast(GetUserJid(), EXmppLoginStatus::LoggedOut);
			}
		}
	}
}

void FXmppConnectionStrophe::DumpState() const
{
	UE_LOGF(LogXmpp, Warning, "JID=%ls Status=%ls", *GetUserJid().ToDebugString(), EXmppLoginStatus::ToString(GetLoginStatus()));
}

#endif
