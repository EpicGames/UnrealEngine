// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppConnection.h"
#include "XmppStrophe/StropheConnection.h"
#include "XmppStrophe/XmppStropheThread.h"
#include "XmppStrophe/StropheWebsocketConnection.h"

#include "Containers/Ticker.h"

#if WITH_XMPP_STROPHE

using FXmppMessagesStrophePtr = TSharedPtr<class FXmppMessagesStrophe, ESPMode::ThreadSafe>;
using FXmppMultiUserChatStrophePtr = TSharedPtr<class FXmppMultiUserChatStrophe, ESPMode::ThreadSafe>;
using FXmppPingStrophePtr = TSharedPtr<class FXmppPingStrophe, ESPMode::ThreadSafe>;
using FXmppPresenceStrophePtr = TSharedPtr<class FXmppPresenceStrophe, ESPMode::ThreadSafe>;
using FXmppPrivateChatStrophePtr = TSharedPtr<class FXmppPrivateChatStrophe, ESPMode::ThreadSafe>;
using FXmppPubSubStrophePtr = TSharedPtr<class FXmppPubSubStrophe, ESPMode::ThreadSafe>;

class FStropheContext;
class FStropheError;

class FXmppConnectionStrophe
	: public IXmppConnection
	, public FTSTickerObjectBase
{
	friend FXmppStropheThread;

public:
	// FXmppConnectionStrophe
	explicit FXmppConnectionStrophe();
	virtual ~FXmppConnectionStrophe();
	FXmppConnectionStrophe(const FXmppConnectionStrophe& Other) = delete;
	FXmppConnectionStrophe(FXmppConnectionStrophe&& Other) = delete;
	FXmppConnectionStrophe& operator=(const FXmppConnectionStrophe& Other) = delete;
	FXmppConnectionStrophe& operator=(FXmppConnectionStrophe&& Other) = delete;

	// Game Thread methods

	// IXmppConnection
	virtual void SetServer(const FXmppServer& NewServerConfiguration) override;
	virtual const FXmppServer& GetServer() const override;

	virtual void Login(const FString& UserId, const FString& Auth) override;
	virtual void Logout() override;

	virtual EXmppLoginStatus::Type GetLoginStatus() const override;

	virtual const FXmppUserJid& GetUserJid() const override;

	virtual FOnXmppLoginComplete& OnLoginComplete() override { return OnXmppLoginCompleteDelegate; }
	virtual FOnXmppLoginChanged& OnLoginChanged() override { return OnXmppLoginChangedDelegate; }
	virtual FOnXmppLogoutComplete& OnLogoutComplete() override { return OnXmppLogoutCompleteDelegate; }
	virtual FOnXmppStanzaSent& OnStanzaSent() override { return OnXmppStanzaSentDelegate; }
	virtual FOnXmppStanzaReceived& OnStanzaReceived() override { return OnXmppStanzaReceivedDelegate; }

	virtual IXmppMessagesPtr Messages() override;
	virtual IXmppMultiUserChatPtr MultiUserChat() override;
	virtual IXmppPresencePtr Presence() override;
	virtual IXmppChatPtr PrivateChat() override;
	virtual IXmppPubSubPtr PubSub() override;

	virtual void DumpState() const override;

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

	bool SendStanza(FStropheStanza&& Stanza);

	void StartXmppThread(const FXmppUserJid& ConnectionUser, const FString& ConnectionAuth);
	void StopXmppThread();

	// XMPP Thread methods

	void ReceiveConnectionStateChange(EStropheConnectionEvent Event, bool bQueue = true);
	void ReceiveConnectionError(const FStropheError& Error, EStropheConnectionEvent Event);
	void ReceiveStanza(const FStropheStanza& Stanza);
	void QueueNewLoginStatus(EXmppLoginStatus::Type NewStatus);
	void ProcessLoginStatusChange(EXmppLoginStatus::Type NewLoginStatus);

	/** Get our Context object */
	const FStropheContext& GetContext() const { return StropheContext; }

	/** Get the MUC domain that was saved right before we connected */
	const FString& GetMucDomain() const { return MucDomain; }

private:
	/** Notify that Login was triggered again while already connected */
	void ReconnectLogin();

protected:
	/** XMPP Context to use across this connection */
	FStropheContext StropheContext;

	/** Our XMPP thread that manages sending/receiving/sending events */
	TUniquePtr<FXmppStropheThread> StropheThread;

	/** If using websockets, use event driven websocket connection instead */
	TUniquePtr<FStropheWebsocketConnection> WebsocketConnection;

	/** Current login status */
	TAtomic<EXmppLoginStatus::Type> LoginStatus;

	/** Queue of login status changes processed on tick so that we do not miss processing any status changes */
	TQueue<EXmppLoginStatus::Type> IncomingLoginStatusChanges;

	/** XMPP Connection information to be used during Login/Connection */
	FXmppServer ServerConfiguration;

	/** The User last logged into */
	FXmppUserJid UserJid;

	/** The Multi-User-Chat domain to use in stanzas */
	FString MucDomain;

	/** Thread-Safe way to request we logout+cleanup from the XMPP Thread*/
	FThreadSafeBool RequestLogout;

	/** Login status delegates */
	FOnXmppLoginComplete OnXmppLoginCompleteDelegate;
	FOnXmppLoginChanged OnXmppLoginChangedDelegate;
	FOnXmppLogoutComplete OnXmppLogoutCompleteDelegate;

	/** Message Status delegates */
	FOnXmppStanzaReceived OnXmppStanzaReceivedDelegate;
	FOnXmppStanzaSent OnXmppStanzaSentDelegate;

	/** XMPP Implementation Shared Pointers */
	FXmppMessagesStrophePtr MessagesStrophe;
	FXmppMultiUserChatStrophePtr MultiUserChatStrophe;
	FXmppPingStrophePtr PingStrophe;
	FXmppPresenceStrophePtr PresenceStrophe;
	FXmppPrivateChatStrophePtr PrivateChatStrophe;
	FXmppPubSubStrophePtr PubSubStrophe;
};

#endif
