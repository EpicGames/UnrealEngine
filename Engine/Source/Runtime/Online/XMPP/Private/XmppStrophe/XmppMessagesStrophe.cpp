// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/XmppMessagesStrophe.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/StropheConnection.h"
#include "XmppStrophe/StropheContext.h"
#include "XmppStrophe/StropheStanzaConstants.h"
#include "XmppStrophe/XmppStrophe.h"
#include "XmppLog.h"

#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Misc/Guid.h"
#include "Misc/EmbeddedCommunication.h"
#include "Containers/BackgroundableTicker.h"
#include "Stats/Stats.h"

#if WITH_XMPP_STROPHE

#define TickRequesterId FName("StropheMessages")

FXmppMessagesStrophe::FXmppMessagesStrophe(FXmppConnectionStrophe& InConnectionManager)
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
	, ConnectionManager(InConnectionManager)
{
}

FXmppMessagesStrophe::~FXmppMessagesStrophe()
{
	CleanupMessages();
}

void FXmppMessagesStrophe::OnDisconnect()
{
	CleanupMessages();
}

void FXmppMessagesStrophe::OnReconnect()
{

}

bool FXmppMessagesStrophe::ReceiveStanza(const FStropheStanza& IncomingStanza)
{
	if (IncomingStanza.GetName() != Strophe::SN_MESSAGE || // Must be a message
		IncomingStanza.GetType() == Strophe::ST_CHAT || // Filter Chat messages
		IncomingStanza.GetType() == Strophe::ST_GROUPCHAT) // Filter MUC messages
	{
		return false;
	}

	const TOptional<const FStropheStanza> ErrorStanza = IncomingStanza.GetChildStropheStanza(Strophe::SN_ERROR);
	if (ErrorStanza.IsSet())
	{
		return HandleMessageErrorStanza(ErrorStanza.GetValue());
	}

	return HandleMessageStanza(IncomingStanza);
}

bool FXmppMessagesStrophe::HandleMessageStanza(const FStropheStanza& IncomingStanza)
{
	FXmppMessage Message;
	Message.FromJid = IncomingStanza.GetFrom();
	Message.ToJid = IncomingStanza.GetTo();

	TOptional<FString> BodyText = IncomingStanza.GetBodyText();
	if (!BodyText.IsSet())
	{
		return true;
	}

	auto JsonReader = TJsonReaderFactory<>::Create(MoveTemp(BodyText.GetValue()));
	TSharedPtr<FJsonObject> JsonBody;
	if (FJsonSerializer::Deserialize(JsonReader, JsonBody) &&
		JsonBody.IsValid())
	{
		JsonBody->TryGetStringField(TEXT("type"), Message.Type);
		const TSharedPtr<FJsonObject>* JsonPayload = NULL;
		if (JsonBody->TryGetObjectField(TEXT("payload"), JsonPayload) &&
			JsonPayload != NULL &&
			(*JsonPayload).IsValid())
		{
			auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&Message.Payload);
			FJsonSerializer::Serialize((*JsonPayload).ToSharedRef(), JsonWriter);
			JsonWriter->Close();
		}
		else if (JsonBody->TryGetStringField(TEXT("payload"), Message.Payload))
		{
			// Payload is now in Message.Payload
		}
		else
		{
			// Treat the entire body as the payload
			Message.Payload = IncomingStanza.GetBodyText().GetValue();
		}
		FString TimestampStr;
		if (JsonBody->TryGetStringField(TEXT("timestamp"), TimestampStr))
		{
			FDateTime::ParseIso8601(*TimestampStr, Message.Timestamp);
		}
	}
	else
	{
		UE_LOGF(LogXmpp, Warning, "Message: Failed to Deserialize JsonBody %ls", *IncomingStanza.GetBodyText().GetValue());
	}

	FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
	IncomingMessages.Enqueue(MakeUnique<FXmppMessage>(MoveTemp(Message)));
	return true;
}

bool FXmppMessagesStrophe::HandleMessageErrorStanza(const FStropheStanza& ErrorStanza)
{
	TArray<FStropheStanza> ErrorList = ErrorStanza.GetChildren();
	if (ErrorList.Num() > 0)
	{
		for (const FStropheStanza& ErrorItem : ErrorList)
		{
			const FString ErrorName = ErrorItem.GetName();
			FString OutError;

			if (ErrorName == Strophe::SN_RECIPIENT_UNAVAILABLE)
			{
				OutError = TEXT("The recipient is no longer available.");
			}
			else
			{
				const FString ErrorStanzaText = ErrorItem.GetText();
				OutError = FString::Printf(TEXT("Unknown Error %s: %s"), *ErrorName, *ErrorStanzaText);
			}

			UE_LOGF(LogXmpp, Error, "Message: Received error %ls", *OutError);
		}
	}
	else
	{
		UE_LOGF(LogXmpp, Warning, "Received unknown message stanza error");
	}
	return true;
}

bool FXmppMessagesStrophe::SendMessage(const FXmppUserJid& RecipientId, const FString& Type, const FString& Payload, bool bPayloadIsSerializedJson)
{
	if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		return false;
	}

	if (!RecipientId.IsValid())
	{
		UE_LOGF(LogXmpp, Warning, "Unable to send message. Invalid jid: %ls", *RecipientId.ToDebugString());
		return false;
	}

	FStropheStanza MessageStanza(ConnectionManager, Strophe::SN_MESSAGE);
	{
		MessageStanza.SetId(FGuid::NewGuid().ToString());
		MessageStanza.SetTo(RecipientId);

		FString StanzaText;
		auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&StanzaText);
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("type"), Type);
		if (bPayloadIsSerializedJson)
		{
			JsonWriter->WriteIdentifierPrefix(TEXT("payload"));
			JsonWriter->WriteRawJSONValue(Payload);
		}
		else
		{
			JsonWriter->WriteValue(TEXT("payload"), Payload);
		}
		JsonWriter->WriteValue(TEXT("timestamp"), FDateTime::UtcNow().ToIso8601());
		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();

		MessageStanza.AddBodyWithText(StanzaText);
	}

	return ConnectionManager.SendStanza(MoveTemp(MessageStanza));
}

bool FXmppMessagesStrophe::SendMessage(const FXmppUserJid& RecipientId, const FString& Type, const TSharedRef<FJsonObject>& Payload)
{
	FString SerializedPayload;
	auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&SerializedPayload);
	verify(FJsonSerializer::Serialize(Payload, JsonWriter));

	return SendMessage(RecipientId, Type, SerializedPayload, true);
}

bool FXmppMessagesStrophe::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FXmppMessagesStrophe_Tick);

	while (!IncomingMessages.IsEmpty())
	{
		TUniquePtr<FXmppMessage> Message;
		if (IncomingMessages.Dequeue(Message))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			OnMessageReceived(MoveTemp(Message));
		}
	}

	return true;
}

void FXmppMessagesStrophe::OnMessageReceived(TUniquePtr<FXmppMessage>&& Message)
{
	const TSharedRef<FXmppMessage> MessageRef = MakeShareable(Message.Release());
	OnMessageReceivedDelegate.Broadcast(ConnectionManager.AsShared(), MessageRef->FromJid, MessageRef);
}

void FXmppMessagesStrophe::CleanupMessages()
{
	while (!IncomingMessages.IsEmpty())
	{
		TUniquePtr<FXmppMessage> Message;
		IncomingMessages.Dequeue(Message);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}
}

#undef TickRequesterId

#endif
