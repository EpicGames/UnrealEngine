// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppMessages.h"

#include "Containers/Ticker.h"
#include "Containers/Queue.h"

#if WITH_XMPP_STROPHE

class FXmppConnectionStrophe;
class FStropheStanza;

class FXmppMessagesStrophe
	: public IXmppMessages
	, public FTSTickerObjectBase
{
public:
	// FXmppMessagesStrophe
	FXmppMessagesStrophe(FXmppConnectionStrophe& InConnectionManager);
	~FXmppMessagesStrophe();

	// XMPP Thread
	bool ReceiveStanza(const FStropheStanza& IncomingStanza);
	bool HandleMessageStanza(const FStropheStanza& IncomingStanza);
	bool HandleMessageErrorStanza(const FStropheStanza& ErrorStanza);

	// Game Thread
	void OnDisconnect();
	void OnReconnect();

	// IXmppMessages
	virtual bool SendMessage(const FXmppUserJid& RecipientId, const FString& Type, const FString& Payload, bool bPayloadIsSerializedJson = false) override;
	virtual bool SendMessage(const FXmppUserJid& RecipientId, const FString& Type, const TSharedRef<class FJsonObject>& Payload) override;
	virtual FOnXmppMessageReceived& OnReceiveMessage() override { return OnMessageReceivedDelegate; }

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

protected:
	void OnMessageReceived(TUniquePtr<FXmppMessage>&& Message);

	/** Remove pending messages and engine KeepAwake calls */
	void CleanupMessages();

protected:
	/** Connection manager controls sending data to XMPP thread */
	FXmppConnectionStrophe& ConnectionManager;

	/**
	 * Queue of Messages needing to be consumed. These are Queued
	 * on the XMPP thread, and Dequeued on the Game thread
	 */
	TQueue<TUniquePtr<FXmppMessage>> IncomingMessages;

	/** Delegate for game to listen to messages */
	FOnXmppMessageReceived OnMessageReceivedDelegate;
};

#endif