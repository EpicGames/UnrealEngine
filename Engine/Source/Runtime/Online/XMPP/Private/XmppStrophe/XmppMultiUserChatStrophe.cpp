// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/XmppMultiUserChatStrophe.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppStrophe/XmppStrophe.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/StropheStanzaConstants.h"
#include "XmppLog.h"

#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/Guid.h"
#include "Misc/EmbeddedCommunication.h"
#include "Containers/BackgroundableTicker.h"
#include "Stats/Stats.h"

#if WITH_XMPP_STROPHE

#define TickRequesterId FName("StropheMultichat")

FXmppMultiUserChatStrophe::FXmppMultiUserChatStrophe(FXmppConnectionStrophe& InConnectionManager)
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
	, ConnectionManager(InConnectionManager)
{

}

FXmppMultiUserChatStrophe::~FXmppMultiUserChatStrophe()
{
	CleanupMessages();
}

void FXmppMultiUserChatStrophe::OnDisconnect()
{
	CleanupMessages();

	Chatrooms.Empty();
	PendingRoomCreateConfigs.Empty();
}

void FXmppMultiUserChatStrophe::OnReconnect()
{

}

bool FXmppMultiUserChatStrophe::ReceiveStanza(const FStropheStanza& IncomingStanza)
{
	// MUC presence are from our MUC domain
	const FString StanzaName = IncomingStanza.GetName();
	if (StanzaName == Strophe::SN_PRESENCE &&
		IncomingStanza.GetFrom().Domain.Equals(ConnectionManager.GetMucDomain(), ESearchCase::CaseSensitive))
	{
		if (IncomingStanza.GetType() != Strophe::ST_ERROR)
		{
			return HandlePresenceStanza(IncomingStanza);
		}
		else
		{
			return HandlePresenceErrorStanza(IncomingStanza);
		}
	}
	else if (StanzaName == Strophe::SN_MESSAGE)
	{
		const FString StanzaType = IncomingStanza.GetType();
		if (StanzaType == Strophe::ST_GROUPCHAT)
		{
			return HandleGroupChatStanza(IncomingStanza);
		}
		else if (StanzaType == Strophe::ST_ERROR)
		{
			return HandleGroupChatErrorStanza(IncomingStanza);
		}
	}
	else if (StanzaName == Strophe::SN_IQ)
	{
		// Ignore pings
		if (IncomingStanza.HasChildByNameAndNamespace(Strophe::SN_PING, Strophe::SNS_PING))
		{
			return false;
		}

		// Config sets/gets (what we're caring about here) don't have queries in the "muc owner" namespace, so filter out those who do
		TOptional<const FStropheStanza> QueryStanza = IncomingStanza.GetChildByNameAndNamespace(Strophe::SN_QUERY, Strophe::SNS_MUC_OWNER);
		if (!QueryStanza.IsSet())
		{
			const FString StanzaType = IncomingStanza.GetType();
			if (StanzaType == Strophe::ST_RESULT)
			{
				// If this is a result type, check the FromJid for JUST a domain (i.e. the server), and drop it if it is
				// These are replies to pings (pongs), and until we keep track of what IDs we use for sent-pings, this filter
				// method will have to do
				const FXmppUserJid FromUser(IncomingStanza.GetFrom());
				const bool bMessageFromOnlyDomain = FromUser.Id.IsEmpty() && !FromUser.Domain.IsEmpty() && FromUser.Resource.IsEmpty();
				if (bMessageFromOnlyDomain)
				{
					// Ignore pongs
					return false;
				}
			}

			if (StanzaType != Strophe::ST_ERROR)
			{
				return HandleRoomConfigStanza(IncomingStanza);
			}
			else
			{
				return HandleRoomConfigErrorStanza(IncomingStanza);
			}
		}
	}

	return false;
}

bool FXmppMultiUserChatStrophe::HandlePresenceStanza(const FStropheStanza& IncomingStanza)
{
	FXmppMucPresence Presence;

	Presence.bIsAvailable = IncomingStanza.GetType() != Strophe::ST_UNAVAILABLE;
	Presence.UserJid = IncomingStanza.GetFrom();

	TOptional<const FStropheStanza> UserStanza = IncomingStanza.GetChildByNameAndNamespace(Strophe::SN_X, Strophe::SNS_MUC_USER);
	if (UserStanza.IsSet())
	{
		TOptional<const FStropheStanza> UserItemStanza = UserStanza->GetChildStropheStanza(Strophe::SN_ITEM);
		if (UserItemStanza.IsSet())
		{
			Presence.MemberJid = FXmppStrophe::JidFromString(UserItemStanza->GetAttribute(Strophe::SA_JID));
			Presence.Role = UserItemStanza->GetAttribute(Strophe::SA_ROLE);
			Presence.Affiliation = UserItemStanza->GetAttribute(Strophe::SA_AFFILIATION);
		}
	}

	FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
	IncomingMucPresenceUpdates.Enqueue(MakeUnique<FXmppMucPresence>(MoveTemp(Presence)));
	return true;
}

bool FXmppMultiUserChatStrophe::HandlePresenceErrorStanza(const FStropheStanza& IncomingStanza)
{
	FXmppStropheErrorPair OutError;
	OutError.RoomId = IncomingStanza.GetFrom().Id;

	TOptional<const FStropheStanza> Error = IncomingStanza.GetChildStropheStanza(Strophe::SN_ERROR);
	if (Error.IsSet())
	{
		const TArray<FStropheStanza> ErrorList = Error->GetChildren();
		for (const FStropheStanza& ErrorItem : ErrorList)
		{
			const FString ErrorName = ErrorItem.GetName();

			if (ErrorName == Strophe::SN_NOT_AUTHORIZED)
			{
				OutError.ErrorMessage += TEXT("A password is required to join this room");
			}
			else if (ErrorName == Strophe::SN_FORBIDDEN)
			{
				OutError.ErrorMessage += TEXT("You are not allowed to join this room");
			}
			else if (ErrorName == Strophe::SN_ITEM_NOT_FOUND)
			{
				OutError.ErrorMessage += TEXT("That room does not exist");
			}
			else if (ErrorName == Strophe::SN_NOT_ALLOWED)
			{
				OutError.ErrorMessage += TEXT("You are unable to create rooms");
			}
			else if (ErrorName == Strophe::SN_NOT_ACCEPTABLE)
			{
				OutError.ErrorMessage += TEXT("You may not change your nickname");
			}
			else if (ErrorName == Strophe::SN_REGISTRATION_REQUIRED)
			{
				OutError.ErrorMessage += TEXT("You are not a member of this room");
			}
			else if (ErrorName == Strophe::SN_CONFLICT)
			{
				OutError.ErrorMessage += TEXT("Your nickname is already in use in this room");
			}
			else if (ErrorName == Strophe::SN_SERVICE_UNAVAILABLE)
			{
				OutError.ErrorMessage += TEXT("The requested room is full");
			}
			else
			{
				OutError.ErrorMessage += FString::Printf(TEXT("Unknown Error %s. "), *ErrorName);
			}

		}
	}

	if (OutError.ErrorMessage.IsEmpty())
	{
		OutError.ErrorMessage = TEXT("Unknown error");
	}

	UE_LOGF(LogXmpp, Warning, "MUC: Received error %ls", *OutError.ErrorMessage);

	if (!OutError.ErrorMessage.IsEmpty())
	{
		FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
		IncomingMucPresenceErrors.Enqueue(MoveTemp(OutError));
	}

	return true;
}

bool FXmppMultiUserChatStrophe::HandleGroupChatStanza(const FStropheStanza& IncomingStanza)
{
	TOptional<const FStropheStanza> SubjectStanza = IncomingStanza.GetChildStropheStanza(Strophe::SN_SUBJECT);
	if (SubjectStanza.IsSet())
	{
		FXmppStropheSubjectUpdate SubjectUpdate;
		SubjectUpdate.NewSubject = SubjectStanza->GetText();
		SubjectUpdate.RoomId = IncomingStanza.GetFrom().Id;

		FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
		IncomingRoomSubjects.Enqueue(MoveTemp(SubjectUpdate));
		return true;
	}

	// Check for room settings update (status code 104)
	TOptional<const FStropheStanza> XStanza = IncomingStanza.GetChildByNameAndNamespace(Strophe::SN_X, Strophe::SNS_MUC_USER);
	if (XStanza.IsSet())
	{
		// We're looking for exactly 1 'status' child
		const TArray<FStropheStanza> XChildren = XStanza->GetChildren();
		if (XChildren.Num() == 1)
		{
			const FStropheStanza& XChild = XChildren[0];
			if (XChild.GetName() == Strophe::SN_STATUS)
			{
				FString Code = XChild.GetAttribute(Strophe::SA_CODE);
				if (Code == Strophe::SC_104)
				{
					FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
					IncomingRoomInfoUpdates.Enqueue(FXmppRoomId(IncomingStanza.GetFrom().Id));
				}
			}
		}

		return true;
	}

	TOptional<FString> BodyText = IncomingStanza.GetBodyText();
	if (!BodyText.IsSet())
	{
		// Bad data, no body
		return true;
	}

	FXmppChatMessage ChatMessage;
	ChatMessage.ToJid = IncomingStanza.GetTo();
	ChatMessage.FromJid = IncomingStanza.GetFrom();
	ChatMessage.Body = MoveTemp(BodyText.GetValue());

	// Parse Timezone
	TOptional<const FStropheStanza> StanzaDelay = IncomingStanza.GetChildStropheStanza(Strophe::SN_DELAY);
	if (StanzaDelay.IsSet())
	{
		if (StanzaDelay->HasAttribute(Strophe::SA_STAMP))
		{
			FString Timestamp = StanzaDelay->GetAttribute(Strophe::SA_STAMP);
			FDateTime::ParseIso8601(*Timestamp, ChatMessage.Timestamp);
		}
	}

	if (ChatMessage.Timestamp == 0)
	{
		ChatMessage.Timestamp = FDateTime::UtcNow();
	}

	FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
	IncomingGroupChatMessages.Enqueue(MakeUnique<FXmppChatMessage>(MoveTemp(ChatMessage)));
	return true;
}

bool FXmppMultiUserChatStrophe::HandleGroupChatErrorStanza(const FStropheStanza& IncomingStanza)
{
	FString ErrorMessage;

	TOptional<const FStropheStanza> Error = IncomingStanza.GetChildStropheStanza(Strophe::SN_ERROR);
	if (Error.IsSet())
	{
		const TArray<FStropheStanza> ErrorList = Error->GetChildren();
		for (const FStropheStanza& ErrorItem : ErrorList)
		{
			const FString ErrorName = ErrorItem.GetName();

			if (ErrorName == Strophe::SN_FORBIDDEN)
			{
				ErrorMessage += TEXT("Unable to send message to room. ");
			}
			else if (ErrorName == Strophe::SN_BAD_REQUEST)
			{
				ErrorMessage += TEXT("Unable to send groupchat message to an individual. ");
			}
			else if (ErrorName == Strophe::SN_NOT_ACCEPTABLE)
			{
				ErrorMessage += TEXT("You may not send messages to rooms you have not joined. ");
			}
			else
			{
				ErrorMessage += FString::Printf(TEXT("%s. "), *ErrorName);
			}

		}
	}

	if (ErrorMessage.IsEmpty())
	{
		ErrorMessage = TEXT("Unknown error");
	}

	UE_LOGF(LogXmpp, Warning, "MUC: Received GroupChat error %ls", *ErrorMessage);

	return true;
}

bool FXmppMultiUserChatStrophe::HandleRoomConfigStanza(const FStropheStanza& IncomingStanza)
{
	// There are four possible outputs from this that mean success:
	// a) No children in iq stanza; this means we successfully set the configuration
	//    for the channel!
	// b) The iq stanza has a query stanza with no children; this means we requested
	//    the config options or the config option defaults and there are none
	// c) The query stanza has children, but those childen have no value children;
	//    this means we got the list of possible config options
	// d) The query stanza has children and those children have value stanzas; this
	//    means the channel already exists and we're querying the options for it

	// Check for config write case (No Query child)
	TOptional<const FStropheStanza> QueryStanza = IncomingStanza.GetChildStropheStanza(Strophe::SN_QUERY);
	if (!QueryStanza.IsSet())
	{
		FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
		IncomingRoomConfigWriteSuccesses.Enqueue(IncomingStanza.GetFrom().Id);
		return true;
	}

	// Right now we only care about the successful write case, but you could totally
	// write a config parser here and pass back the config values for a room to the
	// game thread if you wanted.

	return true;
}

bool FXmppMultiUserChatStrophe::HandleRoomConfigErrorStanza(const FStropheStanza& IncomingStanza)
{
	FXmppStropheErrorPair OutError;
	OutError.RoomId = IncomingStanza.GetFrom().Id;

	TOptional<const FStropheStanza> ErrorStanza = IncomingStanza.GetChildStropheStanza(Strophe::SN_ERROR);
	if (ErrorStanza.IsSet())
	{
		const FString ErrorType = ErrorStanza->GetType();
		if (ErrorType == Strophe::ST_AUTH && ErrorStanza->HasChild(Strophe::SN_FORBIDDEN))
		{
			OutError.ErrorMessage = TEXT("Only the room owner may modify the room configuration");
		}

		// Don't log the error message, as we may not care about failures depending on the CallbackType
	}

	if (!OutError.ErrorMessage.IsEmpty())
	{
		FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
		IncomingRoomConfigErrors.Enqueue(MoveTemp(OutError));
	}

	return true;
}

bool FXmppMultiUserChatStrophe::CreateRoom(const FXmppRoomId& RoomId, const FString& Nickname, const FXmppRoomConfig& RoomConfig)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: CreateRoom=%ls Nickname=%ls", *RoomId, *Nickname);

	bool bSuccess = false;
	FString ErrorStr;

	if (RoomId.IsEmpty())
	{
		ErrorStr = TEXT("Room ID Invalid");
	}
	else if (Nickname.IsEmpty())
	{
		ErrorStr = TEXT("Nickname is Invalid");
	}
	else if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		ErrorStr = TEXT("Not currently connected");
	}
	else
	{
		FXmppRoomStrophe& Room = Chatrooms.FindOrAdd(RoomId);
		// Set the Room's ID if we just created it
		if (Room.Info.Id.IsEmpty() || !Room.RoomJid.IsValid())
		{
			Room.RoomJid = FXmppUserJid(RoomId, ConnectionManager.GetMucDomain(), Nickname);
			Room.Info.Id = Room.RoomJid.Id;
		}

		if (Room.Status == ERoomStatusStrophe::Joined)
		{
			ErrorStr = FString::Printf(TEXT("Already in room %s"), *RoomId);
		}
		else if (Room.Status != ERoomStatusStrophe::NotJoined)
		{
			ErrorStr = FString::Printf(TEXT("Another operation already pending for room %s"), *RoomId);
		}
		else
		{
			Room.Status = ERoomStatusStrophe::CreatePending;

			// cache off the config for use after the room is created & ready to be configured
			PendingRoomCreateConfigs.Emplace(RoomId, RoomConfig);

			bSuccess = SendJoinRoomStanza(Room);
		}
	}

	if (!bSuccess)
	{
		UE_LOGF(LogXmpp, Warning, "MUC: CreateRoom failed. %ls", *ErrorStr);
		OnXmppRoomCreateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), bSuccess, RoomId, ErrorStr);
	}

	return bSuccess;
}

bool FXmppMultiUserChatStrophe::ConfigureRoom(const FXmppRoomId& RoomId, const FXmppRoomConfig& RoomConfig)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: ConfigureRoom RoomId=%ls", *RoomId);

	bool bSuccess = false;
	FString ErrorStr;

	if (RoomId.IsEmpty())
	{
		ErrorStr = TEXT("Room ID Invalid");
	}
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);
	if (RoomPtr == nullptr)
	{
		ErrorStr = FString::Printf(TEXT("Could not find room %s"), *RoomId);
	}
	else if (RoomPtr->Status != ERoomStatusStrophe::Joined)
	{
		ErrorStr = FString::Printf(TEXT("You must be in room %s to configure it."), *RoomId);
	}
	else if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		ErrorStr = TEXT("You are not currently connected to the server");
	}
	else if (RoomPtr->Info.OwnerId != RoomPtr->GetNickname())
	{
		ErrorStr = FString::Printf(TEXT("You must be the owner of room %s to configure it. The current owner is %s"), *RoomId, *RoomPtr->Info.OwnerId);
	}
	else
	{
		bSuccess = InternalConfigureRoom(*RoomPtr, RoomConfig, EConfigureRoomTypeStrophe::UseConfigCallback);
		if (!bSuccess)
		{
			ErrorStr = TEXT("Failed to configure room");
		}
	}

	if (!bSuccess)
	{
		UE_LOGF(LogXmpp, Warning, "MUC: Failed to configure Room=%ls Error=%ls", *RoomId, *ErrorStr);
		OnXmppRoomConfiguredDelegate.Broadcast(ConnectionManager.AsShared(), bSuccess, RoomId, ErrorStr);
	}

	return bSuccess;
}

bool FXmppMultiUserChatStrophe::RefreshRoomInfo(const FXmppRoomId& RoomId)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: RefreshRoomInfo RoomId=%ls", *RoomId);

	// This just prints a bunch of info to the console in our libjingle module, so we're going to
	// skip writing a bunch of code that doesn't do anything and just call our delegate instead

	if (Chatrooms.Find(RoomId) != nullptr)
	{
		OnXmppRoomInfoRefreshedDelegate.Broadcast(ConnectionManager.AsShared(), true, RoomId, FString());
		return true;
	}
	else
	{
		OnXmppRoomInfoRefreshedDelegate.Broadcast(ConnectionManager.AsShared(), false, RoomId, TEXT("Room does not exist"));
		return false;
	}
}

bool FXmppMultiUserChatStrophe::JoinPublicRoom(const FXmppRoomId& RoomId, const FString& Nickname)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: JoinPublicRoom RoomId=%ls Nickname=%ls", *RoomId, *Nickname);

	bool bSuccess = false;
	FString ErrorStr;

	if (RoomId.IsEmpty())
	{
		ErrorStr = TEXT("Room ID Invalid");
	}
	else if (Nickname.IsEmpty())
	{
		ErrorStr = TEXT("Nickname is Invalid");
	}
	else if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		ErrorStr = TEXT("Not currently connected");
	}
	else
	{
		FXmppRoomStrophe& Room = Chatrooms.FindOrAdd(RoomId);
		// Set the Room's ID if we just created it
		if (Room.Info.Id.IsEmpty() || !Room.RoomJid.IsValid())
		{
			Room.RoomJid = FXmppUserJid(RoomId, ConnectionManager.GetMucDomain(), Nickname);
			Room.Info.Id = Room.RoomJid.Id;
		}

		if (Room.Status == ERoomStatusStrophe::Joined)
		{
			ErrorStr = FString::Printf(TEXT("Already in room %s"), *RoomId);
		}
		else if (Room.Status != ERoomStatusStrophe::NotJoined)
		{
			ErrorStr = FString::Printf(TEXT("Another operation already pending for room %s"), *RoomId);
		}
		else
		{
			Room.Status = ERoomStatusStrophe::JoinPublicPending;

			bSuccess = SendJoinRoomStanza(Room);
		}
	}

	if (!bSuccess)
	{
		UE_LOGF(LogXmpp, Warning, "MUC: JoinPublicRoom failed. %ls", *ErrorStr);
		OnXmppRoomJoinPublicCompleteDelegate.Broadcast(ConnectionManager.AsShared(), bSuccess, RoomId, ErrorStr);
	}

	return bSuccess;
}

bool FXmppMultiUserChatStrophe::JoinPrivateRoom(const FXmppRoomId& RoomId, const FString& Nickname, const FString& Password)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: JoinPrivateRoom RoomId=%ls Nickname=%ls", *RoomId, *Nickname);

	bool bSuccess = false;
	FString ErrorStr;

	if (RoomId.IsEmpty())
	{
		ErrorStr = TEXT("Room ID Invalid");
	}
	else if (Nickname.IsEmpty())
	{
		ErrorStr = TEXT("Nickname is Invalid");
	}
	else if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		ErrorStr = TEXT("Not currently connected");
	}
	else
	{
		FXmppRoomStrophe& Room = Chatrooms.FindOrAdd(RoomId);
		// Set the Room's ID if we just created it
		if (Room.Info.Id.IsEmpty() || !Room.RoomJid.IsValid())
		{
			Room.RoomJid = FXmppUserJid(RoomId, ConnectionManager.GetMucDomain(), Nickname);
			Room.Info.Id = Room.RoomJid.Id;
		}

		if (Room.Status == ERoomStatusStrophe::Joined)
		{
			ErrorStr = FString::Printf(TEXT("Already in room %s"), *RoomId);
		}
		else if (Room.Status != ERoomStatusStrophe::NotJoined)
		{
			ErrorStr = FString::Printf(TEXT("Another operation already pending for room %s"), *RoomId);
		}
		else
		{
			Room.Status = ERoomStatusStrophe::JoinPrivatePending;

			bSuccess = SendJoinRoomStanza(Room, Password);
		}
	}

	if (!bSuccess)
	{
		UE_LOGF(LogXmpp, Warning, "MUC: JoinPrivateRoom failed. %ls", *ErrorStr);
		// trigger delegates on error
		OnXmppRoomJoinPrivateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), bSuccess, RoomId, ErrorStr);
	}

	return bSuccess;
}

bool FXmppMultiUserChatStrophe::RegisterMember(const FXmppRoomId& RoomId, const FString& Nickname)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: RegisterMember RoomId=%ls Nickname=%ls", *RoomId, *Nickname);
	// No-op currently in libjingle version
	return false;
}

bool FXmppMultiUserChatStrophe::UnregisterMember(const FXmppRoomId& RoomId, const FString& Nickname)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: UnregisterMember RoomId=%ls Nickname=%ls", *RoomId, *Nickname);
	// No-op currently in libjingle version
	return false;
}

bool FXmppMultiUserChatStrophe::ExitRoom(const FXmppRoomId& RoomId)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: ExitRoom RoomId=%ls", *RoomId);

	bool bSuccess = false;
	FString ErrorStr;
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);

	if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		// Not logged in, so cannot leave the chat room
		ErrorStr = TEXT("User not logged in");
		bSuccess = false;
	}
	else if (RoomPtr == nullptr)
	{
		ErrorStr = TEXT("Room was never created.");
		bSuccess = true;	// Returning true since we can't be in the room at all.
	}
	else
	{
		switch (RoomPtr->Status)
		{
		case ERoomStatusStrophe::ExitPending:
			{
				// We've already queued an exit, so do not queue again or trigger delegates
				bSuccess = true;
			}
			break;
		case ERoomStatusStrophe::Joined:
			{
				ERoomStatusStrophe OldStatus = RoomPtr->Status;

				// Queue our exit
				RoomPtr->Status = ERoomStatusStrophe::ExitPending;
				bSuccess = SendExitRoomStanza(*RoomPtr);

				if (!bSuccess)
				{
					ErrorStr = TEXT("Could not send ExitRoom stanza");
					RoomPtr->Status = OldStatus;	// Restore old room status if we could not queue the exit operation
				}
			}
			break;
		case ERoomStatusStrophe::NotJoined:
		case ERoomStatusStrophe::CreatePending:
		case ERoomStatusStrophe::JoinPublicPending:
		case ERoomStatusStrophe::JoinPrivatePending:
		default:
			{
				// We're (probably) not in this room, so we don't need to exit
				ErrorStr = FString::Printf(TEXT("User not in room: status is %s"), LexToString(RoomPtr->Status));
				bSuccess = false;
			}
			break;
		}
	}

	if (!bSuccess || RoomPtr == nullptr)
	{
		UE_LOGF(LogXmpp, Warning, "MUC: Cannot leave room %ls: %ls", *RoomId, *ErrorStr);
		OnXmppRoomExitCompleteDelegate.Broadcast(ConnectionManager.AsShared(), bSuccess, RoomId, ErrorStr);
	}

	return bSuccess;
}

bool FXmppMultiUserChatStrophe::SendChat(const FXmppRoomId& RoomId, const FString& MsgBody, const FString& ChatInfo)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: SendChat RoomId=%ls", *RoomId);

	if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		return false;
	}

	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);
	if (RoomPtr == nullptr)
	{
		return false;
	}

	FStropheStanza MessageStanza(ConnectionManager, Strophe::SN_MESSAGE);
	{
		MessageStanza.SetId(FGuid::NewGuid().ToString());
		MessageStanza.SetType(Strophe::ST_GROUPCHAT);
		MessageStanza.SetTo(RoomPtr->GetRoomJid().GetBareId());
		MessageStanza.AddBodyWithText(MsgBody);
	}

	return ConnectionManager.SendStanza(MoveTemp(MessageStanza));
}

void FXmppMultiUserChatStrophe::GetJoinedRooms(TArray<FXmppRoomId>& OutRooms)
{
	OutRooms.Empty(Chatrooms.Num());
	for (const TMap<FXmppRoomId, FXmppRoomStrophe>::ElementType& Pair : Chatrooms)
	{
		OutRooms.Emplace(Pair.Key);
	}
}

bool FXmppMultiUserChatStrophe::GetRoomInfo(const FXmppRoomId& RoomId, FXmppRoomInfo& OutRoomInfo)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);
	if (RoomPtr != nullptr)
	{
		OutRoomInfo = RoomPtr->Info;
		return true;
	}

	return false;
}

bool FXmppMultiUserChatStrophe::GetMembers(const FXmppRoomId& RoomId, TArray<FXmppChatMemberRef>& OutMembers)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);
	if (RoomPtr != nullptr)
	{
		OutMembers = RoomPtr->Members;
		return true;
	}
	return false;
}

FXmppChatMemberPtr FXmppMultiUserChatStrophe::GetMember(const FXmppRoomId& RoomId, const FXmppUserJid& MemberJid)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);
	if (RoomPtr != nullptr)
	{
		FXmppChatMemberPtr FoundMember = RoomPtr->GetMember(MemberJid);
		return FoundMember;
	}

	return nullptr;
}

bool FXmppMultiUserChatStrophe::GetLastMessages(const FXmppRoomId& RoomId, int32 NumMessages, TArray< TSharedRef<FXmppChatMessage> >& OutMessages)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);
	if (RoomPtr != nullptr)
	{
		const int32 MessagesToFetch = FMath::Max(FMath::Min(NumMessages, RoomPtr->LastMessages.Num()), 0);
		OutMessages.Empty(MessagesToFetch);

		for (int32 Index = 0; Index < MessagesToFetch; ++Index)
		{
			check(RoomPtr->LastMessages.IsValidIndex(Index));
			OutMessages.Emplace(RoomPtr->LastMessages[Index]);
		}

		return true;
	}

	OutMessages.Empty();
	return false;
}

void FXmppMultiUserChatStrophe::HandleMucPresence(const FXmppMucPresence& MemberPresence)
{
	// We don't use this, but it's built into the interface for reasons?
}

void FXmppMultiUserChatStrophe::DumpMultiUserChatState() const
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogXmpp, ELogVerbosity::Display);
	for (const TMap<FXmppRoomId, FXmppRoomStrophe>::ElementType& Room : Chatrooms)
	{
		const FXmppRoomId& RoomId = Room.Key;
		const FXmppRoomStrophe& XmppRoom = Room.Value;

		UE_LOGF(LogXmpp, Display, "RoomId: %ls", *RoomId);
		UE_LOGF(LogXmpp, Display, " Owner: %ls Subj: %ls Priv: %d", *XmppRoom.Info.OwnerId, *XmppRoom.Info.Subject, XmppRoom.Info.bIsPrivate);
		UE_LOGF(LogXmpp, Display, " Status: %d", (int32)XmppRoom.Status);

		UE_LOGF(LogXmpp, Display, " Members: %d", XmppRoom.Members.Num());
		for (const FXmppChatMemberRef& Member : XmppRoom.Members)
		{
			UE_LOGF(LogXmpp, Display, "  %ls", *Member->ToDebugString());
		}
	}
}

bool FXmppMultiUserChatStrophe::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FXmppMultiUserChatStrophe_Tick);

	while (!IncomingMucPresenceUpdates.IsEmpty())
	{
		TUniquePtr<FXmppMucPresence> MucPresence;
		if (IncomingMucPresenceUpdates.Dequeue(MucPresence))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			check(MucPresence.IsValid());
			OnReceiveMucPresence(MoveTemp(*MucPresence));
		}
	}
	while (!IncomingMucPresenceErrors.IsEmpty())
	{
		FXmppStropheErrorPair ErrorInfo;
		if (IncomingMucPresenceErrors.Dequeue(ErrorInfo))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			OnReceiveMucPresenceError(MoveTemp(ErrorInfo));
		}
	}
	while (!IncomingGroupChatMessages.IsEmpty())
	{
		TUniquePtr<FXmppChatMessage> GroupChatMessage;
		if (IncomingGroupChatMessages.Dequeue(GroupChatMessage))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			OnReceiveGroupChatMessage(MoveTemp(GroupChatMessage));
		}
	}
	while (!IncomingRoomSubjects.IsEmpty())
	{
		FXmppStropheSubjectUpdate NewSubject;
		if (IncomingRoomSubjects.Dequeue(NewSubject))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			OnReceiveGroupChatSubject(MoveTemp(NewSubject));
		}
	}
	while (!IncomingRoomConfigErrors.IsEmpty())
	{
		FXmppStropheErrorPair RoomConfigError;
		if (IncomingRoomConfigErrors.Dequeue(RoomConfigError))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			OnReceiveRoomConfigError(MoveTemp(RoomConfigError));
		}
	}
	while (!IncomingRoomConfigWriteSuccesses.IsEmpty())
	{
		FXmppRoomId RoomId;
		if (IncomingRoomConfigWriteSuccesses.Dequeue(RoomId))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			OnReceiveRoomConfigSuccess(MoveTemp(RoomId));
		}
	}
	while (!IncomingRoomInfoUpdates.IsEmpty())
	{
		FXmppRoomId RoomId;
		if (IncomingRoomInfoUpdates.Dequeue(RoomId))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			OnReceieveRoomInfoUpdate(MoveTemp(RoomId));
		}
	}

	return true;
}

void FXmppMultiUserChatStrophe::OnReceiveMucPresence(FXmppMucPresence&& MemberPresence)
{
	UE_LOGF(LogXmpp, VeryVerbose, "MUC: OnReceiveMucPresence: jid=%ls nick=%ls roomid=%ls role=%ls affiliation=%ls", *MemberPresence.UserJid.ToDebugString(), *MemberPresence.GetNickName(), *MemberPresence.GetRoomId(), *MemberPresence.Role, *MemberPresence.Affiliation);

	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(MemberPresence.GetRoomId());
	if (RoomPtr != nullptr)
	{
		const bool bUpdateIsUs = MemberPresence.GetNickName().Contains(ConnectionManager.GetUserJid().Id, ESearchCase::CaseSensitive);
		const bool bLeftRoom = !MemberPresence.bIsAvailable;
		const bool bCreateRoom = bUpdateIsUs && !bLeftRoom && RoomPtr->Status == ERoomStatusStrophe::CreatePending;
		const bool bJoinPublicRoom = bUpdateIsUs && !bLeftRoom && RoomPtr->Status == ERoomStatusStrophe::JoinPublicPending;
		const bool bJoinPrivateRoom = bUpdateIsUs && !bLeftRoom && RoomPtr->Status == ERoomStatusStrophe::JoinPrivatePending;
		const bool bAlreadyInRoom = RoomPtr->HasMember(MemberPresence.UserJid);

		if (bUpdateIsUs)
		{
			// This presence update is us doing something
			if (bLeftRoom)
			{
				HandleExitRoomComplete(*RoomPtr, MoveTemp(MemberPresence));
			}
			else if (bCreateRoom)
			{
				HandleCreateRoomComplete(*RoomPtr, MoveTemp(MemberPresence));
			}
			else if (bJoinPrivateRoom)
			{
				HandleJoinPrivateRoomComplete(*RoomPtr, MoveTemp(MemberPresence));
			}
			else if (bJoinPublicRoom)
			{
				HandleJoinPublicRoomComplete(*RoomPtr, MoveTemp(MemberPresence));
			}
			else
			{
				ensureMsgf(false, TEXT("Unknown libstrope Presence Self-State update"));
			}
		}
		// Other users
		else
		{
			if (!bAlreadyInRoom && !bLeftRoom)
			{
				// Anyone we didn't know about, that isn't leaving, is joining
				HandleRoomMemberJoined(*RoomPtr, MoveTemp(MemberPresence));
			}
			else if (bLeftRoom)
			{
				HandleRoomMemberLeft(*RoomPtr, MoveTemp(MemberPresence));
			}
			else
			{
				HandleRoomMemberChanged(*RoomPtr, MoveTemp(MemberPresence));
			}
		}
	}
	else
	{
		UE_LOGF(LogXmpp, VeryVerbose, "MUC: OnReceiveMucPresence Ignored presence from room we haven't joined: Room=%ls Connjid=%ls", *MemberPresence.GetRoomId(), *ConnectionManager.GetUserJid().Id);
	}
}

void FXmppMultiUserChatStrophe::OnReceiveMucPresenceError(FXmppStropheErrorPair&& PresenceError)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(PresenceError.RoomId);
	if (RoomPtr != nullptr)
	{
		switch (RoomPtr->Status)
		{
		case ERoomStatusStrophe::CreatePending:
			Chatrooms.Remove(PresenceError.RoomId);
			OnXmppRoomCreateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), false, PresenceError.RoomId, PresenceError.ErrorMessage);
			break;
		case ERoomStatusStrophe::JoinPrivatePending:
			Chatrooms.Remove(PresenceError.RoomId);
			OnXmppRoomJoinPrivateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), false, PresenceError.RoomId, PresenceError.ErrorMessage);
			break;
		case ERoomStatusStrophe::JoinPublicPending:
			Chatrooms.Remove(PresenceError.RoomId);
			OnXmppRoomJoinPublicCompleteDelegate.Broadcast(ConnectionManager.AsShared(), false, PresenceError.RoomId, PresenceError.ErrorMessage);
			break;
		case ERoomStatusStrophe::ExitPending:
		case ERoomStatusStrophe::NotJoined:
		case ERoomStatusStrophe::Joined:
			// No-Op?
			break;
		}
	}
	else
	{
		UE_LOGF(LogXmpp, Warning, "MUC: OnReceiveMucPresenceError Received Error from room we haven't joined: Room=%ls Connjid=%ls Error=%ls", *PresenceError.RoomId, *ConnectionManager.GetUserJid().Id, *PresenceError.ErrorMessage);
	}
}

void FXmppMultiUserChatStrophe::OnReceiveGroupChatMessage(TUniquePtr<FXmppChatMessage>&& GroupChatMessage)
{
	check(GroupChatMessage.IsValid());

	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(GroupChatMessage->FromJid.Id);
	if (RoomPtr != nullptr)
	{
		TSharedRef<FXmppChatMessage> SharedChatMessageRef = MakeShareable(GroupChatMessage.Release());
		RoomPtr->AddNewMessage(SharedChatMessageRef);
		OnXmppRoomChatReceivedDelegate.Broadcast(ConnectionManager.AsShared(), SharedChatMessageRef->FromJid.Id, SharedChatMessageRef->FromJid, SharedChatMessageRef);
	}
	else
	{
		UE_LOGF(LogXmpp, Log, "MUC: OnReceiveGroupChatMessage Ignored GroupChat from room we haven't joined: Room=%ls Connjid=%ls", *GroupChatMessage->FromJid.Id, *ConnectionManager.GetUserJid().Id);
	}
}

void FXmppMultiUserChatStrophe::OnReceiveGroupChatSubject(FXmppStropheSubjectUpdate&& SubjectUpdate)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(SubjectUpdate.RoomId);
	if (RoomPtr != nullptr)
	{
		RoomPtr->Info.Subject = MoveTemp(SubjectUpdate.NewSubject);
		OnXmppRoomInfoRefreshedDelegate.Broadcast(ConnectionManager.AsShared(), true, SubjectUpdate.RoomId, FString());
	}
	else
	{
		UE_LOGF(LogXmpp, Log, "MUC: OnReceiveGroupChatMessage Ignored Updated Room Subject from room we haven't joined: Room=%ls Connjid=%ls NewSubject=%ls", *SubjectUpdate.RoomId, *ConnectionManager.GetUserJid().Id, *SubjectUpdate.NewSubject);
	}}

void FXmppMultiUserChatStrophe::OnReceiveRoomConfigError(FXmppStropheErrorPair&& RoomConfigError)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomConfigError.RoomId);
	if (RoomPtr != nullptr)
	{
		EConfigureRoomTypeStrophe* CallbackTypePtr = PendingRoomConfigCallbacks.Find(RoomPtr->GetRoomId());
		if (ensure(CallbackTypePtr != nullptr))
		{
			switch (*CallbackTypePtr)
			{
			case EConfigureRoomTypeStrophe::NoCallback:
				// No-Op
				return; // Return on purpose so we don't log below
			case EConfigureRoomTypeStrophe::UseConfigCallback:
				OnXmppRoomConfiguredDelegate.Broadcast(ConnectionManager.AsShared(), false, RoomPtr->GetRoomId(), RoomConfigError.ErrorMessage);
				break;
			case EConfigureRoomTypeStrophe::UseCreateCallback:
				SendExitRoomStanza(*RoomPtr);
				break;
			}

			UE_LOGF(LogXmpp, Warning, "MUC: Failed to Configure Room Room=%ls Error=%ls", *RoomConfigError.RoomId, *RoomConfigError.ErrorMessage);
		}
		else
		{
			UE_LOGF(LogXmpp, Warning, "MUC: OnReceiveRoomConfigError Received Error from room we have no callback for: Room=%ls Connjid=%ls Error=%ls", *RoomConfigError.RoomId, *ConnectionManager.GetUserJid().Id, *RoomConfigError.ErrorMessage);
		}
	}
	else
	{
		UE_LOGF(LogXmpp, Warning, "MUC: OnReceiveRoomConfigError Received Error from room we haven't joined: Room=%ls Connjid=%ls Error=%ls", *RoomConfigError.RoomId, *ConnectionManager.GetUserJid().Id, *RoomConfigError.ErrorMessage);
	}
}

void FXmppMultiUserChatStrophe::OnReceiveRoomConfigSuccess(FXmppRoomId&& RoomId)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);
	if (RoomPtr != nullptr)
	{
		EConfigureRoomTypeStrophe* CallbackTypePtr = PendingRoomConfigCallbacks.Find(RoomPtr->GetRoomId());
		if (ensure(CallbackTypePtr != nullptr))
		{
			switch (*CallbackTypePtr)
			{
			case EConfigureRoomTypeStrophe::NoCallback:
				// No-Op
				break;
			case EConfigureRoomTypeStrophe::UseConfigCallback:
				OnXmppRoomConfiguredDelegate.Broadcast(ConnectionManager.AsShared(), true, RoomPtr->GetRoomId(), FString());
				break;
			case EConfigureRoomTypeStrophe::UseCreateCallback:
				if (ensure(RoomPtr->Status == ERoomStatusStrophe::CreatePending))
				{
					RoomPtr->Status = ERoomStatusStrophe::Joined;
				}
				else
				{
					UE_LOGF(LogXmpp, Warning, "OnReceiveRoomConfigSuccess: Room %ls is not in the CreatePending state (%ls)", *RoomPtr->GetRoomId(), LexToString(RoomPtr->Status));
				}
				OnXmppRoomCreateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), true, RoomPtr->GetRoomId(), FString());
				break;
			}

			UE_LOGF(LogXmpp, Verbose, "MUC: OnReceiveRoomConfigSuccess Received success for room %ls", *RoomId);
		}
		else
		{
			UE_LOGF(LogXmpp, Warning, "MUC: OnReceiveRoomConfigSuccess Received success from room with no callback Room=%ls", *RoomId);
		}
	}
	else
	{
		UE_LOGF(LogXmpp, Warning, "MUC: OnReceiveRoomConfigSuccess Received RoomConfig from room we haven't joined: Room=%ls Connjid=%ls", *RoomId, *ConnectionManager.GetUserJid().Id);
	}
}

void FXmppMultiUserChatStrophe::OnReceieveRoomInfoUpdate(FXmppRoomId&& RoomId)
{
	FXmppRoomStrophe* RoomPtr = Chatrooms.Find(RoomId);
	if (RoomPtr != nullptr)
	{
		SendRequestRoomInfoConfigStanza(*RoomPtr);
	}
	else
	{
		UE_LOGF(LogXmpp, Warning, "MUC: OnReceieveRoomInfoUpdate Received RoomInfoUpdate for room we haven't joined: Room=%ls Connjid=%ls", *RoomId, *ConnectionManager.GetUserJid().Id);
	}
}

bool FXmppMultiUserChatStrophe::SendJoinRoomStanza(const FXmppRoomStrophe& Room, const FString& Password)
{
	// Create/Join room
	FStropheStanza JoinRoomStanza(ConnectionManager, Strophe::SN_PRESENCE);
	JoinRoomStanza.SetTo(Room.GetRoomJid());
	{
		FStropheStanza XStanza(ConnectionManager, Strophe::SN_X);
		XStanza.SetNamespace(Strophe::SNS_MUC);
		if (!Password.IsEmpty())
		{
			FStropheStanza PasswordStanza(ConnectionManager, Strophe::SN_PASSWORD);
			PasswordStanza.SetText(Password);
			XStanza.AddChild(PasswordStanza);
		}

		{
			FStropheStanza HistoryStanza(ConnectionManager, Strophe::SN_HISTORY);
			HistoryStanza.SetAttribute(Strophe::SA_MAXSTANZAS, FString::FromInt(MAX_MESSAGE_HISTORY));
			XStanza.AddChild(HistoryStanza);
		}

		JoinRoomStanza.AddChild(XStanza);
	}

	return ConnectionManager.SendStanza(MoveTemp(JoinRoomStanza));
}

bool FXmppMultiUserChatStrophe::SendExitRoomStanza(const FXmppRoomStrophe& Room)
{
	FStropheStanza ExitPresence(ConnectionManager, Strophe::SN_PRESENCE);
	{
		ExitPresence.SetTo(Room.GetRoomJid());
		ExitPresence.SetType(Strophe::ST_UNAVAILABLE);
	}

	return ConnectionManager.SendStanza(MoveTemp(ExitPresence));
}

bool FXmppMultiUserChatStrophe::SendRequestRoomInfoConfigStanza(const FXmppRoomStrophe& Room)
{
	FStropheStanza IQStanza(ConnectionManager, Strophe::SN_IQ);
	{
		IQStanza.SetId(FGuid::NewGuid().ToString());
		IQStanza.SetTo(Room.GetRoomJid().GetBareId());
		IQStanza.SetFrom(ConnectionManager.GetUserJid());
		IQStanza.SetType(Strophe::ST_GET);

		{
			FStropheStanza QueryStanza(ConnectionManager, Strophe::SN_QUERY);
			QueryStanza.SetNamespace(Strophe::SNS_DISCO_INFO);
			IQStanza.AddChild(QueryStanza);
		}
	}

	return ConnectionManager.SendStanza(MoveTemp(IQStanza));
}

bool FXmppMultiUserChatStrophe::InternalConfigureRoom(const FXmppRoomStrophe& Room, const FXmppRoomConfig& RoomConfig, EConfigureRoomTypeStrophe CallbackType)
{
	const auto SetStanzaConfig = [this](FStropheStanza& ParentStanza, const FString& Key, const FString& Value)
	{
		FStropheStanza FieldStanza(ConnectionManager, Strophe::SN_FIELD);
		FieldStanza.SetAttribute(Strophe::SA_VAR, Key);
		{
			FStropheStanza ValueStanza(ConnectionManager, Strophe::SN_VALUE);
			ValueStanza.SetText(Value);
			FieldStanza.AddChild(ValueStanza);
		}
		ParentStanza.AddChild(FieldStanza);
	};

	FStropheStanza IQStanza(ConnectionManager, Strophe::SN_IQ);
	{
		IQStanza.SetId(FGuid::NewGuid().ToString());
		IQStanza.SetTo(Room.GetRoomJid().GetBareId());
		IQStanza.SetFrom(ConnectionManager.GetUserJid());
		IQStanza.SetType(Strophe::ST_SET);

		FStropheStanza QueryStanza(ConnectionManager, Strophe::SN_QUERY);
		{
			QueryStanza.SetNamespace(Strophe::SNS_MUC_OWNER);

			FStropheStanza XStanza(ConnectionManager, Strophe::SN_X);
			{
				XStanza.SetNamespace(Strophe::SNS_X_DATA);
				XStanza.SetType(Strophe::ST_SUBMIT);
				SetStanzaConfig(XStanza, TEXT("FORM_TYPE"),								TEXT("http://jabber.org/protocol/muc#roomconfig"));
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_roomname"),				RoomConfig.RoomName);
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_roomdesc"),				RoomConfig.RoomDesc);
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_persistentroom"),			RoomConfig.bIsPersistent ? TEXT("1") : TEXT("0"));
				SetStanzaConfig(XStanza, TEXT("muc#maxhistoryfetch"),					FString::FromInt(RoomConfig.MaxMsgHistory));
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_changesubject"),			RoomConfig.bAllowChangeSubject ? TEXT("1") : TEXT("0"));
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_anonymity"),				FXmppRoomConfig::ConvertRoomAnonymityToString(RoomConfig.RoomAnonymity));
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_membersonly"),			RoomConfig.bIsMembersOnly ? TEXT("1") : TEXT("0"));
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_moderatedroom"),			RoomConfig.bIsModerated ? TEXT("1") : TEXT("0"));
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_publicroom"),				RoomConfig.bAllowPublicSearch ? TEXT("1") : TEXT("0"));
				SetStanzaConfig(XStanza, TEXT("muc#roomconfig_passwordprotectedroom"),	RoomConfig.bIsPrivate ? TEXT("1") : TEXT("0"));
				if (RoomConfig.bIsPrivate)
				{
					SetStanzaConfig(XStanza, TEXT("muc#roomconfig_roomsecret"), RoomConfig.Password);
				}

				QueryStanza.AddChild(XStanza);
			}
		}
		IQStanza.AddChild(QueryStanza);
	}

	PendingRoomConfigCallbacks.Emplace(Room.GetRoomId(), CallbackType);

	return ConnectionManager.SendStanza(MoveTemp(IQStanza));
}

void FXmppMultiUserChatStrophe::HandleCreateRoomComplete(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: HandleCreateRoomComplete: Room: %ls User: %ls", *Room.GetRoomId(), *MemberPresence.GetNickName());

	FXmppChatMemberRef ChatMember = MakeShareable(new FXmppChatMember(MemberPresence));

	Room.Members.Emplace(ChatMember);

	FXmppRoomConfig* RoomConfigPtr = PendingRoomCreateConfigs.Find(Room.GetRoomId());
	if (ensure(RoomConfigPtr != nullptr))
	{
		const bool bIsOwner = ChatMember->Affiliation == EXmppChatMemberAffiliation::Owner;
		if (bIsOwner)
		{
			const bool bSuccess = InternalConfigureRoom(Room, *RoomConfigPtr, EConfigureRoomTypeStrophe::UseCreateCallback);
			if (!bSuccess)
			{
				SendExitRoomStanza(Room);
			}
		}
		else
		{
			Room.Status = ERoomStatusStrophe::Joined;
			// We joined instead of creating
			OnXmppRoomCreateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), true, Room.GetRoomId(), FString());
		}

		// Remove our pending configurations after we've copied things out in ConfigureRoom
		PendingRoomCreateConfigs.Remove(Room.GetRoomId());
	}
	else
	{
		Room.Status = ERoomStatusStrophe::Joined;
		OnXmppRoomCreateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), false, Room.GetRoomId(), TEXT("Missing Room Configuration"));
	}
}

void FXmppMultiUserChatStrophe::HandleJoinPrivateRoomComplete(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: HandleJoinPrivateRoomComplete: Room: %ls User: %ls", *Room.GetRoomId(), *MemberPresence.GetNickName());

	Room.Status = ERoomStatusStrophe::Joined;
	Room.Members.Emplace(MakeShareable(new FXmppChatMember(MemberPresence)));
	OnXmppRoomJoinPrivateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), true, Room.GetRoomId(), FString());
}

void FXmppMultiUserChatStrophe::HandleJoinPublicRoomComplete(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: HandleJoinPublicRoomComplete: Room: %ls User: %ls", *Room.GetRoomId(), *MemberPresence.GetNickName());

	Room.Status = ERoomStatusStrophe::Joined;
	Room.Members.Emplace(MakeShareable(new FXmppChatMember(MemberPresence)));
	OnXmppRoomJoinPublicCompleteDelegate.Broadcast(ConnectionManager.AsShared(), true, Room.GetRoomId(), FString());
}

void FXmppMultiUserChatStrophe::HandleExitRoomComplete(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: HandleExitRoomComplete: Room: %ls User: %ls", *Room.GetRoomId(), *MemberPresence.GetNickName());

	if (Room.Status == ERoomStatusStrophe::CreatePending)
	{
		OnXmppRoomCreateCompleteDelegate.Broadcast(ConnectionManager.AsShared(), false, Room.GetRoomId(), TEXT("Failed to configure room"));
	}
	else
	{
		if (Room.Status != ERoomStatusStrophe::ExitPending)
		{
			UE_LOGF(LogXmpp, Verbose, "MUC: Server initiated room exit complete; in state %ls", LexToString(Room.Status));
		}
		OnXmppRoomExitCompleteDelegate.Broadcast(ConnectionManager.AsShared(), true, Room.GetRoomId(), FString());
	}

	// Do not use Room after this
	Chatrooms.Remove(Room.GetRoomId());
}

void FXmppMultiUserChatStrophe::HandleRoomMemberJoined(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: HandleRoomMemberJoined: Room: %ls User: %ls", *Room.GetRoomId(), *MemberPresence.GetNickName());

	FXmppChatMemberRef NewMember = MakeShareable(new FXmppChatMember(MemberPresence));

	if (NewMember->Affiliation == EXmppChatMemberAffiliation::Owner)
	{
		Room.Info.OwnerId = MemberPresence.UserJid.Resource;
	}

	Room.Members.Emplace(NewMember);

	OnXmppRoomMemberJoinDelegate.Broadcast(ConnectionManager.AsShared(), Room.GetRoomId(), MemberPresence.UserJid);
}

void FXmppMultiUserChatStrophe::HandleRoomMemberChanged(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: HandleRoomMemberChanged: Room: %ls User: %ls", *Room.GetRoomId(), *MemberPresence.GetNickName());

	for (FXmppChatMemberRef& Member : Room.Members)
	{
		if (Member->RoomMemberJid == MemberPresence.UserJid)
		{
			// We don't need to update Nickname as it's part of the User's JID
			Member->UserPresence = MemberPresence;
			Member->Affiliation = EXmppChatMemberAffiliation::ToType(MemberPresence.Affiliation);
			Member->Role = EXmppChatMemberRole::ToType(MemberPresence.Role);
			break;
		}
	}

	OnXmppRoomMemberChangedDelegate.Broadcast(ConnectionManager.AsShared(), Room.GetRoomId(), MemberPresence.UserJid);
}

void FXmppMultiUserChatStrophe::HandleRoomMemberLeft(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence)
{
	UE_LOGF(LogXmpp, Verbose, "MUC: HandleRoomMemberLeft: Room: %ls User: %ls", *Room.GetRoomId(), *MemberPresence.GetNickName());

	const int32 RoomMemberCount = Room.Members.Num();
	for (int32 Index = 0; Index < RoomMemberCount; ++Index)
	{
		const FXmppChatMemberRef& Member = Room.Members[Index];
		if (Member->RoomMemberJid == MemberPresence.UserJid)
		{
			Room.Members.RemoveAt(Index);
			break;
		}
	}

	OnXmppRoomMemberExitDelegate.Broadcast(ConnectionManager.AsShared(), Room.GetRoomId(), MemberPresence.UserJid);
}

void FXmppMultiUserChatStrophe::CleanupMessages()
{
	while (!IncomingMucPresenceUpdates.IsEmpty())
	{
		TUniquePtr<FXmppMucPresence> MucPresence;
		IncomingMucPresenceUpdates.Dequeue(MucPresence);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}

	while (!IncomingMucPresenceErrors.IsEmpty())
	{
		FXmppStropheErrorPair ErrorInfo;
		IncomingMucPresenceErrors.Dequeue(ErrorInfo);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}

	while (!IncomingGroupChatMessages.IsEmpty())
	{
		TUniquePtr<FXmppChatMessage> GroupChatMessage;
		IncomingGroupChatMessages.Dequeue(GroupChatMessage);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}

	while (!IncomingRoomSubjects.IsEmpty())
	{
		FXmppStropheSubjectUpdate NewSubject;
		IncomingRoomSubjects.Dequeue(NewSubject);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}

	while (!IncomingRoomConfigErrors.IsEmpty())
	{
		FXmppStropheErrorPair RoomConfigError;
		IncomingRoomConfigErrors.Dequeue(RoomConfigError);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}

	while (!IncomingRoomConfigWriteSuccesses.IsEmpty())
	{
		FXmppRoomId RoomId;
		IncomingRoomConfigWriteSuccesses.Dequeue(RoomId);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}

	while (!IncomingRoomInfoUpdates.IsEmpty())
	{
		FXmppRoomId RoomId;
		IncomingRoomInfoUpdates.Dequeue(RoomId);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}
}

#undef TickRequesterId

#endif
