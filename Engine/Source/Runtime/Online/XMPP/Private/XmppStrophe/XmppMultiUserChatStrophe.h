// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppMultiUserChat.h"

#include "Containers/Ticker.h"
#include "Containers/Queue.h"

#if WITH_XMPP_STROPHE

class FXmppConnectionStrophe;
class FStropheStanza;

#define MAX_MESSAGE_HISTORY 50

enum class ERoomStatusStrophe : uint8
{
	NotJoined,
	Joined,
	CreatePending,
	JoinPrivatePending,
	JoinPublicPending,
	ExitPending
};

inline const TCHAR* LexToString(ERoomStatusStrophe Value)
{
	switch (Value)
	{
	case ERoomStatusStrophe::NotJoined:
		return TEXT("NotJoined");
	case ERoomStatusStrophe::Joined:
		return TEXT("Joined");
	case ERoomStatusStrophe::CreatePending:
		return TEXT("CreatePending");
	case ERoomStatusStrophe::JoinPrivatePending:
		return TEXT("JoinPrivatePending");
	case ERoomStatusStrophe::JoinPublicPending:
		return TEXT("JoinPublicPending");
	case ERoomStatusStrophe::ExitPending:
		return TEXT("ExitPending");
	}

	checkf(false, TEXT("Unhandled ERoomStatusStrophe %d"), static_cast<int32>(Value));
	return TEXT("Unknown");
}

/**
 * Info cached about a joined/created room
 */
class FXmppRoomStrophe
{
public:

	FXmppRoomStrophe()
		: Status(ERoomStatusStrophe::NotJoined)
	{
	}

	void AddNewMessage(const TSharedRef<FXmppChatMessage>& ChatMessage)
	{
		LastMessages.Add(ChatMessage);
		if (LastMessages.Num() > MAX_MESSAGE_HISTORY)
		{
			LastMessages.RemoveAt(0);
		}
	}

	/**
	 * Check if a member is in the chat room based on their room JID
	 *
	 * @param RoomMemberJid the member's JID in the room
	 * @return true if the member is in the chat room
	 */
	bool HasMember(const FXmppUserJid& RoomMemberJid) const
	{
		FXmppChatMemberPtr FoundMember = GetMember(RoomMemberJid);
		return FoundMember.IsValid();
	}

	/**
	 * Get a member in the chat room based on their room JID
	 *
	 * @param RoomMemberJid the member's JID in the room
	 * @return pointer to the member, if found in the room
	 */
	FXmppChatMemberPtr GetMember(const FXmppUserJid& RoomMemberJid) const
	{
		for (const FXmppChatMemberRef& Member : Members)
		{
			if (Member->RoomMemberJid == RoomMemberJid)
			{
				return Member;
			}
		}

		return FXmppChatMemberPtr();
	}

	FXmppRoomId& GetRoomId() { return Info.Id; }
	const FXmppRoomId& GetRoomId() const { return Info.Id; }

	FString& GetNickname() { return RoomJid.Resource; };
	const FString& GetNickname() const { return RoomJid.Resource; };

	FXmppUserJid& GetRoomJid() { return RoomJid; }
	const FXmppUserJid& GetRoomJid() const { return RoomJid; }

public:
	FXmppUserJid RoomJid;

	ERoomStatusStrophe Status;
	FXmppRoomInfo Info;
	TArray<FXmppChatMemberRef> Members;
	TArray<TSharedRef<FXmppChatMessage> > LastMessages;
};

/** Struct to hold Error information about a failed command */
struct FXmppStropheErrorPair
{
	FString ErrorMessage;
	FXmppRoomId RoomId;
};

/** Struct to hold Subject information about a joined channel */
struct FXmppStropheSubjectUpdate
{
	FString NewSubject;
	FXmppRoomId RoomId;
};

enum class EConfigureRoomTypeStrophe : uint8
{
	NoCallback, // Trigger no callbacks.  Currently used for global chat config.
	UseCreateCallback, // New room config, trigger create callback when done
	UseConfigCallback // Change existing room, trigger config callback when done
};

class FXmppMultiUserChatStrophe
	: public IXmppMultiUserChat
	, public FTSTickerObjectBase
{
public:
	// FXmppMultiUserChatStrophe
	FXmppMultiUserChatStrophe(class FXmppConnectionStrophe& InConnectionManager);
	virtual ~FXmppMultiUserChatStrophe();

	// XMPP Thread
	bool ReceiveStanza(const FStropheStanza& IncomingStanza);
	bool HandlePresenceStanza(const FStropheStanza& IncomingStanza);
	bool HandlePresenceErrorStanza(const FStropheStanza& IncomingStanza);
	bool HandleGroupChatStanza(const FStropheStanza& IncomingStanza);
	bool HandleGroupChatErrorStanza(const FStropheStanza& IncomingStanza);
	bool HandleRoomConfigStanza(const FStropheStanza& IncomingStanza);
	bool HandleRoomConfigErrorStanza(const FStropheStanza& IncomingStanza);

	// Game Thread
	void OnDisconnect();
	void OnReconnect();

	// IXmppMultiUserChat
	virtual bool CreateRoom(const FXmppRoomId& RoomId, const FString& Nickname, const FXmppRoomConfig& RoomConfig) override;
	virtual bool ConfigureRoom(const FXmppRoomId& RoomId, const FXmppRoomConfig& RoomConfig) override;
	virtual bool RefreshRoomInfo(const FXmppRoomId& RoomId) override;
	virtual bool JoinPublicRoom(const FXmppRoomId& RoomId, const FString& Nickname) override;
	virtual bool JoinPrivateRoom(const FXmppRoomId& RoomId, const FString& Nickname, const FString& Password) override;
	virtual bool RegisterMember(const FXmppRoomId& RoomId, const FString& Nickname) override;
	virtual bool UnregisterMember(const FXmppRoomId& RoomId, const FString& Nickname) override;
	virtual bool ExitRoom(const FXmppRoomId& RoomId) override;
	virtual bool SendChat(const FXmppRoomId& RoomId, const class FString& MsgBody, const FString& ChatInfo) override;
	virtual void GetJoinedRooms(TArray<FXmppRoomId>& OutRooms) override;
	virtual bool GetRoomInfo(const FXmppRoomId& RoomId, FXmppRoomInfo& OutRoomInfo) override;
	virtual bool GetMembers(const FXmppRoomId& RoomId, TArray<FXmppChatMemberRef>& OutMembers) override;
	virtual FXmppChatMemberPtr GetMember(const FXmppRoomId& RoomId, const FXmppUserJid& MemberJid) override;
	virtual bool GetLastMessages(const FXmppRoomId& RoomId, int32 NumMessages, TArray< TSharedRef<FXmppChatMessage> >& OutMessages) override;
	virtual void HandleMucPresence(const FXmppMucPresence& MemberPresence) override;
	virtual void DumpMultiUserChatState() const override;
	virtual FOnXmppRoomCreateComplete& OnRoomCreated() override { return OnXmppRoomCreateCompleteDelegate; }
	virtual FOnXmppRoomConfigureComplete& OnRoomConfigured() override { return OnXmppRoomConfiguredDelegate; }
	virtual FOnXmppRoomInfoRefreshComplete& OnRoomInfoRefreshed() override { return OnXmppRoomInfoRefreshedDelegate; }
	virtual FOnXmppRoomJoinPublicComplete& OnJoinPublicRoom() override { return OnXmppRoomJoinPublicCompleteDelegate; }
	virtual FOnXmppRoomJoinPrivateComplete& OnJoinPrivateRoom() override { return OnXmppRoomJoinPrivateCompleteDelegate; }
	virtual FOnXmppRoomExitComplete& OnExitRoom() override { return OnXmppRoomExitCompleteDelegate; }
	virtual FOnXmppRoomMemberJoin& OnRoomMemberJoin() override { return OnXmppRoomMemberJoinDelegate; }
	virtual FOnXmppRoomMemberExit& OnRoomMemberExit() override { return OnXmppRoomMemberExitDelegate; }
	virtual FOnXmppRoomMemberChanged& OnRoomMemberChanged() override { return OnXmppRoomMemberChangedDelegate; }
	virtual FOnXmppRoomChatReceived& OnRoomChatReceived() override { return OnXmppRoomChatReceivedDelegate; }

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

protected:
	void OnReceiveMucPresence(FXmppMucPresence&& MemberPresence);
	void OnReceiveMucPresenceError(FXmppStropheErrorPair&& PresenceError);
	void OnReceiveGroupChatMessage(TUniquePtr<FXmppChatMessage>&& GroupChatMessage);
	void OnReceiveGroupChatSubject(FXmppStropheSubjectUpdate&& GroupChatMessage);
	void OnReceiveRoomConfigError(FXmppStropheErrorPair&& RoomConfigError);
	void OnReceiveRoomConfigSuccess(FXmppRoomId&& RoomId);
	void OnReceieveRoomInfoUpdate(FXmppRoomId&& RoomId);

	bool SendJoinRoomStanza(const FXmppRoomStrophe& Room, const FString& Password = FString());
	bool SendExitRoomStanza(const FXmppRoomStrophe& Room);
	bool SendRequestRoomInfoConfigStanza(const FXmppRoomStrophe& Room);

	bool InternalConfigureRoom(const FXmppRoomStrophe& Room, const FXmppRoomConfig& RoomConfig, EConfigureRoomTypeStrophe CallbackType);

	// Events that happen to us
	void HandleCreateRoomComplete(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence);
	void HandleJoinPrivateRoomComplete(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence);
	void HandleJoinPublicRoomComplete(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence);
	void HandleExitRoomComplete(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence);

	// Events that happen to others in our rooms
	void HandleRoomMemberJoined(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence);
	void HandleRoomMemberChanged(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence);
	void HandleRoomMemberLeft(FXmppRoomStrophe& Room, FXmppMucPresence&& MemberPresence);

	/** Remove pending messages and engine KeepAwake calls */
	void CleanupMessages();

protected:
	/** Connection manager controls sending data to XMPP thread */
	FXmppConnectionStrophe& ConnectionManager;

	/** Cache of known Rooms we belong to */
	TMap<FXmppRoomId, FXmppRoomStrophe> Chatrooms;

	/** Cache of Room configs while we wait for the corresponding room to be created */
	TMap<FXmppRoomId, FXmppRoomConfig> PendingRoomCreateConfigs;
	/** Cache of Room config callbacks while we wait for the corresponding room to be configured */
	TMap<FXmppRoomId, EConfigureRoomTypeStrophe> PendingRoomConfigCallbacks;

	/** Queue of presence updates from the server */
	TQueue<TUniquePtr<FXmppMucPresence>> IncomingMucPresenceUpdates;
	/** Queue of presence errors from the server */
	TQueue<FXmppStropheErrorPair> IncomingMucPresenceErrors;
	/** Queue of new chat messages from the server */
	TQueue<TUniquePtr<FXmppChatMessage>> IncomingGroupChatMessages;
	/** Queue of new Subjects for chat rooms */
	TQueue<FXmppStropheSubjectUpdate> IncomingRoomSubjects;
	/** Queue of room configuration errors from the server */
	TQueue<FXmppStropheErrorPair> IncomingRoomConfigErrors;
	/** Queue of room configuration writes that came back successful */
	TQueue<FXmppRoomId> IncomingRoomConfigWriteSuccesses;
	/** Queue of room configuration updates to be queried */
	TQueue<FXmppRoomId> IncomingRoomInfoUpdates;

	/** Delegates for game to listen for MUC events*/
	FOnXmppRoomCreateComplete OnXmppRoomCreateCompleteDelegate;
	FOnXmppRoomConfigureComplete OnXmppRoomConfiguredDelegate;
	FOnXmppRoomInfoRefreshComplete OnXmppRoomInfoRefreshedDelegate;
	FOnXmppRoomJoinPublicComplete OnXmppRoomJoinPublicCompleteDelegate;
	FOnXmppRoomJoinPrivateComplete OnXmppRoomJoinPrivateCompleteDelegate;
	FOnXmppRoomExitComplete OnXmppRoomExitCompleteDelegate;
	FOnXmppRoomMemberJoin OnXmppRoomMemberJoinDelegate;
	FOnXmppRoomMemberExit OnXmppRoomMemberExitDelegate;
	FOnXmppRoomMemberChanged OnXmppRoomMemberChangedDelegate;
	FOnXmppRoomChatReceived OnXmppRoomChatReceivedDelegate;
};
#endif