// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/XmppPingStrophe.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/StropheStanzaConstants.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppLog.h"
#include "Misc/Guid.h"
#include "Misc/EmbeddedCommunication.h"
#include "Containers/BackgroundableTicker.h"
#include "Stats/Stats.h"

#if WITH_XMPP_STROPHE

#define TickRequesterId FName("StrophePing")

FXmppPingStrophe::FXmppPingStrophe(FXmppConnectionStrophe& InConnectionManager)
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
	, ConnectionManager(InConnectionManager)
	, TimeSinceLastClientPing(0.0f)
	, bWaitingForPong(false)
	, SecondsSinceLastServerPong(0.0f)
	, bHasReceievedServerPong(false)
	, MissedPongs(0)
{
}

FXmppPingStrophe::~FXmppPingStrophe()
{
	CleanupMessages();
}

bool FXmppPingStrophe::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FXmppPingStrophe_Tick);

	// Process our ping queue and send pongs
	while (!IncomingPings.IsEmpty())
	{
		FXmppPingReceivedInfo ReceivedPing;
		if (IncomingPings.Dequeue(ReceivedPing))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			ReplyToPing(MoveTemp(ReceivedPing));
		}
	}

	if (ConnectionManager.GetLoginStatus() == EXmppLoginStatus::LoggedIn)
	{
		// Handle XMPP client pings
		TimeSinceLastClientPing += DeltaTime;
		if (TimeSinceLastClientPing >= ConnectionManager.GetServer().PingInterval)
		{
			SendClientPing();
		}
		else if (bWaitingForPong)
		{
			CheckXmppPongTimeout(DeltaTime);
		}
	}

	// Continue ticking
	return true;
}

void FXmppPingStrophe::OnDisconnect()
{
	CleanupMessages();

	// Reset our client ping timer
	ResetPingTimer();

	// Reset our client ping response timer
	ResetPongTimer();
}

void FXmppPingStrophe::OnReconnect()
{

}

bool FXmppPingStrophe::ReceiveStanza(const FStropheStanza& IncomingStanza)
{
	// All ping stanzas are IQ
	if (IncomingStanza.GetName() != Strophe::SN_IQ) // Must be an IQ element
	{
		return false;
	}

	// Store StanzaType for multiple comparisons
	const FString StanzaType = IncomingStanza.GetType();

	const bool bIsErrorType = StanzaType == Strophe::ST_ERROR;
	const bool bIsGetType = StanzaType == Strophe::ST_GET;
	const bool bIsResultType = StanzaType == Strophe::ST_RESULT;
	// Check if this is a ping stanza type (type of "get", "Result" or "error")
	if (!bIsGetType && !bIsResultType && !bIsErrorType )
	{
		return false;
	}

	// Check if this is a server-pong (result type)
	if (bIsResultType)
	{
		// Log that a pong has happened
		bHasReceievedServerPong = true;
		return true;
	}

	// Check if we have a ping child in the ping namespace
	if (!IncomingStanza.HasChildByNameAndNamespace(Strophe::SN_PING, Strophe::SNS_PING))
	{
		return false;
	}

	// Ignore ping errors (it means whoever we pinged just didn't support pings)
	if (bIsErrorType)
	{
		return true;
	}

	return HandlePingStanza(IncomingStanza);
}

void FXmppPingStrophe::ResetPingTimer()
{
	UE_LOGF(LogXmpp, VeryVerbose, "Resetting timer for Client-To-Server XMPP Ping");
	TimeSinceLastClientPing = 0.0f;
}

void FXmppPingStrophe::ResetPongTimer()
{
	UE_LOGF(LogXmpp, VeryVerbose, "Resetting timer for Client-To-Server XMPP Pong");

	bWaitingForPong = false;
	bHasReceievedServerPong = false;
	SecondsSinceLastServerPong = 0.0f;
	MissedPongs = 0;
}

bool FXmppPingStrophe::HandlePingStanza(const FStropheStanza& PingStanza)
{
	FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
	IncomingPings.Enqueue(FXmppPingReceivedInfo(PingStanza.GetFrom(), PingStanza.GetId()));
	return true;
}

void FXmppPingStrophe::ReplyToPing(FXmppPingReceivedInfo&& ReceivedPing)
{
	if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		return;
	}

	FStropheStanza PingReply(ConnectionManager, Strophe::SN_IQ);
	{
		PingReply.SetFrom(ConnectionManager.GetUserJid());
		PingReply.SetTo(ReceivedPing.FromJid);
		PingReply.SetId(MoveTemp(ReceivedPing.PingId));
		PingReply.SetType(Strophe::ST_RESULT);
	}

	ConnectionManager.SendStanza(MoveTemp(PingReply));
}

void FXmppPingStrophe::SendClientPing()
{
	UE_LOGF(LogXmpp, Verbose, "Sending Client-To-Server XMPP Ping");

	FStropheStanza ClientPing(ConnectionManager, Strophe::SN_IQ);
	{
		ClientPing.SetFrom(ConnectionManager.GetUserJid());
		ClientPing.SetTo(ConnectionManager.GetServer().Domain);
		ClientPing.SetType(Strophe::ST_GET);
		ClientPing.SetId(FGuid::NewGuid().ToString());

		FStropheStanza PingChild(ConnectionManager, Strophe::SN_PING);
		{
			PingChild.SetNamespace(Strophe::SNS_PING);
		}
		ClientPing.AddChild(PingChild);
	}

	if (ConnectionManager.SendStanza(MoveTemp(ClientPing)))
	{
		ResetPingTimer();
		bWaitingForPong = true;
	}
}

void FXmppPingStrophe::CheckXmppPongTimeout(float DeltaTime)
{
	// Check if we have received our ping response
	if (bHasReceievedServerPong)
	{
		// Reset our ping tracking
		ResetPongTimer();
	}
	else
	{
		const FXmppServer& XmppServer = ConnectionManager.GetServer();

		// We haven't received our ping response yet, so increment our timer
		SecondsSinceLastServerPong += DeltaTime;
		if (SecondsSinceLastServerPong >= XmppServer.PingTimeout)
		{
			++MissedPongs;
			UE_LOGF(LogXmpp, Warning, "XMPP Ping Timeout of %0.2f seconds reached, of %d/%d allowed", XmppServer.PingTimeout, MissedPongs, XmppServer.MaxPingRetries);
			if (MissedPongs > XmppServer.MaxPingRetries)
			{
				// We have detected a timeout, so disconnect from xmpp
				UE_LOGF(LogXmpp, Warning, "Max missed XMPP pings detected (%d/%d)", MissedPongs, XmppServer.MaxPingRetries);
				ResetPongTimer();
				ConnectionManager.Logout();
			}
			else
			{
				// We have not reached our maximum amount of missed ping responses
				bWaitingForPong = false;
				SecondsSinceLastServerPong = 0.0f;
			}
		}
	}
}

void FXmppPingStrophe::CleanupMessages()
{
	while (!IncomingPings.IsEmpty())
	{
		FXmppPingReceivedInfo ReceivedPing;
		IncomingPings.Dequeue(ReceivedPing);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}
}

#undef TickRequesterId

#endif
