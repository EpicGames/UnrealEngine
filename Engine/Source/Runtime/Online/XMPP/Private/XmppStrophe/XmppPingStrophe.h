// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "XmppConnection.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "HAL/ThreadSafeBool.h"

class FXmppConnectionStrophe;
class FStropheStanza;

#if WITH_XMPP_STROPHE

struct FXmppPingReceivedInfo
{
public:
	FXmppUserJid FromJid;
	FString PingId;

public:
	FXmppPingReceivedInfo()
	{
	}

	FXmppPingReceivedInfo(FXmppUserJid&& InFromJid, FString&& InPingId)
		: FromJid(MoveTemp(InFromJid))
		, PingId(MoveTemp(InPingId))
	{
	}
};

class FXmppPingStrophe
	: public FTSTickerObjectBase
{
public:
	// FXmppPingStrophe
	FXmppPingStrophe(FXmppConnectionStrophe& InConnectionManager);
	virtual ~FXmppPingStrophe();

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

	/** Clear any caches on disconnect */
	void OnDisconnect();
	void OnReconnect();

	/** Determine if an incoming stanza is a ping stanza */
	bool ReceiveStanza(const FStropheStanza& IncomingStanza);

	/** Reset the ping timer (usually due to incoming/outgoing traffic) */
	void ResetPingTimer();

	/** Reset the pong timer (usually due to incoming traffic) */
	void ResetPongTimer();

protected:
	/** Process parsing a ping we have received and queue it to be replied to */
	bool HandlePingStanza(const FStropheStanza& PingStanza);

	/** Queue a reply to a specific ping we received */
	void ReplyToPing(FXmppPingReceivedInfo&& ReceivedPing);

	/** Send a ping from the client to the server */
	void SendClientPing();

	/** Check the current status of our Xmpp ping responses; may lead to disconnection requests */
	void CheckXmppPongTimeout(float DeltaTime);

	/** Remove pending messages and engine KeepAwake calls */
	void CleanupMessages();

private:
	/** Connection manager controls sending data to XMPP thread */
	FXmppConnectionStrophe& ConnectionManager;

	/** Queued pings we have received but haven't processed */
	TQueue<FXmppPingReceivedInfo> IncomingPings;

	/** The time since we last sent the server a ping */
	float TimeSinceLastClientPing;

	/** Are we waiting for a pong? */
	bool bWaitingForPong;

	/** The amount of seconds since the server last responded to our ping request */
	float SecondsSinceLastServerPong;

	/** Have we received a pong since the last tick? */
	FThreadSafeBool bHasReceievedServerPong;

	/** Number of missed pongs */
	int32 MissedPongs;
};

#endif
