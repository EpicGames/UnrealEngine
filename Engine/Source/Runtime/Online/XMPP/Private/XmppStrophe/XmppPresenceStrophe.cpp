// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/XmppPresenceStrophe.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/StropheStanzaConstants.h"
#include "Misc/EmbeddedCommunication.h"
#include "Containers/BackgroundableTicker.h"
#include "Stats/Stats.h"

#if WITH_XMPP_STROPHE

#define TickRequesterId FName("StrophePresence")

FXmppPresenceStrophe::FXmppPresenceStrophe(FXmppConnectionStrophe& InConnectionManager)
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
	, ConnectionManager(InConnectionManager)
{

}

FXmppPresenceStrophe::~FXmppPresenceStrophe()
{
	CleanupMessages();
}

void FXmppPresenceStrophe::OnDisconnect()
{
	if (RosterMembers.Num() > 0)
	{
		RosterMembers.Empty();
	}

	CleanupMessages();
}

void FXmppPresenceStrophe::OnReconnect()
{
	// Triggered by login request when already connected
	// re-broadcast all cached presence entries
	for (const auto& Pair : RosterMembers)
	{
		const TSharedRef<FXmppUserPresence>& Presence = Pair.Value;
		OnXmppPresenceReceivedDelegate.Broadcast(ConnectionManager.AsShared(), Presence->UserJid, Presence);
	}
}

bool FXmppPresenceStrophe::ReceiveStanza(const FStropheStanza& IncomingStanza)
{
	if (IncomingStanza.GetName() != Strophe::SN_PRESENCE)
	{
		return false;
	}

	FXmppUserJid FromJid = IncomingStanza.GetFrom();

	bool bIsMucPresence = FromJid.Domain.Equals(ConnectionManager.GetMucDomain(), ESearchCase::CaseSensitive);
	if (bIsMucPresence)
	{
		// Our MultiUserChat interface will handle this stanza
		return false;
	}
	else if (FromJid.Resource.IsEmpty())
	{
		// Skip user presence updates that are missing a resource
		return true;
	}

	// We build this into a MucPresence and slice it down to a UserPresence if it's a user presence
	FXmppMucPresence Presence;

	Presence.UserJid = MoveTemp(FromJid);

	if (IncomingStanza.GetType() == Strophe::ST_UNAVAILABLE)
	{
		Presence.bIsAvailable = false;
	}
	else
	{
		Presence.bIsAvailable = true;

		TOptional<const FStropheStanza> StatusTextStanza = IncomingStanza.GetChildStropheStanza(Strophe::SN_STATUS);
		if (StatusTextStanza.IsSet())
		{
			Presence.StatusStr = StatusTextStanza->GetText();
		}

		Presence.Status = EXmppPresenceStatus::Online;

		TOptional<const FStropheStanza> StatusEnumStanza = IncomingStanza.GetChildStropheStanza(Strophe::SN_SHOW);
		if (StatusEnumStanza.IsSet())
		{
			FString StatusEnum(StatusEnumStanza->GetText());
			if (StatusEnum == TEXT("away"))
			{
				Presence.Status = EXmppPresenceStatus::Away;
			}
			else if (StatusEnum == TEXT("chat"))
			{
				Presence.Status = EXmppPresenceStatus::Chat;
			}
			else if (StatusEnum == TEXT("dnd"))
			{
				Presence.Status = EXmppPresenceStatus::DoNotDisturb;
			}
			else if (StatusEnum == TEXT("xa"))
			{
				Presence.Status = EXmppPresenceStatus::ExtendedAway;
			}
		}

		TOptional<const FStropheStanza> TimestampStanza = IncomingStanza.GetChildStropheStanza(Strophe::SN_DELAY);
		if (TimestampStanza.IsSet())
		{
			FDateTime::ParseIso8601(*TimestampStanza->GetAttribute(Strophe::SA_STAMP), Presence.SentTime);
		}

		Presence.ReceivedTime = FDateTime::UtcNow();

		FString UnusedPlatformUserId;
		Presence.UserJid.ParseResource(Presence.AppId, Presence.Platform, UnusedPlatformUserId);
	}

	FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
	return IncomingPresenceUpdates.Enqueue(MakeUnique<FXmppUserPresence>(MoveTemp(Presence)));
}

bool FXmppPresenceStrophe::UpdatePresence(const FXmppUserPresence& NewPresence)
{
	FStropheStanza PresenceStanza(ConnectionManager, Strophe::SN_PRESENCE);

	if (NewPresence.bIsAvailable)
	{
		// Update Availability
		{
			FStropheStanza AvailabilityStanza(ConnectionManager, Strophe::SN_SHOW);
			switch (NewPresence.Status)
			{
			case EXmppPresenceStatus::Away:
				AvailabilityStanza.SetText(TEXT("away"));
				PresenceStanza.AddChild(AvailabilityStanza);
				break;
			case EXmppPresenceStatus::Chat:
				AvailabilityStanza.SetText(TEXT("chat"));
				PresenceStanza.AddChild(AvailabilityStanza);
				break;
			case EXmppPresenceStatus::DoNotDisturb:
				AvailabilityStanza.SetText(TEXT("dnd"));
				PresenceStanza.AddChild(AvailabilityStanza);
				break;
			case EXmppPresenceStatus::ExtendedAway:
				AvailabilityStanza.SetText(TEXT("xa"));
				PresenceStanza.AddChild(AvailabilityStanza);
				break;
			}
		}

		// Update Status String
		if (!NewPresence.StatusStr.IsEmpty())
		{
			FStropheStanza StatusStanza(ConnectionManager, Strophe::SN_STATUS);
			StatusStanza.SetText(NewPresence.StatusStr);
			PresenceStanza.AddChild(StatusStanza);
		}

		// Update Sent Time
		{
			FStropheStanza DelayStanza(ConnectionManager, Strophe::SN_DELAY);
			DelayStanza.SetNamespace(Strophe::SNS_DELAY);
			DelayStanza.SetAttribute(Strophe::SA_STAMP, FDateTime::UtcNow().ToIso8601());
			PresenceStanza.AddChild(DelayStanza);
		}
	}
	else
	{
		PresenceStanza.SetType(Strophe::ST_UNAVAILABLE);
	}

	const bool bSuccess = ConnectionManager.SendStanza(MoveTemp(PresenceStanza));
	if (bSuccess)
	{
		CachedPresence = NewPresence;
	}

	return bSuccess;
}

const FXmppUserPresence& FXmppPresenceStrophe::GetPresence() const
{
	return CachedPresence;
}

bool FXmppPresenceStrophe::QueryPresence(const FString& UserId)
{
	// Not supported by tigase
	return false;
}

TArray<TSharedPtr<FXmppUserPresence>> FXmppPresenceStrophe::GetRosterPresence(const FString& UserId)
{
	TArray<TSharedPtr<FXmppUserPresence>> OutRoster;
	for (const TMap<FString, TSharedRef<FXmppUserPresence>>::ElementType& Pair : RosterMembers)
	{
		if (Pair.Value->UserJid.Id == UserId)
		{
			OutRoster.Emplace(Pair.Value);
		}
	}

	return OutRoster;
}

void FXmppPresenceStrophe::GetRosterMembers(TArray<FXmppUserJid>& Members)
{
	check(IsInGameThread());
	Members.Empty(RosterMembers.Num());
	for (const TMap<FString, TSharedRef<FXmppUserPresence>>::ElementType& Pair : RosterMembers)
	{
		Members.Emplace(Pair.Value->UserJid);
	}
}

bool FXmppPresenceStrophe::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FXmppPresenceStrophe_Tick);

	while (!IncomingPresenceUpdates.IsEmpty())
	{
		TUniquePtr<FXmppUserPresence> PresencePtr;
		if (IncomingPresenceUpdates.Dequeue(PresencePtr))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			check(PresencePtr.IsValid());
			OnPresenceUpdate(MoveTemp(PresencePtr));
		}
	}

	return true;
}

void FXmppPresenceStrophe::OnPresenceUpdate(TUniquePtr<FXmppUserPresence>&& NewPresencePtr)
{
	TSharedRef<FXmppUserPresence> Presence = MakeShareable(NewPresencePtr.Release());

	RosterMembers.Emplace(Presence->UserJid.GetFullPath(), Presence);
	OnXmppPresenceReceivedDelegate.Broadcast(ConnectionManager.AsShared(), Presence->UserJid, Presence);
}

void FXmppPresenceStrophe::CleanupMessages()
{
	while (!IncomingPresenceUpdates.IsEmpty())
	{
		TUniquePtr<FXmppUserPresence> PresencePtr;
		IncomingPresenceUpdates.Dequeue(PresencePtr);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}
}

#undef TickRequesterId

#endif