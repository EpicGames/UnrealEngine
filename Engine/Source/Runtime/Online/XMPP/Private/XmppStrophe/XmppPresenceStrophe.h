// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppPresence.h"

#include "Containers/Ticker.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"

#if WITH_XMPP_STROPHE

class FXmppConnectionStrophe;
class FStropheStanza;

class FXmppPresenceStrophe
	: public IXmppPresence
	, public FTSTickerObjectBase
{
public:
	// FXmppPresenceStrophe
	FXmppPresenceStrophe(FXmppConnectionStrophe& InConnectionManager);
	virtual ~FXmppPresenceStrophe();

	// XMPP Thread
	bool ReceiveStanza(const FStropheStanza& IncomingStanza);

	// Game Thread
	void OnDisconnect();
	void OnReconnect();

	// IXmppPresence
	virtual bool UpdatePresence(const FXmppUserPresence& NewPresence) override;
	virtual const FXmppUserPresence& GetPresence() const override;
	virtual bool QueryPresence(const FString& UserId) override;
	virtual TArray<TSharedPtr<FXmppUserPresence>> GetRosterPresence(const FString& UserId) override;
	virtual void GetRosterMembers(TArray<FXmppUserJid>& Members) override;
	virtual FOnXmppPresenceReceived& OnReceivePresence() override { return OnXmppPresenceReceivedDelegate; }

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

protected:
	void OnPresenceUpdate(TUniquePtr<FXmppUserPresence>&& UpdatedPresence);

	/** Remove pending messages and engine KeepAwake calls */
	void CleanupMessages();

protected:
	/** Connection manager controls sending data to XMPP thread */
	FXmppConnectionStrophe& ConnectionManager;

	/** Local user's Presence information*/
	FXmppUserPresence CachedPresence;

	/** Presence information for players on our roster */
	TMap<FString, TSharedRef<FXmppUserPresence>> RosterMembers;

	/** Queue of stanzas needing to be processed on the game thread */
	TQueue<TUniquePtr<FXmppUserPresence>> IncomingPresenceUpdates;

	/** Delegate to signal presence information has been received for a user */
	FOnXmppPresenceReceived OnXmppPresenceReceivedDelegate;
};

#endif
