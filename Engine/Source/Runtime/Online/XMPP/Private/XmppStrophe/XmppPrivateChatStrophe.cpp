// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/XmppPrivateChatStrophe.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppStrophe/XmppPresenceStrophe.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppStrophe/StropheStanzaConstants.h"
#include "XmppLog.h"
#include "Misc/EmbeddedCommunication.h"
#include "Containers/BackgroundableTicker.h"
#include "Stats/Stats.h"

#if WITH_XMPP_STROPHE

#define TickRequesterId FName("StrophePrivateChat")

FXmppPrivateChatStrophe::FXmppPrivateChatStrophe(FXmppConnectionStrophe& InConnectionManager)
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
	, ConnectionManager(InConnectionManager)
{
}

FXmppPrivateChatStrophe::~FXmppPrivateChatStrophe()
{
	CleanupMessages();
}

void FXmppPrivateChatStrophe::OnDisconnect()
{
	CleanupMessages();
}

void FXmppPrivateChatStrophe::OnReconnect()
{

}

bool FXmppPrivateChatStrophe::ReceiveStanza(const FStropheStanza& IncomingStanza)
{
	if (IncomingStanza.GetName() != Strophe::SN_MESSAGE || // Must be a message
		IncomingStanza.GetType() != Strophe::ST_CHAT || // Filter Non-Chat messages
		IncomingStanza.GetFrom().Domain.StartsWith(TEXT("muc"), ESearchCase::CaseSensitive)) // Filter MUC messages
	{
		return false;
	}

	TOptional<FString> BodyText = IncomingStanza.GetBodyText();
	if (!BodyText.IsSet())
	{
		// Bad data, no body
		return true;
	}

	FXmppChatMessage ChatMessage;
	ChatMessage.ToJid = IncomingStanza.GetTo();
	ChatMessage.FromJid = IncomingStanza.GetFrom();
	ChatMessage.Body = MoveTemp(BodyText.GetValue());

	// Parse Timezone
	TOptional<const FStropheStanza> StanzaDelay = IncomingStanza.GetChildStropheStanza(Strophe::SN_DELAY);
	if (StanzaDelay.IsSet())
	{
		if (StanzaDelay->HasAttribute(Strophe::SA_STAMP))
		{
			FString Timestamp = StanzaDelay->GetAttribute(Strophe::SA_STAMP);
			FDateTime::ParseIso8601(*Timestamp, ChatMessage.Timestamp);
		}
	}

	if (ChatMessage.Timestamp == 0)
	{
		ChatMessage.Timestamp = FDateTime::UtcNow();
	}

	FEmbeddedCommunication::KeepAwake(TickRequesterId, false);
	IncomingChatMessages.Enqueue(MakeUnique<FXmppChatMessage>(MoveTemp(ChatMessage)));
	return true;
}

bool FXmppPrivateChatStrophe::SendChat(const FXmppUserJid& RecipientId, const FString& Message)
{
	if (ConnectionManager.GetLoginStatus() != EXmppLoginStatus::LoggedIn)
	{
		return false;
	}

	if (!RecipientId.IsValid())
	{
		UE_LOGF(LogXmpp, Warning, "Unable to send chat message. Invalid jid: %ls", *RecipientId.ToDebugString());
		return false;
	}

	FStropheStanza ChatStanza(ConnectionManager, Strophe::SN_MESSAGE);
	{
		ChatStanza.SetType(Strophe::ST_CHAT);
		ChatStanza.SetTo(RecipientId);
		ChatStanza.AddBodyWithText(Message);
	}

	return ConnectionManager.SendStanza(MoveTemp(ChatStanza));
}

bool FXmppPrivateChatStrophe::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FXmppPrivateChatStrophe_Tick);

	while (!IncomingChatMessages.IsEmpty())
	{
		TUniquePtr<FXmppChatMessage> ChatMessage;
		if (IncomingChatMessages.Dequeue(ChatMessage))
		{
			FEmbeddedCommunication::AllowSleep(TickRequesterId);
			check(ChatMessage);
			OnChatReceived(MoveTemp(ChatMessage));
		}
	}

	return true;
}

void FXmppPrivateChatStrophe::OnChatReceived(TUniquePtr<FXmppChatMessage>&& Chat)
{
	TSharedRef<FXmppChatMessage> ChatRef = MakeShareable(Chat.Release());

	// Potentially filter out non-friends/non-admins
	if (ConnectionManager.GetServer().bPrivateChatFriendsOnly && ConnectionManager.Presence().IsValid())
	{
		if (ChatRef->FromJid.Id != TEXT("xmpp-admin"))
		{
			TArray<FXmppUserJid> RosterMembers;
			ConnectionManager.Presence()->GetRosterMembers(RosterMembers);
			if (!RosterMembers.Contains(ChatRef->FromJid))
			{
				return;
			}
		}
	}

	OnChatReceivedDelegate.Broadcast(ConnectionManager.AsShared(), ChatRef->FromJid, ChatRef);
}

void FXmppPrivateChatStrophe::CleanupMessages()
{
	while (!IncomingChatMessages.IsEmpty())
	{
		TUniquePtr<FXmppChatMessage> ChatMessage;
		IncomingChatMessages.Dequeue(ChatMessage);
		FEmbeddedCommunication::AllowSleep(TickRequesterId);
	}
}

#undef TickRequesterId

#endif
