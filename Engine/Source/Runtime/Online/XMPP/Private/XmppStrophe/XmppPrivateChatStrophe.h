// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppChat.h"

#include "Containers/Ticker.h"
#include "Containers/Queue.h"

#if WITH_XMPP_STROPHE

class FXmppChatMessage;
class FXmppConnectionStrophe;
class FStropheStanza;

class FXmppPrivateChatStrophe
	: public IXmppChat
	, public FTSTickerObjectBase
{
	friend FXmppConnectionStrophe;

public:
	// FXmppPrivateChatStrophe
	FXmppPrivateChatStrophe(FXmppConnectionStrophe& InConnectionManager);
	virtual ~FXmppPrivateChatStrophe();

	// XMPP Thread
	bool ReceiveStanza(const FStropheStanza& IncomingStanza);

	// Game Thread
	void OnDisconnect();
	void OnReconnect();

	// IXmppChat
	virtual bool SendChat(const FXmppUserJid& RecipientId, const FString& Message) override;
	virtual FOnXmppChatReceived& OnReceiveChat() override { return OnChatReceivedDelegate; }

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

protected:
	void OnChatReceived(TUniquePtr<FXmppChatMessage>&& Chat);

	/** Remove pending messages and engine KeepAwake calls */
	void CleanupMessages();

protected:
	/** Connection manager controls sending data to XMPP thread */
	FXmppConnectionStrophe& ConnectionManager;

	/**
	 * Queue of Chat Messages needing to be consumed. These are Queued
	 * on the XMPP thread, and Dequeued on the Game thread
	 */
	TQueue<TUniquePtr<FXmppChatMessage>> IncomingChatMessages;

	/** Delegate for game to listen to Chat Messages */
	FOnXmppChatReceived OnChatReceivedDelegate;
};

#endif
