// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppPubSubStrophe.h"
#include "XmppConnectionStrophe.h"
#include "Containers/BackgroundableTicker.h"
#include "Stats/Stats.h"

#if WITH_XMPP_STROPHE

FXmppPubSubStrophe::FXmppPubSubStrophe(FXmppConnectionStrophe& InConnectionManager)
	: FTSTickerObjectBase(0.0f, FTSBackgroundableTicker::GetCoreTicker())
	, ConnectionManager(InConnectionManager)
{

}

void FXmppPubSubStrophe::OnDisconnect()
{

}

void FXmppPubSubStrophe::OnReconnect()
{

}

bool FXmppPubSubStrophe::ReceiveStanza(const FStropheStanza& IncomingStanza)
{
	return false;
}

bool FXmppPubSubStrophe::CreateNode(const FXmppPubSubId& NodeId, const FXmppPubSubConfig& NodeConfig)
{
	return false;
}

bool FXmppPubSubStrophe::ConfigureNode(const FXmppPubSubId& NodeId, const FXmppPubSubConfig& NodeConfig)
{
	return false;
}

bool FXmppPubSubStrophe::DestroyNode(const FXmppPubSubId& NodeId)
{
	return false;
}

bool FXmppPubSubStrophe::QueryNode(const FXmppPubSubId& NodeId)
{
	return false;
}

bool FXmppPubSubStrophe::QuerySubscriptions()
{
	return false;
}

bool FXmppPubSubStrophe::Subscribe(const FXmppPubSubId& NodeId)
{
	return false;
}

bool FXmppPubSubStrophe::Unsubscribe(const FXmppPubSubId& NodeId)
{
	return false;
}

bool FXmppPubSubStrophe::PublishMessage(const FXmppPubSubId& NodeId, const FXmppPubSubMessage& Message)
{
	return false;
}

TArray<FXmppPubSubId> FXmppPubSubStrophe::GetOwnedNodes() const
{
	return TArray<FXmppPubSubId>();
}

TArray<FXmppPubSubId> FXmppPubSubStrophe::GetSubscribedNodes() const
{
	return TArray<FXmppPubSubId>();
}

TSharedPtr<FXmppPubSubNode> FXmppPubSubStrophe::GetNodeInfo(const FXmppPubSubId& NodeId) const
{
	return nullptr;
}

bool FXmppPubSubStrophe::GetLastMessages(const FXmppPubSubId& NodeId, int32 NumMessages, TArray<TSharedRef<FXmppPubSubMessage>>& OutMessages) const
{
	OutMessages.Empty();
	return false;
}

bool FXmppPubSubStrophe::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FXmppPubSubStrophe_Tick);
	return true;
}

#endif
