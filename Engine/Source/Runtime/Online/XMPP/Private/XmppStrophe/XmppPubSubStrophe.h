// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppPubSub.h"

#include "Containers/Ticker.h"

#if WITH_XMPP_STROPHE

class FXmppConnectionStrophe;
class FStropheStanza;

class FXmppPubSubStrophe
	: public IXmppPubSub
	, public FTSTickerObjectBase
{
public:
	// FXmppPubSubStrophe
	FXmppPubSubStrophe(class FXmppConnectionStrophe& InConnectionManager);
	virtual ~FXmppPubSubStrophe() = default;

	// XMPP Thread
	bool ReceiveStanza(const FStropheStanza& IncomingStanza);

	// Game Thread
	void OnDisconnect();
	void OnReconnect();

	// IXmppPubSub
	virtual bool CreateNode(const FXmppPubSubId& NodeId, const FXmppPubSubConfig& NodeConfig) override;
	virtual bool ConfigureNode(const FXmppPubSubId& NodeId, const FXmppPubSubConfig& NodeConfig) override;
	virtual bool DestroyNode(const FXmppPubSubId& NodeId) override;
	virtual bool QueryNode(const FXmppPubSubId& NodeId) override;
	virtual bool QuerySubscriptions() override;
	virtual bool Subscribe(const FXmppPubSubId& NodeId) override;
	virtual bool Unsubscribe(const FXmppPubSubId& NodeId) override;
	virtual bool PublishMessage(const FXmppPubSubId& NodeId, const FXmppPubSubMessage& Message) override;
	virtual TArray<FXmppPubSubId> GetOwnedNodes() const override;
	virtual TArray<FXmppPubSubId> GetSubscribedNodes() const override;
	virtual TSharedPtr<FXmppPubSubNode> GetNodeInfo(const FXmppPubSubId& NodeId) const override;
	virtual bool GetLastMessages(const FXmppPubSubId& NodeId, int32 NumMessages, TArray<TSharedRef<FXmppPubSubMessage>>& OutMessages) const override;

	virtual FOnXmppPubSubCreateNodeComplete& OnCreateNodeComplete() override { return OnXmppPubSubCreateNodeCompleteDelegate; }
	virtual FOnXmppPubSubConfigureNodeComplete& OnConfigureNodeComplete() override { return OnXmppPubSubConfigureNodeCompleteDelegate; }
	virtual FOnXmppPubSubDestroyNodeComplete& OnDestroyNodeComplete() override { return OnXmppPubSubDestroyNodeCompleteDelegate; }
	virtual FOnXmppPubSubQueryNodeComplete& OnQueryNodeComplete() override { return OnXmppPubSubQueryNodeCompleteDelegate; }
	virtual FOnXmppPubSubQuerySubscriptionsComplete& OnQuerySubscriptionsComplete() override { return OnXmppPubSubQuerySubscriptionsCompleteDelegate; }
	virtual FOnXmppPubSubSubscribed& OnSubscribed() override { return OnXmppPubSubSubscribedDelegate; }
	virtual FOnXmppPubSubUnsubscribed& OnUnsubscribed() override { return OnXmppPubSubUnsubscribedDelegate; }
	virtual FOnXmppPubSubMessageReceived& OnMessageReceived() override { return OnXmppPubSubMessageReceivedDelegate; }

	// FTSTickerObjectBase
	virtual bool Tick(float DeltaTime) override;

protected:
	/** Connection manager controls sending data to XMPP thread */
	FXmppConnectionStrophe& ConnectionManager;

	FOnXmppPubSubCreateNodeComplete OnXmppPubSubCreateNodeCompleteDelegate;
	FOnXmppPubSubConfigureNodeComplete OnXmppPubSubConfigureNodeCompleteDelegate;
	FOnXmppPubSubDestroyNodeComplete OnXmppPubSubDestroyNodeCompleteDelegate;
	FOnXmppPubSubQueryNodeComplete OnXmppPubSubQueryNodeCompleteDelegate;
	FOnXmppPubSubQuerySubscriptionsComplete OnXmppPubSubQuerySubscriptionsCompleteDelegate;
	FOnXmppPubSubSubscribed OnXmppPubSubSubscribedDelegate;
	FOnXmppPubSubUnsubscribed OnXmppPubSubUnsubscribedDelegate;
	FOnXmppPubSubMessageReceived OnXmppPubSubMessageReceivedDelegate;
};

#endif
