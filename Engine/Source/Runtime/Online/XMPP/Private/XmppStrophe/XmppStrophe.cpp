// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/XmppStrophe.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppStrophe/StropheContext.h"

#if WITH_XMPP_STROPHE

THIRD_PARTY_INCLUDES_START
#include "strophe.h"
THIRD_PARTY_INCLUDES_END

void FXmppStrophe::Init()
{
	// Init libstrophe
	xmpp_initialize();
}

void FXmppStrophe::Cleanup()
{
	// Cleanup libstrophe
	xmpp_shutdown();
}

TSharedRef<IXmppConnection> FXmppStrophe::CreateConnection()
{
	return MakeShareable(new FXmppConnectionStrophe());
}

FString FXmppStrophe::JidToString(const FXmppUserJid& UserJid)
{
	return UserJid.GetFullPath();
}

FXmppUserJid FXmppStrophe::JidFromString(const FString& JidString)
{
	return FXmppUserJid::FromFullJid(JidString);
}

FXmppUserJid FXmppStrophe::JidFromStropheString(const char* StropheJidString)
{
	return JidFromString(FString(UTF8_TO_TCHAR(StropheJidString)));
}

#endif
