// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_XMPP_STROPHE

class IXmppConnection;
class FXmppUserJid;

/**
 * Entry for access to Xmpp connections implemented via libstrophe
 */
class FXmppStrophe
{
public:
	// FXmppStrophe
	static void Init();
	static void Cleanup();
	static TSharedRef<IXmppConnection> CreateConnection();

	static FString JidToString(const FXmppUserJid& UserJid);

	static FXmppUserJid JidFromString(const FString& JidString);
	static FXmppUserJid JidFromStropheString(const char* StropheJidPtr);
};

#endif
