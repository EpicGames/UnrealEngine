// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppStrophe/XmppStropheThread.h"
#include "XmppStrophe/XmppConnectionStrophe.h"
#include "XmppStrophe/StropheContext.h"
#include "XmppStrophe/StropheConnection.h"
#include "XmppStrophe/StropheStanza.h"
#include "XmppLog.h"

#include "HAL/RunnableThread.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_XMPP_STROPHE

FXmppStropheThread::FXmppStropheThread(FXmppConnectionStrophe& InConnectionManager, const FXmppUserJid& InUser, const FString& InAuth, const FXmppServer& InServerConfiguration)
	: ConnectionManager(InConnectionManager)
	, StropheConnection(ConnectionManager.StropheContext)
	, ServerConfiguration(InServerConfiguration)
{
	// Setup connection
	StropheConnection.SetUserId(InUser);
	StropheConnection.SetPassword(InAuth);
	StropheConnection.SetKeepAlive(ServerConfiguration.PingTimeout, ServerConfiguration.PingInterval);

	bConnectRequest = true;

	static int32 ThreadInstanceIdx = 0;
	int32 StackSize = 128 * 1024;
	GConfig->GetInt(TEXT("XMPP"), TEXT("ThreadStackSize"), StackSize, GEngineIni);
	ThreadPtr.Reset(FRunnableThread::Create(this, *FString::Printf(TEXT("XmppConnectionThread_%d"), ThreadInstanceIdx++), StackSize, TPri_Normal));
}

FXmppStropheThread::~FXmppStropheThread()
{
	if (ThreadPtr.IsValid())
	{
		// Stop ticking our thread before we exit (and kill the underlying thread)
		ThreadPtr->Kill(true);
	}
}

bool FXmppStropheThread::SendStanza(FStropheStanza&& Stanza)
{
	return StanzaSendQueue.Enqueue(MakeUnique<FStropheStanza>(MoveTemp(Stanza)));
}

bool FXmppStropheThread::Init()
{
	StropheConnection.RegisterStropheHandler(ConnectionManager);
	return true;
}

uint32 FXmppStropheThread::Run()
{
	while (!bExitRequested)
	{
		if (bConnectRequest)
		{
			bConnectRequest = false;

			if (StropheConnection.GetConnectionState() == EStropheConnectionState::Disconnected)
			{
				ConnectionManager.QueueNewLoginStatus(EXmppLoginStatus::ProcessingLogin);
				if (!StropheConnection.Connect(ServerConfiguration.ServerAddr, ServerConfiguration.ServerPort, ConnectionManager))
				{
					ConnectionManager.QueueNewLoginStatus(EXmppLoginStatus::LoggedOut);
				}
			}
		}
		else if (bDisconnectRequest)
		{
			bDisconnectRequest = false;

			if (StropheConnection.GetConnectionState() != EStropheConnectionState::Disconnected)
			{
				ConnectionManager.QueueNewLoginStatus(EXmppLoginStatus::ProcessingLogout);
				StropheConnection.Disconnect();
				ConnectionManager.QueueNewLoginStatus(EXmppLoginStatus::LoggedOut);
			}

			bExitRequested = true;
		}

		SendQueuedStanza();

		StropheConnection.XmppThreadTick();
	}

	return 0;
}

void FXmppStropheThread::Stop()
{
	bDisconnectRequest = true;
}

void FXmppStropheThread::Exit()
{
	if (StropheConnection.GetConnectionState() == EStropheConnectionState::Connected)
	{
		StropheConnection.Disconnect();
	}
	StropheConnection.RemoveStropheHandler();
}

void FXmppStropheThread::SendQueuedStanza()
{
	// Send all our queued stanzas
	while (!StanzaSendQueue.IsEmpty() && StropheConnection.GetConnectionState() == EStropheConnectionState::Connected)
	{
		TUniquePtr<FStropheStanza> StanzaPtr;
		if (StanzaSendQueue.Dequeue(StanzaPtr))
		{
			// Add a Correlation Id if we haven't set one already
			static const FString CorrelationId("corr-id");
			if (!StanzaPtr->HasAttribute(CorrelationId))
			{
				StanzaPtr->SetAttribute(CorrelationId, FGuid::NewGuid().ToString());
			}

			if (!StropheConnection.SendStanza(*StanzaPtr))
			{
				UE_LOGF(LogXmpp, Warning, "Unable to send stanza %ls", *StanzaPtr->GetAttribute(CorrelationId));
			}
		}
	}
}

#endif
