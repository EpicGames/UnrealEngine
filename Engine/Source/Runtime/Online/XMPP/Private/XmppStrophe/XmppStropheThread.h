// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppConnection.h"
#include "XmppStrophe/StropheConnection.h"

#include "HAL/ThreadSafeBool.h"
#include "HAL/Runnable.h"
#include "Containers/Queue.h"

#if WITH_XMPP_STROPHE

class FRunnableThread;
class FXmppConnectionStrophe;
class FStropheStanza;

class FXmppStropheThread
	: public FRunnable
{
public:
	// FXmppStropheThread
	explicit FXmppStropheThread(FXmppConnectionStrophe& InConnectionManager, const FXmppUserJid& InUser, const FString& InAuth, const FXmppServer& InServerConfiguration);
	virtual ~FXmppStropheThread();

	bool SendStanza(FStropheStanza&& Stanza);

	// FRunnable
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

protected:
	void SendQueuedStanza();

protected:
	/** Connection Manager */
	FXmppConnectionStrophe& ConnectionManager;

	/** Strophe Connection */
	FStropheConnection StropheConnection;

	/** Server Configuration to use for this connection */
	const FXmppServer& ServerConfiguration;

	/** Our runnable thread to be cleaned up before we're destroyed */
	TUniquePtr<FRunnableThread> ThreadPtr;

	/** Queue of stanzas needing to be sent */
	TQueue<TUniquePtr<FStropheStanza>> StanzaSendQueue;

	/** signal request for connection */
	FThreadSafeBool bConnectRequest;
	/** signal request for disconnect */
	FThreadSafeBool bDisconnectRequest;
	/** Thread-Safe way to trigger a thread exit */
	FThreadSafeBool bExitRequested;
};

#endif
