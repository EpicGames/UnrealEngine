// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmppTests.h"
#include "XmppPresence.h"
#include "XmppModule.h"
#include "XmppLog.h"

FXmppTest::FXmppTest()
	: bRunBasicPresenceTest(true)
	, bRunPubSubTest(false)
	, bRunChatTest(false)
{}

void FXmppTest::Test(const FString& UserId, const FString& Password, const FXmppServer& XmppServer)
{
	XmppConnection = FXmppModule::Get().CreateConnection(UserId);
	XmppConnection->SetServer(XmppServer);

	OnLoginCompleteHandle = XmppConnection->OnLoginComplete().AddRaw(this, &FXmppTest::OnLoginComplete);
	XmppConnection->Login(UserId, Password);
}

void FXmppTest::StartNextTest()
{
	if (bRunBasicPresenceTest)
	{	
		FXmppUserPresence Presence;
		Presence.bIsAvailable = true;
		Presence.Status = EXmppPresenceStatus::DoNotDisturb;
		Presence.StatusStr = (TEXT("Test rich presence status"));

		XmppConnection->Presence()->UpdatePresence(Presence); 

		bRunBasicPresenceTest = false;
		StartNextTest();
	}
	else if (bRunPubSubTest)
	{
		//@todo samz - not implemented
		bRunPubSubTest = false;
	}
	else if (bRunChatTest)
	{
		//@todo samz - not implemented
		bRunChatTest = false;
	}
	else
	{
		FinishTest();
	}
}

void FXmppTest::FinishTest()
{
	bool bDone = false;
	if (XmppConnection.IsValid())
	{
		if (XmppConnection->GetLoginStatus() == EXmppLoginStatus::LoggedIn)
		{
			OnLogoutCompleteHandle = XmppConnection->OnLogoutComplete().AddRaw(this, &FXmppTest::OnLogoutComplete);
			XmppConnection->Logout();
		}
		else
		{
			XmppConnection->OnLoginComplete ().Remove(OnLoginCompleteHandle);
			XmppConnection->OnLogoutComplete().Remove(OnLogoutCompleteHandle);
			FXmppModule::Get().RemoveConnection(XmppConnection.ToSharedRef());
			bDone = true;
		}
	}
	if (bDone)
	{	
		delete this;
	}
}

void FXmppTest::OnLoginComplete(const FXmppUserJid& UserJid, bool bWasSuccess, const FString& Error)
{
	UE_LOGF(LogXmpp, Log, "FXmppTest::OnLoginComplete UserJid=%ls Success=%ls Error=%ls",
		*UserJid.GetFullPath(), bWasSuccess ? TEXT("true") : TEXT("false"), *Error);

	if (bWasSuccess)
	{
		StartNextTest();
	}
	else
	{
		FinishTest();
	}
}

void FXmppTest::OnLogoutComplete(const FXmppUserJid& UserJid, bool bWasSuccess, const FString& Error)
{
	UE_LOGF(LogXmpp, Log, "FXmppTest::OnLogoutComplete UserJid=%ls Success=%ls Error=%ls",
		*UserJid.GetFullPath(), bWasSuccess ? TEXT("true") : TEXT("false"), *Error);

	FinishTest();
}
