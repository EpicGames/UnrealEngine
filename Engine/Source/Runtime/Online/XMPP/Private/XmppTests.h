// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class Error;
class FXmppServer;
class FXmppUserJid;
class IXmppConnection;

/**
 * Create a new Xmpp connection 
 * And run some basic tests for login/presence/pubsub/chat
 */
class FXmppTest
{
public:

	/**
	 * Constructor
	 */
	FXmppTest();

	/**
	* Destructor
	*/
	virtual ~FXmppTest() = default;

	/**
	* Kicks off all of the testing process
	*/
	void Test(const FString& UserId, const FString& Password, const FXmppServer& XmppServer);

private:

	/**
	* Step through the various tests that should be run and initiate the next one
	*/
	void StartNextTest();

	/**
	* Finish/cleanup the tests
	*/
	void FinishTest();

	/** tests to run */
	bool bRunBasicPresenceTest;
	bool bRunPubSubTest;
	bool bRunChatTest;

	TSharedPtr<IXmppConnection> XmppConnection;

	/** completion delegates */
	void OnLoginComplete(const FXmppUserJid& UserJid, bool bWasSuccess, const FString& Error);
	void OnLogoutComplete(const FXmppUserJid& UserJid, bool bWasSuccess, const FString& Error);

	/** Handles for the registered delegates */
	FDelegateHandle OnLoginCompleteHandle;
	FDelegateHandle OnLogoutCompleteHandle;
};

