// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

// Module includes
#include "XmppConnection.h"
#include "XmppMessages.h"
#include "XmppChat.h"
#include "XmppPresence.h"
#include "XmppMultiUserChat.h"
#include "XmppPubSub.h"
#include "XmppModule.h"

