// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppConnection.h"

/**
* Chat message received from another Xmpp user
*/
class FXmppChatMessage
{
public:
	/** constructor */
	FXmppChatMessage()
		: Timestamp(0)
	{}

	/** id of message sender */
	FXmppUserJid FromJid;
	/** id of message recipient */
	FXmppUserJid ToJid;
	/** body of the message */
	FString Body;
	/** server provided timestamp for message */
	FDateTime Timestamp;
};

/**
 * Interface for sending/receiving chat messages between users
 */
class IXmppChat
{
public:

	/** destructor */
	virtual ~IXmppChat() = default;

	/**
	 * Send a chat to a user via xmpp service
	 *
	 * @param RecipientId user to send chat to (must be online)
	 * @param Message chat message to send
	 *
	 * @return true if successfully sent
	 */
	virtual bool SendChat(const FXmppUserJid& RecipientId, const FString& Message) = 0;

	/**
	 * Delegate callback for when a new chat message is received
	 *
	 * @param Connection the xmpp connection that received the chat 
	 * @param FromId id of user that sent the chat
	 * @param Chat data that was received
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppChatReceived, const TSharedRef<IXmppConnection>& /*Connection*/, const FXmppUserJid& /*FromJid*/, const TSharedRef<FXmppChatMessage>& /*ChatMsg*/);
	/** @return chat received delegate */
	virtual FOnXmppChatReceived& OnReceiveChat() = 0;
};

