// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class IXmppChat;
class IXmppMessages;
class IXmppMultiUserChat;
class IXmppPresence;
class IXmppPubSub;
class IXmppStanza;

/** Possible XMPP login states */
namespace EXmppLoginStatus
{
	enum Type
	{
		NotStarted,
		ProcessingLogin,
		ProcessingLogout,
		LoggedIn,
		LoggedOut
	};

	inline const TCHAR* ToString(EXmppLoginStatus::Type EnumVal)
	{
		switch (EnumVal)
		{
			case NotStarted:
				return TEXT("NotStarted");
			case ProcessingLogin:
				return TEXT("ProcessingLogin");
			case ProcessingLogout:
				return TEXT("ProcessingLogout");
			case LoggedIn:
				return TEXT("LoggedIn");
			case LoggedOut:
				return TEXT("LoggedOut");
		};
		return TEXT("Unknown");
	}
};

/**
 * Info needed for connecting to an XMPP server
 */
class FXmppServer
{
public:
	FXmppServer() 
		: ServerPort(5222) 
		, bUseSSL(true)
		, bUsePlainTextAuth(false)
		, PingInterval(60.0f)
		, PingTimeout(30.0f)
		, MaxPingRetries(1)
		, bPrivateChatFriendsOnly(false)
		, bResetPingTimeoutOnReceiveStanza(true)
	{}

	/** ip/host to connect to */
	FString ServerAddr;
	/** port number 5222 typically */
	int32 ServerPort;
	/** Platform user id, if applicable */
	FString PlatformUserId;
	/** domain for user jid */
	FString Domain;
	/** client id user is logging in from (constructed from other fields) */
	FString ClientResource;
	/** app id associated with this client */
	FString AppId;
	/** platform the player is using */
	FString Platform;
	/** true to enable SSL handshake for connection */
	bool bUseSSL;
	/** true to allow the usage of plain text authentication */
	bool bUsePlainTextAuth;
	/** seconds between sending server ping requests */
	float PingInterval;
	/** seconds before a server ping request is treated as a timeout */
	float PingTimeout;
	/** max number of retries on ping timeout before connection is abandoned and logged out */
	int32 MaxPingRetries;
	/** limit private chat to friends only */
	bool bPrivateChatFriendsOnly;
	/** reset ping timeout on received stanzas */
	bool bResetPingTimeoutOnReceiveStanza;
};

/**
 * Jid for identifying user on the current connection as well as roster members
 */
class FXmppUserJid
{
public:

	explicit FXmppUserJid(
		FString InId = FString(),
		FString InDomain = FString(),
		FString InResource = FString()
		)
		: Id(MoveTemp(InId))
		, Domain(MoveTemp(InDomain))
		, Resource(MoveTemp(InResource))
	{
	}

	/**
	 * Create an FXmppUserJid from a formatted JID
	 * e.g., user@domain/resource
	 *
	 * @param JidString the formatted JID
	 * @return FXmppUserJid populated from the formatted JID
	 */
	static XMPP_API FXmppUserJid FromFullJid(const FString& JidString);

	/** unique id for the user */
	FString Id;
	/** domain user has access to */
	FString Domain;
	/** client resource user is logged in from */
	FString Resource;

	/**
	 * Get the components that comprise the resource
	 *
	 * @param InResource The resource to parse
	 * @param OutAppId The app id the user is using
	 * @param OutPlatform The platform the user is using
	 * @param OutPlatformUserId The platform user id (optional)
	 *
	 * @return Whether the Resource was successfully parsed or not
	 */
	static XMPP_API bool ParseResource(const FString& InResource, FString& OutAppId, FString& OutPlatform, FString& OutPlatformUserId);

	static XMPP_API FString CreateResource(const FString& AppId, const FString& Platform, const FString& PlatformUserId);

	/** 
	 * Get the components that comprise the resource
	 *
	 * @param OutAppId The app id the user is using
	 * @param OutPlatform The platform the user is using
	 * @param OutPlatformUserId The platform user id (optional)
	 *
	 * @return Whether the Resource was successfully parsed or not
	 */
	bool ParseResource(FString& OutAppId, FString& OutPlatform, FString& OutPlatformUserId) const
	{
		return ParseResource(Resource, OutAppId, OutPlatform, OutPlatformUserId);
	}

	/**
	 * Separate the MUC half of the resource (nickname:userid) from the UserJid resource portion (Vx:AppId:Platform:etc)
	 * This is highly dependent on FChatRoomMemberMcp::BuildMemberJidResourceString
	 */
	static XMPP_API FString ParseMucUserResource(const FString& InResource);

	bool operator==(const FXmppUserJid& Other) const
	{
		return 
			Other.Id == Id && 
			Other.Domain == Domain && 
			Other.Resource == Resource;
	}

	bool operator!=(const FXmppUserJid& Other) const
	{
		return !operator==(Other);
	}

	/** full jid path <id@domain/resource> */
	FString GetFullPath() const
	{
		FString Result(Id);
		if (!Domain.IsEmpty())
		{
			Result += TEXT("@") + Domain;

			if (!Resource.IsEmpty())
			{
				Result += TEXT("/") + Resource;
			}
		}

		return Result;
	}

	/** Return Bare id (id@domain) */
	FString GetBareId() const
	{
		return FString::Printf(TEXT("%s@%s"), *Id, *Domain);
	}

	/** @return true if jid has all valid elements */
	bool IsValid() const
	{
		return !Id.IsEmpty() && !Domain.IsEmpty();
	}

	XMPP_API FString ToDebugString() const;
};

typedef TSharedPtr<IXmppPresence, ESPMode::ThreadSafe> IXmppPresencePtr;
typedef TSharedPtr<IXmppPubSub, ESPMode::ThreadSafe> IXmppPubSubPtr;
typedef TSharedPtr<IXmppMessages, ESPMode::ThreadSafe> IXmppMessagesPtr;
typedef TSharedPtr<IXmppMultiUserChat, ESPMode::ThreadSafe> IXmppMultiUserChatPtr;
typedef TSharedPtr<IXmppChat, ESPMode::ThreadSafe> IXmppChatPtr;

/**
 * Base interface for connecting to Xmpp
 */
class IXmppConnection 
	: public TSharedFromThis<IXmppConnection>
{
public:

	/** destructor */
	virtual ~IXmppConnection() = default;

	/**
	 * Configure the connection with server details
	 *
	 * @param Server server details
	 */
	virtual void SetServer(const FXmppServer& Server) = 0;
	
	/**
	 * Obtain last server details associated with the connection
	 *
	 * @return Server server details
	 */
	virtual const FXmppServer& GetServer() const = 0;
	
	/**
	 * Login on the connection. No socket connection is created until user attempts to login
	 * See OnLoginComplete(), OnLoginChanged() delegates for completion
	 *
	 * @param UserId just the id portion of the jid (domain is in server config)
	 * @param Auth plain text auth credentials for user
	 */
	virtual void Login(const FString& UserId, const FString& Auth) = 0;	

	/**
	 * Logout on the connection with a user that has previously logged in. 
	 * This will close the socket connection and cleanup.
	 * See OnLogoutComplete(), OnLoginChanged() delegates for completion
	 */
	virtual void Logout() = 0;

	/**
	 * Obtain currently cached login status
	 *
	 * @return EXmppLoginStatus based on logged in and socket connection state
	 */
	virtual EXmppLoginStatus::Type GetLoginStatus() const = 0;

	/**
	 * Get the jid of the last user login attempt
	 * Note that they may be logged out
	 *
	 * @param jid to identify user
	 */
	virtual const FXmppUserJid& GetUserJid() const = 0;

	/**
	 * Delegate called when login completes
	 *
	 * @param UserJid jid of user that attempted to login
	 * @param bWasSuccess true if successfully logged in user, false otherwise
	 * @param Error string representing error or empty if success
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppLoginComplete, const FXmppUserJid& /*UserJid*/, bool /*bWasSuccess*/, const FString& /*Error*/);

	/**
	* Delegate called when logout completes
	*
	* @param UserJid jid of user that logged out
	* @param bWasSuccess true if successfully logged out, false otherwise
	* @param Error string representing error or empty if success
	*/
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppLogoutComplete, const FXmppUserJid& /*UserJid*/, bool /*bWasSuccess*/, const FString& /*Error*/);

	/**
	* Delegate called when login completes
	*
	* @param UserJid jid of user that changed login state
	* @param LoginState new login state (see EXmppLoginStatus)
	*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnXmppLoginChanged, const FXmppUserJid& /*UserJid*/, EXmppLoginStatus::Type /*LoginState*/);

	/** Alias for old typo, remove in 4.24 */
	using FOnXmppLogingChanged UE_DEPRECATED(4.23, "Please update usages of 'FOnXmppLogingChanged' to 'FOnXmppLoginChanged'") = FOnXmppLoginChanged;
	/**
	* Delegate called when a message is received (ONLY STROPHE)
	*
	* @param reference to stanza
	*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnXmppStanzaReceived, const IXmppStanza& /* Stanza */);

	/**
	* Delegate called when a message is sent (ONLY STROPHE)
	*
	* @param reference to stanza
	*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnXmppStanzaSent, const IXmppStanza& /* Stanza */);

	/** @return login complete delegate */
	virtual FOnXmppLoginComplete& OnLoginComplete() = 0;
	/** @return login changed delegate */
	virtual FOnXmppLoginChanged& OnLoginChanged() = 0;
	/** @return logout complete delegate */
	virtual FOnXmppLogoutComplete& OnLogoutComplete() = 0;
	/** @return Stanza sent delegate */
	virtual FOnXmppStanzaSent& OnStanzaSent() = 0;
	/** @return Stanza received delegate */
	virtual FOnXmppStanzaReceived& OnStanzaReceived() = 0;

	/** @return Presence interface if available. NULL otherwise */
	virtual IXmppPresencePtr Presence() = 0;
	/** @return PubSub interface if available. NULL otherwise */
	virtual IXmppPubSubPtr PubSub() = 0;
	/** @return Messages interface if available. NULL otherwise */
	virtual IXmppMessagesPtr Messages() = 0;
	/** @return MultiUserChat interface if available. NULL otherwise */
	virtual IXmppMultiUserChatPtr MultiUserChat() = 0;
	/** @return PrivateChat interface if available. NULL otherwise */
	virtual IXmppChatPtr PrivateChat() = 0;
	
	/** Debug dump of this connection */
	virtual void DumpState() const = 0;
};

