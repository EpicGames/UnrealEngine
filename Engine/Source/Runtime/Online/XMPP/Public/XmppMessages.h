// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppConnection.h"

/**
 * Message received from Xmpp user/admin
 */
class FXmppMessage
{
public:
	/** constructor */
	FXmppMessage()
	{}

	/** id of message sender */
	FXmppUserJid FromJid;
	/** id of message recipient */
	FXmppUserJid ToJid;
	/** body of the message */
	FString Payload;
	/** type of the message */
	FString Type;
	/** date sent */
	FDateTime Timestamp;
};

/**
 * Interface for sending/receiving messages between users (also admin to user notifications)
 */
class IXmppMessages
{
public:

	/** destructor */
	virtual ~IXmppMessages() = default;

	/**
	 * Send a message to a user via xmpp service
	 *
	 * @param RecipientId user to send message to (must be online)
	 * @param Type message type
	 * @param Payload message data to send
	 *
	 * @return true if successfully sent
	 */
	virtual bool SendMessage(const FXmppUserJid& RecipientId, const FString& Type, const FString& Payload, bool bPayloadIsSerializedJson = false) = 0;
	virtual bool SendMessage(const FXmppUserJid& RecipientId, const FString& Type, const TSharedRef<class FJsonObject>& Payload) = 0;

	/**
	 * Delegate callback for when a new message is received
	 *
	 * @param Connection the xmpp connection this message was received on
	 * @param FromId id of user that sent the message (might be admin)
	 * @param Message data that was received
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppMessageReceived, const TSharedRef<IXmppConnection>& /*Connection*/, const FXmppUserJid& /*FromJid*/, const TSharedRef<FXmppMessage>& /*Message*/);
	/** @return message received delegate */
	virtual FOnXmppMessageReceived& OnReceiveMessage() = 0;
};

