// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "XmppConnection.h"
#include "XmppMultiUserChat.h"

class Error;
class FOutputDevice;
class IXmppConnection;
class UWorld;

/**
 * Module for Xmpp connections
 * Use CreateConnection to create a new Xmpp connection
 */
class FXmppModule :
	public IModuleInterface, public FSelfRegisteringExec, public FTSTickerObjectBase
{

protected:

	// FSelfRegisteringExec

	/**
	 * Handle exec commands starting with "XMPP"
	 *
	 * @param InWorld	the world context
	 * @param Cmd		the exec command being executed
	 * @param Ar		the archive to log results to
	 *
	 * @return true if the handler consumed the input, false to continue searching handlers
	 */
	XMPP_API virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

public:

	/** 
	 * Exec command handlers
	 */
	XMPP_API bool HandleXmppCommand( const TCHAR* Cmd, FOutputDevice& Ar );

	// FXmppModule

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static XMPP_API FXmppModule& Get();

	/**
	 * Checks to see if this module is loaded and ready.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static XMPP_API bool IsAvailable();

	/**
	 * Creates a new Xmpp connection for the current platform and associated it with the user
	 *
	 * @return new Xmpp connection instance
	 */
	XMPP_API TSharedRef<class IXmppConnection> CreateConnection(const FString& UserId);

	/**
	 * Return an existing Xmpp connection associated with a user
	 *
	 * @return new Xmpp connection instance
	 */
	XMPP_API TSharedPtr<class IXmppConnection> GetConnection(const FString& UserId) const;

	/**
	 * Remove an existing Xmpp connection associated with a user
	 *
	 * @param UserId user to find connection for
	 */
	XMPP_API void RemoveConnection(const FString& UserId);

	/**
	 * Clean up any pending connection removals
	 */
	XMPP_API void ProcessPendingRemovals();

	/**
	 * Remove an existing Xmpp connection
	 *
	 * @param Connection reference to find/remove
	 */
	XMPP_API void RemoveConnection(const TSharedRef<class IXmppConnection>& Connection);

	/**
	 * @return true if Xmpp requests are globally enabled
	 */
	inline bool IsXmppEnabled() const
	{
		return bEnabled;
	}

	// FTSTickerObjectBase
	XMPP_API virtual bool Tick(float DeltaTime) override;

	/**
	 * Delegate callback when a system acquires ownership over an XMPP connection
	 *
	 * @param XmppConnection The connection that is acquired
	 * @param SystemName The name of the system initiating this request
	 */
	DECLARE_EVENT_TwoParams(FXmppModule, FOnXmppConnectionAcquired, const TSharedRef<IXmppConnection>& /*XmppConnection*/, const FString& /*SystemName*/);
	FOnXmppConnectionAcquired OnXmppConnectionAcquired;

	/**
	 * Delegate callback when a system relinquishes ownership of an XMPP connection
	 *
	 * @param XmppConnection The connection that is relinquished
	 * @param SystemName The name of the system initiating this request
	 */
	DECLARE_EVENT_TwoParams(FXmppModule, FOnXmppConnectionRelinquished, const TSharedRef<IXmppConnection>& /*XmppConnection*/, const FString& /*SystemName*/);
	FOnXmppConnectionRelinquished OnXmppConnectionRelinquished;

	/**
	* Delegate fired when an Xmpp connection is created.
	*
	* @param NewConnection Reference to newly created Xmpp connection
	*
	*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnXmppConnectionCreated, const TSharedRef<IXmppConnection>& /*NewConnection*/);
	FOnXmppConnectionCreated OnXmppConnectionCreated;


private:

	// IModuleInterface

	XMPP_API void OnXmppRoomCreated(const TSharedRef<IXmppConnection>& Connection, bool bSuccess, const FXmppRoomId& RoomId, const FString& Error);
	XMPP_API void OnXmppRoomConfigured(const TSharedRef<IXmppConnection>& Connection, bool bSuccess, const FXmppRoomId& RoomId, const FString& Error);

	/**
	 * Called when Xmpp module is loaded
	 * Initialize platform specific parts of Xmpp handling
	 */
	XMPP_API virtual void StartupModule() override;
	
	/**
	 * Called when Xmpp module is unloaded
	 * Shutdown platform specific parts of Xmpp handling
	 */
	XMPP_API virtual void ShutdownModule() override;

	/**
	 * Connection cleanup before removal
	 */
	XMPP_API void CleanupConnection(const TSharedRef<class IXmppConnection>& Connection);

	/** toggles Xmpp requests */
	bool bEnabled;
	/** singleton for the module while loaded and available */
	static XMPP_API FXmppModule* Singleton;

	/** Active Xmpp server connections mapped by user id */
	TMap<FString, TSharedRef<class IXmppConnection>> ActiveConnections;
	/** Xmpp connections pending removal on next tick */
	TSet<TSharedPtr<IXmppConnection>> PendingRemovals;
	/** Keep track of removed connections pending cleanup */
	TArray<TSharedRef<class IXmppConnection>> PendingDeleteConnections;
};
