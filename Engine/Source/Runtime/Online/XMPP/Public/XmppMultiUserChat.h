// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppConnection.h"
#include "XmppPresence.h"
#include "XmppChat.h"

/**
 * Id of a chat room
 */
typedef FString FXmppRoomId;

/**
 * Affiliation of a chat room member
 * (long term associations with a room, persists across visits)
 * http://xmpp.org/extensions/xep-0045.html#affil
 */
namespace EXmppChatMemberAffiliation
{
	enum Type
	{	
		/** Owner of the room */
		Owner,
		/* An admin or owner enters a room as a moderator */
		Admin, 
		/* A member enters a room as a participant. */
		/* As a default, an unaffiliated user enters a moderated room as a visitor, and enters an open room as a participant. */
		Member,
		/** Banned from the room */
		Outcast,
		/** Absence of an affiliation */
		None
	};

	inline const TCHAR* ToString(Type Affiliation)
	{
		switch (Affiliation)
		{
			case Owner:
				return TEXT("Owner");
			case Admin:
				return TEXT("Admin");
			case Member:
				return TEXT("Member");
			case Outcast:
				return TEXT("Outcast");
			case None:
			default:
				return TEXT("None");
		}
	}

	inline Type ToType(const FString& Affiliation)
	{
		if (Affiliation == TEXT("Owner"))
		{
			return Owner;
		}
		else if (Affiliation == TEXT("Admin"))
		{
			return Admin;
		}
		else if (Affiliation == TEXT("Member"))
		{
			return Member;
		}
		else if (Affiliation == TEXT("Outcast"))
		{
			return Outcast;
		}
		else
		{
			return None;
		}
	}
}

/**
 * Role of a chat room member
 * (temporary role while in the room)
 * http://xmpp.org/extensions/xep-0045.html#roles
 */
namespace EXmppChatMemberRole
{
	enum Type
	{
		/* A moderator is the most powerful role within the context of the room, and can to some extent manage other occupants' roles in the room. */
		Moderator,
	    /* A participant has fewer privileges than a moderator, although they always have the right to speak. */
		Participant,
		/* A visitor is a more restricted role within the context of a moderated room */
		Visitor,
		None
	};

	inline const TCHAR* ToString(Type Role)
	{
		switch (Role)
		{
			case Moderator:
				return TEXT("Moderator");
			case Participant:
				return TEXT("Participant");
			case Visitor:
				return TEXT("Visitor");
			case None:
			default:
				return TEXT("None");
		}
	}

	inline Type ToType(const FString& Role)
	{
		if (Role == TEXT("Moderator"))
		{
			return Moderator;
		}
		else if (Role == TEXT("Participant"))
		{
			return Participant;
		}
		else if (Role == TEXT("Visitor"))
		{
			return Visitor;
		}
		else
		{
			return None;
		}
	}
}

/**
 * Member of a chat room
 */
class FXmppChatMember
{
public:
	FXmppChatMember()
		: Affiliation(EXmppChatMemberAffiliation::None)
		, Role(EXmppChatMemberRole::None)
	{}

	explicit FXmppChatMember(const FXmppMucPresence& MemberPresence)
		: Nickname(MemberPresence.GetNickName())
		, RoomMemberJid(MemberPresence.UserJid)
		, UserJid(MemberPresence.MemberJid)
		, UserPresence(MemberPresence)
		, Affiliation(EXmppChatMemberAffiliation::ToType(MemberPresence.Affiliation))
		, Role(EXmppChatMemberRole::ToType(MemberPresence.Role))
	{
	}

	/** Nickname of the user */
	FString Nickname;
	/**
	 * JID of the member for the chat room
	 * Use this to identify the user in the chat room (such as iterating the chat room's members)
	 * ex: RoomId@MucDomain/MemberMucResource
	 */
	FXmppUserJid RoomMemberJid;
	/**
	 * JID of the member outside of the chat room, if the room is non-anonymous. May be blank.
	 * Use this to identify the user outside of the chat room (such as private messages)
	 * ex: AccountId@Domain/Resource
	 * @see https://xmpp.org/extensions/xep-0045.html#enter-nonanon
	 */
	FXmppUserJid UserJid;
	/** Presence of the member */
	FXmppUserPresence UserPresence;
	/** Affiliation of the member in the chat room */
	EXmppChatMemberAffiliation::Type Affiliation;
	/** Role of the member in the chat room */
	EXmppChatMemberRole::Type Role;

	/** @return string representation of this object suitable for logging */
	FString ToDebugString() const
	{
		return FString::Printf(TEXT("Nick=[%s] RoomMemberJid=[%s] UserJid=[%s] Affiliation=%d Role=%d"), *Nickname, *RoomMemberJid.ToDebugString(), *UserJid.ToDebugString(), static_cast<int32>(Affiliation), static_cast<int32>(Role));
	}
};

typedef TSharedPtr<FXmppChatMember, ESPMode::ThreadSafe> FXmppChatMemberPtr;
typedef TSharedRef<FXmppChatMember, ESPMode::ThreadSafe> FXmppChatMemberRef;

/**
 * Info for a joined/created chat room
 */
class FXmppRoomInfo
{
public:
	FXmppRoomInfo()
		: bIsPrivate(false)
	{}

	FXmppRoomId Id;
	FString OwnerId;
	FString Subject;
	bool bIsPrivate;

	FString ToDebugString() const
	{
		return FString::Printf(TEXT("%s Owner: %s Subj: %s Priv: %d"), *Id, *OwnerId, *Subject, bIsPrivate);
	}
};

/**
 * Info to configure a new room
 */
class FXmppRoomConfig
{
public:
	FXmppRoomConfig()
		: RoomName(TEXT(""))
		, RoomDesc(TEXT(""))
		, bIsPrivate(true)
		, Password(TEXT(""))
		, bIsPersistent(false)
		, bAllowPublicSearch(false)
		, bIsModerated(false)
		, bIsMembersOnly(false)
		, bAllowChangeSubject(false)
		, MaxMsgHistory(0)
		, RoomAnonymity(ERoomAnonymity::Semianonymous)
	{}

	enum class ERoomAnonymity
	{
		Semianonymous,
		Nonanonymous,
		Fullanonymous
	};

	static const FString ConvertRoomAnonymityToString(ERoomAnonymity InRoomAnonymity)
	{
		if (InRoomAnonymity == ERoomAnonymity::Nonanonymous)
		{
			return TEXT("nonanonymous");
		}
		else if (InRoomAnonymity == ERoomAnonymity::Semianonymous)
		{
			return TEXT("semianonymous");
		}
		else if (InRoomAnonymity == ERoomAnonymity::Fullanonymous)
		{
			return TEXT("fullanonymous");
		}
		else
		{
			return TEXT("INVALID ROOMANONYMITY ENUM");
		}
	}

	// <field type = "text-single" label = "Natural-Language Room Name" var = "muc#roomconfig_roomname">
	FString RoomName;
	// <field type = "text-single" label = "Short Description of Room" var = "muc#roomconfig_roomdesc">
	FString RoomDesc;
	// <field type = "boolean" label = "Password Required to Enter?" var = "muc#roomconfig_passwordprotectedroom">
	bool bIsPrivate;
	// <field type = "text-single" label = "Password" var = "muc#roomconfig_roomsecret">
	FString Password;
	// <field type = "boolean" label = "Make Room Persistent?" var = "muc#roomconfig_persistentroom">
	bool bIsPersistent;

private:
	// These fields are not used yet.  Move to public when hooking up
	// <field type = "boolean" label = "Make Room Publicly Searchable?" var = "muc#roomconfig_publicroom">
	bool bAllowPublicSearch;
	// <field type = "boolean" label = "Make Room Moderated?" var = "muc#roomconfig_moderatedroom">
	bool bIsModerated;
	// <field type = "boolean" label = "Make Room Members Only?" var = "muc#roomconfig_membersonly">
	bool bIsMembersOnly;
	// <field type = "boolean" label = "Allow Occupants to Change Subject?" var = "muc#roomconfig_changesubject">
	bool bAllowChangeSubject;
	// <field type = "text-single" label = "Maximum Number of History Messages Returned by Room" var = "muc#maxhistoryfetch">
	int32 MaxMsgHistory;
	// <field type = "list-single" label = "Room anonymity level:" var = "muc#roomconfig_anonymity">
	ERoomAnonymity RoomAnonymity;
	// <field type = "list-single" label = "Presence delivery logic" var = "tigase#presence_delivery_logic">
	// <field type = "boolean" label = "Enable filtering of presence (broadcasting presence only between selected groups" var = "tigase#presence_filtering">
	// <field type = "list-multi" label = "Affiliations for which presence should be delivered" var = "tigase#presence_filtered_affiliations">

	// Allow RoomConfigOp to use the defaults we don't expose yet for consumers to change
	friend class FXmppChatRoomConfigOp;
	friend class FXmppMultiUserChatStrophe;
};

/**
 * Interface for creating/joining/chatting in a MUC
 */
class IXmppMultiUserChat
{
public:

	virtual ~IXmppMultiUserChat() = default;

	virtual bool CreateRoom(const FXmppRoomId& RoomId, const FString& Nickname, const FXmppRoomConfig& RoomConfig = FXmppRoomConfig()) = 0;
	virtual bool ConfigureRoom(const FXmppRoomId& RoomId, const FXmppRoomConfig& RoomConfig) = 0;
	virtual bool JoinPublicRoom(const FXmppRoomId& RoomId, const FString& Nickname) = 0;
	virtual bool JoinPrivateRoom(const FXmppRoomId& RoomId, const FString& Nickname, const FString& Password) = 0;
	virtual bool RegisterMember(const FXmppRoomId& RoomId, const FString& Nickname) = 0;
	virtual bool UnregisterMember(const FXmppRoomId& RoomId, const FString& Nickname) = 0;
	virtual bool ExitRoom(const FXmppRoomId& RoomId) = 0;
	virtual bool SendChat(const FXmppRoomId& RoomId, const FString& MsgBody, const FString& ChatInfo) = 0;
	virtual void GetJoinedRooms(TArray<FXmppRoomId>& OutRooms) = 0;
	virtual bool RefreshRoomInfo(const FXmppRoomId& RoomId) = 0;
	virtual bool GetRoomInfo(const FXmppRoomId& RoomId, FXmppRoomInfo& OutRoomInfo) = 0;
	virtual bool GetMembers(const FXmppRoomId& RoomId, TArray<FXmppChatMemberRef>& OutMembers) = 0;
	virtual FXmppChatMemberPtr GetMember(const FXmppRoomId& RoomId, const FXmppUserJid& MemberJid) = 0;
	virtual bool GetLastMessages(const FXmppRoomId& RoomId, int32 NumMessages, TArray< TSharedRef<FXmppChatMessage> >& OutMessages) = 0;
	virtual void HandleMucPresence(const FXmppMucPresence& MemberPresence) = 0;
	virtual void DumpMultiUserChatState() const = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppRoomCreateComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppRoomId& /*RoomId*/, const FString& /*Error*/);
	/** @return room created delegate */
	virtual FOnXmppRoomCreateComplete& OnRoomCreated() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppRoomConfigureComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppRoomId& /*RoomId*/, const FString& /*Error*/);
	/** @return room configured delegate */
	virtual FOnXmppRoomConfigureComplete& OnRoomConfigured() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppRoomInfoRefreshComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppRoomId& /*RoomId*/, const FString& /*Error*/);
	/** @return room refresh delegate */
	virtual FOnXmppRoomInfoRefreshComplete& OnRoomInfoRefreshed() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppRoomJoinPublicComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppRoomId& /*RoomId*/, const FString& /*Error*/);
	/** @return public room joined delegate */
	virtual FOnXmppRoomJoinPublicComplete& OnJoinPublicRoom() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppRoomJoinPrivateComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppRoomId& /*RoomId*/, const FString& /*Error*/);
	/** @return private room joined delegate */
	virtual FOnXmppRoomJoinPrivateComplete& OnJoinPrivateRoom() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppRoomExitComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppRoomId& /*RoomId*/, const FString& /*Error*/);
	/** @return exit room delegate */
	virtual FOnXmppRoomExitComplete& OnExitRoom() = 0;

	/**
	 * Delegate when a member joins a room
	 * @param Connection the XMPP connection
	 * @param RoomId the ID of the room that a member joined
	 * @param RoomMemberJid the JID the member that joined, used to lookup the member inside of the room
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppRoomMemberJoin, const TSharedRef<IXmppConnection>& /*Connection*/, const FXmppRoomId& /*RoomId*/, const FXmppUserJid& /*RoomMemberJid*/);
	/** @return room member joined room delegate */
	virtual FOnXmppRoomMemberJoin& OnRoomMemberJoin() = 0;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppRoomMemberExit, const TSharedRef<IXmppConnection>& /*Connection*/, const FXmppRoomId& /*RoomId*/, const FXmppUserJid& /*RoomMemberJid*/);
	/** @return room member exited room delegate */
	virtual FOnXmppRoomMemberExit& OnRoomMemberExit() = 0;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppRoomMemberChanged, const TSharedRef<IXmppConnection>& /*Connection*/, const FXmppRoomId& /*RoomId*/, const FXmppUserJid& /*RoomMemberJid*/);
	/** @return room member changed delegate */
	virtual FOnXmppRoomMemberChanged& OnRoomMemberChanged() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppRoomChatReceived, const TSharedRef<IXmppConnection>& /*Connection*/, const FXmppRoomId& /*RoomId*/, const FXmppUserJid& /*RoomMemberJid*/, const TSharedRef<FXmppChatMessage>& /*ChatMsg*/);
	/** @return chat room message received delegate */
	virtual FOnXmppRoomChatReceived& OnRoomChatReceived() = 0;
};
