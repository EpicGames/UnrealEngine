// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppConnection.h"

/**
 * Basic presence online states
 */
namespace EXmppPresenceStatus
{
	enum Type
	{
		// online while connected
		Online,
		// offline if not connected
		Offline,
		// online but away due to being afk or manually set
		Away,
		// online but away for a long period or manually set
		ExtendedAway,
		// manually set to avoid interruptions
		DoNotDisturb,
		// currently chatting. implies online
		Chat
	};

	/** @return string version of enum */
	inline const TCHAR* ToString(EXmppPresenceStatus::Type EnumVal)
	{
		switch (EnumVal)
		{
			case Online:
				return TEXT("Online");
			case Offline:
				return TEXT("Offline");
			case Away:
				return TEXT("Away");
			case ExtendedAway:
				return TEXT("ExtendedAway");
			case DoNotDisturb:
				return TEXT("DoNotDisturb");
			case Chat:
				return TEXT("Chat");
		}
		return TEXT("");
	}
}

/**
 * User presence info obtained from Xmpp roster
 */
class FXmppUserPresence
{
public:
	/** state of basic online status */
	EXmppPresenceStatus::Type Status = EXmppPresenceStatus::Offline;
	/** connected an available to receive messages */
	bool bIsAvailable = true;
	/** time when presence was sent by the user */
	FDateTime SentTime = FDateTime::MinValue();
	/** time when presence was sent by the user */
	FDateTime ReceivedTime = FDateTime::MinValue();
	/** app id user is logged in from */
	FString AppId;
	/** platform associated with this client */
	FString Platform;
	/** string that will be parsed for further displayed presence info */
	FString StatusStr;
	/** full jid for user that sent this presence update */
	FXmppUserJid UserJid;

	inline bool operator==(const FXmppUserPresence& Presence) const
	{
		// SentTime is explicitly not checked
		return Status == Presence.Status &&
			bIsAvailable == Presence.bIsAvailable &&
			AppId == Presence.AppId &&
			Platform == Presence.Platform &&
			StatusStr == Presence.StatusStr &&
			UserJid == Presence.UserJid;
	}
	inline bool operator!=(const FXmppUserPresence& Presence) const
	{
		return !(*this == Presence);
	}
};

/**
 * Muc room presence from an Xmpp muc room member
 */
class FXmppMucPresence : public FXmppUserPresence
{
public:
	/** constructor */
	FXmppMucPresence()
		: FXmppUserPresence()
	{}

	/** Role of the member in the room */
	FString Role;
	/** Affiliation of the member in the room */
	FString Affiliation;
	/**
	 * JID of the member outside of the chat room, if the room is non-anonymous. May be blank.
	 * Different from UserJid, where UserJid is the identifier of this member inside of the chat room.
	 * Use this to identify the user outside of the chat room (such as private messages)
	 * ex: AccountId@Domain/Resource
	 * @see https://xmpp.org/extensions/xep-0045.html#enter-nonanon
	 */
	FXmppUserJid MemberJid;

	const FString& GetRoomId() const { return UserJid.Id; }
	const FString& GetNickName() const { return UserJid.Resource; }
};

/**
 * Interface for updating presence for current user and for obtaining updates of roster members
 */
class IXmppPresence
{
public:

	/** destructor */
	virtual ~IXmppPresence() = default;

	/**
	 * Send a presence update to Xmpp service for current user logged in with the connection.
	 * Also updates cached copy of presence data
	 *
	 * @param Presence status data to update
	 * @return true if successfully sent
	 */
	virtual bool UpdatePresence(const FXmppUserPresence& Presence) = 0;

	/**
	 * Get the last cached presence for the current user
	 *
	 * @return Presence status data
	 */
	virtual const FXmppUserPresence& GetPresence() const = 0;

	/**
	 * Kick off a query for presence data for a given user
	 *
	 * @return true if successfully sent
	 */
	virtual bool QueryPresence(const FString& UserId) = 0;

	/**
	 * Obtain a list of current roster member id
	 *
	 * @param OutMembers [out] members to update
	 */
	virtual void GetRosterMembers(TArray<FXmppUserJid>& OutMembers) = 0;

	/**
	 * Obtain presence info for a given roster member id
	 *
	 * @return cached presence or NULL if not found
	 */
	virtual TArray<TSharedPtr<FXmppUserPresence>> GetRosterPresence(const FString& UserId) = 0;

	/**
	 * Delegate callback for when new presence data is received for a given user.
	 * Can also obtain cached data via GetRosterPresence()
	 *
	 * @param Connection the xmpp connection this message was received on
	 * @param UserId roster member or current user that updated
	 * @param Presence presence data that was received
	 *
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppPresenceReceived, const TSharedRef<IXmppConnection>& /*Connection*/, const FXmppUserJid& /*FromJid*/, const TSharedRef<FXmppUserPresence>& /*Presence*/);
	/** @return presence received delegate */
	virtual FOnXmppPresenceReceived& OnReceivePresence() = 0;
};


