// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "XmppConnection.h"

/**
 * Id of a PubSub node
 */
typedef FString FXmppPubSubId;

/**
 * Info to configure a new PubSub node
 */
class FXmppPubSubConfig
{
public:
	FXmppPubSubConfig()
		: bPersistItems(false) 
		, bCollection(false)
		, MaxItems(1)
		, MaxPayloadSize(4*1024)
	{}

	/** items published to the node are not persisted */
	bool bPersistItems;
	/** node is a collection of node instead of a leaf node. can only publish to leaf nodes */
	bool bCollection;
	/** maximum number of items that the node will persist */
	int32 MaxItems;
	/** maximum size of item payload in bytes */
	int32 MaxPayloadSize;
	/** optional id of collection node to use as container/parent for this node */
	FXmppPubSubId CollectionId;
};

/**
 * Message received from PubSub node
 */
class FXmppPubSubMessage
{
public:
	/** constructor */
	FXmppPubSubMessage()
	{}

	/** jid of node that sent the message */
	FXmppUserJid FromJid;
	/** jid of message recipient */
	FXmppUserJid ToJid;
	/** body of the message */
	FString Payload;
	/** type of the message */
	FString Type;
	/** date sent */
	FDateTime Timestamp;
};

/**
 * Info cached about a PubSub node
 */
class FXmppPubSubNode
{
public:
	FXmppPubSubNode()
	{}

	/** id/path of the pubsub node */
	FXmppPubSubId Id;
	/** configuration of the node */
	FXmppPubSubConfig Config;
};

/**
 * Interface for publishing/subscribing to events
 */
class IXmppPubSub
{
public:

	virtual ~IXmppPubSub() = default;

	virtual bool CreateNode(const FXmppPubSubId& NodeId, const FXmppPubSubConfig& NodeConfig) = 0;
	virtual bool ConfigureNode(const FXmppPubSubId& NodeId, const FXmppPubSubConfig& NodeConfig) = 0;
	virtual bool DestroyNode(const FXmppPubSubId& NodeId) = 0;
	virtual bool QueryNode(const FXmppPubSubId& NodeId) = 0;
	virtual bool QuerySubscriptions() = 0;
	virtual bool Subscribe(const FXmppPubSubId& NodeId) = 0;
	virtual bool Unsubscribe(const FXmppPubSubId& NodeId) = 0;
	virtual bool PublishMessage(const FXmppPubSubId& NodeId, const FXmppPubSubMessage& Message) = 0;
	virtual TArray<FXmppPubSubId> GetOwnedNodes() const = 0;
	virtual TArray<FXmppPubSubId> GetSubscribedNodes() const = 0;
	virtual TSharedPtr<FXmppPubSubNode> GetNodeInfo(const FXmppPubSubId& NodeId) const = 0;
	virtual bool GetLastMessages(const FXmppPubSubId& NodeId, int32 NumMessages, TArray< TSharedRef<FXmppPubSubMessage> >& OutMessages) const = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppPubSubCreateNodeComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppPubSubId& /*NodeId*/, const FString& /*Error*/);
	/** @return pubsub node created delegate */
	virtual FOnXmppPubSubCreateNodeComplete& OnCreateNodeComplete() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppPubSubConfigureNodeComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppPubSubId& /*NodeId*/, const FString& /*Error*/);
	/** @return pubsub node configured delegate */
	virtual FOnXmppPubSubConfigureNodeComplete& OnConfigureNodeComplete() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppPubSubDestroyNodeComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppPubSubId& /*NodeId*/, const FString& /*Error*/);
	/** @return pubsub node destroyed delegate */
	virtual FOnXmppPubSubDestroyNodeComplete& OnDestroyNodeComplete() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppPubSubQueryNodeComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppPubSubId& /*NodeId*/, const FString& /*Error*/);
	/** @return pubsub node query delegate */
	virtual FOnXmppPubSubQueryNodeComplete& OnQueryNodeComplete() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppPubSubQuerySubscriptionsComplete, const TSharedRef<IXmppConnection>& /*Connection*/, bool /*bSuccess*/, const FXmppPubSubId& /*NodeId*/, const FString& /*Error*/);
	/** @return pubsub node created delegate */
	virtual FOnXmppPubSubQuerySubscriptionsComplete& OnQuerySubscriptionsComplete() = 0;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppPubSubSubscribed, const TSharedRef<IXmppConnection>& /*Connection*/, FXmppPubSubId& /*NodeId*/, const FXmppUserJid& /*UserJid*/);
	/** @return pubsub node subscription delegate */
	virtual FOnXmppPubSubSubscribed& OnSubscribed() = 0;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnXmppPubSubUnsubscribed, const TSharedRef<IXmppConnection>& /*Connection*/, FXmppPubSubId& /*NodeId*/, const FXmppUserJid& /*UserJid*/);
	/** @return pubsub node unsubscribe delegate */
	virtual FOnXmppPubSubUnsubscribed& OnUnsubscribed() = 0;

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnXmppPubSubMessageReceived, const TSharedRef<IXmppConnection>& /*Connection*/, FXmppPubSubId& /*NodeId*/, const FXmppUserJid& /*UserJid*/, const TSharedRef<FXmppPubSubMessage>& /*PubSubMsg*/);
	/** @return pubsub message received delegate */
	virtual FOnXmppPubSubMessageReceived& OnMessageReceived() = 0;
};

