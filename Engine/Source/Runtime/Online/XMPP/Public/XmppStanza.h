// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FXmppUserJid;

/**
 * Xmpp stanza abstract interface
 * Exposes getters for various attributes and fields of an Xmpp stanza
 */
class IXmppStanza
{
public:

	virtual ~IXmppStanza() = default;

	/**
	 * Get Stanza type (presence, query, message)
	 * @return stanza type
	 */
	virtual FString GetName() const = 0;
	
	/**
	 * Get Stanza text (concatenates all child text nodes)
	 * @return stanza text
	 */
	virtual FString GetText() const = 0;

	/**
	 * Get Stanza type
	 * @return stanza type
	 */
	virtual FString GetType() const = 0;

	/**
	 * Get Stanza Id
	 * @return Stanza Id
	 */
	virtual FString GetId() const = 0;

	/**
	 * Get Stanza to (destination) user Jid
	 * @return Stanza to Jid
	 */
	virtual FXmppUserJid GetTo() const = 0;

	/**
	 * Get Stanza from (source) user Jid
	 * @return Stanza from Jid
	 */
	virtual FXmppUserJid GetFrom() const = 0;

	/**
	 * Get Stanza attribute
	 * @param Key attribute name
	 * @return attribute text
	 */
	virtual FString GetAttribute(const FString& Key) const = 0;

	/**
	 * Returns whether or not stanza has a given attribute
	 * @param Key attribute name
	 * @return whether or not stanza has attribute named Key
	 */
	virtual bool HasAttribute(const FString& Key) const = 0;

	/**
	 * Get stanza body text (if any)
	 * @return Stanza body text
	 */
	virtual TOptional<FString> GetBodyText() const = 0;

	/**
	 * Get given Stanza child, if existing
	 * @param ChildName name of child to return
	 * @return Stanza ChildName, if it exists
	 */
	virtual TUniquePtr<IXmppStanza> GetChild(const FString& ChildName) const = 0;

	/**
	 * Whether or not Stanza has a given child
	 * @param ChildName name of child to check for 
	 * @return Whether or not Stanza has child ChildName
	 */
	virtual bool HasChild(const FString& ChildName) const = 0;
};
