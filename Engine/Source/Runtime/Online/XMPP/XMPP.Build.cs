// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class XMPP : ModuleRules
{
	protected virtual bool bTargetPlatformSupportsStrophe
	{
		get =>
			Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) ||
			Target.Platform == UnrealTargetPlatform.Android ||
			Target.Platform == UnrealTargetPlatform.IOS ||
			Target.Platform == UnrealTargetPlatform.Mac ||
			Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
	}

	protected virtual bool bRequireOpenSSL { get { return false; } }

	public XMPP(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDefinitions.Add("XMPP_PACKAGE=1");

		PrivateDependencyModuleNames.AddRange(
			new string[] 
			{ 
				"Core",
				"Json"
			}
		);

		if (bTargetPlatformSupportsStrophe)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "libstrophe");
			PrivateDependencyModuleNames.Add("WebSockets");
			PrivateDefinitions.Add("WITH_XMPP_STROPHE=1");
		}
		else
		{
			PrivateDefinitions.Add("WITH_XMPP_STROPHE=0");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) ||
			Target.Platform == UnrealTargetPlatform.Mac ||
			bRequireOpenSSL)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "libcurl");
		}
	}
}
